// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Collections.Immutable;
using System.Diagnostics;
using System.Linq;
using System.Threading;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// Enumerates build action types.
	/// </summary>
	enum ActionType
	{
		BuildProject,

		GatherModuleDependencies,

		CompileModuleInterface,

		Compile,

		CreateAppBundle,

		GenerateDebugInfo,

		Link,

		WriteMetadata,

		PostBuildStep,

		ParseTimingInfo,
	}

	/// <summary>
	/// Defines the action's support for artifacts
	/// </summary>
	[Flags]
	enum ArtifactMode : byte
	{

		/// <summary>
		/// Cached artifacts aren't enabled
		/// </summary>
		None = 0,

		/// <summary>
		/// If set, the outputs should be cached but respect the flags below
		/// </summary>
		Enabled = 1 << 0,

		/// <summary>
		/// Absolute file paths must be used when recording inputs and outputs
		/// </summary>
		AbsolutePath = 1 << 1,

		/// <summary>
		/// For actions that can't be cached, by setting this flag, the inputs for the action in question
		/// will be used as additional inputs for any action that references outputs of this action.  
		/// For example, PCH files shouldn't be cached, but compiles that use the PCH should still be 
		/// valid if consider the inputs used to create the PCH file.
		/// </summary>
		PropagateInputs = 1 << 2,
	}

	interface IExternalAction
	{
		/// <summary>
		/// The type of this action (for debugging purposes).
		/// </summary>
		ActionType ActionType { get; }

		/// <summary>
		/// Artifact support for this step
		/// </summary>
		ArtifactMode ArtifactMode { get; }

		/// <summary>
		/// Every file this action depends on.  These files need to exist and be up to date in order for this action to even be considered
		/// </summary>
		IEnumerable<FileItem> PrerequisiteItems { get; }

		/// <summary>
		/// The files that this action produces after completing
		/// </summary>
		IEnumerable<FileItem> ProducedItems { get; }

		/// <summary>
		/// Items that should be deleted before running this action
		/// </summary>
		IEnumerable<FileItem> DeleteItems { get; }

		/// <summary>
		/// Root paths for this action (generally engine root project root, toolchain root, sdk root)
		/// </summary>
		CppRootPaths RootPaths { get; }

		/// <summary>
		/// For C++ source files, specifies a dependency list file used to check changes to header files
		/// </summary>
		FileItem? DependencyListFile { get; }

		/// <summary>
		/// Directory from which to execute the program to create produced items
		/// </summary>
		DirectoryReference WorkingDirectory { get; }

		/// <summary>
		/// The command to run to create produced items
		/// </summary>
		FileReference CommandPath { get; }

		/// <summary>
		/// Command-line parameters to pass to the program
		/// </summary>
		string CommandArguments { get; }

		/// <summary>
		/// Any arguments used in a response file to specify arguments to the executable.
		/// The response file is expected to be passed as part of <see cref="CommandArguments"/>.
		/// </summary>
		ImmutableArray<string> ResponseFileContents { get; }

		/// <summary>
		/// Version of the command used for this action. This will be considered a dependency.
		/// </summary>
		string CommandVersion { get; }

		/// <summary>
		/// Optional friendly description of the type of command being performed, for example "Compile" or "Link".  Displayed by some executors.
		/// </summary>
		string CommandDescription { get; }

		/// <summary>
		/// Human-readable description of this action that may be displayed as status while invoking the action.  This is often the name of the file being compiled, or an executable file name being linked.  Displayed by some executors.
		/// </summary>
		string StatusDescription { get; }

		/// <summary>
		/// True if this action is allowed to be run on a remote machine when a distributed build system is being used, such as XGE
		/// </summary>
		bool bCanExecuteRemotely { get; }

		/// <summary>
		/// True if this action is allowed to be run on a remote machine with SNDBS. Files with #import directives must be compiled locally. Also requires bCanExecuteRemotely = true.
		/// </summary>
		bool bCanExecuteRemotelyWithSNDBS { get; }

		/// <summary>
		/// True if this action is allowed to be run on a remote machine with XGE. Files with #import directives must be compiled locally. Also requires bCanExecuteRemotely = true.
		/// </summary>
		bool bCanExecuteRemotelyWithXGE { get; }

		/// <summary>
		/// True if this action can be executed by Unreal Build Accelerator
		/// </summary>
		bool bCanExecuteInUBA { get; }

		/// <summary>
		/// True if this action can be executed on a cross-architecture Unreal Build Accelerator helper. If no cross-architecture toolchain is found this setting has no effect
		/// </summary>
		bool bCanExecuteInUBACrossArchitecture { get; }

		/// <summary>
		/// True if this action is using the Clang compiler.  Some build systems may be able to optimize for this case.
		/// </summary>
		bool bIsClangCompiler { get; }

		/// <summary>
		/// True if the action executor should delete output files on an error.
		/// May not work for XGE, SN-DBS, and FASTBuild executors.
		/// </summary>
		bool bDeleteProducedItemsOnError { get; }

		/// <summary>
		/// True if the action executor should force warning output to fail the action when the exit code would otherwise be zero.
		/// Used if the command to be run does not support a warnings as error flag.
		/// May not work for XGE, SN-DBS, and FASTBuild executors.
		/// </summary>
		bool bForceWarningsAsError { get; }

		/// <summary>
		/// Whether we should log this action, whether executed locally or remotely.  This is useful for actions that take time
		/// but invoke tools without any console output.
		/// </summary>
		bool bShouldOutputStatusDescription { get; }

		/// <summary>
		/// Whether we should output log of this action, whether executed locally or remotely.  This is useful for actions that spam
		/// logs to console output with non critical data
		/// </summary>
		bool bShouldOutputLog { get; }

		/// <summary>
		/// True if any libraries produced by this action should be considered 'import libraries'
		/// </summary>
		bool bProducesImportLibrary { get; }

		/// <summary>
		/// Whether changes in the command line used to generate these produced items should invalidate the action
		/// </summary>
		bool bUseActionHistory { get; }

		/// <summary>
		/// True if this action should be scheduled early. Some actions can take a long time (pch) and need to start very early
		/// If set to true it will propagate high priority to all its prerequisite tasks
		/// </summary>
		bool bIsHighPriority { get; }

		/// <summary>
		/// Used to determine how much weight(CPU and Memory work) this action is.
		/// </summary>
		double Weight { get; }

		/// <summary>
		/// Used to determine which cache bucket should be used for this action
		/// Using different cache buckets is an optimization where we want one build to use as few buckets as possible while 
		/// two different builds should not use the same bucket if there is no chance their cache entries will be the same
		/// </summary>
		public uint CacheBucket { get; }
	}

	/// <summary>
	/// A build action.
	/// </summary>
	class Action : IExternalAction
	{
		///
		/// Preparation and Assembly (serialized)
		/// 

		/// <summary>
		/// The type of this action (for debugging purposes).
		/// </summary>
		public ActionType ActionType { get; set; }

		/// <summary>
		/// Artifact support for this step
		/// </summary>
		public ArtifactMode ArtifactMode { get; set; } = ArtifactMode.None;

		/// <summary>
		/// Every file this action depends on.  These files need to exist and be up to date in order for this action to even be considered
		/// </summary>
		public SortedSet<FileItem> PrerequisiteItems { get; set; } = new(FileItem.SortByPath);

		/// <summary>
		/// The files that this action produces after completing
		/// </summary>
		public SortedSet<FileItem> ProducedItems { get; set; } = new(FileItem.SortByPath);

		/// <summary>
		/// Items that should be deleted before running this action
		/// </summary>
		public SortedSet<FileItem> DeleteItems { get; set; } = new(FileItem.SortByPath);

		/// <summary>
		/// Root paths for this action (generally engine root project root, toolchain root, sdk root)
		/// Order needs to be maintained, so a list is used
		/// </summary>
		public CppRootPaths RootPaths { get; set; } = new();

		/// <summary>
		/// For C++ source files, specifies a dependency list file used to check changes to header files
		/// </summary>
		public FileItem? DependencyListFile { get; set; }

		/// <summary>
		/// Directory from which to execute the program to create produced items
		/// </summary>
		public DirectoryReference WorkingDirectory { get; set; } = null!;

		/// <summary>
		/// The command to run to create produced items
		/// </summary>
		public FileReference CommandPath { get; set; } = null!;

		/// <summary>
		/// Command-line parameters to pass to the program
		/// </summary>
		public string CommandArguments { get; set; } = null!;

		public ImmutableArray<string> ResponseFileContents { get; set; } = ImmutableArray<string>.Empty;

		/// <summary>
		/// Version of the command used for this action. This will be considered a dependency.
		/// </summary>
		public string CommandVersion { get; set; } = "0";

		/// <summary>
		/// Optional friendly description of the type of command being performed, for example "Compile" or "Link".  Displayed by some executors.
		/// </summary>
		public string CommandDescription { get; set; } = null!;

		/// <summary>
		/// Human-readable description of this action that may be displayed as status while invoking the action.  This is often the name of the file being compiled, or an executable file name being linked.  Displayed by some executors.
		/// </summary>
		public string StatusDescription { get; set; } = "...";

		/// <summary>
		/// True if this action is allowed to be run on a remote machine when a distributed build system is being used, such as XGE
		/// </summary>
		public bool bCanExecuteRemotely { get; set; } = false;

		/// <summary>
		/// True if this action is allowed to be run on a remote machine with SNDBS. Files with #import directives must be compiled locally. Also requires bCanExecuteRemotely = true.
		/// </summary>
		public bool bCanExecuteRemotelyWithSNDBS { get; set; } = true;

		/// <summary>
		/// True if this action is allowed to be run on a remote machine with XGE. Files with #import directives must be compiled locally. Also requires bCanExecuteRemotely = true.
		/// </summary>
		public bool bCanExecuteRemotelyWithXGE { get; set; } = true;

		/// <summary>
		/// True if this action can be executed in uba
		/// </summary>
		public bool bCanExecuteInUBA { get; set; } = true;

		/// <summary>
		/// True if this action can be executed on a cross-architecture Unreal Build Accelerator helper. If no cross-architecture toolchain is found this setting has no effect
		/// </summary>
		public bool bCanExecuteInUBACrossArchitecture { get; set; } = true;

		/// <summary>
		/// True if this action is using the Clang compiler.  Some build systems may be able to optimize for this case.
		/// </summary>
		public bool bIsClangCompiler { get; set; } = false;

		/// <summary>
		/// True if the action executor should delete output files on an error.
		/// May not work for XGE, SN-DBS, and FASTBuild executors.
		/// </summary>
		public bool bDeleteProducedItemsOnError { get; set; } = false;

		/// <summary>
		/// True if the action executor should force warning output to fail the action when the exit code would otherwise be zero.
		/// Used if the command to be run does not support a warnings as error flag.
		/// May not work for XGE, SN-DBS, and FASTBuild executors.
		/// </summary>
		public bool bForceWarningsAsError { get; set; } = false;

		/// <summary>
		/// Whether we should log this action, whether executed locally or remotely.  This is useful for actions that take time
		/// but invoke tools without any console output.
		/// </summary>
		public bool bShouldOutputStatusDescription { get; set; } = true;

		/// <inheritdoc/>
		public bool bShouldOutputLog { get; set; } = true;

		/// <summary>
		/// True if any libraries produced by this action should be considered 'import libraries'
		/// </summary>
		public bool bProducesImportLibrary { get; set; } = false;

		/// <inheritdoc/>
		public bool bUseActionHistory { get; set; } = true;

		/// <inheritdoc/>
		public bool bIsHighPriority { get; set; } = false;

		/// <inheritdoc/>
		public double Weight { get; set; } = 1.0;

		/// <inheritdoc/>
		public uint CacheBucket { get; set; }

		IEnumerable<FileItem> IExternalAction.PrerequisiteItems => PrerequisiteItems;
		IEnumerable<FileItem> IExternalAction.ProducedItems => ProducedItems;
		IEnumerable<FileItem> IExternalAction.DeleteItems => DeleteItems;
		public Action(ActionType InActionType)
		{
			ActionType = InActionType;

			// link actions are going to run locally on SN-DBS so don't try to distribute them as that generates warnings for missing tool templates
			if (ActionType == ActionType.Link)
			{
				bCanExecuteRemotelyWithSNDBS = false;
			}
		}

		public Action(IExternalAction InOther)
		{
			ActionType = InOther.ActionType;
			ArtifactMode = InOther.ArtifactMode;
			PrerequisiteItems = new (InOther.PrerequisiteItems, FileItem.SortByPath);
			ProducedItems = new(InOther.ProducedItems, FileItem.SortByPath);
			DeleteItems = new(InOther.DeleteItems, FileItem.SortByPath);
			RootPaths = new(InOther.RootPaths);
			DependencyListFile = InOther.DependencyListFile;
			WorkingDirectory = InOther.WorkingDirectory;
			CommandPath = InOther.CommandPath;
			CommandArguments = InOther.CommandArguments;
			ResponseFileContents = InOther.ResponseFileContents;
			CommandVersion = InOther.CommandVersion;
			CommandDescription = InOther.CommandDescription;
			StatusDescription = InOther.StatusDescription;
			bCanExecuteRemotely = InOther.bCanExecuteRemotely;
			bCanExecuteRemotelyWithSNDBS = InOther.bCanExecuteRemotelyWithSNDBS;
			bCanExecuteRemotelyWithXGE = InOther.bCanExecuteRemotelyWithXGE;
			bCanExecuteInUBA = InOther.bCanExecuteInUBA;
			bCanExecuteInUBACrossArchitecture = InOther.bCanExecuteInUBACrossArchitecture;
			bIsClangCompiler = InOther.bIsClangCompiler;
			bDeleteProducedItemsOnError = InOther.bDeleteProducedItemsOnError;
			bForceWarningsAsError = InOther.bForceWarningsAsError;
			bShouldOutputStatusDescription = InOther.bShouldOutputStatusDescription;
			bShouldOutputLog = InOther.bShouldOutputLog;
			bProducesImportLibrary = InOther.bProducesImportLibrary;
			bUseActionHistory = InOther.bUseActionHistory;
			bIsHighPriority = InOther.bIsHighPriority;
			Weight = InOther.Weight;
			CacheBucket = InOther.CacheBucket;
		}

		public Action(BinaryArchiveReader Reader)
		{
			ActionType = (ActionType)Reader.ReadByte();
			ArtifactMode = (ArtifactMode)Reader.ReadByte();
			WorkingDirectory = Reader.ReadDirectoryReferenceNotNull();
			CommandPath = Reader.ReadFileReference();
			CommandArguments = Reader.ReadString()!;
			ResponseFileContents = [.. Reader.ReadList(() => Reader.ReadString()!)!];
			CommandVersion = Reader.ReadString()!;
			CommandDescription = Reader.ReadString()!;
			StatusDescription = Reader.ReadString()!;
			bCanExecuteRemotely = Reader.ReadBool();
			bCanExecuteRemotelyWithSNDBS = Reader.ReadBool();
			bCanExecuteRemotelyWithXGE = Reader.ReadBool();
			bCanExecuteInUBA = Reader.ReadBool();
			bCanExecuteInUBACrossArchitecture = Reader.ReadBool();
			bIsClangCompiler = Reader.ReadBool();
			bDeleteProducedItemsOnError = Reader.ReadBool();
			bForceWarningsAsError = Reader.ReadBool();
			bShouldOutputStatusDescription = Reader.ReadBool();
			bShouldOutputLog = Reader.ReadBool();
			bProducesImportLibrary = Reader.ReadBool();
			PrerequisiteItems = Reader.ReadSortedSet(() => Reader.ReadFileItem(), FileItem.SortByPath)!;
			ProducedItems = Reader.ReadSortedSet(() => Reader.ReadFileItem(), FileItem.SortByPath)!;
			DeleteItems = Reader.ReadSortedSet(() => Reader.ReadFileItem(), FileItem.SortByPath)!;
			RootPaths = new(Reader);
			DependencyListFile = Reader.ReadFileItem();
			bUseActionHistory = Reader.ReadBool();
			bIsHighPriority = Reader.ReadBool();
			Weight = Reader.ReadDouble();
			CacheBucket = Reader.ReadUnsignedInt();
		}

		/// <summary>
		/// ISerializable: Called when serialized to report additional properties that should be saved
		/// </summary>
		public void Write(BinaryArchiveWriter Writer)
		{
			Writer.WriteByte((byte)ActionType);
			Writer.WriteByte((byte)ArtifactMode);
			Writer.WriteDirectoryReference(WorkingDirectory);
			Writer.WriteFileReference(CommandPath);
			Writer.WriteString(CommandArguments);
			Writer.WriteList(ResponseFileContents, Writer.WriteString);
			Writer.WriteString(CommandVersion);
			Writer.WriteString(CommandDescription);
			Writer.WriteString(StatusDescription);
			Writer.WriteBool(bCanExecuteRemotely);
			Writer.WriteBool(bCanExecuteRemotelyWithSNDBS);
			Writer.WriteBool(bCanExecuteRemotelyWithXGE);
			Writer.WriteBool(bCanExecuteInUBA);
			Writer.WriteBool(bCanExecuteInUBACrossArchitecture);
			Writer.WriteBool(bIsClangCompiler);
			Writer.WriteBool(bDeleteProducedItemsOnError);
			Writer.WriteBool(bForceWarningsAsError);
			Writer.WriteBool(bShouldOutputStatusDescription);
			Writer.WriteBool(bShouldOutputLog);
			Writer.WriteBool(bProducesImportLibrary);
			Writer.WriteSortedSet(PrerequisiteItems, Item => Writer.WriteFileItem(Item));
			Writer.WriteSortedSet(ProducedItems, Item => Writer.WriteFileItem(Item));
			Writer.WriteSortedSet(DeleteItems, Item => Writer.WriteFileItem(Item));
			RootPaths.Write(Writer);
			Writer.WriteFileItem(DependencyListFile);
			Writer.WriteBool(bUseActionHistory);
			Writer.WriteBool(bIsHighPriority);
			Writer.WriteDouble(Weight);
			Writer.WriteUnsignedInt(CacheBucket);
		}

		/// <summary>
		/// Writes an action to a json file
		/// </summary>
		/// <param name="Object">The object to parse</param>
		public static Action ImportJson(JsonObject Object)
		{
			Action Action = new(Object.GetEnumField<ActionType>("Type"));

			if (Object.TryGetEnumField("ArtifactMode", out ArtifactMode ArtifactMode))
			{
				Action.ArtifactMode = ArtifactMode;
			}

			if (Object.TryGetStringField("WorkingDirectory", out string? WorkingDirectory))
			{
				Action.WorkingDirectory = new DirectoryReference(WorkingDirectory);
			}

			if (Object.TryGetStringField("CommandPath", out string? CommandPath))
			{
				Action.CommandPath = new FileReference(CommandPath);
			}

			if (Object.TryGetStringField("CommandArguments", out string? CommandArguments))
			{
				Action.CommandArguments = CommandArguments;
			}

			if (Object.TryGetStringArrayField("ResponseFileContents", out string[]? ResponseFileContents))
			{
				Action.ResponseFileContents = [.. ResponseFileContents];
			}

			if (Object.TryGetStringField("CommandVersion", out string? CommandVersion))
			{
				Action.CommandVersion = CommandVersion;
			}

			if (Object.TryGetStringField("CommandDescription", out string? CommandDescription))
			{
				Action.CommandDescription = CommandDescription;
			}

			if (Object.TryGetStringField("StatusDescription", out string? StatusDescription))
			{
				Action.StatusDescription = StatusDescription;
			}

			if (Object.TryGetBoolField("bCanExecuteRemotely", out bool bCanExecuteRemotely))
			{
				Action.bCanExecuteRemotely = bCanExecuteRemotely;
			}

			if (Object.TryGetBoolField("bCanExecuteRemotelyWithSNDBS", out bool bCanExecuteRemotelyWithSNDBS))
			{
				Action.bCanExecuteRemotelyWithSNDBS = bCanExecuteRemotelyWithSNDBS;
			}

			if (Object.TryGetBoolField("bCanExecuteRemotelyWithXGE", out bool bCanExecuteRemotelyWithXGE))
			{
				Action.bCanExecuteRemotelyWithXGE = bCanExecuteRemotelyWithXGE;
			}

			if (Object.TryGetBoolField("bCanExecuteInUBA", out bool bCanExecuteInUBA))
			{
				Action.bCanExecuteInUBA = bCanExecuteInUBA;
			}

			if (Object.TryGetBoolField("bCanExecuteInUBACrossArchitecture", out bool bCanExecuteInUBACrossArchitecture))
			{
				Action.bCanExecuteInUBACrossArchitecture = bCanExecuteInUBACrossArchitecture;
			}

			if (Object.TryGetBoolField("bIsClangCompiler", out bool bIsClangCompiler))
			{
				Action.bIsClangCompiler = bIsClangCompiler;
			}

			if (Object.TryGetBoolField("bDeleteProducedItemsOnError", out bool bDeleteProducedItemsOnError))
			{
				Action.bDeleteProducedItemsOnError = bDeleteProducedItemsOnError;
			}

			if (Object.TryGetBoolField("bForceWarningsAsError", out bool bForceWarningsAsError))
			{
				Action.bForceWarningsAsError = bForceWarningsAsError;
			}

			if (Object.TryGetBoolField("bShouldOutputStatusDescription", out bool bShouldOutputStatusDescription))
			{
				Action.bShouldOutputStatusDescription = bShouldOutputStatusDescription;
			}

			if (Object.TryGetBoolField("bShouldOutputLog", out bool bShouldOutputLog)) 
			{
				Action.bShouldOutputLog= bShouldOutputLog;
			}

			if (Object.TryGetBoolField("bProducesImportLibrary", out bool bProducesImportLibrary))
			{
				Action.bProducesImportLibrary = bProducesImportLibrary;
			}

			if (Object.TryGetStringArrayField("PrerequisiteItems", out string[]? PrerequisiteItems))
			{
				Action.PrerequisiteItems.UnionWith(PrerequisiteItems.Select(x => FileItem.GetItemByPath(x)));
			}

			if (Object.TryGetStringArrayField("ProducedItems", out string[]? ProducedItems))
			{
				Action.ProducedItems.UnionWith(ProducedItems.Select(x => FileItem.GetItemByPath(x)));
			}

			if (Object.TryGetStringArrayField("DeleteItems", out string[]? DeleteItems))
			{
				Action.DeleteItems.UnionWith(DeleteItems.Select(x => FileItem.GetItemByPath(x)));
			}

			if (Object.TryGetObjectField("RootPaths", out JsonObject? RootPaths))
			{
				Action.RootPaths.Read(RootPaths);
			}

			if (Object.TryGetStringField("DependencyListFile", out string? DependencyListFile))
			{
				Action.DependencyListFile = FileItem.GetItemByPath(DependencyListFile);
			}

			if (Object.TryGetDoubleField("Weight", out double Weight))
			{
				Action.Weight = Weight;
			}

			if (Object.TryGetUnsignedIntegerField("CacheBucket", out uint CacheBucket))
			{
				Action.CacheBucket = CacheBucket;
			}

			return Action;
		}

		public override string ToString()
		{
			string ReturnString = "";
			if (CommandPath != null)
			{
				ReturnString += CommandPath + " - ";
			}
			if (CommandArguments != null)
			{
				ReturnString += CommandArguments;
			}
			return ReturnString;
		}
	}

	/// <summary>
	/// Interface that is used when -SingleFile=xx is used on the cmd line to make specific files compile
	/// Actions that has this interface can be used to create real actions that can compile the specific files
	/// </summary>
	interface ISpecificFileAction
	{
		/// <summary>
		/// The directory this action can create actions for
		/// </summary>
		DirectoryReference RootDirectory { get; }

		/// <summary>
		/// Creates an action for a specific file. It can return null if for example file extension is not handled
		/// </summary>
		IExternalAction? CreateAction(FileItem SourceFile, ILogger Logger);
	}

	/// <summary>
	/// Extension methods for action classes
	/// </summary>
	static class ActionExtensions
	{
		/// <summary>
		/// Writes an action to a json file
		/// </summary>
		/// <param name="Action">The action to write</param>
		/// <param name="LinkedActionToId">Map of action to unique id</param>
		/// <param name="Writer">Writer to receive the output</param>
		public static void ExportJson(this LinkedAction Action, Dictionary<LinkedAction, int> LinkedActionToId, JsonWriter Writer)
		{
			Writer.WriteValue("Id", LinkedActionToId[Action]);
			Writer.WriteEnumValue("Type", Action.ActionType);
			Writer.WriteEnumValue("ArtifactMode", Action.ArtifactMode);
			Writer.WriteValue("WorkingDirectory", Action.WorkingDirectory.FullName);
			Writer.WriteValue("CommandPath", Action.CommandPath.FullName);
			Writer.WriteValue("CommandArguments", Action.CommandArguments);
			Writer.WriteStringArrayField("ResponseFileContents", Action.ResponseFileContents);
			Writer.WriteValue("CommandVersion", Action.CommandVersion);
			Writer.WriteValue("CommandDescription", Action.CommandDescription);
			Writer.WriteValue("StatusDescription", Action.StatusDescription);
			Writer.WriteValue("bCanExecuteRemotely", Action.bCanExecuteRemotely);
			Writer.WriteValue("bCanExecuteRemotelyWithSNDBS", Action.bCanExecuteRemotelyWithSNDBS);
			Writer.WriteValue("bCanExecuteRemotelyWithXGE", Action.bCanExecuteRemotelyWithXGE);
			Writer.WriteValue("bCanExecuteInUBA", Action.bCanExecuteInUBA);
			Writer.WriteValue("bCanExecuteInUBACrossArchitecture", Action.bCanExecuteInUBACrossArchitecture);
			Writer.WriteValue("bIsClangCompiler", Action.bIsClangCompiler);
			Writer.WriteValue("bDeleteProducedItemsOnError", Action.bDeleteProducedItemsOnError);
			Writer.WriteValue("bForceWarningsAsError", Action.bForceWarningsAsError);			
			Writer.WriteValue("bShouldOutputStatusDescription", Action.bShouldOutputStatusDescription);
			Writer.WriteValue("bShouldOutputLog", Action.bShouldOutputLog);
			Writer.WriteValue("bProducesImportLibrary", Action.bProducesImportLibrary);
			Writer.WriteValue("Weight", Action.Weight);
			Writer.WriteValue("CacheBucket", Action.CacheBucket);

			Writer.WriteArrayStart("PrerequisiteActions");
			foreach (LinkedAction PrerequisiteAction in Action.PrerequisiteActions)
			{
				Writer.WriteValue(LinkedActionToId[PrerequisiteAction]);
			}
			Writer.WriteArrayEnd();

			Writer.WriteArrayStart("ProducedItems");
			foreach (FileItem ProducedItem in Action.ProducedItems)
			{
				Writer.WriteValue(ProducedItem.AbsolutePath);
			}
			Writer.WriteArrayEnd();

			Writer.WriteArrayStart("DeleteItems");
			foreach (FileItem DeleteItem in Action.DeleteItems)
			{
				Writer.WriteValue(DeleteItem.AbsolutePath);
			}
			Writer.WriteArrayEnd();

			Writer.WriteObjectStart("RootPaths");
			Action.RootPaths.Write(Writer);
			Writer.WriteObjectEnd();

			if (Action.DependencyListFile != null)
			{
				Writer.WriteValue("DependencyListFile", Action.DependencyListFile.AbsolutePath);
			}
		}

		/// <summary>
		/// Determine if this Action ignores output conflicts.
		/// </summary>
		/// <param name="Action"> The Action to check</param>
		/// <returns>True if conflicts are ignored, else false.</returns>
		public static bool IgnoreConflicts(this IExternalAction Action) =>
			Action.ActionType == ActionType.WriteMetadata ||
			Action.ActionType == ActionType.CreateAppBundle;
	}

	/// <summary>
	/// Default serializer for <see cref="Action"/> instances
	/// </summary>
	class DefaultActionSerializer : ActionSerializerBase<Action>
	{
		/// <inheritdoc/>
		public override Action Read(BinaryArchiveReader Reader) => new Action(Reader);

		/// <inheritdoc/>
		public override void Write(BinaryArchiveWriter Writer, Action Action) => Action.Write(Writer);
	}

	/// <summary>
	/// Information about an action queued to be executed
	/// </summary>
	[DebuggerDisplay("{StatusDescription}")]
	class LinkedAction : IExternalAction
	{
		/// <summary>
		/// The inner action instance
		/// </summary>
		public IExternalAction Inner;

		/// <summary>
		/// A target that this action contributes to
		/// </summary>
		public TargetDescriptor? Target;

		/// <summary>
		/// Set of other actions that this action depends on. This set is built when the action graph is linked.
		/// </summary>
		public HashSet<LinkedAction> PrerequisiteActions = null!;

		/// <summary>
		/// Total number of actions depending on this one.
		/// </summary>
		public int NumTotalDependentActions = 0;

		/// <summary>
		/// Additional field used for sorting that can be used to control sorting of link actions from outside
		/// </summary>
		public int SortIndex = 0;

		/// <summary>
		/// True if this action should be scheduled early. Some actions can take a long time (pch) and need to start very early
		/// If set to 1 it will propagate high priority to all its prerequisite tasks. (Note it is an int to be able to do interlocked operations)
		/// </summary>
		public int IsHighPriority;

		/// <summary>
		/// If set, will be output whenever the group differs to the last executed action. Set when executing multiple targets at once.
		/// </summary>
		public SortedSet<string> GroupNames = [];

		#region Wrapper implementation of IAction

		public ActionType ActionType => Inner.ActionType;
		public ArtifactMode ArtifactMode => Inner.ArtifactMode;
		public IEnumerable<FileItem> PrerequisiteItems => Inner.PrerequisiteItems;
		public IEnumerable<FileItem> ProducedItems => Inner.ProducedItems;
		public IEnumerable<FileItem> DeleteItems => Inner.DeleteItems;
		public CppRootPaths RootPaths => Inner.RootPaths;
		public FileItem? DependencyListFile => Inner.DependencyListFile;
		public DirectoryReference WorkingDirectory => Inner.WorkingDirectory;
		public FileReference CommandPath => Inner.CommandPath;
		public string CommandArguments => Inner.CommandArguments;
		public ImmutableArray<string> ResponseFileContents => Inner.ResponseFileContents;
		public string CommandVersion => Inner.CommandVersion;
		public string CommandDescription => Inner.CommandDescription;
		public string StatusDescription => Inner.StatusDescription;
		public bool bCanExecuteRemotely => Inner.bCanExecuteRemotely;
		public bool bCanExecuteRemotelyWithSNDBS => Inner.bCanExecuteRemotelyWithSNDBS;
		public bool bCanExecuteRemotelyWithXGE => Inner.bCanExecuteRemotelyWithXGE;
		public bool bCanExecuteInUBA => Inner.bCanExecuteInUBA;
		public bool bCanExecuteInUBACrossArchitecture => Inner.bCanExecuteInUBACrossArchitecture;
		public bool bIsClangCompiler => Inner.bIsClangCompiler;
		public bool bDeleteProducedItemsOnError => Inner.bDeleteProducedItemsOnError;
		public bool bForceWarningsAsError => Inner.bForceWarningsAsError;
		public bool bShouldOutputStatusDescription => Inner.bShouldOutputStatusDescription;
		public bool bShouldOutputLog => Inner.bShouldOutputLog;
		public bool bProducesImportLibrary => Inner.bProducesImportLibrary;
		public bool bUseActionHistory => Inner.bUseActionHistory;
		public bool bIsHighPriority => IsHighPriority != 0;
		public double Weight => Inner.Weight;
		public uint CacheBucket => Inner.CacheBucket;

		#endregion

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="Inner">The inner action instance</param>
		/// <param name="Target"></param>
		public LinkedAction(IExternalAction Inner, TargetDescriptor? Target)
		{
			this.Inner = Inner;
			this.Target = Target;
			IsHighPriority = Inner.bIsHighPriority ? 1 : 0;
		}

		/// <summary>
		/// Increment the number of dependents, recursively
		/// </summary>
		/// <param name="VisitedActions">Set of visited actions</param>
		/// <param name="bIsHighPriority">Propagate high priority to this and prerequisite actions</param>
		public void IncrementDependentCount(HashSet<LinkedAction> VisitedActions, bool bIsHighPriority)
		{
			if (VisitedActions.Add(this))
			{
				if (bIsHighPriority)
				{
					Interlocked.Exchange(ref IsHighPriority, 1);
				}

				Interlocked.Increment(ref NumTotalDependentActions);
				foreach (LinkedAction PrerequisiteAction in PrerequisiteActions)
				{
					PrerequisiteAction.IncrementDependentCount(VisitedActions, bIsHighPriority);
				}
			}
		}

		/// <summary>
		/// Compares two actions based on total number of sort index, priority and dependent items, descending.
		/// </summary>
		/// <param name="A">Action to compare</param>
		/// <param name="B">Action to compare</param>
		public static int Compare(LinkedAction A, LinkedAction B)
		{
			if (A.SortIndex != B.SortIndex)
			{
				return Math.Sign(A.SortIndex - B.SortIndex);
			}

			if (A.IsHighPriority != B.IsHighPriority)
			{
				return B.IsHighPriority - A.IsHighPriority;
			}

			// Primary sort criteria is total number of dependent files, up to max depth.
			if (B.NumTotalDependentActions != A.NumTotalDependentActions)
			{
				return Math.Sign(B.NumTotalDependentActions - A.NumTotalDependentActions);
			}

			// Secondary sort criteria is directory name of first produced item.
			// This will sort actions from same module closer to each other
			// Also sort in alphabetic order since when building with unity files it is likely that the last unity file is the fastest to compile
			// and we want longer actions to run earlier
			int Result = String.Compare(A.ProducedItems.FirstOrDefault()?.FullName, B.ProducedItems.FirstOrDefault()?.FullName, StringComparison.Ordinal);
			if (Result != 0)
			{
				return Math.Sign(Result);
			}

			// Third sort criteria is number of pre-requisites.
			return Math.Sign(B.PrerequisiteItems.Count() - A.PrerequisiteItems.Count());
		}
	}
}
