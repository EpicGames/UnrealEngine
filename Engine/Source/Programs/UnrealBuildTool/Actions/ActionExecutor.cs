// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.LogMatchers;
using Microsoft.Extensions.Logging;

namespace UnrealBuildTool
{
	abstract class ActionExecutor : IDisposable
	{
		public abstract string Name
		{
			get;
		}

		protected static double MemoryPerActionBytesOverride
		{
			get;
			private set;
		} = 0.0;

		readonly LogEventParser Parser;

		public ActionExecutor(ILogger Logger)
		{
			Parser = new LogEventParser(Logger);
			Parser.AddMatchers(LogMatcherCollections.UBTMatchers);
		}

		public void Dispose()
		{
			Dispose(true);
			GC.SuppressFinalize(this);
		}

		protected virtual void Dispose(bool disposing)
		{
			if (disposing)
			{
				Parser.Dispose();
			}
		}

		/// <summary>
		/// Allow targets to override the expected amount of memory required for compiles, used to control the number
		/// of parallel action processes.
		/// </summary>
		/// <param name="MemoryPerActionOverrideGB"></param>
		public static void SetMemoryPerActionOverride(double MemoryPerActionOverrideGB)
		{
			MemoryPerActionBytesOverride = Math.Max(MemoryPerActionBytesOverride, MemoryPerActionOverrideGB * 1024 * 1024 * 1024);
		}

		/// <summary>
		/// Execute the given actions
		/// </summary>
		/// <param name="ActionsToExecute">Actions to be executed</param>
		/// <param name="Logger">Logger for output</param>
		/// <returns>True if the build succeeded, false otherwise</returns>
		public abstract Task<bool> ExecuteActionsAsync(IEnumerable<LinkedAction> ActionsToExecute, ILogger Logger);

		/// <summary>
		/// When completed, get a populated telemetry event for this executor
		/// </summary>
		/// <returns></returns>
		public abstract TelemetryExecutorEvent? GetTelemetryEvent();

		/// <summary>
		/// Post a telemetry event if enabled
		/// </summary>
		public void PostTelemetryEvent()
		{
			TelemetryExecutorEvent? telemetryEvent = GetTelemetryEvent();
			if (telemetryEvent != null)
			{
				TelemetryService.Get().RecordEvent(telemetryEvent);
			}
		}

		/// <summary>
		/// Will verify that produced items exists on disk
		/// </summary>
		public virtual bool VerifyOutputs => true;

		protected void WriteToolOutput(string Line)
		{
			lock (Parser)
			{
				Parser.WriteLine(Line);
			}
		}

		protected void FlushToolOutput()
		{
			lock (Parser)
			{
				Parser.Flush();
			}
		}
	}
}
