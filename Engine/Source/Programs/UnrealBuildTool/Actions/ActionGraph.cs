// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Collections.Immutable;
using System.Diagnostics;
using System.Globalization;
using System.IO;
using System.Linq;
using System.Runtime;
using System.Text;
using System.Text.Json;
using System.Text.Json.Serialization;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using OpenTracing.Util;
using UnrealBuildBase;
using UnrealBuildTool.Executors;

namespace UnrealBuildTool
{
	internal static class ActionGraph
	{
		/// <summary>
		/// Enum describing why an Action is in conflict with another Action
		/// </summary>
		[Flags]
		private enum ActionConflictReasonFlags : byte
		{
			None = 0,
			ActionType = 1 << 0,
			PrerequisiteItems = 1 << 1,
			DeleteItems = 1 << 2,
			DependencyListFile = 1 << 3,
			WorkingDirectory = 1 << 4,
			CommandPath = 1 << 5,
			CommandArguments = 1 << 6,
		};

		/// <summary>
		/// Links the actions together and sets up their dependencies
		/// </summary>
		/// <param name="Actions">List of actions in the graph</param>
		/// <param name="Logger">Logger for output</param>
		/// <param name="Sort">Optional sorting of actions</param>
		public static void Link(List<LinkedAction> Actions, ILogger Logger, bool Sort = true)
		{
			using ITimelineEvent _ = Timeline.ScopeEvent($"LinkActions");

			// Build a map from item to its producing action
			Dictionary<FileItem, LinkedAction> ItemToProducingAction = new (Actions.Count * 3);
			foreach (LinkedAction Action in Actions)
			{
				foreach (FileItem ProducedItem in Action.ProducedItems)
				{
					ItemToProducingAction[ProducedItem] = Action;
				}
			}

			// Check for cycles
			DetectActionGraphCycles(Actions, ItemToProducingAction, Logger);

			// Use this map to add all the prerequisite actions
			foreach (LinkedAction Action in Actions)
			{
				Action.PrerequisiteActions = new HashSet<LinkedAction>();
				foreach (FileItem PrerequisiteItem in Action.PrerequisiteItems)
				{
					if (ItemToProducingAction.TryGetValue(PrerequisiteItem, out LinkedAction? PrerequisiteAction))
					{
						Action.PrerequisiteActions.Add(PrerequisiteAction);
					}
				}
			}

			// Sort the action graph
			if (Sort)
			{
				SortActionList(Actions);
			}
		}

		/// <summary>
		/// Checks a set of actions for conflicts (ie. different actions producing the same output items)
		/// </summary>
		/// <param name="Actions">The set of actions to check</param>
		/// <param name="Logger">Logger for output</param>
		public static void CheckForConflicts(IEnumerable<IExternalAction> Actions, ILogger Logger)
		{
			bool bResult = true;

			Dictionary<FileItem, IExternalAction> ItemToProducingAction = new Dictionary<FileItem, IExternalAction>();
			foreach (IExternalAction Action in Actions)
			{
				HashSet<IExternalAction> Conflicted = new HashSet<IExternalAction>();
				foreach (FileItem ProducedItem in Action.ProducedItems)
				{
					if (ItemToProducingAction.TryGetValue(ProducedItem, out IExternalAction? ExistingAction))
					{
						if (!Conflicted.Contains(ExistingAction))
						{
							bResult &= CheckForConflicts(ExistingAction, Action, Logger);
							Conflicted.Add(ExistingAction);
						}
					}
					else
					{
						ItemToProducingAction.Add(ProducedItem, Action);
					}
				}
			}

			if (!bResult)
			{
				throw new CompilationResultException(CompilationResult.ActionGraphInvalid, "Action graph is invalid; unable to continue. See log for additional details.");
			}
		}

		/// <summary>
		/// Finds conflicts between two actions, and prints them to the log
		/// </summary>
		/// <param name="A">The first action</param>
		/// <param name="B">The second action</param>
		/// <param name="Logger">Logger for output</param>
		/// <returns>True if no conflicts were found, false otherwise.</returns>
		private static bool CheckForConflicts(IExternalAction A, IExternalAction B, ILogger Logger)
		{
			ActionConflictReasonFlags Reason = ActionConflictReasonFlags.None;
			if (A.ActionType != B.ActionType)
			{
				Reason |= ActionConflictReasonFlags.ActionType;
			}

			if (!A.PrerequisiteItems.SequenceEqual(B.PrerequisiteItems))
			{
				Reason |= ActionConflictReasonFlags.PrerequisiteItems;
			}

			if (!A.DeleteItems.SequenceEqual(B.DeleteItems))
			{
				Reason |= ActionConflictReasonFlags.DeleteItems;
			}

			if (A.DependencyListFile != B.DependencyListFile)
			{
				Reason |= ActionConflictReasonFlags.DependencyListFile;
			}

			if (A.WorkingDirectory != B.WorkingDirectory)
			{
				Reason |= ActionConflictReasonFlags.WorkingDirectory;
			}

			if (A.CommandPath != B.CommandPath)
			{
				Reason |= ActionConflictReasonFlags.CommandPath;
			}

			if (A.CommandArguments != B.CommandArguments)
			{
				Reason |= ActionConflictReasonFlags.CommandArguments;
			}

			if (Reason != ActionConflictReasonFlags.None)
			{
				LogConflict(A, B, Reason, Logger);
				return false;
			}

			return true;
		}

		private class LogActionActionTypeConverter : JsonConverter<ActionType>
		{
			public override ActionType Read(ref Utf8JsonReader Reader, Type TypeToConvert, JsonSerializerOptions Options)
			{
				throw new NotImplementedException();
			}

			public override void Write(Utf8JsonWriter Writer, ActionType Value, JsonSerializerOptions Options)
			{
				Writer.WriteStringValue(Value.ToString());
			}
		}

		private class LogActionFileItemConverter : JsonConverter<FileItem>
		{
			public override FileItem Read(ref Utf8JsonReader Reader, Type TypeToConvert, JsonSerializerOptions Options)
			{
				throw new NotImplementedException();
			}

			public override void Write(Utf8JsonWriter Writer, FileItem Value, JsonSerializerOptions Options)
			{
				Writer.WriteStringValue(Value.FullName);
			}
		}

		private class LogActionDirectoryReferenceConverter : JsonConverter<DirectoryReference>
		{
			public override DirectoryReference Read(ref Utf8JsonReader Reader, Type TypeToConvert, JsonSerializerOptions Options)
			{
				throw new NotImplementedException();
			}

			public override void Write(Utf8JsonWriter Writer, DirectoryReference Value, JsonSerializerOptions Options)
			{
				Writer.WriteStringValue(Value.FullName);
			}
		}

		private class LogActionFileReferenceConverter : JsonConverter<FileReference>
		{
			public override FileReference Read(ref Utf8JsonReader Reader, Type TypeToConvert, JsonSerializerOptions Options)
			{
				throw new NotImplementedException();
			}

			public override void Write(Utf8JsonWriter Writer, FileReference Value, JsonSerializerOptions Options)
			{
				Writer.WriteStringValue(Value.FullName);
			}
		}

		/// <summary>
		/// Adds the description of a merge error to an output message
		/// </summary>
		/// <param name="A">The first action with the conflict</param>
		/// <param name="B">The second action with the conflict</param>
		/// <param name="Reason">Enum flags for which properties are in conflict</param>
		/// <param name="Logger">Logger for output</param>
		static void LogConflict(IExternalAction A, IExternalAction B, ActionConflictReasonFlags Reason, ILogger Logger)
		{
			// Convert some complex types in IExternalAction to strings when printing json
			JsonSerializerOptions Options = new JsonSerializerOptions
			{
				WriteIndented = true,
				DefaultIgnoreCondition = JsonIgnoreCondition.WhenWritingNull,
				Converters =
				{
					new LogActionActionTypeConverter(),
					new LogActionFileItemConverter(),
					new LogActionDirectoryReferenceConverter(),
					new LogActionFileReferenceConverter(),
				},
			};

			string AJson = JsonSerializer.Serialize(A, Options);
			string BJson = JsonSerializer.Serialize(B, Options);
			string AJsonPath = Path.Combine(Path.GetTempPath(), "UnrealBuildTool", $"{IoHash.Compute(Encoding.Default.GetBytes(AJson)).GetHashCode():X}") + ".json";
			string BJsonPath = Path.Combine(Path.GetTempPath(), "UnrealBuildTool", $"{IoHash.Compute(Encoding.Default.GetBytes(BJson)).GetHashCode():X}") + ".json";

			Directory.CreateDirectory(Path.Combine(Path.GetTempPath(), "UnrealBuildTool"));
			File.WriteAllText(AJsonPath, AJson);
			File.WriteAllText(BJsonPath, BJson);

			Logger.LogError("Unable to merge actions '{StatusA}' and '{StatusB}': {Reason} are different", A.StatusDescription, B.StatusDescription, Reason);
			Logger.LogInformation("  First Action: {AJson}", AJson);
			Logger.LogInformation("  Second Action: {BJson}", BJson);
			Logger.LogInformation("  First Action json written to '{AJsonPath}'", AJsonPath);
			Logger.LogInformation("  Second Action json written to '{BJsonPath}'", BJsonPath);
		}

		/// <summary>
		/// Builds a list of actions that need to be executed to produce the specified output items.
		/// </summary>
		public static List<LinkedAction> GetActionsToExecute(List<LinkedAction> Actions,
			CppDependencyCache CppDependencies, ActionHistory History, bool bIgnoreOutdatedImportLibraries, ILogger Logger)
		{
			using (GlobalTracer.Instance.BuildSpan("ActionGraph.GetActionsToExecute()").StartActive())
			{
				// For all targets, build a set of all actions that are outdated.
				ConcurrentDictionary<LinkedAction, bool> OutdatedActionDictionary = new ConcurrentDictionary<LinkedAction, bool>();
				GatherAllOutdatedActions(Actions, History, OutdatedActionDictionary, CppDependencies, bIgnoreOutdatedImportLibraries, Logger);

				// Build a list of actions that are both needed for this target and outdated.
				return Actions.Where(Action => OutdatedActionDictionary[Action]).ToList();
			}
		}

		/// <summary>
		/// Checks that there aren't any intermediate files longer than the max allowed path length
		/// </summary>
		/// <param name="BuildConfiguration">The build configuration</param>
		/// <param name="Actions">List of actions in the graph</param>
		/// <param name="Logger"></param>
		public static void CheckPathLengths(BuildConfiguration BuildConfiguration, IEnumerable<IExternalAction> Actions, ILogger Logger)
		{
			if (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Win64)
			{
				const int MAX_PATH = 260;

				bool ShouldFail(FileItem item)
				{
					return item.Location.FullName.Length >= MAX_PATH &&
						!(item.Location.ContainsName("Intermediate", 0) && item.Location.ContainsName("H", 1)); // Ignore -IncludeHeader items
				}

				bool ShouldWarn(FileItem item)
				{
					if (item.Location.FullName.Length <= Unreal.RootDirectory.FullName.Length + BuildConfiguration.MaxNestedPathLength)
					{
						return false;
					}

					if (!item.Location.IsUnderDirectory(Unreal.RootDirectory))
					{
						return false;
					}

					if (!item.Location.FullName.Contains("/Restricted/NotForLicensees/", StringComparison.OrdinalIgnoreCase)) //Be more relaxed for internal only code
					{
						return false;
					}

					if (item.Location.FullName.Contains("/Intermediate/", StringComparison.OrdinalIgnoreCase))
					{
						if (item.Location.ContainsName("H", 1)) // Ignore -IncludeHeader items
						{
							return false;
						}

						if (item.Location.FullName.EndsWith(".i.PVS-Studio.log", StringComparison.Ordinal)) // Ignore PVS Studio intermediate items
						{
							return false;
						}
					}

					return true;
				}

				HashSet<FileReference> FailPaths = new();
				HashSet<FileReference> WarnPaths = new();
				foreach (IExternalAction Action in Actions)
				{
					foreach (FileItem PrerequisiteItem in Action.PrerequisiteItems)
					{
						if (ShouldFail(PrerequisiteItem))
						{
							FailPaths.Add(PrerequisiteItem.Location);
						}

						if (ShouldWarn(PrerequisiteItem))
						{
							WarnPaths.Add(PrerequisiteItem.Location);
						}
					}

					foreach (FileItem ProducedItem in Action.ProducedItems)
					{
						if (ShouldFail(ProducedItem))
						{
							FailPaths.Add(ProducedItem.Location);
						}

						if (ShouldWarn(ProducedItem))
						{
							WarnPaths.Add(ProducedItem.Location);
						}
					}
				}

				if (FailPaths.Count > 0)
				{
					StringBuilder Message = new StringBuilder();
					Message.Append($"The following action paths are longer than {MAX_PATH} characters. Please move the engine to a directory with a shorter path.");
					foreach (FileReference Path in FailPaths.OrderBy(x => x.FullName))
					{
						Message.Append($"\n[{Path.FullName.Length.ToString()} characters] {Path}");
					}

					throw new BuildException(Message.ToString());
				}
				if (WarnPaths.Count > 0)
				{
					StringBuilder Message = new StringBuilder();
					Message.Append($"Detected paths more than {BuildConfiguration.MaxNestedPathLength.ToString()} characters below UE root directory. This may cause portability issues due to the {MAX_PATH.ToString()} character maximum path length on Windows:\n");
					foreach (FileReference Path in WarnPaths.OrderBy(x => x.FullName))
					{
						string RelativePath = Path.MakeRelativeTo(Unreal.RootDirectory);
						Message.Append($"\n[{RelativePath.Length.ToString()} characters] {RelativePath}");
					}

					Message.Append($"\n\nConsider setting {nameof(ModuleRules.ShortName)} = ... in module *.Build.cs files to use alternative names for intermediate paths.");
					Logger.LogWarning("{Message}", Message.ToString());
				}
			}
		}

		/// <summary>
		/// Executes a list of actions.
		/// </summary>
		public static async Task ExecuteActionsAsync(BuildConfiguration BuildConfiguration, List<LinkedAction> ActionsToExecute, List<TargetDescriptor> TargetDescriptors, ILogger Logger, ActionExecutor? executor = null)
		{
			if (ActionsToExecute.Count == 0)
			{
				Logger.LogInformation("Target is up to date");

				if (executor != null)
				{
					// We still want to run the executor to give it a chance to clean up in case it has tasks running that can be cancelled
					await InternalExecuteActions(executor, ActionsToExecute, Logger);
				}
			}
			else
			{
				// Figure out which executor to use
				if (executor == null)
				{
					using ActionExecutor SelectedExecutor = ExecutorFactory.Instance.GetExecutor(BuildConfiguration, ActionsToExecute.Count, TargetDescriptors, Logger);
					await InternalExecuteActions(SelectedExecutor, ActionsToExecute, Logger);
				}
				else
				{
					await InternalExecuteActions(executor, ActionsToExecute, Logger);
				}
			}
		}

		static async Task InternalExecuteActions(ActionExecutor Executor, List<LinkedAction> ActionsToExecute, ILogger Logger)
		{
			Logger.LogInformation("Using {ExecutorName} executor to run {ActionCount} action(s)", Executor.Name, ActionsToExecute.Count);

			// Force an aggressive garbage collection in the background before running actions to free up as much memory as possible as compiling and linking can use a significant amount of memory
			Task gcTask = Task.Run(() => {
				using ITimelineEvent _ = Timeline.ScopeEvent("GarbageCollect");
				GCSettings.LargeObjectHeapCompactionMode = GCLargeObjectHeapCompactionMode.CompactOnce;
				GC.Collect(GC.MaxGeneration, GCCollectionMode.Aggressive, true, true);
			});

			// Execute the build
			Stopwatch Timer = Stopwatch.StartNew();
			bool Result = await Executor.ExecuteActionsAsync(ActionsToExecute, Logger);
			Executor.PostTelemetryEvent();

			Logger.LogInformation("Total time in {ExecutorName} executor: {TotalSeconds:0.00} seconds", Executor.Name, Timer.Elapsed.TotalSeconds);

			if (!Result)
			{
				throw new CompilationResultException(CompilationResult.OtherCompilationError);
			}

			// Reset the file info for all the produced items
			foreach (LinkedAction BuildAction in ActionsToExecute)
			{
				foreach (FileItem ProducedItem in BuildAction.ProducedItems)
				{
					ProducedItem.ResetCachedInfo();
				}
			}

			// Verify the link outputs were created (seems to happen with Win64 compiles)
			if (Executor.VerifyOutputs)
			{
				foreach (LinkedAction BuildAction in ActionsToExecute)
				{
					if (BuildAction.ActionType == ActionType.Link)
					{
						foreach (FileItem Item in BuildAction.ProducedItems)
						{
							if (!Item.Exists)
							{
								throw new BuildException($"Failed to produce item: {Item.AbsolutePath}");
							}
						}
					}
				}
			}
		}
		/// <summary>
		/// Sorts the action list for improved parallelism with local execution.
		/// </summary>
		static void SortActionList(List<LinkedAction> Actions)
		{
			// Clear the current dependent count and SortIndex if there is any
			foreach (LinkedAction Action in Actions)
			{
				Action.NumTotalDependentActions = 0;
				Action.SortIndex = 0;
			}

			// Increment all the dependencies plus propagate high priority flag
			Parallel.ForEach(Actions, (Action) =>
			{
				Action.IncrementDependentCount(new HashSet<LinkedAction>(), Action.bIsHighPriority);
			});

			// Sort actions by number of actions depending on them, descending. Secondary sort criteria is file size.
			Actions.Sort(LinkedAction.Compare);

			// Now when everything is sorted we want to move link actions and actions that can't run remotely to as early as possible
			// Find the last prereq action and set the sort index to the same. since action has at least one more dependency it will end up after in the next sorting call
			int I = 0;
			foreach (LinkedAction Action in Actions)
			{
				Action.SortIndex = ++I;
				if ((!Action.bCanExecuteRemotely || Action.ActionType == ActionType.Link) && Action.PrerequisiteActions.Count > 0)
				{
					int LastSortIndex = 0;
					foreach (LinkedAction Prereq in Action.PrerequisiteActions)
					{
						LastSortIndex = Math.Max(LastSortIndex, Prereq.SortIndex);
					}
					Action.SortIndex = LastSortIndex;
				}
			}

			// Sort again now when we have set sorting index and will put link actions earlier
			Actions.Sort(LinkedAction.Compare);
		}

		/// <summary>
		/// Checks for cycles in the action graph.
		/// </summary>
		static void DetectActionGraphCycles(List<LinkedAction> Actions, Dictionary<FileItem, LinkedAction> ItemToProducingAction, ILogger Logger)
		{
			using ITimelineEvent _ = Timeline.ScopeEvent($"DetectActionGraphCycles");

			// Starting with actions that only depend on non-produced items, iteratively expand a set of actions that are only dependent on
			// non-cyclical actions.
			Dictionary<LinkedAction, bool> ActionIsNonCyclical = new Dictionary<LinkedAction, bool>();
			Dictionary<LinkedAction, List<LinkedAction>> CyclicActions = new Dictionary<LinkedAction, List<LinkedAction>>();
			while (true)
			{
				bool bFoundNewNonCyclicalAction = false;

				foreach (LinkedAction Action in Actions)
				{
					if (!ActionIsNonCyclical.ContainsKey(Action))
					{
						// Determine if the action depends on only actions that are already known to be non-cyclical.
						bool bActionOnlyDependsOnNonCyclicalActions = true;
						foreach (FileItem PrerequisiteItem in Action.PrerequisiteItems)
						{
							if (ItemToProducingAction.TryGetValue(PrerequisiteItem, out LinkedAction? ProducingAction))
							{
								if (!ActionIsNonCyclical.ContainsKey(ProducingAction))
								{
									bActionOnlyDependsOnNonCyclicalActions = false;
									if (!CyclicActions.ContainsKey(Action))
									{
										CyclicActions.Add(Action, new List<LinkedAction>());
									}

									List<LinkedAction> CyclicPrerequisite = CyclicActions[Action];
									if (!CyclicPrerequisite.Contains(ProducingAction))
									{
										CyclicPrerequisite.Add(ProducingAction);
									}
								}
							}
						}

						// If the action only depends on known non-cyclical actions, then add it to the set of known non-cyclical actions.
						if (bActionOnlyDependsOnNonCyclicalActions)
						{
							ActionIsNonCyclical.Add(Action, true);
							bFoundNewNonCyclicalAction = true;
							CyclicActions.Remove(Action);
						}
					}
				}

				// If this iteration has visited all actions without finding a new non-cyclical action, then all non-cyclical actions have
				// been found.
				if (!bFoundNewNonCyclicalAction)
				{
					break;
				}
			}

			// If there are any cyclical actions, throw an exception.
			if (ActionIsNonCyclical.Count < Actions.Count)
			{
				// Find the index of each action
				Dictionary<LinkedAction, int> ActionToIndex = new Dictionary<LinkedAction, int>();
				for (int Idx = 0; Idx < Actions.Count; Idx++)
				{
					ActionToIndex[Actions[Idx]] = Idx;
				}

				// Describe the cyclical actions.
				foreach (LinkedAction Action in Actions)
				{
					if (!ActionIsNonCyclical.ContainsKey(Action))
					{
						string CycleDescription = "";
						CycleDescription += $"Action #{ActionToIndex[Action]}: {Action.CommandPath}\n";
						CycleDescription += $"\twith arguments: {Action.CommandArguments}\n";
						foreach (FileItem PrerequisiteItem in Action.PrerequisiteItems)
						{
							CycleDescription += $"\tdepends on: {PrerequisiteItem.AbsolutePath}\n";
						}

						foreach (FileItem ProducedItem in Action.ProducedItems)
						{
							CycleDescription += $"\tproduces:   {ProducedItem.AbsolutePath}\n";
						}

						CycleDescription += "\tDepends on cyclic actions:\n";
						if (CyclicActions.ContainsKey(Action))
						{
							foreach (LinkedAction CyclicPrerequisiteAction in CyclicActions[Action])
							{
								if (CyclicActions.ContainsKey(CyclicPrerequisiteAction))
								{
									List<FileItem> CyclicProducedItems =
										CyclicPrerequisiteAction.ProducedItems.ToList();
									if (CyclicProducedItems.Count == 1)
									{
										CycleDescription += $"\t\t{ActionToIndex[CyclicPrerequisiteAction]} (produces: {CyclicProducedItems[0].AbsolutePath})\n";
									}
									else
									{
										CycleDescription += $"\t\t{ActionToIndex[CyclicPrerequisiteAction]}\n";
										foreach (FileItem CyclicProducedItem in CyclicProducedItems)
										{
											CycleDescription += $"\t\t\tproduces:   {CyclicProducedItem.AbsolutePath}\n";
										}
									}
								}
							}

							CycleDescription += "\n";
						}
						else
						{
							CycleDescription += "\t\tNone?? Coding error!\n";
						}

						Logger.LogInformation("{CycleDescription}", CycleDescription);
					}
				}

				throw new CompilationResultException(CompilationResult.ActionGraphInvalid, "Action graph contains cycle!");
			}
		}

		/// <summary>
		/// Determines the full set of actions that must be built to produce an item.
		/// </summary>
		/// <param name="Actions">All the actions in the graph</param>
		/// <param name="OutputItems">Set of output items to be built</param>
		/// <returns>Set of prerequisite actions</returns>
		public static List<LinkedAction> GatherPrerequisiteActions(List<LinkedAction> Actions, HashSet<FileItem> OutputItems)
		{
			using ITimelineEvent _ = Timeline.ScopeEvent($"GatherPrerequisiteActions");

			HashSet<LinkedAction> PrerequisiteActions = new HashSet<LinkedAction>();
			foreach (LinkedAction Action in Actions)
			{
				if (Action.ProducedItems.Any(OutputItems.Contains))
				{
					GatherPrerequisiteActions(Action, PrerequisiteActions);
				}
			}

			return PrerequisiteActions.ToList();
		}

		/// <summary>
		/// Determines the full set of actions that must be built to produce an item.
		/// </summary>
		/// <param name="Action">The root action to scan</param>
		/// <param name="PrerequisiteActions">Set of prerequisite actions</param>
		private static void GatherPrerequisiteActions(LinkedAction Action, HashSet<LinkedAction> PrerequisiteActions)
		{
			if (PrerequisiteActions.Add(Action))
			{
				foreach (LinkedAction PrerequisiteAction in Action.PrerequisiteActions)
				{
					GatherPrerequisiteActions(PrerequisiteAction, PrerequisiteActions);
				}
			}
		}

		/// <summary>
		/// Determines whether an action is outdated based on the modification times for its prerequisite
		/// and produced items, without considering the full set of prerequisites.
		/// Writes to OutdatedActionDictionary iff the action is found to be outdated.
		/// Safe to run in parallel, but only with different RootActions.
		/// </summary>
		/// <param name="RootAction">- The action being considered.</param>
		/// <param name="OutdatedActionDictionary">-</param>
		/// <param name="ActionHistory"></param>
		/// <param name="CppDependencies"></param>
		/// <param name="bIgnoreOutdatedImportLibraries"></param>
		/// <param name="Logger"></param>
		/// <returns>true if outdated</returns>
		private static void IsIndividualActionOutdated(LinkedAction RootAction,
			ConcurrentDictionary<LinkedAction, bool> OutdatedActionDictionary,
			ActionHistory ActionHistory, CppDependencyCache CppDependencies, bool bIgnoreOutdatedImportLibraries, ILogger Logger)
		{
			// Only compute the outdated-ness for actions that don't aren't cached in the outdated action dictionary.
			bool bIsOutdated = false;
			{
				// OutdatedActionDictionary may have already been populated for RootAction as part of a previously processed target
				if (OutdatedActionDictionary.ContainsKey(RootAction))
				{
					return;
				}
			}

			// Determine the last time the action was run based on the write times of its produced files.
			DateTimeOffset LastExecutionTimeUtc = DateTimeOffset.MaxValue;
			string NewProducingAttributes = $"{RootAction.CommandPath.FullName} {RootAction.CommandArguments} (ver {RootAction.CommandVersion})";
			foreach (FileItem ProducedItem in RootAction.ProducedItems)
			{
				ProducedItem.Directory.CacheFiles();

				if (!ProducedItem.HasInfo)
				{
					//Console.WriteLine("HASINFO " + ProducedItem);
				}

				// Check if the command-line of the action previously used to produce the item is outdated.
				if (ActionHistory.UpdateProducingAttributes(ProducedItem, NewProducingAttributes, Logger) && RootAction.bUseActionHistory)
				{
					if (ProducedItem.Exists)
					{
						Logger.LogTrace("{StatusDescription}: Produced item \"{ProducedItem}\" was produced by outdated attributes.", RootAction.StatusDescription, ProducedItem.Location);
						Logger.LogTrace("  New attributes: {NewProducingAttributes}", NewProducingAttributes);
					}

					bIsOutdated = true;
				}

				// If the produced file doesn't exist or has zero size, consider it outdated.  The zero size check is to detect cases
				// where aborting an earlier compile produced invalid zero-sized obj files, but that may cause actions where that's
				// legitimate output to always be considered outdated.
				if (ProducedItem.Exists && (RootAction.ActionType != ActionType.Compile || ProducedItem.Length > 0 ||
											(!ProducedItem.Location.HasExtension(".obj") && !ProducedItem.Location.HasExtension(".o"))))
				{
					// Find the newer of LastWriteTime and CreationTime, as copied files (such as from a build cache) can have a newer creation time in some cases
					DateTime ExecutionTimeUtc = ProducedItem.LastWriteTimeUtc > ProducedItem.CreationTimeUtc ? ProducedItem.LastWriteTimeUtc : ProducedItem.CreationTimeUtc;
					// Use the oldest produced item's time as the last execution time.
					if (ExecutionTimeUtc < LastExecutionTimeUtc)
					{
						LastExecutionTimeUtc = ExecutionTimeUtc;
					}
				}
				else
				{
					// If any of the produced items doesn't exist, the action is outdated.
					Logger.LogTrace("{StatusDescription}: Produced item \"{ProducedItem}\" doesn't exist.", RootAction.StatusDescription, ProducedItem.Location);
					bIsOutdated = true;
				}
			}

			// Check if any prerequisite item has a newer timestamp than the last execution time of this action
			if (!bIsOutdated)
			{
				foreach (FileItem PrerequisiteItem in RootAction.PrerequisiteItems)
				{
					// Need to check for import libraries here too
					if (bIgnoreOutdatedImportLibraries && IsImportLibraryDependency(RootAction, PrerequisiteItem))
					{
						continue;
					}

					PrerequisiteItem.Directory.CacheFiles();

					if (PrerequisiteItem.Exists)
					{
						// allow a 1 second slop for network copies
						TimeSpan TimeDifference = PrerequisiteItem.LastWriteTimeUtc - LastExecutionTimeUtc;
						bool bPrerequisiteItemIsNewerThanLastExecution = TimeDifference.TotalSeconds > 1;
						if (bPrerequisiteItemIsNewerThanLastExecution)
						{
							Logger.LogTrace("{StatusDescription}: Prerequisite {PrerequisiteItem} is newer than the last execution of the action: {Message}", RootAction.StatusDescription, PrerequisiteItem.Location,
									$"{PrerequisiteItem.LastWriteTimeUtc.ToLocalTime().ToString(CultureInfo.CurrentCulture)} vs {LastExecutionTimeUtc.LocalDateTime.ToString(CultureInfo.CurrentCulture)}");
							bIsOutdated = true;
							break;
						}
					}
					else
					{
						// If the PrerequisiteItem doesn't exist, it means it's being produced right now and we need to update the ProducedItem that
						// depends on it
						Logger.LogTrace("{StatusDescription}: Prerequisite {PrerequisiteItem} doesn't exist.", RootAction.StatusDescription, PrerequisiteItem.Location);
						bIsOutdated = true;
						break;
					}
				}
			}

			// Check the dependency list
			if (!bIsOutdated && RootAction.DependencyListFile != null)
			{
				if (!CppDependencies.TryGetDependencies(RootAction.DependencyListFile, Logger, out List<FileItem>? DependencyFiles))
				{
					Logger.LogTrace("{StatusDescription}: Missing dependency list file \"{DependencyListFile}\"", RootAction.StatusDescription, RootAction.DependencyListFile);
					bIsOutdated = true;
				}
				else
				{
					foreach (FileItem DependencyFile in DependencyFiles.Except(RootAction.PrerequisiteItems))
					{
						if (!DependencyFile.Exists)
						{
							Logger.LogTrace("{StatusDescription}: Dependency {DependencyFile} doesn't exist", RootAction.StatusDescription, DependencyFile.AbsolutePath);
							bIsOutdated = true;
							break;
						}
						else if (DependencyFile.LastWriteTimeUtc > LastExecutionTimeUtc)
						{
							Logger.LogTrace("{StatusDescription}: Dependency {DependencyFile} is newer than the last execution of the action: {Message}", RootAction.StatusDescription, DependencyFile.AbsolutePath,
								$"{DependencyFile.LastWriteTimeUtc.ToLocalTime().ToString(CultureInfo.CurrentCulture)} vs {LastExecutionTimeUtc.LocalDateTime.ToString(CultureInfo.CurrentCulture)}");
							bIsOutdated = true;
							break;
						}
					}
				}
			}

			// if the action is known to be out of date, record that fact 
			// We don't yet know that the action is up-to-date - to determine that requires traversal of the graph of prerequisites.
			if (bIsOutdated)
			{
				OutdatedActionDictionary.TryAdd(RootAction, bIsOutdated);
			}
		}

		/// <summary>
		/// Determines whether an action is outdated by examining the up-to-date state of all of its prerequisites, recursively.
		/// Not thread safe. Typically very fast.
		/// </summary>
		/// <param name="RootAction">- The action being considered.</param>
		/// <param name="OutdatedActionDictionary">-</param>
		/// <param name="bIgnoreOutdatedImportLibraries"></param>
		/// <param name="Logger">Logger instance</param>
		/// <returns>true if outdated</returns>
		private static bool IsActionOutdatedDueToPrerequisites(LinkedAction RootAction, ConcurrentDictionary<LinkedAction, bool> OutdatedActionDictionary, bool bIgnoreOutdatedImportLibraries, ILogger Logger)
		{
			// Only compute the outdated-ness for actions that aren't already cached in the outdated action dictionary.
			if (OutdatedActionDictionary.TryGetValue(RootAction, out bool bIsOutdated))
			{
				return bIsOutdated;
			}

			// Check if any of the prerequisite actions are out of date
			foreach (LinkedAction PrerequisiteAction in RootAction.PrerequisiteActions)
			{
				if (IsActionOutdatedDueToPrerequisites(PrerequisiteAction, OutdatedActionDictionary, bIgnoreOutdatedImportLibraries, Logger))
				{
					// Only check for outdated import libraries if we were configured to do so.  Often, a changed import library
					// won't affect a dependency unless a public header file was also changed, in which case we would be forced
					// to recompile anyway.  This just allows for faster iteration when working on a subsystem in a DLL, as we
					// won't have to wait for dependent targets to be relinked after each change.
					if (!bIgnoreOutdatedImportLibraries || !IsImportLibraryDependency(RootAction, PrerequisiteAction))
					{
						Logger.LogTrace("{StatusDescription}: Prerequisite {PrereqStatusDescription} is produced by outdated action.", RootAction.StatusDescription, PrerequisiteAction.StatusDescription);
						bIsOutdated = true;
						break;
					}
				}
			}

			// Cache the outdated-ness of this action.
			OutdatedActionDictionary.TryAdd(RootAction, bIsOutdated);

			return bIsOutdated;
		}

		/// <summary>
		/// Determines if the dependency between two actions is only for an import library
		/// </summary>
		/// <param name="RootAction">The action to check</param>
		/// <param name="PrerequisiteAction">The action that it depends on</param>
		/// <returns>True if the only dependency between two actions is for an import library</returns>
		static bool IsImportLibraryDependency(LinkedAction RootAction, LinkedAction PrerequisiteAction)
		{
			if (PrerequisiteAction.bProducesImportLibrary)
			{
				return PrerequisiteAction.ProducedItems.All(I => IsLibraryFile(I.Location) || !RootAction.PrerequisiteItems.Contains(I));
			}
			else
			{
				return false;
			}
		}

		/// <summary>
		/// Determines if the dependency on a between two actions is only for an import library
		/// </summary>
		/// <param name="RootAction">The action to check</param>
		/// <param name="PrerequisiteItem">The dependency that is out of date</param>
		/// <returns>True if the only dependency between two actions is for an import library</returns>
		static bool IsImportLibraryDependency(LinkedAction RootAction, FileItem PrerequisiteItem)
		{
			if (IsLibraryFile(PrerequisiteItem.Location))
			{
				foreach (LinkedAction PrerequisiteAction in RootAction.PrerequisiteActions)
				{
					if (PrerequisiteAction.bProducesImportLibrary && PrerequisiteAction.ProducedItems.FirstOrDefault(PrerequisiteItem) != null)
					{
						return true;
					}
				}
			}

			return false;
		}

		/// <summary>
		/// Test to see if the given file is a library file (or in the case of linux/mac, a dynamic library)
		/// </summary>
		/// <param name="Location">File to test</param>
		/// <returns>True if the file is a library file</returns>
		static bool IsLibraryFile(FileReference Location)
		{
			return Location.HasExtension(".lib") || Location.HasExtension(".so") || Location.HasExtension(".dylib");
		}

		/// <summary>
		/// Builds a dictionary containing the actions from AllActions that are outdated by calling
		/// IsActionOutdated.
		/// </summary>
		public static void GatherAllOutdatedActions(IReadOnlyList<LinkedAction> Actions, ActionHistory ActionHistory,
			ConcurrentDictionary<LinkedAction, bool> OutdatedActions, CppDependencyCache CppDependencies,
			bool bIgnoreOutdatedImportLibraries, ILogger Logger)
		{
			using ITimelineEvent __ = Timeline.ScopeEvent($"GatherOutdatedActions");

			using (GlobalTracer.Instance.BuildSpan("Cache individual outdated actions").StartActive())
			{
				Parallel2.ForEach(Actions,
					Action => IsIndividualActionOutdated(Action, OutdatedActions,
						ActionHistory, CppDependencies, bIgnoreOutdatedImportLibraries, Logger));
			}

			using (GlobalTracer.Instance.BuildSpan("Cache outdated actions based on recursive prerequisites").StartActive())
			{
				foreach (LinkedAction Action in Actions)
				{
					IsActionOutdatedDueToPrerequisites(Action, OutdatedActions, bIgnoreOutdatedImportLibraries, Logger);
				}
			}
		}

		/// <summary>
		/// Deletes all the items produced by actions in the provided outdated action dictionary.
		/// </summary>
		/// <param name="OutdatedActions">List of outdated actions</param>
		/// <param name="Logger">Logger for output</param>
		public static void DeleteOutdatedProducedItems(List<LinkedAction> OutdatedActions, ILogger Logger)
		{
			using ITimelineEvent __ = Timeline.ScopeEvent($"DeleteOutdatedProducedItems");

			Parallel2.ForEach(OutdatedActions.SelectMany(oa => oa.DeleteItems), DeleteItem =>
			{
				if (DeleteItem.Exists)
				{
					Logger.LogDebug("Deleting outdated item: {DeleteItem}", DeleteItem.Location);
					DeleteItem.Delete(Logger);
				}
			});
		}

		/// <summary>
		/// Creates directories for all the items produced by actions in the provided outdated action
		/// dictionary.
		/// </summary>
		public static void CreateDirectoriesForProducedItems(List<LinkedAction> OutdatedActions)
		{
			using ITimelineEvent __ = Timeline.ScopeEvent($"CreateDirectoriesForProducedItems");

			HashSet<DirectoryReference> OutputDirectories = new HashSet<DirectoryReference>();
			foreach (LinkedAction OutdatedAction in OutdatedActions)
			{
				foreach (FileItem ProducedItem in OutdatedAction.ProducedItems)
				{
					OutputDirectories.Add(ProducedItem.Location.Directory);
				}
			}

			foreach (DirectoryReference OutputDirectory in OutputDirectories)
			{
				if (!DirectoryReference.Exists(OutputDirectory))
				{
					DirectoryReference.CreateDirectory(OutputDirectory);
				}
			}
		}

		/// <summary>
		/// Imports an action graph from a JSON file
		/// </summary>
		/// <param name="InputFile">The file to read from</param>
		/// <returns>List of actions</returns>
		public static List<Action> ImportJson(FileReference InputFile)
		{
			JsonObject Object = JsonObject.Read(InputFile);

			JsonObject EnvironmentObject = Object.GetObjectField("Environment");
			foreach (string KeyName in EnvironmentObject.KeyNames)
			{
				Environment.SetEnvironmentVariable(KeyName, EnvironmentObject.GetStringField(KeyName));
			}

			List<Action> Actions = new List<Action>();
			foreach (JsonObject ActionObject in Object.GetObjectArrayField("Actions"))
			{
				Actions.Add(Action.ImportJson(ActionObject));
			}

			return Actions;
		}

		/// <summary>
		/// Exports an action graph to a JSON file
		/// </summary>
		/// <param name="Actions">The actions to write</param>
		/// <param name="OutputFile">Output file to write the actions to</param>
		public static void ExportJson(IReadOnlyList<LinkedAction> Actions, FileReference OutputFile)
		{
			DirectoryReference.CreateDirectory(OutputFile.Directory);
			using JsonWriter Writer = new JsonWriter(OutputFile);
			Writer.WriteObjectStart();

			Writer.WriteObjectStart("Environment");
			foreach (object? Object in Environment.GetEnvironmentVariables())
			{
				System.Collections.DictionaryEntry Pair = (System.Collections.DictionaryEntry)Object!;
				if (!UnrealBuildTool.InitialEnvironment!.Contains(Pair.Key) ||
					(string)(UnrealBuildTool.InitialEnvironment[Pair.Key]!) != (string)(Pair.Value!))
				{
					Writer.WriteValue((string)Pair.Key, (string)Pair.Value!);
				}
			}

			Writer.WriteObjectEnd();

			Dictionary<LinkedAction, int> ActionToId = new Dictionary<LinkedAction, int>();
			foreach (LinkedAction Action in Actions)
			{
				ActionToId[Action] = ActionToId.Count;
			}

			Writer.WriteArrayStart("Actions");
			foreach (LinkedAction Action in Actions)
			{
				Writer.WriteObjectStart();
				Action.ExportJson(ActionToId, Writer);
				Writer.WriteObjectEnd();
			}

			Writer.WriteArrayEnd();
			Writer.WriteObjectEnd();
		}
	}
}
