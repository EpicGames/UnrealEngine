// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// Interface for toolchain operations that produce output
	/// </summary>
	interface IActionGraphBuilder
	{
		/// <summary>
		/// Adds an action to this graph
		/// </summary>
		/// <param name="Action">Action to add</param>
		void AddAction(IExternalAction Action);

		/// <summary>
		/// Creates a response file for use in the action graph
		/// </summary>
		/// <param name="Location">Location of the response file</param>
		/// <param name="Contents">Contents of the file</param>
		/// <param name="AllowAsync">Allows the backend to write the file in a separate task.</param>
		/// <returns>New file item</returns>
		void CreateIntermediateTextFile(FileItem Location, string Contents, bool AllowAsync = true);

		/// <summary>
		/// Creates a response file for use in the action graph, with a newline between each string in ContentLines
		/// </summary>
		/// <param name="Location">Location of the response file</param>
		/// <param name="ContentLines">Contents of the file</param>
		/// <param name="AllowAsync">Allows the backend to write the file in a separate task.</param>
		/// <returns>New file item</returns>
		void CreateIntermediateTextFile(FileItem Location, IEnumerable<string> ContentLines, bool AllowAsync = true);

		/// <summary>
		/// Adds a file which is in the non-unity working set
		/// </summary>
		/// <param name="File">The file to add to the working set</param>
		void AddFileToWorkingSet(FileItem File);

		/// <summary>
		/// Adds a file which is a candidate for being in the non-unity working set
		/// </summary>
		/// <param name="File">The file to add to the working set</param>
		void AddCandidateForWorkingSet(FileItem File);

		/// <summary>
		/// Adds a source directory. These folders are scanned recursively for C++ source files.
		/// </summary>
		/// <param name="SourceDir">Base source directory</param>
		void AddSourceDir(DirectoryItem SourceDir);

		/// <summary>
		/// Adds the given source files as dependencies
		/// </summary>
		/// <param name="SourceDir">Source directory containing files to build</param>
		/// <param name="SourceFiles">Contents of the directory</param>
		void AddSourceFiles(DirectoryItem SourceDir, FileItem[] SourceFiles);

		/// <summary>
		/// Adds a list of known header files
		/// </summary>
		/// <param name="HeaderFiles">List of header files to track</param>
		void AddHeaderFiles(FileItem[] HeaderFiles);

		/// <summary>
		/// Sets the output items which belong to a particular module
		/// </summary>
		/// <param name="ModuleName">Name of the module</param>
		/// <param name="OutputItems">Array of output items for this module</param>
		void SetOutputItemsForModule(string ModuleName, FileItem[] OutputItems);

		/// <summary>
		/// Adds a diagnostic message
		/// </summary>
		/// <param name="Message">Message to display</param>
		void AddDiagnostic(string Message);
	}

	/// <summary>
	/// Implementation of IActionGraphBuilder which discards all unnecessary operations
	/// </summary>
	sealed class NullActionGraphBuilder : IActionGraphBuilder
	{
		private readonly ILogger Logger;
		private readonly FileHashCache? _fileHashCache;

		/// <summary>
		/// Constructor
		/// </summary>
		public NullActionGraphBuilder(ILogger InLogger, FileHashCache? fileHashCache = null)
		{
			Logger = InLogger;
			_fileHashCache = fileHashCache;
		}

		/// <inheritdoc/>
		public void AddAction(IExternalAction Action)
		{
		}

		/// <inheritdoc/>
		public void CreateIntermediateTextFile(FileItem FileItem, string Contents, bool AllowAsync = true)
		{
			if (_fileHashCache != null)
			{
				_fileHashCache.WriteFileIfChanged(FileItem, Contents, Logger, false);
			}
			else
			{
				Utils.WriteFileIfChanged(FileItem, Contents, Logger);
			}
		}

		/// <inheritdoc/>
		public void CreateIntermediateTextFile(FileItem FileItem, IEnumerable<string> ContentLines, bool AllowAsync = true)
		{
			if (_fileHashCache != null)
			{
				_fileHashCache.WriteFileIfChanged(FileItem, ContentLines, Logger, false);
			}
			else
			{
				Utils.WriteFileIfChanged(FileItem, ContentLines, Logger);
			}
		}

		/// <inheritdoc/>
		public void AddSourceDir(DirectoryItem SourceDir)
		{
		}

		/// <inheritdoc/>
		public void AddSourceFiles(DirectoryItem SourceDir, FileItem[] SourceFiles)
		{
		}

		/// <inheritdoc/>
		public void AddHeaderFiles(FileItem[] HeaderFiles)
		{
		}

		/// <inheritdoc/>
		public void AddFileToWorkingSet(FileItem File)
		{
		}

		/// <inheritdoc/>
		public void AddCandidateForWorkingSet(FileItem File)
		{
		}

		/// <inheritdoc/>
		public void AddDiagnostic(string Message)
		{
		}

		/// <inheritdoc/>
		public void SetOutputItemsForModule(string ModuleName, FileItem[] OutputItems)
		{
		}
	}

	/// <summary>
	/// Implementation of IActionGraphBuilder which forwards calls to an underlying implementation, allowing derived classes to intercept certain calls
	/// </summary>
	class ForwardingActionGraphBuilder : IActionGraphBuilder
	{
		/// <summary>
		/// The inner graph builder
		/// </summary>
		readonly IActionGraphBuilder Inner;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="Inner">Builder to pass all calls to</param>
		public ForwardingActionGraphBuilder(IActionGraphBuilder Inner)
		{
			this.Inner = Inner;
		}

		/// <inheritdoc/>
		public virtual void AddAction(IExternalAction Action)
		{
			Inner.AddAction(Action);
		}

		/// <inheritdoc/>
		public virtual void CreateIntermediateTextFile(FileItem FileItem, string Contents, bool AllowAsync = true)
		{
			Inner.CreateIntermediateTextFile(FileItem, Contents, AllowAsync);
		}

		/// <inheritdoc/>
		public virtual void CreateIntermediateTextFile(FileItem FileItem, IEnumerable<string> ContentLines, bool AllowAsync = true)
		{
			Inner.CreateIntermediateTextFile(FileItem, ContentLines, AllowAsync);
		}

		/// <inheritdoc/>
		public virtual void AddSourceDir(DirectoryItem SourceDir)
		{
			Inner.AddSourceDir(SourceDir);
		}

		/// <inheritdoc/>
		public virtual void AddSourceFiles(DirectoryItem SourceDir, FileItem[] SourceFiles)
		{
			Inner.AddSourceFiles(SourceDir, SourceFiles);
		}

		/// <inheritdoc/>
		public virtual void AddHeaderFiles(FileItem[] HeaderFiles)
		{
			Inner.AddHeaderFiles(HeaderFiles);
		}

		/// <inheritdoc/>
		public virtual void AddFileToWorkingSet(FileItem File)
		{
			Inner.AddFileToWorkingSet(File);
		}

		/// <inheritdoc/>
		public virtual void AddCandidateForWorkingSet(FileItem File)
		{
			Inner.AddCandidateForWorkingSet(File);
		}

		/// <inheritdoc/>
		public virtual void AddDiagnostic(string Message)
		{
			Inner.AddDiagnostic(Message);
		}

		/// <inheritdoc/>
		public virtual void SetOutputItemsForModule(string ModuleName, FileItem[] OutputItems)
		{
			Inner.SetOutputItemsForModule(ModuleName, OutputItems);
		}
	}

	/// <summary>
	/// Extension methods for IActionGraphBuilder classes
	/// </summary>
	static class ActionGraphBuilderExtensions
	{
		/// <summary>
		/// Creates a new action to be built as part of this target
		/// </summary>
		/// <param name="Graph">Graph to add the action to</param>
		/// <param name="Type">Type of action to create</param>
		/// <returns>New action</returns>
		public static Action CreateAction(this IActionGraphBuilder Graph, ActionType Type)
		{
			Action Action = new Action(Type);
			Graph.AddAction(Action);
			return Action;
		}

		/// <summary>
		/// Creates an action which copies a file from one location to another
		/// </summary>
		/// <param name="Graph">The action graph</param>
		/// <param name="SourceFile">The source file location</param>
		/// <param name="TargetFile">The target file location</param>
		/// <returns>File item for the output file</returns>
		public static Action CreateCopyAction(this IActionGraphBuilder Graph, FileItem SourceFile, FileItem TargetFile)
		{
			Action CopyAction = Graph.CreateAction(ActionType.BuildProject);
			CopyAction.CommandDescription = "Copy";
			CopyAction.CommandPath = BuildHostPlatform.Current.Shell;
			if (BuildHostPlatform.Current.ShellType == ShellType.Cmd)
			{
				CopyAction.CommandArguments = String.Format("/C \"copy /Y \"{0}\" \"{1}\" 1>nul\"", SourceFile.AbsolutePath, TargetFile.AbsolutePath);
			}
			else
			{
				CopyAction.CommandArguments = String.Format("-c \"cp -f \\\"{0}\\\" \\\"{1}\\\"\"", SourceFile.AbsolutePath, TargetFile.AbsolutePath);
			}
			CopyAction.WorkingDirectory = Unreal.EngineSourceDirectory;
			CopyAction.PrerequisiteItems.Add(SourceFile);
			CopyAction.ProducedItems.Add(TargetFile);
			CopyAction.DeleteItems.Add(TargetFile);
			CopyAction.StatusDescription = TargetFile.Location.GetFileName();
			CopyAction.bCanExecuteRemotely = false;
			return CopyAction;
		}

		/// <summary>
		/// Creates an action which copies a file from one location to another
		/// </summary>
		/// <param name="Graph">List of actions to be executed. Additional actions will be added to this list.</param>
		/// <param name="SourceFile">The source file location</param>
		/// <param name="TargetFile">The target file location</param>
		/// <returns>File item for the output file</returns>
		public static FileItem CreateCopyAction(this IActionGraphBuilder Graph, FileReference SourceFile, FileReference TargetFile)
		{
			FileItem SourceFileItem = FileItem.GetItemByFileReference(SourceFile);
			FileItem TargetFileItem = FileItem.GetItemByFileReference(TargetFile);

			Graph.CreateCopyAction(SourceFileItem, TargetFileItem);

			return TargetFileItem;
		}

		/// <summary>
		/// Creates an action which calls UBT recursively
		/// </summary>
		/// <param name="Graph">The action graph</param>
		/// <param name="Type">Type of the action</param>
		/// <param name="Arguments">Arguments for the action</param>
		/// <returns>New action instance</returns>
		public static Action CreateRecursiveAction<T>(this IActionGraphBuilder Graph, ActionType Type, string Arguments) where T : IToolMode<T>, new()
		{
			Action NewAction = Graph.CreateAction(Type);
			NewAction.CommandPath = Unreal.DotnetPath;
			NewAction.WorkingDirectory = Unreal.EngineSourceDirectory;
			NewAction.CommandArguments = $"\"{Unreal.UnrealBuildToolDllPath}\" -Session=\"{UnrealBuildTool.SessionIdentifier}\" -Mode={T.Name} {Arguments}";
			NewAction.CommandDescription = T.Name;
			NewAction.bCanExecuteRemotely = false;
			NewAction.bCanExecuteRemotelyWithSNDBS = false;
			NewAction.bCanExecuteInUBA = false;
			return NewAction;
		}

		/// <summary>
		/// Creates a text file with the given contents.  If the contents of the text file aren't changed, it won't write the new contents to
		/// the file to avoid causing an action to be considered outdated.
		/// </summary>
		/// <param name="Graph">The action graph</param>
		/// <param name="AbsolutePath">Path to the intermediate file to create</param>
		/// <param name="Contents">Contents of the new file</param>
		/// <param name="AllowAsync">Allows the backend to write the file in a separate task.</param>
		/// <returns>File item for the newly created file</returns>
		public static FileItem CreateIntermediateTextFile(this IActionGraphBuilder Graph, FileReference AbsolutePath, string Contents, bool AllowAsync = true)
		{
			FileItem FileItem = FileItem.GetItemByFileReference(AbsolutePath);
			Graph.CreateIntermediateTextFile(FileItem, Contents, AllowAsync);
			return FileItem;
		}

		/// <summary>
		/// Creates a text file with the given contents.  If the contents of the text file aren't changed, it won't write the new contents to
		/// the file to avoid causing an action to be considered outdated.
		/// </summary>
		/// <param name="Graph">The action graph</param>
		/// <param name="AbsolutePath">Path to the intermediate file to create</param>
		/// <param name="ContentLines">Contents of the new file</param>
		/// <param name="AllowAsync">Allows the backend to write the file in a separate task.</param>
		/// <returns>File item for the newly created file</returns>
		public static FileItem CreateIntermediateTextFile(this IActionGraphBuilder Graph, FileReference AbsolutePath, IEnumerable<string> ContentLines, bool AllowAsync = true)
		{
			FileItem FileItem = UnrealBuildBase.FileItem.GetItemByFileReference(AbsolutePath);
			Graph.CreateIntermediateTextFile(FileItem, ContentLines, AllowAsync);
			return FileItem;
		}
	}
}
