// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Text;
using System.Threading;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// Caches include dependency information to speed up preprocessing on subsequent runs.
	/// </summary>
	[DebuggerDisplay("{Location}")]
	class ActionHistoryLayer
	{
		/// <summary>
		/// Version number to check
		/// </summary>
		const int CurrentVersion = 3;

		/// <summary>
		/// Path to store the cache data to.
		/// </summary>
		public FileReference Location
		{
			get;
		}

		/// <summary>
		/// The Attributes used to produce files, keyed by the absolute file paths.
		/// </summary>
		ConcurrentDictionary<FileItem, IoHash> OutputItemToAttributeHash = new ConcurrentDictionary<FileItem, IoHash>();

		/// <summary>
		/// Whether the dependency cache is dirty and needs to be saved.
		/// </summary>
		bool bModified;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="Location">File to store this history in</param>
		/// <param name="Logger">Logger for output</param>
		public ActionHistoryLayer(FileReference Location, ILogger Logger)
		{
			this.Location = Location;

			if (FileReference.Exists(Location))
			{
				Load(Logger);
			}
		}

		/// <summary>
		/// Attempts to load this action history from disk
		/// </summary>
		void Load(ILogger Logger)
		{
			try
			{
				using (BinaryArchiveReader Reader = new BinaryArchiveReader(Location))
				{
					int Version = Reader.ReadInt();
					if (Version != CurrentVersion)
					{
						Logger.LogDebug("Unable to read action history from {Location}; version {Version} vs current {CurrentVersion}", Location, Version, CurrentVersion);
						return;
					}

					OutputItemToAttributeHash = new ConcurrentDictionary<FileItem, IoHash>(Reader.ReadDictionary(() => Reader.ReadFileItem()!, () => Reader.ReadIoHash() ?? IoHash.Zero)!);
				}
			}
			catch (Exception Ex)
			{
				Logger.LogWarning("Unable to read {Location}. See log for additional information.", Location);
				Logger.LogDebug("{Ex}", ExceptionUtils.FormatExceptionDetails(Ex));
			}
		}

		/// <summary>
		/// Saves this action history to disk
		/// </summary>
		public void Save()
		{
			if (bModified)
			{
				DirectoryReference.CreateDirectory(Location.Directory);
				using (BinaryArchiveWriter Writer = new BinaryArchiveWriter(Location))
				{
					Writer.WriteInt(CurrentVersion);
					Writer.WriteDictionary(OutputItemToAttributeHash, Key => Writer.WriteFileItem(Key), Value => Writer.WriteIoHash(Value));
				}
				bModified = false;
			}
		}

		/// <summary>
		/// Computes the case-invariant IoHash for a string
		/// </summary>
		/// <param name="Text">The text to hash</param>
		/// <returns>IoHash of the string</returns>
		static IoHash ComputeHash(string Text) => IoHash.Compute(Encoding.Unicode.GetBytes(Text.ToUpperInvariant()));

		/// <summary>
		/// Gets the producing attributes for the given file
		/// </summary>
		/// <param name="File">The output file to look for</param>
		/// <param name="Attributes">Receives the Attributes  used to produce this file</param>
		/// <returns>True if Attributes have changed and is updated, false otherwise</returns>
		public bool UpdateProducingCommandLine(FileItem File, string Attributes)
		{
			IoHash NewHash = ComputeHash(Attributes);

			for (; ; )
			{
				if (OutputItemToAttributeHash.TryAdd(File, NewHash))
				{
					// If this is a new entry we're done
					bModified = true;
					return true;
				}
				else
				{
					if (OutputItemToAttributeHash.TryGetValue(File, out IoHash OldHash))
					{
						if (NewHash == OldHash)
						{
							// hashes are the same, no update needed
							return false;
						}
						else
						{
							// Try to update with the new value
							if (OutputItemToAttributeHash.TryUpdate(File, NewHash, OldHash))
							{
								bModified = true;
								return true;
							}
						}
					}
				}
			}
		}

		/// <summary>
		/// Gets the location for the engine action history
		/// </summary>
		/// <param name="TargetName">Target name being built</param>
		/// <param name="Platform">The platform being built</param>
		/// <param name="TargetType">Type of the target being built</param>
		/// <param name="Architectures">The target architecture(s)</param>
		/// <returns>Path to the engine action history for this target</returns>
		public static FileReference GetEngineLocation(string TargetName, UnrealTargetPlatform Platform, TargetType TargetType, UnrealArchitectures Architectures)
		{
			string AppName;
			if (TargetType == TargetType.Program)
			{
				AppName = TargetName;
			}
			else
			{
				AppName = UEBuildTarget.GetAppNameForTargetType(TargetType);
			}

			return FileReference.Combine(Unreal.EngineDirectory, UEBuildTarget.GetPlatformIntermediateFolder(Platform, Architectures, false), AppName, "ActionHistory.bin");
		}

		/// <summary>
		/// Gets the location of the project action history
		/// </summary>
		/// <param name="ProjectFile">Path to the project file</param>
		/// <param name="Platform">Platform being built</param>
		/// <param name="TargetName">Name of the target being built</param>
		/// <param name="Architectures">The target architecture(s)</param>
		/// <returns>Path to the project action history</returns>
		public static FileReference GetProjectLocation(FileReference ProjectFile, string TargetName, UnrealTargetPlatform Platform, UnrealArchitectures Architectures)
		{
			return FileReference.Combine(ProjectFile.Directory, UEBuildTarget.GetPlatformIntermediateFolder(Platform, Architectures, false), TargetName, "ActionHistory.dat");
		}

		/// <summary>
		/// Enumerates all the locations of action history files for the given target
		/// </summary>
		/// <param name="ProjectFile">Project file for the target being built</param>
		/// <param name="TargetName">Name of the target</param>
		/// <param name="Platform">Platform being built</param>
		/// <param name="TargetType">The target type</param>
		/// <param name="Architectures">The target architecture(s)</param>
		/// <returns>Dependency cache hierarchy for the given project</returns>
		public static IEnumerable<FileReference> GetFilesToClean(FileReference ProjectFile, string TargetName, UnrealTargetPlatform Platform, TargetType TargetType, UnrealArchitectures Architectures)
		{
			if (ProjectFile == null || !Unreal.IsEngineInstalled())
			{
				yield return GetEngineLocation(TargetName, Platform, TargetType, Architectures);
			}
			if (ProjectFile != null)
			{
				yield return GetProjectLocation(ProjectFile, TargetName, Platform, Architectures);
			}
		}
	}

	/// <summary>
	/// Information about actions producing artifacts under a particular directory
	/// </summary>
	[DebuggerDisplay("{BaseDir}")]
	class ActionHistoryPartition
	{
		/// <summary>
		/// The base directory for this partition
		/// </summary>
		public DirectoryReference BaseDir { get; }

		/// <summary>
		/// Used to ensure exclusive access to the layers list
		/// </summary>
		readonly Lock LockObject = new();

		/// <summary>
		/// Map of filename to layer
		/// </summary>
		IReadOnlyList<ActionHistoryLayer> Layers = new List<ActionHistoryLayer>();

		/// <summary>
		/// Construct a new partition
		/// </summary>
		/// <param name="BaseDir">The base directory for this partition</param>
		public ActionHistoryPartition(DirectoryReference BaseDir)
		{
			this.BaseDir = BaseDir;
		}

		/// <summary>
		/// Attempt to update the producing command line for the given file
		/// </summary>
		/// <param name="File">The file to update</param>
		/// <param name="Attributes">The new attributes</param>
		/// <param name="Logger">Logger for output</param>
		/// <returns>True if the attributes were updated, false otherwise</returns>
		public bool UpdateProducingAttributes(FileItem File, string Attributes, ILogger Logger)
		{
			FileReference LayerLocation = GetLayerLocationForFile(File.Location);

			ActionHistoryLayer? Layer = Layers.FirstOrDefault(x => x.Location == LayerLocation);
			if (Layer == null)
			{
				lock (LockObject)
				{
					Layer = Layers.FirstOrDefault(x => x.Location == LayerLocation);
					if (Layer == null)
					{
						Layer = new ActionHistoryLayer(LayerLocation, Logger);

						List<ActionHistoryLayer> NewLayers = new List<ActionHistoryLayer>(Layers);
						NewLayers.Add(Layer);
						Layers = NewLayers;
					}
				}
			}
			return Layer.UpdateProducingCommandLine(File, Attributes);
		}

		/// <summary>
		/// Get the path to the action history layer to use for the given file
		/// </summary>
		/// <param name="Location">Path to the file to use</param>
		/// <returns>Path to the file</returns>
		public FileReference GetLayerLocationForFile(FileReference Location)
		{
			int Offset = BaseDir.FullName.Length;
			for (; ; )
			{
				int NameOffset = Offset + 1;

				// Get the next directory separator
				Offset = Location.FullName.IndexOf(Path.DirectorySeparatorChar, NameOffset + 1);
				if (Offset == -1)
				{
					break;
				}

				// Get the length of the name
				int NameLength = Offset - NameOffset;

				// Try to find Binaries/<Platform>/ in the path
				if (MatchPathFragment(Location, NameOffset, NameLength, "Binaries"))
				{
					int PlatformOffset = Offset + 1;
					int PlatformEndOffset = Location.FullName.IndexOf(Path.DirectorySeparatorChar, PlatformOffset);
					if (PlatformEndOffset != -1)
					{
						string PlatformName = Location.FullName.Substring(PlatformOffset, PlatformEndOffset - PlatformOffset);
						return FileReference.Combine(BaseDir, "Intermediate", "Build", PlatformName, "ActionHistory.bin");
					}
				}

				// Try to find /Intermediate/Build/<Platform>/<Target>/<Configuration> in the path
				if (MatchPathFragment(Location, NameOffset, NameLength, "Intermediate"))
				{
					int BuildOffset = Offset + 1;
					int BuildEndOffset = Location.FullName.IndexOf(Path.DirectorySeparatorChar, BuildOffset);
					if (BuildEndOffset != -1 && MatchPathFragment(Location, BuildOffset, BuildEndOffset - BuildOffset, "Build"))
					{
						// Skip the platform, target/app name, and configuration
						int EndOffset = BuildEndOffset;
						for (int Idx = 0; ; Idx++)
						{
							EndOffset = Location.FullName.IndexOf(Path.DirectorySeparatorChar, EndOffset + 1);
							if (EndOffset == -1)
							{
								break;
							}
							if (Idx == 2)
							{
								return FileReference.Combine(BaseDir, Location.FullName.Substring(NameOffset, EndOffset - NameOffset), "ActionHistory.bin");
							}
						}
					}
				}
			}
			return FileReference.Combine(BaseDir, "Intermediate", "Build", "ActionHistory.bin");
		}

		/// <summary>
		/// Attempts to match a substring of a path with the given fragment
		/// </summary>
		/// <param name="Location">Path to match against</param>
		/// <param name="Offset">Offset of the substring to match</param>
		/// <param name="Length">Length of the substring to match</param>
		/// <param name="Fragment">The path fragment</param>
		/// <returns>True if the substring matches</returns>
		static bool MatchPathFragment(FileReference Location, int Offset, int Length, string Fragment)
		{
			return Length == Fragment.Length && String.Compare(Location.FullName, Offset, Fragment, 0, Fragment.Length, FileReference.Comparison) == 0;
		}

		/// <summary>
		/// Saves the modified layers
		/// </summary>
		public void Save()
		{
			foreach (ActionHistoryLayer Layer in Layers)
			{
				Layer.Save();
			}
		}
	}

	/// <summary>
	/// A collection of ActionHistory layers
	/// </summary>
	class ActionHistory
	{
		/// <summary>
		/// The lock object for this history
		/// </summary>
		readonly Lock LockObject = new();

		/// <summary>
		/// List of partitions
		/// </summary>
		readonly List<ActionHistoryPartition> Partitions = new List<ActionHistoryPartition>();

		/// <summary>
		/// Constructor
		/// </summary>
		public ActionHistory()
		{
			Partitions.Add(new ActionHistoryPartition(Unreal.EngineDirectory));
		}

		/// <summary>
		/// Reads a cache from the given location, or creates it with the given settings
		/// </summary>
		/// <param name="BaseDir">Base directory for files that this cache should store data for</param>
		/// <returns>Reference to a dependency cache with the given settings</returns>
		public void Mount(DirectoryReference BaseDir)
		{
			lock (LockObject)
			{
				ActionHistoryPartition? Partition = Partitions.FirstOrDefault(x => x.BaseDir == BaseDir);
				if (Partition == null)
				{
					Partition = new ActionHistoryPartition(BaseDir);
					Partitions.Add(Partition);
				}
			}
		}

		/// <summary>
		/// Gets the producing command line for the given file
		/// </summary>
		/// <param name="File">The output file to look for</param>
		/// <param name="Attributes">Receives the Attributes used to produce this file</param>
		/// <param name="Logger">Logger for output</param>
		/// <returns>True if the output item exists</returns>
		public bool UpdateProducingAttributes(FileItem File, string Attributes, ILogger Logger)
		{
			foreach (ActionHistoryPartition Partition in Partitions)
			{
				if (File.Location.IsUnderDirectory(Partition.BaseDir))
				{
					return Partition.UpdateProducingAttributes(File, Attributes, Logger);
				}
			}

			Logger.LogWarning("File {FileLocation} is not under any action history root directory", File.Location);
			return false;
		}

		/// <summary>
		/// Saves all layers of this action history
		/// </summary>
		public void Save()
		{
			lock (LockObject)
			{
				foreach (ActionHistoryPartition Partition in Partitions)
				{
					Partition.Save();
				}
			}
		}
	}
}
