// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace UnrealBuildTool;

/// <summary>
/// Metrics for the execution of an action.
/// </summary>
/// <param name="ExecutionTime">The wall-clock time the action took to execute.</param>
/// <param name="ProcessorTime">The amount of CPU time the action took to execute.</param>
/// <param name="PeakMemoryBytes">The peak amount of memory, in bytes, used by the action.</param>
internal readonly record struct ActionMetrics(TimeSpan ExecutionTime, TimeSpan ProcessorTime, ulong PeakMemoryBytes);
