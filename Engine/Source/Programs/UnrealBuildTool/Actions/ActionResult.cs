// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Collections.Immutable;
using EpicGames.Core;

namespace UnrealBuildTool;

/// <summary>
/// The result of the execution of an action.
/// </summary>
/// <param name="CommandPath">The file path of the command that was ran.</param>
/// <param name="CommandArguments">The arguments to the command that was ran.</param>
/// <param name="ResponseFileContents">Contents of a response file, if one was passed as an argument.</param>
/// <param name="CommandVersion">The version of the command that was ran.</param>
/// <param name="Description">A human-readable description of the action.</param>
/// <param name="ExitCode">The exit code the action returned after executing.</param>
/// <param name="Success">Whether the action succeeded - actions may still fail even if the exit code was 0.</param>
/// <param name="Output">The output of the action.</param>
/// <param name="Metrics">The metrics associated with the action.</param>
internal record class ActionResult(
	FileReference CommandPath,
	string CommandArguments,
	ImmutableArray<string> ResponseFileContents,
	string CommandVersion,
	string Description,
	int ExitCode,
	bool Success,
	IReadOnlyList<string> Output,
	ActionMetrics Metrics)
{
	public static ActionResult From(ExecuteResults executeResults)
	{
		ActionMetrics metrics = new(executeResults.ExecutionTime, executeResults.ProcessorTime, executeResults.PeakMemoryUsed);
		return new(
			executeResults.Action.CommandPath,
			executeResults.Action.CommandArguments,
			executeResults.Action.ResponseFileContents,
			executeResults.Action.CommandVersion,
			executeResults.GetDescription(),
			executeResults.ExitCode,
			executeResults.ExitCode == 0,
			executeResults.LogLines,
			metrics);
	}
}
