// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Immutable;

namespace UnrealBuildTool.Actions;

internal record class ActionResultManifest(OriginalTargetDescriptor TargetDescriptor, ImmutableArray<ActionResult> Results);
