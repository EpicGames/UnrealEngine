// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Reflection;
using EpicGames.Core;

namespace UnrealBuildTool
{
	/// <summary>
	/// Interface for a class that can serialize an action type
	/// </summary>
	interface IActionSerializer
	{
		/// <summary>
		/// The action type
		/// </summary>
		Type Type { get; }

		/// <summary>
		/// Read the action from an archive
		/// </summary>
		/// <param name="Reader">Reader for the action</param>
		/// <returns>New action</returns>
		IExternalAction Read(BinaryArchiveReader Reader);

		/// <summary>
		/// Writes an action to an archive
		/// </summary>
		/// <param name="Writer">Writer for the archive</param>
		/// <param name="Action">The action to write</param>
		void Write(BinaryArchiveWriter Writer, IExternalAction Action);
	}

	/// <summary>
	/// Generic base class for an action serializer
	/// </summary>
	/// <typeparam name="TAction"></typeparam>
	abstract class ActionSerializerBase<TAction> : IActionSerializer where TAction : IExternalAction
	{
		/// <inheritdoc/>
		public Type Type => typeof(TAction);

		/// <inheritdoc/>
		IExternalAction IActionSerializer.Read(BinaryArchiveReader Reader) => Read(Reader);

		/// <inheritdoc/>
		void IActionSerializer.Write(BinaryArchiveWriter Writer, IExternalAction Action) => Write(Writer, (TAction)Action);

		/// <summary>
		/// Read the action from an archive
		/// </summary>
		/// <param name="Reader">Reader for the action</param>
		/// <returns>New action</returns>
		public abstract TAction Read(BinaryArchiveReader Reader);

		/// <summary>
		/// Writes an action to an archive
		/// </summary>
		/// <param name="Writer">Writer for the archive</param>
		/// <param name="Action">The action to write</param>
		public abstract void Write(BinaryArchiveWriter Writer, TAction Action);
	}

	/// <summary>
	/// Helper methods for registering serializers and serializing actions
	/// </summary>
	static class ActionSerialization
	{
		/// <summary>
		/// Map from type name to deserializing constructor
		/// </summary>
		static readonly IReadOnlyDictionary<Type, IActionSerializer> TypeToSerializer;

		/// <summary>
		/// Map from serializer name to instance
		/// </summary>
		static readonly IReadOnlyDictionary<string, IActionSerializer> NameToSerializer;

#pragma warning disable CA1810 // Initialize reference type static fields inline
		/// <summary>
		/// Creates a map of type name to constructor
		/// </summary>
		/// <returns></returns>
		static ActionSerialization()
		{
			Dictionary<Type, IActionSerializer> TypeToSerializerDict = new Dictionary<Type, IActionSerializer>();
			Dictionary<string, IActionSerializer> NameToSerializerDict = new Dictionary<string, IActionSerializer>(StringComparer.Ordinal);

			Type[] Types = Assembly.GetExecutingAssembly().GetTypes();
			foreach (Type Type in Types)
			{
				if (Type.IsClass && !Type.IsAbstract && typeof(IActionSerializer).IsAssignableFrom(Type))
				{
					IActionSerializer Serializer = (IActionSerializer)Activator.CreateInstance(Type)!;
					TypeToSerializerDict[Serializer.Type] = Serializer;
					NameToSerializerDict[Type.Name] = Serializer;
				}
			}

			TypeToSerializer = TypeToSerializerDict;
			NameToSerializer = NameToSerializerDict;
		}
#pragma warning restore CA1810

		/// <summary>
		/// Read an action from the given archive
		/// </summary>
		/// <param name="Reader">Reader to deserialize from</param>
		/// <returns>New action</returns>
		public static IExternalAction ReadAction(this BinaryArchiveReader Reader)
		{
			IActionSerializer Serializer = Reader.ReadObjectReference(() => ReadSerializer(Reader))!;
			return Serializer.Read(Reader);
		}

		/// <summary>
		/// Reads a type name and find its registered constructor from an archive
		/// </summary>
		/// <param name="Reader">Archive to read from</param>
		/// <returns>New constructor info</returns>
		static IActionSerializer ReadSerializer(BinaryArchiveReader Reader)
		{
			string Name = Reader.ReadString()!;

			IActionSerializer? Serializer;
			if (!NameToSerializer.TryGetValue(Name, out Serializer))
			{
				throw new BuildException("Unable to find action type '{0}'", Name);
			}

			return Serializer;
		}

		/// <summary>
		/// Writes an action to the given archive
		/// </summary>
		/// <param name="Writer">Writer to serialize the action to</param>
		/// <param name="Action">Action to serialize</param>
		public static void WriteAction(this BinaryArchiveWriter Writer, IExternalAction Action)
		{
			Type Type = Action.GetType();

			IActionSerializer? Serializer;
			if (!TypeToSerializer.TryGetValue(Type, out Serializer))
			{
				throw new BuildException("Unable to find serializer for action type '{0}'", Type.Name);
			}

			Writer.WriteObjectReference(Serializer, () => Writer.WriteString(Serializer.GetType().Name));
			Serializer.Write(Writer, Action);
		}
	}
}
