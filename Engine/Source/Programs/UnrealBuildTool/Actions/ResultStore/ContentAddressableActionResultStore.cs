// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Collections.Immutable;
using System.IO;
using System.Linq;
using System.Text.Json;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildTool.Storage;

namespace UnrealBuildTool.Actions.ResultStore;

internal class ContentAddressableActionResultStore(IStorageProvider storageProvider, ILogger logger)
	: IActionResultStore
{
	/// <inheritdoc />
	public async Task<ActionResult?> LoadResultAsync(LinkedAction action, CancellationToken cancellationToken)
	{
		IoHash actionHash = GetActionHash(action);
		return await LoadResultImplAsync(actionHash, cancellationToken);
	}

	/// <inheritdoc />
	public async Task<ActionResultManifest?> LoadManifestAsync(OriginalTargetDescriptor targetDescriptor, CancellationToken cancellationToken)
	{
		IoHash targetHash = _targetToHash.GetOrAdd(targetDescriptor, GetTargetHash);
		await using Stream? stream = storageProvider.CreateReader(targetHash);
		if (stream is null)
		{
			return null;
		}

		PartialActionResultManifest? partialManifest = await JsonSerializer.DeserializeAsync<PartialActionResultManifest>(stream, _jsonSerializerOptions, cancellationToken);
		if (partialManifest is null)
		{
			logger.LogInformation("Found empty manifest for hash {TargetHash}", targetHash);
			return null;
		}

		ActionResult?[] actionResults = await Task.WhenAll(partialManifest.Actions.AsParallel().Select(a => LoadResultImplAsync(a, cancellationToken)));
		IEnumerable<ActionResult> resultsWithoutNulls = actionResults.OfType<ActionResult>();
		return new ActionResultManifest(partialManifest.Target, [.. resultsWithoutNulls]);
	}

	/// <inheritdoc />
	public async Task StoreResultAsync(LinkedAction action, ActionResult result, CancellationToken cancellationToken)
	{
		IoHash actionHash = GetActionHash(action);
		await using Stream stream = storageProvider.CreateWriter(actionHash, cancellationToken);
		await JsonSerializer.SerializeAsync(stream, result, _jsonSerializerOptions, cancellationToken);

		if (action.Target is null)
		{
			return;
		}

		IoHash targetHash = _targetToHash.GetOrAdd(action.Target.Original, GetTargetHash);
		ManifestBuilder manifestBuilder = _targetHashToStream
			.GetOrAdd(targetHash, h => new(() => new(storageProvider, action.Target.Original, h, cancellationToken)))
			.Value;
		manifestBuilder.Actions.Add(actionHash);
	}

	/// <summary>
	/// Write out the complete manifests to the path given by the hash of the target descriptors.
	/// </summary>
	public async ValueTask DisposeAsync()
	{
		await Task.WhenAll(_targetHashToStream.Values.AsParallel().Select(async st => await st.Value.DisposeAsync()));
	}

	// Reference equality should be sufficient for our purposes, and is much faster.
	private readonly ConcurrentDictionary<OriginalTargetDescriptor, IoHash> _targetToHash = new(ReferenceEqualityComparer.Instance);
	// Lazy allows us to make sure only one stream actually gets constructed.
	private readonly ConcurrentDictionary<IoHash, Lazy<ManifestBuilder>> _targetHashToStream = [];

	private static readonly JsonSerializerOptions _jsonSerializerOptions = new()
	{
		PreferredObjectCreationHandling = System.Text.Json.Serialization.JsonObjectCreationHandling.Populate,
		IncludeFields = true,
	};

	private async Task<ActionResult?> LoadResultImplAsync(IoHash actionHash, CancellationToken cancellationToken)
	{
		await using Stream? stream = storageProvider.CreateReader(actionHash);
		if (stream is null)
		{
			return null;
		}

		return await JsonSerializer.DeserializeAsync<ActionResult>(stream, _jsonSerializerOptions, cancellationToken);
	}

	private static IoHash GetActionHash(LinkedAction action)
	{
		using Blake3.Hasher hasher = Blake3.Hasher.New();
		hasher.Update(action.CommandPath.FullName.AsSpan());
		hasher.Update(action.CommandArguments.AsSpan());
		hasher.Update(action.CommandVersion.AsSpan());
		return IoHash.FromBlake3(hasher);
	}

	private IoHash GetTargetHash(OriginalTargetDescriptor targetDescriptor)
	{
		const int byteEstimateForSmallTargetDescriptor = 2048;
		using MemoryStream memoryStream = new MemoryStream(byteEstimateForSmallTargetDescriptor);
		JsonSerializer.Serialize(memoryStream, targetDescriptor, _jsonSerializerOptions);
		ReadOnlySpan<byte> usedStream = memoryStream.GetBuffer().AsSpan()[..(int)memoryStream.Position];
		return IoHash.Compute(usedStream);
	}

	private record class PartialActionResultManifest(OriginalTargetDescriptor Target, ImmutableList<IoHash> Actions);

	/// <summary>
	/// Tracks actions and serializes a PartialActionResultManifest on dispose.
	/// </summary>
	private class ManifestBuilder(
		IStorageProvider storageProvider,
		OriginalTargetDescriptor targetDescriptor,
		IoHash targetHash,
		CancellationToken cancellationToken
	) : IAsyncDisposable
	{
		/// <summary>
		/// Even for 50,000 actions, this should only take about 1MB of storage, so keep it simple and just write out the manifest
		/// once all actions are known.
		/// </summary>
		public ConcurrentBag<IoHash> Actions { get; } = [];

		public async ValueTask DisposeAsync()
		{
			try
			{
				await using Stream stream = storageProvider.CreateWriter(targetHash, cancellationToken);
				PartialActionResultManifest manifest = new(targetDescriptor, [.. Actions]);
				await JsonSerializer.SerializeAsync(stream, manifest, _jsonSerializerOptions, cancellationToken);
			}
			catch (TaskCanceledException)
			{
				// Don't throw on cancellation.
			}
		}
	}
}
