// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Threading;
using System.Threading.Tasks;

namespace UnrealBuildTool.Actions.ResultStore;

/// <summary>
/// Persists the result of executing actions.
/// </summary>
internal interface IActionResultStore : IAsyncDisposable
{
	/// <summary>
	/// Load the result of executing a given action.
	/// </summary>
	/// <param name="action">The action to check results of.</param>
	/// <param name="cancellationToken">A token to allow cancelling the operation.</param>
	/// <returns>The result of the action, if it was previously persisted, otherwise null.</returns>
	Task<ActionResult?> LoadResultAsync(LinkedAction action, CancellationToken cancellationToken = default);

	/// <summary>
	/// Load an action result manifest for a given target descriptor.
	/// This will only exist if the target descriptor has been built previously.
	/// </summary>
	/// <param name="targetDescriptor">The target descriptor for which to load the manifest.</param>
	/// <param name="cancellationToken">A token to allow cancelling the operation.</param>
	/// <returns>The action result manifest, if it exists and is valid, otherwise null.</returns>
	Task<ActionResultManifest?> LoadManifestAsync(OriginalTargetDescriptor targetDescriptor, CancellationToken cancellationToken = default);

	/// <summary>
	/// Store the result of executing a given action.
	/// </summary>
	/// <param name="action">The action to store results of.</param>
	/// <param name="result">The result of the action to store.</param>
	/// <param name="cancellationToken">A token to allow cancelling the operation.</param>
	Task StoreResultAsync(LinkedAction action, ActionResult result, CancellationToken cancellationToken = default);
}
