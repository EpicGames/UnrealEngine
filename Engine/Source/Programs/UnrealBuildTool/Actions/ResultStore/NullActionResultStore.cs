// Copyright Epic Games, Inc. All Rights Reserved.

using System.Threading;
using System.Threading.Tasks;

namespace UnrealBuildTool.Actions.ResultStore;

internal class NullActionResultStore : IActionResultStore
{

	/// <inheritdoc />
	public Task<ActionResult?> LoadResultAsync(LinkedAction action, CancellationToken cancellationToken)
	{
		return Task.FromResult<ActionResult?>(null);
	}

	public Task<ActionResultManifest?> LoadManifestAsync(OriginalTargetDescriptor targetDescriptor, CancellationToken cancellationToken = default)
	{
		return Task.FromResult<ActionResultManifest?>(null);
	}

	/// <inheritdoc />
	public Task StoreResultAsync(LinkedAction action, ActionResult result, CancellationToken cancellationToken)
	{
		return Task.CompletedTask;
	}

	/// <inheritdoc />
	public ValueTask DisposeAsync()
	{
		return ValueTask.CompletedTask;
	}
}
