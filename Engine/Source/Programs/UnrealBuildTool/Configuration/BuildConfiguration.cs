// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using EpicGames.Core;

namespace UnrealBuildTool
{
	/// <summary>
	/// Global settings for building. Should not contain any target-specific settings.
	/// </summary>
	[System.Diagnostics.CodeAnalysis.SuppressMessage("Style", "IDE1006:Naming Styles", Justification = "UnrealBuildTool naming style")]
	internal sealed class BuildConfiguration
	{
		/// <summary>
		/// Whether to ignore import library files that are out of date when building targets. Set this to true to improve iteration time.
		/// By default, we do not bother re-linking targets if only a dependent .lib has changed, as chances are that
		/// the import library was not actually different unless a dependent header file of this target was also changed,
		/// in which case the target would automatically be rebuilt.
		/// </summary>
		[XmlConfigFile]
		public bool bIgnoreOutdatedImportLibraries { get; set; } = true;

		/// <summary>
		/// Use existing static libraries for all engine modules in this target.
		/// </summary>
		[CommandLine("-UsePrecompiled")]
		public bool bUsePrecompiled { get; set; } = false;

		/// <summary>
		/// Whether debug info should be written to the console.
		/// </summary>
		[XmlConfigFile]
		[CommandLine("-PrintDebugInfo", Value = "true")]
		public bool bPrintDebugInfo { get; set; } = false;

		/// <summary>
		/// Whether the hybrid executor will be used (a remote executor and local executor). No longer supported.
		/// </summary>
		[XmlConfigFile(Deprecated = true)]
		[Obsolete("Deprecated in UE5.8. HybridExecutor is no longer supported")]
		public bool bAllowHybridExecutor { get; set; } = false;

		/// <summary>
		/// Priority order for remote executors (XGE, SNDBS, FASTBuild, UBA)
		/// </summary>
		[XmlConfigFile]
		public string[] RemoteExecutorPriority { get; set; } = [XGE.ExecutorName, SNDBS.ExecutorName, FASTBuild.ExecutorName, UBAExecutor.ExecutorName];

		/// <summary>
		/// Whether the UnrealBuildAccelerator executor will be used.
		/// </summary>
		[XmlConfigFile]
		[CommandLine("-UBA", Value = "true")]
		[CommandLine("-NoUBA", Value = "false")]
		public bool bAllowUBAExecutor { get; set; } = true;

		/// <summary>
		/// Whether the UnrealBuildAccelerator (local only) executor will be used.
		/// </summary>
		[XmlConfigFile(Deprecated = true)]
		[Obsolete("Deprecated in UE5.8. Replace with bAllowUBAExecutor")]
		public bool bAllowUBALocalExecutor
		{
			get => bAllowUBAExecutor;
			set => bAllowUBAExecutor = value;
		}

		/// <summary>
		/// Whether XGE may be used if available, default is true.
		/// </summary>
		[XmlConfigFile]
		[CommandLine("-NoXGE", Value = "false")]
		public bool bAllowXGE { get; set; } = true;

		/// <summary>
		/// Whether FASTBuild may be used if available, default is true.
		/// </summary>
		[XmlConfigFile]
		[CommandLine("-NoFASTBuild", Value = "false")]
		public bool bAllowFASTBuild { get; set; } = true;

		/// <summary>
		/// Whether SN-DBS may be used if available, default is true.
		/// </summary>
		[XmlConfigFile]
		[CommandLine("-NoSNDBS", Value = "false")]
		public bool bAllowSNDBS { get; set; } = true;

		/// <summary>
		/// Enables support for very fast iterative builds by caching target data. Turning this on causes Unreal Build Tool to emit
		/// 'UBT Makefiles' for targets when they are built the first time. Subsequent builds will load these Makefiles and begin
		/// outdatedness checking and build invocation very quickly. The caveat is that if source files are added or removed to
		/// the project, UBT will need to gather information about those in order for your build to complete successfully. Currently,
		/// you must run the project file generator after adding/removing source files to tell UBT to re-gather this information.
		/// 
		/// Events that can invalidate the 'UBT Makefile':  
		///		- Adding/removing .cpp files
		///		- Adding/removing .h files with UObjects
		///		- Adding new UObject types to a file that did not previously have any
		///		- Changing global build settings (most settings in this file qualify)
		///		- Changed code that affects how Unreal Header Tool works
		///	
		///	You can force regeneration of the 'UBT Makefile' by passing the '-gather' argument, or simply regenerating project files.
		///
		///	This also enables the fast include file dependency scanning and caching system that allows Unreal Build Tool to detect out 
		/// of date dependencies very quickly. When enabled, a deep C++ include graph does not have to be generated, and instead,
		/// we only scan and cache indirect includes for after a dependent build product was already found to be out of date. During the
		/// next build, we will load those cached indirect includes and check for outdatedness.
		/// </summary>
		[XmlConfigFile]
		[CommandLine("-NoUBTMakefiles", Value = "false")]
		public bool bUseUBTMakefiles { get; set; } = true;

		/// <summary>
		/// Number of actions that can be executed in parallel. If 0 then code will pick a default based
		/// on the number of cores and memory available. Applies to the ParallelExecutor, HybridExecutor, and LocalExecutor
		/// </summary>
		[XmlConfigFile]
		[CommandLine("-MaxParallelActions")]
		public int MaxParallelActions { get; set; } = 0;

		/// <summary>
		/// Consider logical cores when determining how many total cpu cores are available.
		/// </summary>
		[XmlConfigFile(Name = "bAllCores")]
		[CommandLine("-AllCores")]
		public bool bAllCores { get; set; } = false;

		/// <summary>
		/// If true, force header regeneration. Intended for the build machine.
		/// </summary>
		[CommandLine("-ForceHeaderGeneration")]
		[XmlConfigFile(Category = "UEBuildConfiguration")]
		public bool bForceHeaderGeneration { get; set; } = false;

		/// <summary>
		/// If true, enable the UHT input cache for both reading and writing
		/// </summary>
		[CommandLine("-EnableUHTInputCache")]
		[XmlConfigFile(Category = "UEBuildConfiguration")]
		public bool bEnableUHTInputCache { get; set; } = false;

		/// <summary>
		/// If true, do not build UHT, assume it is already built.
		/// </summary>
		[Obsolete("Deprecated in UE5.8.")]
		[XmlConfigFile(Category = "UEBuildConfiguration", Deprecated = true)]
		public bool bDoNotBuildUHT { get; set; } = false;

		/// <summary>
		/// If true, fail if any of the generated header files is out of date.
		/// </summary>
		[CommandLine("-FailIfGeneratedCodeChanges")]
		[XmlConfigFile(Category = "UEBuildConfiguration")]
		public bool bFailIfGeneratedCodeChanges { get; set; } = false;

		/// <summary>
		/// True if hot-reload from IDE is allowed.
		/// </summary>
		[CommandLine("-NoHotReloadFromIDE", Value = "false")]
		[XmlConfigFile(Category = "UEBuildConfiguration")]
		public bool bAllowHotReloadFromIDE { get; set; } = true;

		/// <summary>
		/// If true, the Debug version of UnrealHeaderTool will be built and run instead of the Development version.
		/// </summary>
		[Obsolete("Deprecated in UE5.8.")]
		[XmlConfigFile(Category = "UEBuildConfiguration", Deprecated = true)]
		public bool bForceDebugUnrealHeaderTool { get; set; } = false;

		/// <summary>
		/// If true, use C# UHT internal to UBT
		/// </summary>
		[Obsolete("Deprecated in UE5.8.")]
		[XmlConfigFile(Category = "UEBuildConfiguration", Deprecated = true)]
		public bool bUseBuiltInUnrealHeaderTool { get; set; } = true;

		/// <summary>
		/// If true, generate warnings when C++ UHT is used
		/// </summary>
		[Obsolete("Deprecated in UE5.8.")]
		[XmlConfigFile(Category = "UEBuildConfiguration", Deprecated = true)]
		public bool bWarnOnCppUnrealHeaderTool { get; set; } = true;

		/// <summary>
		/// Whether to skip compiling rules assemblies and just assume they are valid
		/// </summary>
		[CommandLine("-SkipRulesCompile")]
		public bool bSkipRulesCompile { get; set; } = false;

		/// <summary>
		/// Whether to force compiling rules assemblies, regardless of whether they are valid
		/// </summary>
		[CommandLine("-ForceRulesCompile")]
		public bool bForceRulesCompile { get; set; } = false;

		/// <summary>
		/// Maximum recommended root path length.
		/// </summary>
		[XmlConfigFile(Category = "WindowsPlatform")]
		public int MaxRootPathLength { get; set; } = 50;

		/// <summary>
		/// Maximum length of a path relative to the root directory. Used on Windows to ensure paths are portable between machines.
		/// </summary>
		[XmlConfigFile(Category = "WindowsPlatform")]
		public int MaxNestedPathLength { get; set; } = 200;

		/// <summary>
		/// When single file targets are specified, via -File=, -SingleFile=, or -FileList=
		/// If this option is set, no error will be produced if the source file is not included in the target.
		/// Additionally, if any file or file list is specified, the target will not be built if none of the specified files are part of that target,
		/// including the case where a file specified via -FileList= is empty.
		/// </summary>
		[CommandLine("-IgnoreInvalidFiles")]
		public bool bIgnoreInvalidFiles { get; set; } = false;

		/// <summary>
		/// Instruct the executor to write compact output e.g. only errors, if supported by the executor.
		/// This field is used to hold the value when specified from the command line or XML
		/// </summary>
		[XmlConfigFile(Name = "bCompactOutput")]
		[CommandLine("-CompactOutput")]
		private bool bCompactOutputCommandLine { get; set; } = false;

		/// <summary>
		/// If set, artifacts will be read
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		[CommandLine("-ArtifactReads", Value = "True")]
		[CommandLine("-NoArtifactReads", Value = "False")]
		public bool bArtifactRead { get; set; } = true;

		/// <summary>
		/// If set, artifacts will be written
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		[CommandLine("-ArtifactWrites", Value = "True")]
		[CommandLine("-NoArtifactWrites", Value = "False")]
		public bool bArtifactWrites { get; set; } = true;

		/// <summary>
		/// If true, log all artifact cache misses as informational messages
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		[CommandLine("-LogArtifactCacheMisses", Value = "True")]
		public bool bLogArtifactCacheMisses { get; set; } = false;

		/// <summary>
		/// Location to store the artifacts.
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		[CommandLine("-ArtifactDirectory=")]
		public string ArtifactDirectory { get; set; } = String.Empty;

		/// <summary>
		/// Instruct the executor to write compact output e.g. only errors, if supported by the executor,
		/// and only if output is not being redirected e.g. during a build from within Visual Studio
		/// </summary>
		public bool bCompactOutput => bCompactOutputCommandLine && !Console.IsOutputRedirected;

		/// <summary>
		/// Whether to unify C++ code into larger files for faster compilation.
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool? bUseUnityBuild { get; set; }

		/// <summary>
		/// Whether to force C++ source files to be combined into larger files for faster compilation.
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool? bForceUnityBuild { get; set; }

		/// <summary>
		/// Whether to generate target makefiles in parallel.
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bParallelMakefileGeneration { get; set; } = true;

		/// <summary>
		/// Intermediate environment. Determines if the intermediates end up in a different folder than normal.
		/// </summary>
		public UnrealIntermediateEnvironment? IntermediateEnvironment
		{
			get
			{
				if (bForceUnityBuild == true)
				{
					return UnrealIntermediateEnvironment.Default;
				}
				else if (bUseUnityBuild == false)
				{
					return UnrealIntermediateEnvironment.NonUnity;
				}
				return null;
			}
		}
	}
}
