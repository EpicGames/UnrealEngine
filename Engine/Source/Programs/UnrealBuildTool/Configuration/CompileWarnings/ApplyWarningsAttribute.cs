// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using EpicGames.Core;

namespace UnrealBuildTool
{
	/// <summary>
	/// Denotes the position of a resolved compile warnings argument relative to other warnings.
	/// </summary>
	public enum ArgPosition
	{
		/// <summary>
		/// At the beginning of the argument list.
		/// </summary>
		Beginning,
		/// <summary>
		/// In the middle of the argument list.
		/// </summary>
		Middle,
		/// <summary>
		/// At the end of the argument list.
		/// </summary>
		End
	}

	/// <summary>
	/// Attribute used to mark fields which describes a set of <see cref="WarningLevel"/> arguments.
	/// </summary>
	[AttributeUsage(AttributeTargets.Field | AttributeTargets.Property, AllowMultiple = true)]
	internal abstract class ApplyWarningsAttribute : Attribute
	{
		private readonly string[]? _offArgs;
		private readonly string[]? _warningArgs;
		private readonly string[]? _errorArgs;
		protected readonly List<IApplyWarningsFilter> ActiveFilters = [];
		public readonly ArgPosition Position;

		/// <summary>
		/// Defines the argument metadata around a <see cref="CppCompileWarnings"/> <see cref="WarningLevel"/> property.
		/// </summary>
		/// <param name="offArgs">The arguments to use when the property is defined as <see cref="WarningLevel.Off"/>.</param>
		/// <param name="warningArgs">The arguments to use when the property is defined as <see cref="WarningLevel.Warning"/>.</param>
		/// <param name="errorArgs">The arguments to use when the property is defined as <see cref="WarningLevel.Error"/>.</param>
		/// <param name="applicableFilters">The applicable filter criteria.</param>
		/// <param name="argPosition">The phase of when to apply the warning.</param>
		internal ApplyWarningsAttribute(string[]? offArgs = null, string[]? warningArgs = null, string[]? errorArgs = null, string[]? applicableFilters = null, ArgPosition argPosition = ArgPosition.Middle)
		{
			_offArgs = offArgs;
			_warningArgs = warningArgs;
			_errorArgs = errorArgs;
			AppendNewFilters(applicableFilters);
			Position = argPosition;
		}

		/// <summary>
		/// Applies the warnings to the provided arguments list.
		/// </summary>
		/// <param name="level">The warning level to use when applying.</param>
		/// <param name="toolChainContext">The toolchain context to use in application.</param>
		/// <param name="arguments">The arugments list to add values to.</param>
		internal void ApplyWarningsToArguments(WarningLevel level, CompilerWarningsToolChainContext toolChainContext, List<string> arguments)
		{
			ApplyWarningsToArgumentsInternal(level, toolChainContext, arguments);
		}

		/// <summary>
		/// Applies the warnings to the provided arguments list.
		/// </summary>
		/// <param name="level">The warning level to use when applying.</param>
		/// <param name="toolChainContext">The toolchain context to use in application.</param>
		/// <param name="arguments">The arugments list to add values to.</param>
		/// <remarks>Used to provide custom overriding logic.</remarks>
		protected virtual void ApplyWarningsToArgumentsInternal(WarningLevel level, CompilerWarningsToolChainContext toolChainContext, List<string> arguments)
		{
			switch (level)
			{
				case WarningLevel.Off:
					if (_offArgs != null)
					{
						arguments.AddRange(_offArgs);
					}
					break;
				case WarningLevel.Warning:
					if (_warningArgs != null)
					{
						arguments.AddRange(_warningArgs);
					}
					break;
				case WarningLevel.Error:
					if (_errorArgs != null)
					{
						arguments.AddRange(_errorArgs);
					}
					break;
			}
		}

		/// <summary>
		/// Whether the given <see cref="ApplyWarningsAttribute"/> can be applied to the provided context.
		/// </summary>
		/// <param name="toolChainContext">The toolchain context to be applied to.</param>
		/// <returns>True if applicable, false otherwise.</returns>
		internal virtual bool CanApplyToContext(CompilerWarningsToolChainContext toolChainContext)
		{
			return (ActiveFilters == null || (ActiveFilters.TrueForAll(x => x.CanApply(toolChainContext))));
		}

		protected static bool TryAppendNewFilter(string rawFilterName, List<IApplyWarningsFilter> container)
		{
			IApplyWarningsFilter? filter = ApplyWarningsFilterRegistry.RequestFilter(rawFilterName);
			if (filter != null && !container.Contains(filter))
			{
				container.Add(filter);
				return true;
			}

			return false;
		}

		protected void AppendNewFilters(IEnumerable<string>? newFilters)
		{
			if (newFilters != null)
			{
				foreach (string rawFilterName in newFilters)
				{
					TryAppendNewFilter(rawFilterName, ActiveFilters);
				}
			}
		}

		protected void AppendNewFilters(IEnumerable<FilterID>? newFilters)
		{
			if (newFilters != null)
			{
				foreach (FilterID filterID in newFilters)
				{
					TryAppendNewFilter(filterID.ToString(), ActiveFilters);
				}
			}
		}

		/// <summary>
		/// The list of off arugments corresponding to <see cref="WarningLevel.Off"/>
		/// </summary>
		internal string[]? OffArgs => _offArgs;

		/// <summary>
		/// The list of off arugments corresponding to <see cref="WarningLevel.Warning"/>
		/// </summary>
		internal string[]? WarningArgs => _warningArgs;

		/// <summary>
		/// The list of off arugments corresponding to <see cref="WarningLevel.Error"/>
		/// </summary>
		internal string[]? ErrorArgs => _errorArgs;
	}

	#region -- ToolChains --

#pragma warning disable CA1813
	internal class WarningsVCToolChainAttribute : ApplyWarningsAttribute
#pragma warning restore CA1813
	{
		/// <inheritdoc/>
		internal WarningsVCToolChainAttribute(string[]? offArgs = null, string[]? warningArgs = null, string[]? errorArgs = null, string[]? filterNames = null, ArgPosition argPosition = ArgPosition.Middle) : base(offArgs, warningArgs, errorArgs, filterNames, argPosition)
		{
			AppendNewFilters([nameof(FilterID.VCToolChain)]);
		}
	}

#pragma warning disable CA1813
	internal class WarningsClangToolChainAttribute : ApplyWarningsAttribute
	{
#pragma warning restore CA1813
		/// <inheritdoc/>
		internal WarningsClangToolChainAttribute(string[]? offArgs = null, string[]? warningArgs = null, string[]? errorArgs = null, string[]? filterNames = null, ArgPosition argPosition = ArgPosition.Middle) : base(offArgs, warningArgs, errorArgs, filterNames, argPosition)
		{
			AppendNewFilters([nameof(FilterID.ClangToolChain)]);
		}
	}

#pragma warning disable CA1813
	internal class WarningsIntelCompilerAttribute : ApplyWarningsAttribute
#pragma warning restore CA1813
	{
		/// <inheritdoc/>
		internal WarningsIntelCompilerAttribute(string[]? offArgs = null, string[]? warningArgs = null, string[]? errorArgs = null, string[]? filterNames = null, ArgPosition argPosition = ArgPosition.Middle) : base(offArgs, warningArgs, errorArgs, filterNames, argPosition)
		{
			AppendNewFilters([nameof(StandardFilters.IntelCompilerFilter)]);
		}
	}

	#endregion -- ToolChains --

	#region -- Compiler Specializations -- 

#pragma warning disable CA1813
	internal class NonVCClangWarningsClangToolChainAttribute : WarningsClangToolChainAttribute
	{
#pragma warning restore CA1813
		/// <inheritdoc/>
		internal NonVCClangWarningsClangToolChainAttribute(string[]? offArgs = null, string[]? warningArgs = null, string[]? errorArgs = null, string[]? filterNames = null, ArgPosition argPosition = ArgPosition.Middle) : base(offArgs, warningArgs, errorArgs, filterNames, argPosition)
		{
			AppendNewFilters([nameof(StandardFilters.NonVCClangCompilerFilter)]);
		}
	}

	/// <summary>
	/// Specialized <see cref="WarningsVCToolChainAttribute"/> for MSVC compiler within the <see cref="VCToolChain"/>.
	/// </summary>
	internal sealed class WarningsMSCVAttribute : WarningsVCToolChainAttribute
	{
		/// <inheritdoc/>
		internal WarningsMSCVAttribute(string[]? offArgs = null, string[]? warningArgs = null, string[]? errorArgs = null, string[]? filterNames = null, ArgPosition argPosition = ArgPosition.Middle) : base(offArgs, warningArgs, errorArgs, filterNames, argPosition)
		{
			AppendNewFilters([nameof(StandardFilters.MSVCCompilerFilter)]);
		}
	}

	/// <summary>
	/// Specialized <see cref="WarningsVCToolChainAttribute"/> for Clang compiler within the <see cref="VCToolChain"/>.
	/// </summary>
	internal sealed class WarningsVCClangAttribute : WarningsVCToolChainAttribute
	{
		/// <inheritdoc/>
		internal WarningsVCClangAttribute(string[]? offArgs = null, string[]? warningArgs = null, string[]? errorArgs = null, ArgPosition argPosition = ArgPosition.Middle) : base(offArgs, warningArgs, errorArgs, argPosition: argPosition)
		{
			AppendNewFilters([nameof(StandardFilters.VCClangCompilerFilter)]);
		}
	}

	#endregion -- Compiler Specializations --

	#region -- ToolChain Specializations --

	/// <summary>
	/// / Specialized <see cref="WarningsClangToolChainAttribute"/> for Unused Value.
	/// </summary>
	internal sealed class UnusedValueClangToolChainAttribute : WarningsClangToolChainAttribute
	{
		/// <inheritdoc/>
		internal UnusedValueClangToolChainAttribute(string[]? offArgs = null, string[]? warningArgs = null, string[]? errorArgs = null) : base(offArgs, warningArgs, errorArgs)
		{
			AppendNewFilters([FilterID.ConfigShipping]);
		}

		/// <inheritdoc/>
		internal override bool CanApplyToContext(CompilerWarningsToolChainContext toolChainContext)
		{
			return base.CanApplyToContext(toolChainContext) || (toolChainContext._analyzer != StaticAnalyzer.None);
		}
	}

	/// <summary>
	/// Specialized <see cref="WarningsClangToolChainAttribute"/> for Shadow Variable Warnings.
	/// </summary>
	internal sealed class ShadowVariableWarningsClangToolChainAttribute : WarningsClangToolChainAttribute
	{
		/// <inheritdoc/>
		internal ShadowVariableWarningsClangToolChainAttribute(string[]? offArgs, string[]? warningArgs, string[]? errorArgs) : base(offArgs, warningArgs, errorArgs) { }

		/// <inheritdoc/>
		/// <remarks>No matter what our <see cref="CppCompileWarnings.ShadowVariableWarningLevel"/> is, in the clang 17-18.1.3 range we always disable.</remarks>
		protected override void ApplyWarningsToArgumentsInternal(WarningLevel level, CompilerWarningsToolChainContext toolChainContext, List<string> arguments)
		{
			if (toolChainContext._toolChainVersion == null || (toolChainContext._toolChainVersion >= new EpicGames.Core.VersionNumber(17) && toolChainContext._toolChainVersion < new VersionNumber(18, 1, 3)))
			{
				if (OffArgs != null)
				{
					arguments.AddRange(OffArgs);
				}
			}
			else
			{
				// Explicitly remove the OffArgs for consideration
				switch (level)
				{
					case WarningLevel.Warning:
						if (WarningArgs != null)
						{
							arguments.AddRange(WarningArgs);
						}
						break;
					case WarningLevel.Error:
						if (ErrorArgs != null)
						{
							arguments.AddRange(ErrorArgs);
						}
						break;
				}
			}
		}
	}

	/// <summary>
	/// Specialized <see cref="WarningsVCToolChainAttribute"/> for Undefined identifier.
	/// </summary>
	internal sealed class UndefinedIdentifierWarningsVCToolChainAttribute : WarningsVCToolChainAttribute
	{
		/// <inheritdoc/>
		internal UndefinedIdentifierWarningsVCToolChainAttribute(string[]? offArgs, string[]? warningArgs, string[]? errorArgs) : base(offArgs, warningArgs, errorArgs) { }

		/// <inheritdoc/>
		protected override void ApplyWarningsToArgumentsInternal(WarningLevel level, CompilerWarningsToolChainContext toolChainContext, List<string> arguments)
		{
			if (!toolChainContext._compileEnvironment.bPreprocessOnly)
			{
				base.ApplyWarningsToArgumentsInternal(level, toolChainContext, arguments);
			}
		}
	}

	/// <summary>
	/// Specialized <see cref="WarningsVCToolChainAttribute"/> for Unsafe Type Casts.
	/// </summary>
	internal sealed class UnsafeTypeCastWarningsVCToolChainAttribute : WarningsVCToolChainAttribute
	{
		/// <inheritdoc/>
		internal UnsafeTypeCastWarningsVCToolChainAttribute(string[] offArgs, string[] warningArgs, string[] errorArgs) : base(offArgs, warningArgs, errorArgs) { }

		/// <inheritdoc/>
		protected override void ApplyWarningsToArgumentsInternal(WarningLevel level, CompilerWarningsToolChainContext toolChainContext, List<string> arguments)
		{
			WarningLevel effectiveCastWarningLevel = (toolChainContext._buildSystemContext.GetReadOnlyTargetRules()?.Platform == UnrealTargetPlatform.Win64) ? level : WarningLevel.Off;
			base.ApplyWarningsToArgumentsInternal(effectiveCastWarningLevel, toolChainContext, arguments);
		}
	}

	#endregion -- ToolChain Specializations --
}