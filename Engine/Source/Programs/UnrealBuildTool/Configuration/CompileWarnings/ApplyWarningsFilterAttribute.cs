// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using EpicGames.Core;

namespace UnrealBuildTool
{
	/// <summary>
	/// Shorthand enum to be used in the <see cref="ApplyWarningsFilterAttribute"/> extensibility context.
	/// </summary>
	public enum FilterID
	{
		/// <summary>
		/// Filter ID for <see cref="ToolChainFilterAttribute"/>, specifically for <see cref="VCToolChain"/>.
		/// </summary>
		VCToolChain,

		/// <summary>
		/// Filter ID for <see cref="ToolChainFilterAttribute"/>, specifically for <see cref="ClangToolChain"/>.
		/// </summary>
		ClangToolChain,

		/// <summary>
		/// Filter ID for <see cref="ConfigurationFilterAttribute"/>, specifically for <see cref="CppConfiguration.Debug"/>.
		/// </summary>
		ConfigDebug,

		/// <summary>
		/// Filter ID for <see cref="ConfigurationFilterAttribute"/>, specifically for <see cref="CppConfiguration.Shipping"/>.
		/// </summary>
		ConfigShipping,

		/// <summary>
		/// Filter ID for <see cref="ToolChainVersionConstrainedFilterAttribute"/>, specifically for <see cref="VersionNumber"/> 14.0 as a min, with no upper bound.
		/// </summary>
		Version14Min,

		/// <summary>
		/// Filter ID for <see cref="ToolChainVersionConstrainedFilterAttribute"/>, specifically for <see cref="VersionNumber"/> 16.0 as a min, with no upper bound.
		/// </summary>
		Version16Min,

		/// <summary>
		/// Filter ID for <see cref="ToolChainVersionConstrainedFilterAttribute"/>, specifically for <see cref="VersionNumber"/> 18.0 as a min, with no upper bound.
		/// </summary>
		Version18Min,

		/// <summary>
		/// Filter ID for <see cref="ToolChainVersionConstrainedFilterAttribute"/>, specifically for <see cref="VersionNumber"/> 18.1 as a min, with no upper bound.
		/// </summary>
		Version18Point1Min,

		/// <summary>
		/// Filter ID for <see cref="ToolChainVersionConstrainedFilterAttribute"/>, specifically for <see cref="VersionNumber"/> 19.0 as a min, with no upper bound.
		/// </summary>
		Version19Min,

		/// <summary>
		/// Filter ID for <see cref="ToolChainVersionConstrainedFilterAttribute"/>, specifically for <see cref="VersionNumber"/> 12.0 as a min, with no upper bound.
		/// </summary>
		Version20Min,

		/// <summary>
		/// Filter ID for <see cref="CppStandardConstrainedFilterAttribute"/>, specifically for <see cref="CppStandardVersion.Cpp20"/> as a minimum, with no upper bound.
		/// </summary>
		Cpp20Min
	}

	/// <summary>
	///  Defines an attribute that is used in the class context for the purposes of ApplyWarnings filters. 
	/// </summary>
	/// <remarks>
	/// Use this extensibility point when instance data is requred, and your need to have the same logic applied for different instacnce data.
	/// The target class of the attribute is <see cref="ApplyWarningsFilterRegistry"/>.
	/// </remarks>
	[AttributeUsage(AttributeTargets.Class, AllowMultiple = true)]
	internal abstract class ApplyWarningsFilterAttribute : Attribute, IApplyWarningsFilter
	{
		private readonly string _filterName;
		public string FilterName => _filterName;

		/// <summary>
		/// Constructs the attribute.
		/// </summary>
		/// <param name="filterID">The filter id that describes the instance of the ApplyWarningsFilter attribute.</param>
#pragma warning disable CA1019 // Define accessors for attribute arguments
		public ApplyWarningsFilterAttribute(FilterID filterID)
#pragma warning restore CA1019 // Define accessors for attribute arguments
		{
			_filterName = filterID.ToString();
		}

		/// <summary>
		/// Constructs the attribute.
		/// </summary>
		/// <param name="filterName">The filter name that describes the instance of the ApplyWarningsFilter attribute.</param>
		public ApplyWarningsFilterAttribute(string filterName)
		{
			_filterName = filterName;
		}

		/// <inheritdoc/>
		public abstract bool CanApply(CompilerWarningsToolChainContext context);
	}

	#region -- Filter Attributes --

	/// <summary>
	/// Configuration filter attribute that will only apply the filter under a speciic <see cref="CppConfiguration"/>.
	/// </summary>
	internal sealed class ConfigurationFilterAttribute : ApplyWarningsFilterAttribute
	{
		private readonly CppConfiguration _config = CppConfiguration.Debug;
		public CppConfiguration Config => _config;

		/// Constructs a ConfigurationFilterAttribute.
		/// <param name="filterID">The filter id that describes the instance of the ApplyWarningsFilter attribute.</param>
		/// <param name="config">The config to filter on.</param>
#pragma warning disable CA1019 // Define accessors for attribute arguments
		public ConfigurationFilterAttribute(FilterID filterID, CppConfiguration config) : base(filterID)
#pragma warning restore CA1019 // Define accessors for attribute arguments
		{
			_config = config;
		}

		/// Constructs a ConfigurationFilterAttribute.
		/// <param name="filterName">The filter name that describes the instance of the ApplyWarningsFilter attribute.</param>
		/// <param name="config">The config to filter on.</param>
		public ConfigurationFilterAttribute(string filterName, CppConfiguration config) : base(filterName)
		{
			_config = config;
		}

		/// <inheritdoc/>
		public override bool CanApply(CompilerWarningsToolChainContext context)
		{
			return _config == context._compileEnvironment.Configuration;
		}
	}

	/// <summary>
	/// Specialized <see cref="ApplyWarningsFilterAttribute"/> to toolchain version constraints.
	/// </summary>
	internal sealed class ToolChainVersionConstrainedFilterAttribute : ApplyWarningsFilterAttribute
	{
		private readonly VersionNumber? _lowerBound;
		private readonly VersionNumber? _upperBound;

		public VersionNumber? LowerBound => _lowerBound;
		public VersionNumber? UpperBound => _upperBound;

		/// <summary>
		/// Constructs a ToolChainVersionConstrainedFilterAttribute.
		/// </summary>
		/// <param name="filterID">The filter id that describes the instance of the ApplyWarningsFilter attribute.</param>
		/// <param name="lowerBoundVersionComponents">The lower bound of the <see cref="VersionNumber"/> to constrain this warning attribute to.</param>
		/// <param name="upperBoundVersionComponents">The upper bound of the <see cref="VersionNumber"/> to constrain this warning attribute to.</param>
		/// <remarks><see cref="VersionNumber"/> is not a viable parameter to this attribute, as it's a generic (via Comparable interface), and as such is not viable for attributes.</remarks>
#pragma warning disable CA1019 // Define accessors for attribute arguments
		public ToolChainVersionConstrainedFilterAttribute(FilterID filterID, int[]? lowerBoundVersionComponents = null, int[]? upperBoundVersionComponents = null) : base(filterID)
#pragma warning restore CA1019 // Define accessors for attribute arguments
		{
			_lowerBound = lowerBoundVersionComponents == null ? null : new VersionNumber(lowerBoundVersionComponents);
			_upperBound = upperBoundVersionComponents == null ? null : new VersionNumber(upperBoundVersionComponents);
		}

		/// <summary>
		/// Constructs a ToolChainVersionConstrainedFilterAttribute.
		/// </summary>
		/// <param name="filterName">The filter name that describes the instance of the ApplyWarningsFilter attribute.</param>
		/// <param name="lowerBoundVersionComponents">The lower bound of the <see cref="VersionNumber"/> to constrain this warning attribute to.</param>
		/// <param name="upperBoundVersionComponents">The upper bound of the <see cref="VersionNumber"/> to constrain this warning attribute to.</param>
		/// <remarks><see cref="VersionNumber"/> is not a viable parameter to this attribute, as it's a generic (via Comparable interface), and as such is not viable for attributes.</remarks>
		public ToolChainVersionConstrainedFilterAttribute(string filterName, int[]? lowerBoundVersionComponents = null, int[]? upperBoundVersionComponents = null) : base(filterName)
		{
			_lowerBound = lowerBoundVersionComponents == null ? null : new VersionNumber(lowerBoundVersionComponents);
			_upperBound = upperBoundVersionComponents == null ? null : new VersionNumber(upperBoundVersionComponents);
		}

		/// <inheritdoc/>
		public override bool CanApply(CompilerWarningsToolChainContext context)
		{
			// If our evaluation version is null, we can't possibly make a statement about whether it fits within a range
			if (context._toolChainVersion == null)
			{
				return false;
			}

			bool isAboveLowerBound = _lowerBound == null || context._toolChainVersion >= _lowerBound;
			bool isBelowUpperBound = _upperBound == null || context._toolChainVersion <= _upperBound;

			return isAboveLowerBound && isBelowUpperBound;
		}
	}

	/// <summary>
	/// Specialized <see cref="ApplyWarningsFilterAttribute"/> to enforce cpp standard constraints.
	/// </summary>
	internal sealed class CppStandardConstrainedFilterAttribute : ApplyWarningsFilterAttribute
	{
		private readonly CppStandardVersion _lowerBound;
		private readonly CppStandardVersion _upperBound;

		public CppStandardVersion LowerBound => _lowerBound;
		public CppStandardVersion UpperBound => _upperBound;

		/// <summary>
		/// Constructor for CppStandardConstrainedFilterAttribute that enforces a cpp standard constraint.
		/// </summary>
		/// <param name="filterID">The filter id that describes the instance of the ApplyWarningsFilter attribute.</param>
		/// <param name="lowerBound">The lower bound of the <see cref="CppStandardVersion"/> to constrain this warning attribute to.</param>
		/// <param name="upperBound">The upper bound of the <see cref="CppStandardVersion"/> to constrain this warning attribute to.</param>
#pragma warning disable CA1019 // Define accessors for attribute arguments
		public CppStandardConstrainedFilterAttribute(FilterID filterID, CppStandardVersion lowerBound = CppStandardVersion.Minimum, CppStandardVersion upperBound = CppStandardVersion.Latest) : base(filterID)
#pragma warning restore CA1019 // Define accessors for attribute arguments
		{
			_lowerBound = lowerBound;
			_upperBound = upperBound;
		}

		/// <summary>
		/// Constructor for CppStandardConstrainedFilterAttribute that enforces a cpp standard constraint.
		/// </summary>
		/// <param name="filterName">The filter name that describes the instance of the ApplyWarningsFilter attribute.</param>
		/// <param name="lowerBound">The lower bound of the <see cref="CppStandardVersion"/> to constrain this warning attribute to.</param>
		/// <param name="upperBound">The upper bound of the <see cref="CppStandardVersion"/> to constrain this warning attribute to.</param>
		public CppStandardConstrainedFilterAttribute(string filterName, CppStandardVersion lowerBound = CppStandardVersion.Minimum, CppStandardVersion upperBound = CppStandardVersion.Latest) : base(filterName)
		{
			_lowerBound = lowerBound;
			_upperBound = upperBound;
		}

		/// <inheritdoc/>
		public override bool CanApply(CompilerWarningsToolChainContext context)
		{
			bool isAboveLowerBound = context._compileEnvironment.CppStandard >= _lowerBound;
			bool isBelowUpperBound = context._compileEnvironment.CppStandard <= _upperBound;

			return isAboveLowerBound && isBelowUpperBound;
		}
	}

	/// <summary>
	/// Specialized <see cref="ApplyWarningsFilterAttribute"/> to enforce specific tool chain contexts.
	/// </summary>
	internal sealed class ToolChainFilterAttribute : ApplyWarningsFilterAttribute
	{
		private readonly Type _toolChainType;
		public Type ToolChainType => _toolChainType;

		/// <summary>
		/// Constructor for ToolChainFilterAttribute that enforces specific tool chain contexts.
		/// </summary>
		/// <param name="filterID">The filter id that describes the instance of the ApplyWarningsFilter attribute.</param>
		/// <param name="toolChainType">The toolchain of which this filter should particiapte.</param>
		/// <exception cref="ArgumentException">If the provided toolchain type is not derived from <see cref="UEToolChain"/>, throws.</exception>
#pragma warning disable CA1019 // Define accessors for attribute arguments
		public ToolChainFilterAttribute(FilterID filterID, Type toolChainType) : base(filterID)
#pragma warning restore CA1019 // Define accessors for attribute arguments
		{
			if (!typeof(UEToolChain).IsAssignableFrom(toolChainType))
			{
				throw new ArgumentException($"Invalid type provided for toolchain: {toolChainType.ToString()}");
			}

			_toolChainType = toolChainType;
		}

		/// <summary>
		/// Constructor for ToolChainFilterAttribute that enforces specific tool chain contexts.
		/// </summary>
		/// <param name="filterName">The filter name that describes the instance of the ApplyWarningsFilter attribute.</param>
		/// <param name="toolChainType">The toolchain of which this filter should particiapte.</param>
		/// <exception cref="ArgumentException">If the provided toolchain type is not derived from <see cref="UEToolChain"/>, throws.</exception>
		public ToolChainFilterAttribute(string filterName, Type toolChainType) : base(filterName)
		{
			if (!typeof(UEToolChain).IsAssignableFrom(toolChainType))
			{
				throw new ArgumentException($"Invalid type provided for toolchain: {toolChainType}");
			}

			_toolChainType = toolChainType;
		}

		/// <inheritdoc/>
		public override bool CanApply(CompilerWarningsToolChainContext context)
		{
			return _toolChainType == context._toolChainType;
		}
	}

	#endregion
}