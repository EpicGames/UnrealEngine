// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Reflection;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace UnrealBuildTool
{
	/// <summary>
	/// Defines an attribute that is used to support reflective delegates for the purposes of ApplyWarnings filters.
	/// </summary>
	/// <remarks>Use this extensibility point when no such instance data is required, or you don't need to have the same logic with different instance data.</remarks>
	[AttributeUsage(AttributeTargets.Method, AllowMultiple = false)]
	internal sealed class ApplyWarningsFilterDelegateAttribute : Attribute
	{
		private readonly string _filterName;
		public string FilterName => _filterName;

		/// <summary>
		/// Constructor for ApplyWarningsFilterDelegateAttribute.
		/// </summary>
		/// <param name="filterName">The name of the filter.</param>
		public ApplyWarningsFilterDelegateAttribute(string filterName)
		{
			_filterName = filterName;
		}
	}

	/// <summary>
	/// A wrapper for the methods annotated by the <see cref="ApplyWarningsFilterDelegateAttribute"/> to support <see cref="IApplyWarningsFilter"/>.
	/// </summary>
	internal struct ApplyWarningsFilterWrapper : IApplyWarningsFilter
	{
		public string FilterName => _applyWarningsFilterDelegateAttribute.FilterName;

		private readonly ILogger? _logger = Log.Logger;
		private readonly ApplyWarningsFilterDelegateAttribute _applyWarningsFilterDelegateAttribute;
		private readonly MethodInfo _method;

		/// <summary>
		/// Constructs a ApplyWarningsFilterWrapper.
		/// </summary>
		/// <param name="filterAttribute">Instance of the filter attribute that is being wrapped.</param>
		/// <param name="method">The method that will be used for reflective invocation in <see cref="CanApply(CompilerWarningsToolChainContext)"/>.</param>
		public ApplyWarningsFilterWrapper(ApplyWarningsFilterDelegateAttribute filterAttribute, MethodInfo method)
		{
			_applyWarningsFilterDelegateAttribute = filterAttribute;
			_method = method;
		}

		/// <inheritdoc/>
		public bool CanApply(CompilerWarningsToolChainContext context)
		{
			bool canApply = _method != null;

			if (canApply)
			{
				try
				{
					object? result = _method?.Invoke(null, [context]);
					canApply = result != null ? (bool)result : false;
				}
				catch (Exception e)
				{
					_logger?.LogWarning("Have failed to reflectively invoke (name:{MethodName}) in the CanApply context. Exception: {Exception}.", _method?.Name, e.Message);
				}
			}

			return canApply;
		}
	}
}
