// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Reflection;
using EpicGames.Core;
using Microsoft.CodeAnalysis;
using Microsoft.Extensions.Logging;

namespace UnrealBuildTool
{
	/// <summary>
	/// Interface that participates in defining whether a filter can be applied in a <see cref="ApplyWarningsAttribute"/>.
	/// </summary>
	internal interface IApplyWarningsFilter
	{
		/// <summary>
		/// Determines whether the <see cref="IApplyWarningsFilter"/> should be applied for the given <see cref="CompilerWarningsToolChainContext"/>.
		/// </summary>
		/// <param name="context">The context in which to evaluate the applicability of the filter.</param>
		/// <returns>True if the filter should be applied, false otherwise.</returns>
		bool CanApply(CompilerWarningsToolChainContext context);
	}

	/// <summary>
	/// ApplyWarningsFilterRegistry is the container for all of the filters applicable within the <see cref="ApplyWarningsAttribute.CanApplyToContext(CompilerWarningsToolChainContext)"/> context.
	/// </summary>
	/// <remarks>The two primary entry points for extensibility are <see cref="ApplyWarningsFilterDelegateAttribute"/> and <see cref="ApplyWarningsFilterAttribute"/>.</remarks>
	[ConfigurationFilter(FilterID.ConfigDebug, CppConfiguration.Debug)]
	[ConfigurationFilter(FilterID.ConfigShipping, CppConfiguration.Shipping)]
	[ToolChainFilter(FilterID.VCToolChain, typeof(VCToolChain))]
	[ToolChainFilter(FilterID.ClangToolChain, typeof(ClangToolChain))]
	[ToolChainVersionConstrainedFilter(FilterID.Version14Min, [14])]
	[ToolChainVersionConstrainedFilter(FilterID.Version16Min, [16])]
	[ToolChainVersionConstrainedFilter(FilterID.Version18Min, [18])]
	[ToolChainVersionConstrainedFilter(FilterID.Version18Point1Min, [18, 1])]
	[ToolChainVersionConstrainedFilter(FilterID.Version19Min, [19])]
	[ToolChainVersionConstrainedFilter(FilterID.Version20Min, [20])]
	[CppStandardConstrainedFilter(FilterID.Cpp20Min, CppStandardVersion.Cpp20)]
	internal class ApplyWarningsFilterRegistry
	{
		private readonly ILogger? _logger = Log.Logger;
		private static ApplyWarningsFilterRegistry? s_filterRegistry;
		private readonly Dictionary<string, IApplyWarningsFilter> _filterNameToBehaviour = [];

		/// <summary>
		/// The filter name to filter representation of the registry.
		/// </summary>
		internal static IReadOnlyDictionary<string, IApplyWarningsFilter> Filters
		{
			get
			{
				if (s_filterRegistry == null)
				{
					s_filterRegistry = new ApplyWarningsFilterRegistry();
				}

				return s_filterRegistry._filterNameToBehaviour;
			}
		}

		/// <summary>
		/// Requests a filter by name.
		/// </summary>
		/// <param name="filterName">The name of the filter to obtain.</param>
		/// <returns>The ApplyWarningsFilter if it has been registered, null if it was not found.</returns>
		internal static IApplyWarningsFilter? RequestFilter(string filterName)
		{
			IApplyWarningsFilter? returnFilter;
			Filters.TryGetValue(filterName, out returnFilter);

			return returnFilter;
		}

		private ApplyWarningsFilterRegistry()
		{
			// Check the entire assembly for the ApplyWarningsFilterAttribute
			Type filterRegistryType = typeof(ApplyWarningsFilterRegistry);

			IEnumerable<ApplyWarningsFilterAttribute> attributes = filterRegistryType
				.GetCustomAttributes(typeof(ApplyWarningsFilterAttribute), inherit: false)
				.Cast<ApplyWarningsFilterAttribute>();

			foreach (ApplyWarningsFilterAttribute attribute in attributes)
			{
				if (!_filterNameToBehaviour.TryAdd(attribute.FilterName, attribute))
				{
					_logger?.LogWarning("Duplicate filter names ({DuplicateName}) encountered when building the ApplyWarningsFilterRegistry.", attribute.FilterName);
				}
			}

			// Process all of the static methods in the assembly that are annotated wtih ApplyWarningsFilterDelegateAttribute, and evaluate given they match
			// the execution delegate, which is effectively IApplyWarningsFilter.CanApply
			MethodInfo? reflectiveInterfaceMethodInfo = typeof(IApplyWarningsFilter).GetMethod(nameof(IApplyWarningsFilter.CanApply));

			if (reflectiveInterfaceMethodInfo != null)
			{
				Assembly assembly = Assembly.GetExecutingAssembly();
				Type[] allTypes = assembly.GetTypes();
				List<MethodInfo> delegatesForConsideration = new List<MethodInfo>();

				foreach (Type type in allTypes)
				{
					// We could place all of the additional filter requirements in with the initial reflection call, but we want to provide better messaging to better guide users
					// attempting to extend the functionality.
					IEnumerable<MethodInfo> methodsWithAttribute = type
						.GetMethods(BindingFlags.Static | BindingFlags.Public | BindingFlags.NonPublic)
						.Where(m => m.GetCustomAttributes(typeof(ApplyWarningsFilterDelegateAttribute), false).Length > 0);

					delegatesForConsideration.AddRange(methodsWithAttribute);
				}

				// Verify the method infos match the signature of our fundamental interface: IApplyWarningsFilter.CanApply
				foreach (MethodInfo mi in delegatesForConsideration)
				{
					if (mi.ReturnType != reflectiveInterfaceMethodInfo.ReturnType)
					{
						_logger?.LogWarning("Skipping MethodInfo (name:{MethodName}) annotated with ApplyWarningsFilterDelegateAttribute as it doesn't return {ReturnType}.", mi.Name, reflectiveInterfaceMethodInfo.ReturnType.Name);
						continue;
					}

					if (mi.GetParameters().Length != reflectiveInterfaceMethodInfo.GetParameters().Length)
					{
						_logger?.LogWarning("Skipping MethodInfo (name:{MethodName}) annotated with ApplyWarningsFilterDelegateAttribute as it doesn't contain the same number of parameters as the base method interface ({Count})", mi.Name, reflectiveInterfaceMethodInfo.GetParameters().Length);
						continue;
					}
					if (!mi.GetParameters().SequenceEqual(reflectiveInterfaceMethodInfo.GetParameters(), EqualityComparer<ParameterInfo>.Create((a, b) => a?.ParameterType == b?.ParameterType)))
					{
						_logger?.LogWarning("Skipping MethodInfo (name:{MethodName}) annotated with ApplyWarningsFilterDelegateAttribute it's parameters don't match the base method interface.", mi.Name);
						continue;
					}

					ApplyWarningsFilterDelegateAttribute? attributeInstance = mi.GetCustomAttribute<ApplyWarningsFilterDelegateAttribute>();
					if (attributeInstance == null)
					{
						_logger?.LogWarning("Skipping MethodInfo (name:{MethodName}) annotated with ApplyWarningsFilterDelegateAttribute as reflection has failed to obtain the attribute instance.", mi.Name);
						continue;
					}

					if (!_filterNameToBehaviour.TryAdd(attributeInstance.FilterName, new ApplyWarningsFilterWrapper(attributeInstance, mi)))
					{
						_logger?.LogWarning("Duplicate filter names ({DuplicateName}) encountered when building the ApplyWarningsFilterRegistry.", attributeInstance.FilterName);
					}
				}
			}
			else
			{
				_logger?.LogWarning("Base method interface was null. Ignoring all method reflective methods used for filters.");
			}
		}
	}
}