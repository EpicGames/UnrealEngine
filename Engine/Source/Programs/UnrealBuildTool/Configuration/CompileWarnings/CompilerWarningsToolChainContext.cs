// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using EpicGames.Core;

namespace UnrealBuildTool
{
	/// <summary>
	/// Context object used to pass data to <see cref="CppCompileWarnings"/> in apply warnings phase.
	/// </summary>
	/// <remarks>This class should be used as the common place to add new context as necessary for <see cref="ApplyWarningsAttribute.ApplyWarningsToArguments"/> extensibility.</remarks>
	internal class CompilerWarningsToolChainContext
	{
		/// <summary>
		/// Constructor for toolchain context.
		/// </summary>
		/// <param name="compileEnvironment">Compile environment for the tool chain.</param>
		/// <param name="buildSystemContext">Build context of the <see cref="CppCompileWarnings"/>.</param>
		/// <param name="toolChainType">The type of the toolchain.</param>
		/// <param name="toolChainVersion">The version of the toolchain.</param>
		/// <param name="analyzer">The static analyzer utilized within the current toolchain context.</param>
		public CompilerWarningsToolChainContext(CppCompileEnvironment compileEnvironment, BuildSystemContext buildSystemContext, Type toolChainType, VersionNumber? toolChainVersion, StaticAnalyzer analyzer = StaticAnalyzer.None)
		{
			_compileEnvironment = compileEnvironment;
			_buildSystemContext = buildSystemContext;
			_toolChainType = toolChainType;
			_toolChainVersion = toolChainVersion;
			_analyzer = analyzer;
		}

		internal CppCompileEnvironment _compileEnvironment;
		internal BuildSystemContext _buildSystemContext;
		internal Type _toolChainType;
		internal VersionNumber? _toolChainVersion;
		internal StaticAnalyzer _analyzer;
	}
}