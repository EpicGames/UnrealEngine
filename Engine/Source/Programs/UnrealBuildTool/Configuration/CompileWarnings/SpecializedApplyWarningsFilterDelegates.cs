// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;

namespace UnrealBuildTool
{
	/// <summary>
	/// Specialized filters with more complex logic that aren't typically reusable.
	/// </summary>
	internal static class SpecializedFilters
	{
		/// <summary>
		///  Specialized filter to enforce Android as an excluded platform, at a specific toolchain version, at a specific C++ standard.
		/// </summary>
		/// <param name="context">The context in which to evaluate the applicability of the filter.</param>
		/// <returns>True if the ApplyWarnings will apply, false otherwise.</returns>
		[ApplyWarningsFilterDelegate(nameof(AndroidNDKR26ToolChainVersionExclusion))]
		internal static bool AndroidNDKR26ToolChainVersionExclusion(CompilerWarningsToolChainContext context)
		{
			if (context._toolChainVersion == null)
			{
				return false;
			}

			bool versionMin = context._toolChainVersion >= new VersionNumber(17);
			bool bIsAndroidClang17 = context._toolChainVersion == new VersionNumber(17, 0, 2) && context._compileEnvironment.Platform == UnrealTargetPlatform.Android;

			return versionMin && !bIsAndroidClang17 && context._compileEnvironment.CppStandard < CppStandardVersion.Latest;
		}

		/// <summary>
		/// Specialized filter to enforce Android as an excluded platform, at a specific toolchain version.
		/// </summary>
		/// <param name="context">The context in which to evaluate the applicability of the filter.</param>
		/// <returns>True if the ApplyWarnings will apply, false otherwise.</returns>
		[ApplyWarningsFilterDelegate(nameof(AndroidNDKR28ToolChainVersionExclusion))]
		internal static bool AndroidNDKR28ToolChainVersionExclusion(CompilerWarningsToolChainContext context)
		{
			if (context._toolChainVersion == null)
			{
				return false;
			}

			bool versionMin = context._toolChainVersion >= new VersionNumber(19);
			bool isExcludedVersion = context._toolChainVersion == new VersionNumber(19, 0, 0);
			bool bIsAndroidClang19 = isExcludedVersion && context._compileEnvironment.Platform == UnrealTargetPlatform.Android;

			return versionMin && !bIsAndroidClang19;
		}
	}
}
