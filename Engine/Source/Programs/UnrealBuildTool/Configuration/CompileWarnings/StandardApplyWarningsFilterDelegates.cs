// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool
{
	/// <summary>
	/// Standard filters that can be reused, and often in a composable manner.
	/// </summary>
	internal static class StandardFilters
	{
		/// <summary>
		/// Filter to enforce deterministic compilation constraints.
		/// </summary>
		/// <param name="context">The context in which to evaluate the applicability of the filter.</param>
		/// <returns>True if the ApplyWarnings will apply, false otherwise.</returns>
		[ApplyWarningsFilterDelegate(nameof(DeterministcFlagSetFilter))]
		internal static bool DeterministcFlagSetFilter(CompilerWarningsToolChainContext context)
		{
			return context._compileEnvironment.bDeterministic;
		}

		/// <summary>
		/// Filter to profile guided optimization constraints.
		/// </summary>
		/// <param name="context">The context in which to evaluate the applicability of the filter.</param>
		/// <returns>True if the ApplyWarnings will apply, false otherwise.</returns>
		[ApplyWarningsFilterDelegate(nameof(PGOOptimizedFilter))]
		internal static bool PGOOptimizedFilter(CompilerWarningsToolChainContext context)
		{
			return context._compileEnvironment.bPGOOptimize;
		}

		/// <summary>
		/// Filter for MSVC compiler within the <see cref="VCToolChain"/>.
		/// </summary>
		/// <param name="context">The context in which to evaluate the applicability of the filter.</param>
		/// <returns>True if the ApplyWarnings will apply, false otherwise.</returns>
		[ApplyWarningsFilterDelegate(nameof(MSVCCompilerFilter))]
		internal static bool MSVCCompilerFilter(CompilerWarningsToolChainContext context)
		{
			ReadOnlyTargetRules? readOnlyTargetRules = context._buildSystemContext.GetReadOnlyTargetRules();

			return readOnlyTargetRules?.WindowsPlatform?.Compiler.IsMSVC() == true;
		}

		/// <summary>
		/// Filter for Intel compiler.
		/// </summary>
		/// <param name="context">The context in which to evaluate the applicability of the filter.</param>
		/// <returns>True if the ApplyWarnings will apply, false otherwise.</returns>
		[ApplyWarningsFilterDelegate(nameof(IntelCompilerFilter))]
		internal static bool IntelCompilerFilter(CompilerWarningsToolChainContext context)
		{
			ReadOnlyTargetRules? readOnlyTargetRules = context._buildSystemContext.GetReadOnlyTargetRules();

			return readOnlyTargetRules?.WindowsPlatform?.Compiler.IsIntel() == true;
		}

		/// <summary>
		/// Filter for Clang compiler within the <see cref="VCToolChain"/>.
		/// </summary>
		/// <param name="context">The context in which to evaluate the applicability of the filter.</param>
		/// <returns>True if the ApplyWarnings will apply, false otherwise.</returns>
		[ApplyWarningsFilterDelegate(nameof(VCClangCompilerFilter))]
		internal static bool VCClangCompilerFilter(CompilerWarningsToolChainContext context)
		{
			ReadOnlyTargetRules? readOnlyTargetRules = context._buildSystemContext.GetReadOnlyTargetRules();

			return readOnlyTargetRules?.WindowsPlatform?.Compiler.IsClang() == true;
		}

		/// <summary>
		/// Negation filter for !VCClang.
		/// </summary>
		/// <param name="context">The context in which to evaluate the applicability of the filter.</param>
		/// <returns>True if the ApplyWarnings will apply, false otherwise.</returns>
		[ApplyWarningsFilterDelegate(nameof(NonVCClangCompilerFilter))]
		internal static bool NonVCClangCompilerFilter(CompilerWarningsToolChainContext context)
		{
			return !VCClangCompilerFilter(context);
		}
	}
}
