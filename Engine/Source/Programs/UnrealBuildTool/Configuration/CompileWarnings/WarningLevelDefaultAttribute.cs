// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace UnrealBuildTool.Configuration.CompileWarnings
{
	/// <summary>
	/// Flag used to communicate which context the <see cref="VersionWarningLevelDefaultAttribute"/> should be applied.
	/// </summary>
	internal enum InitializationContext
	{
		/// <summary>
		/// Apply the <see cref="VersionWarningLevelDefaultAttribute"/> in all contexts.
		/// </summary>
		Any,
		/// <summary>
		/// Apply the <see cref="VersionWarningLevelDefaultAttribute"/> exclusively in the <see cref="CppCompileWarnings.ApplyTargetDefaults(CppCompileWarnings, Boolean)"/> context.
		/// </summary>
		Target,
		/// <summary>
		/// Apply the <see cref="VersionWarningLevelDefaultAttribute"/> exclusively in the <see cref="CppCompileWarnings.ApplyDefaults"/> context.
		/// </summary>
		Constructor
	}

	internal static class WarningLevelDefaultHelpers
	{
		public static IList<T>? CastWarningLevelDefaultAttirbuteToTarget<T>(IList<WarningLevelDefaultAttribute> uncastedAttributes, ILogger? logger)
		{
			List<T>? castedAttributes = null;
			try
			{
				castedAttributes = uncastedAttributes.Cast<T>().ToList();
			}
			catch (Exception ex)
			{
				logger?.LogError("Exception occurred {ExMessage}. Received a set of attributes that weren't of the correct warning attribute type ({Type})", ex.Message, nameof(T));
			}

			return castedAttributes;
		}
	}

	/// <summary>
	/// Abstract attribute used to specify the default <see cref="WarningLevel"/> to assign a member, and the interface,
	/// </summary>
	[AttributeUsage(AttributeTargets.Field | AttributeTargets.Property, AllowMultiple = true)]
	internal abstract class WarningLevelDefaultAttribute : Attribute
	{
		protected static readonly ILogger? Logger = Log.Logger;

		/// <summary>
		/// Constructor a WarningLevelDefaultAttribute.
		/// </summary>
		/// <param name="context">The context in which to apply this default.</param>
		public WarningLevelDefaultAttribute(InitializationContext context = InitializationContext.Any)
		{
			Context = context;
		}

		/// <summary>
		/// Gets the default <see cref="WarningLevel"/> to apply to the member.
		/// </summary>
		/// <param name="buildSystemContext">The build system context used to consider defaults and applicability.</param>
		/// <returns></returns>
		public abstract WarningLevel GetDefaultLevel(BuildSystemContext? buildSystemContext);

		/// <summary>
		/// The context in which this attribute should be applied.
		/// </summary>
		public InitializationContext Context { get; }
	}

	/// <summary>
	/// Basic attribute used to set the default values of a <see cref="WarningLevel"/> under the context of <see cref="CppCompileWarnings.ApplyDefaults"/> context and <see cref="CppCompileWarnings.ApplyTargetDefaults(CppCompileWarnings, Boolean)"/> context.
	/// </summary>
	/// <remarks>When combined with any other <see cref="WarningLevelDefaultAttribute"/>, it will act as the fallback in resolution if no other attribute sets a non-<see cref="WarningLevel.Default"/>.</remarks>
	[AttributeUsage(AttributeTargets.Field | AttributeTargets.Property, AllowMultiple = false)]
#pragma warning disable CA1813 // Avoid unsealed attributes
	internal class BasicWarningLevelDefaultAttribute : WarningLevelDefaultAttribute
#pragma warning restore CA1813 // Avoid unsealed attributes
	{
		/// <summary>
		/// Constructor a BasicWarningLevelDefaultAttribute.
		/// </summary>
		/// <param name="defaultLevel">The default warning level to apply for this bound.</param>
		/// <param name="context">The context in which to apply this default.</param>
#pragma warning disable CA1019 // Define accessors for attribute arguments
		public BasicWarningLevelDefaultAttribute(WarningLevel defaultLevel = WarningLevel.Default, InitializationContext context = InitializationContext.Any) : base(context)
#pragma warning restore CA1019 // Define accessors for attribute arguments
		{
			DefaultLevel = defaultLevel;
		}

		/// <summary>
		/// The default warning level.
		/// </summary>
		/// <param name="buildSystemContext">The build system context used to conditionally augment the <see cref="DefaultLevel"/> by.</param>
		public override WarningLevel GetDefaultLevel(BuildSystemContext? buildSystemContext = null)
		{
			return DefaultLevel;
		}

		/// <summary>
		/// Resolver for <see cref="BasicWarningLevelDefaultAttribute"/>.
		/// </summary>
		/// <param name="unsortedDefaultValueAttributes">The set of attributes of which to consider for resolution.</param>
		/// <param name="buildSystemContext">The build system context used to consider defaults and applicability.</param>
		/// <returns>The corresponding <see cref="WarningLevel"/> of which to apply to the property with the default value attributes.</returns>
		[WarningLevelResolverDelegate(nameof(BasicWarningLevelDefaultAttribute))]
		internal static WarningLevel ResolveWarningLevelDefault(IList<WarningLevelDefaultAttribute> unsortedDefaultValueAttributes, BuildSystemContext? buildSystemContext)
		{
			if (unsortedDefaultValueAttributes.Count != 1)
			{
				Logger?.LogError("Invalid attribute definition for {Name}.", nameof(WarningLevelDefaultAttribute));
				return WarningLevel.Default;
			}

			return unsortedDefaultValueAttributes[0].GetDefaultLevel(buildSystemContext);
		}

		protected readonly WarningLevel DefaultLevel;
	}

	#region -- Compiler Constrained Warning Level Defaults --
	/// <summary>
	/// Compiler constrained Attribute used to set the default values of a <see cref="WarningLevel"/> under the context of <see cref="CppCompileWarnings.ApplyDefaults"/> context and <see cref="CppCompileWarnings.ApplyTargetDefaults(CppCompileWarnings, Boolean)"/> context.
	/// </summary>
	[AttributeUsage(AttributeTargets.Field | AttributeTargets.Property, AllowMultiple = true)]
	internal abstract class CompilerWarningLevelDefaultAttribute : BasicWarningLevelDefaultAttribute
	{
		public WindowsCompiler Compiler { get; }

		/// <summary>
		/// Constructs a CompilerWarningLevelDefaultAttribute.
		/// </summary>
		/// <param name="compiler">The compiler which this attribute should be applied.</param>
		/// <param name="defaultLevel">The default warning level to apply for this bound.</param>
		/// <param name="context">The context in which to apply this default.</param>
#pragma warning disable CA1019 // Define accessors for attribute arguments
		public CompilerWarningLevelDefaultAttribute(WindowsCompiler compiler, WarningLevel defaultLevel = WarningLevel.Default, InitializationContext context = InitializationContext.Any) : base(defaultLevel, context)
#pragma warning restore CA1019 // Define accessors for attribute arguments
		{
			Compiler = compiler;
		}

		/// <summary>
		/// Resolves the set of <see cref="CompilerWarningLevelDefaultAttribute"/> to a <see cref="WarningLevel"/> provided the current <see cref="BuildSystemContext"/>.
		/// </summary>
		/// <param name="unsortedDefaultValueAttributes">The set of attributes of which to consider for resolution.</param>
		/// <param name="buildSystemContext">The build system context used to consider defaults and applicability.</param>
		/// <returns>The corresponding <see cref="WarningLevel"/> of which to apply to the property with the default value attributes.</returns>
		[WarningLevelResolverDelegate(nameof(CompilerWarningLevelDefaultAttribute))]
#pragma warning disable IDE0060 // Remove unused parameter
		internal static WarningLevel ResolveVersionWarningLevelDefault(IList<WarningLevelDefaultAttribute> unsortedDefaultValueAttributes, BuildSystemContext? buildSystemContext)
#pragma warning restore IDE0060 // Remove unused parameter
		{
			IList<CompilerWarningLevelDefaultAttribute>? castedAttributes = WarningLevelDefaultHelpers.CastWarningLevelDefaultAttirbuteToTarget<CompilerWarningLevelDefaultAttribute>(unsortedDefaultValueAttributes, Logger);

			if (castedAttributes == null)
			{
				Logger?.LogWarning("Unable to discern default warning level through resolution. Defaulting to WarningLevel.Default.");
				return WarningLevel.Default;
			}

			int nonDefaultCount = castedAttributes.Select(x => x.GetDefaultLevel(buildSystemContext) != WarningLevel.Default).Count();

			if (nonDefaultCount > 1)
			{
				Logger?.LogWarning("Unable to disambiguate the warning level through resolution. Too many CompilerWarningLevelDefaultAttribute returned a resolved value. Defaulting to WarningLevel.Default.");
				return WarningLevel.Default;
			}

			if (nonDefaultCount == 1)
			{
				return castedAttributes.Where(x => x.GetDefaultLevel(buildSystemContext) != WarningLevel.Default).Select(x => x.GetDefaultLevel(buildSystemContext)).First();
			}

			return WarningLevel.Default;
		}
	}

	/// <summary>
	/// Version constrained attribute used to set the default values of a <see cref="WarningLevel"/> under the context of <see cref="CppCompileWarnings.ApplyDefaults"/> context and <see cref="CppCompileWarnings.ApplyTargetDefaults(CppCompileWarnings, Boolean)"/> context.
	/// </summary>
	[AttributeUsage(AttributeTargets.Field | AttributeTargets.Property, AllowMultiple = false)]
	internal sealed class IntelCompilerWarningLevelDefaultAttribute : CompilerWarningLevelDefaultAttribute
	{
		/// <summary>
		/// Constructs a IntelCompilerWarningLevelDefaultAttribute.
		/// </summary>
		/// <param name="defaultLevel">The default warning level to apply for this bound.</param>
		/// <param name="context">The context in which to apply this default.</param>
#pragma warning disable CA1019 // Define accessors for attribute arguments
		public IntelCompilerWarningLevelDefaultAttribute(WarningLevel defaultLevel = WarningLevel.Default, InitializationContext context = InitializationContext.Any) : base(WindowsCompiler.Intel, defaultLevel, context) { }
#pragma warning restore CA1019 // Define accessors for attribute arguments

		public override WarningLevel GetDefaultLevel(BuildSystemContext? buildSystemContext = null)
		{
			if (buildSystemContext == null)
			{
				return WarningLevel.Default;
			}

			ReadOnlyTargetRules? readOnlyTargetRules = buildSystemContext.GetReadOnlyTargetRules();

			if (readOnlyTargetRules?.WindowsPlatform?.Compiler.IsIntel() == true)
			{
				return DefaultLevel;
			}

			return WarningLevel.Default;
		}
	}

	/// <summary>
	/// Version constrained attribute used to set the default values of a <see cref="WarningLevel"/> under the context of <see cref="CppCompileWarnings.ApplyDefaults"/> context and <see cref="CppCompileWarnings.ApplyTargetDefaults(CppCompileWarnings, Boolean)"/> context.
	/// </summary>
	[AttributeUsage(AttributeTargets.Field | AttributeTargets.Property, AllowMultiple = false)]
	internal sealed class MSVCCompilerWarningLevelDefaultAttribute : CompilerWarningLevelDefaultAttribute
	{
		/// <summary>
		/// Constructs a MSVCCompilerWarningLevelDefaultAttribute.
		/// </summary>
		/// <param name="defaultLevel">The default warning level to apply for this bound.</param>
		/// <param name="context">The context in which to apply this default.</param>
#pragma warning disable CA1019 // Define accessors for attribute arguments
		public MSVCCompilerWarningLevelDefaultAttribute(WarningLevel defaultLevel = WarningLevel.Default, InitializationContext context = InitializationContext.Any) : base(WindowsCompiler.VisualStudio2022, defaultLevel, context) { }
#pragma warning restore CA1019 // Define accessors for attribute arguments

		public override WarningLevel GetDefaultLevel(BuildSystemContext? buildSystemContext = null)
		{
			if (buildSystemContext == null)
			{
				return WarningLevel.Default;
			}

			ReadOnlyTargetRules? readOnlyTargetRules = buildSystemContext.GetReadOnlyTargetRules();

			if (readOnlyTargetRules?.WindowsPlatform?.Compiler.IsMSVC() == true)
			{
				return DefaultLevel;
			}

			return WarningLevel.Default;
		}
	}

	/// <summary>
	/// Version constrained attribute used to set the default values of a <see cref="WarningLevel"/> under the context of <see cref="CppCompileWarnings.ApplyDefaults"/> context and <see cref="CppCompileWarnings.ApplyTargetDefaults(CppCompileWarnings, Boolean)"/> context.
	/// </summary>
	/// <remarks>Although <see cref="WindowsCompiler.Intel"/> is considered clang, this specificaly removes that.</remarks>
	[AttributeUsage(AttributeTargets.Field | AttributeTargets.Property, AllowMultiple = false)]
	internal sealed class ClangNonIntelSpecializedCompilerWarningLevelDefaultAttribute : CompilerWarningLevelDefaultAttribute
	{
		/// <summary>
		/// Constructs a ClangNonIntelSpecializedCompilerWarningLevelDefaultAttribute.
		/// </summary>
		/// <param name="defaultLevel">The default warning level to apply for this bound.</param>
		/// <param name="context">The context in which to apply this default.</param>
#pragma warning disable CA1019 // Define accessors for attribute arguments
		public ClangNonIntelSpecializedCompilerWarningLevelDefaultAttribute(WarningLevel defaultLevel = WarningLevel.Default, InitializationContext context = InitializationContext.Any) : base(WindowsCompiler.Clang, defaultLevel, context) { }
#pragma warning restore CA1019 // Define accessors for attribute arguments

		public override WarningLevel GetDefaultLevel(BuildSystemContext? buildSystemContext = null)
		{
			if (buildSystemContext == null)
			{
				return WarningLevel.Default;
			}

			ReadOnlyTargetRules? readOnlyTargetRules = buildSystemContext.GetReadOnlyTargetRules();

			if (readOnlyTargetRules?.WindowsPlatform?.Compiler.IsClang() == true && !readOnlyTargetRules.WindowsPlatform.Compiler.IsIntel())
			{
				return DefaultLevel;
			}

			return WarningLevel.Default;
		}
	}

	#endregion -- Compiler Constrained Warning Level Defaults --

	/// <summary>
	/// Version constrained attribute used to set the default values of a <see cref="WarningLevel"/> under the context of <see cref="CppCompileWarnings.ApplyDefaults"/> context and <see cref="CppCompileWarnings.ApplyTargetDefaults(CppCompileWarnings, Boolean)"/> context.
	/// </summary>
	[AttributeUsage(AttributeTargets.Field | AttributeTargets.Property, AllowMultiple = true)]
	internal sealed class VersionWarningLevelDefaultAttribute : BasicWarningLevelDefaultAttribute
	{
		/// <summary>
		/// The lower bound (inclusive) <see cref="BuildSettingsVersion"/> of which to apply this <see cref="BasicWarningLevelDefaultAttribute.DefaultLevel"/> to.
		/// </summary>
		public BuildSettingsVersion MinVersion { get; }

		/// <summary>
		/// The upper bound (inclusive) <see cref="BuildSettingsVersion"/> of which to apply this <see cref="BasicWarningLevelDefaultAttribute.DefaultLevel"/> to.
		/// </summary>
		public BuildSettingsVersion MaxVersion { get; }

		/// <summary>
		/// Constructs a VersionWarningLevelDefaultAttribute.
		/// </summary>
		/// <param name="minVersion">Minimum version of the bound to apply this setting to. Inclusive.</param>
		/// <param name="maxVersion">Maximum version of the bound to apply this setting to. Inclusive.</param>
		/// <param name="defaultLevel">The default warning level to apply for this bound.</param>
		/// <param name="context">The context in which to apply this default.</param>
#pragma warning disable CA1019
		public VersionWarningLevelDefaultAttribute(WarningLevel defaultLevel = WarningLevel.Default, BuildSettingsVersion minVersion = BuildSettingsVersion.V1, BuildSettingsVersion maxVersion = BuildSettingsVersion.Latest, InitializationContext context = InitializationContext.Any) : base(defaultLevel, context)
#pragma warning restore CA1019
		{
			MinVersion = minVersion;
			MaxVersion = maxVersion;
		}

		/// <summary>
		/// Resolves the set of <see cref="VersionWarningLevelDefaultAttribute"/> to a <see cref="WarningLevel"/> provided the current <see cref="BuildSystemContext"/>.
		/// </summary>
		/// <param name="unsortedDefaultValueAttributes">The set of attributes of which to consider for resolution.</param>
		/// <param name="buildSystemContext">The build system context used to consider defaults and applicability.</param>
		/// <returns>The corresponding <see cref="WarningLevel"/> of which to apply to the property with the default value attributes.</returns>
		/// <remarks>Will invoke <see cref="EnsureWarningLevelDefaultBounds(IList{VersionWarningLevelDefaultAttribute})"/> on the input <see cref="VersionWarningLevelDefaultAttribute"/>.</remarks>
		[WarningLevelResolverDelegate(nameof(VersionWarningLevelDefaultAttribute))]
		internal static WarningLevel ResolveVersionWarningLevelDefault(IList<WarningLevelDefaultAttribute> unsortedDefaultValueAttributes, BuildSystemContext? buildSystemContext)
		{
			IList<VersionWarningLevelDefaultAttribute>? castedAttributes = WarningLevelDefaultHelpers.CastWarningLevelDefaultAttirbuteToTarget<VersionWarningLevelDefaultAttribute>(unsortedDefaultValueAttributes, Logger);

			if (castedAttributes == null)
			{
				Logger?.LogWarning("Unable to discern default warning level through resolution. Defaulting to WarningLevel.Default.");
				return WarningLevel.Default;
			}

			BuildSettingsVersion activeVersion = buildSystemContext != null ? buildSystemContext._buildContext.GetBuildSettings() : BuildSettingsVersion.V1;
			List<VersionWarningLevelDefaultAttribute> sortedAndMerged = EnsureWarningLevelDefaultBounds(castedAttributes);

			WarningLevel returnWarningLevel = WarningLevel.Default;

			foreach (VersionWarningLevelDefaultAttribute attr in sortedAndMerged)
			{
				// If we find our appropriate range, we early out.
				if (attr.MinVersion <= activeVersion && activeVersion <= attr.MaxVersion)
				{
					returnWarningLevel = attr.GetDefaultLevel(buildSystemContext);
					break;
				}
			}

			return returnWarningLevel;
		}

		/// <summary>
		/// Ensures the attributes collection represents a contiguous, non-overlapping range.
		/// </summary>
		/// <param name="attributes">The list of attributes to verify.</param>
		/// <returns>A list of attributes that has no overlaps, and is contiguous.</returns>
		internal static List<VersionWarningLevelDefaultAttribute> EnsureWarningLevelDefaultBounds(IList<VersionWarningLevelDefaultAttribute> attributes)
		{
			List<VersionWarningLevelDefaultAttribute> sorted = attributes.OrderBy(a => a.MinVersion).ToList();
			List<VersionWarningLevelDefaultAttribute> merged = new List<VersionWarningLevelDefaultAttribute>(sorted.Count);

			for (int i = 0; i < sorted.Count - 1; i++)
			{
				VersionWarningLevelDefaultAttribute current = sorted[i];
				VersionWarningLevelDefaultAttribute next = sorted[i + 1];

				if (current.MaxVersion < next.MinVersion - 1)
				{
					Logger?.LogWarning("Malformed VersionWarningLevelDefaultAttribute collection; taking corrective action to address gap in range (CurrentMax: {CurrentMax} NextMin: {NextMin}-1).", next.MinVersion, current.MaxVersion);

					// Extend current range to cover the gap up to the next MinVersion,using the old build settings version standard.
					current = new VersionWarningLevelDefaultAttribute(
						current.DefaultLevel,
						current.MinVersion,
						next.MinVersion - 1
					);
				}
				else if (next.MinVersion <= current.MaxVersion)
				{
					Logger?.LogWarning("Malformed VersionWarningLevelDefaultAttribute collection; taking corrective action to address overlap in range (NextMin: {NextMin} <= CurrentMax: {CurrentMax}).", next.MinVersion, current.MaxVersion);

					// Reduce next range to be more constrained, leaving the larger current range at the old build settings version standard.
					next = new VersionWarningLevelDefaultAttribute(
						next.DefaultLevel,
						current.MaxVersion + 1,
						next.MaxVersion
					);

					// Flatten the old value with the updated one.
					sorted[i + 1] = next;
				}

				merged.Add(current);
			}

			merged.Add(sorted.Last());

			return merged;
		}
	}
}