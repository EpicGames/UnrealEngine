// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Reflection;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildTool.Configuration.CompileWarnings;

namespace UnrealBuildTool
{
	/// <summary>
	/// A wrapper for the methods annotated by the <see cref="WarningLevelResolverDelegateAttribute"/> to support <see cref="IWarningLevelResolver"/>.
	/// </summary>
	/// <remarks>Primarily invoked by <see cref="CppCompileWarnings.CoalesceDefaultWarningFromMemberInfo"/>.</remarks>
	internal struct WarningLevelResolverWrapper : IWarningLevelResolver
	{
		public string FilterName => _applyWarningsFilterDelegateAttribute.ResolverName;

		private readonly ILogger? _logger = Log.Logger;
		private readonly WarningLevelResolverDelegateAttribute _applyWarningsFilterDelegateAttribute;
		private readonly MethodInfo _method;

		/// <summary>
		/// Constructs a ApplyWarningsFilterWrapper.
		/// </summary>
		/// <param name="filterAttribute">Instance of the filter attribute that is being wrapped.</param>
		/// <param name="method">The method that will be used for reflective invocation in <see cref="ResolveWarningLevelDefault"/>.</param>
		public WarningLevelResolverWrapper(WarningLevelResolverDelegateAttribute filterAttribute, MethodInfo method)
		{
			_applyWarningsFilterDelegateAttribute = filterAttribute;
			_method = method;
		}

		/// <inheritdoc/>
		public WarningLevel ResolveWarningLevelDefault(IList<WarningLevelDefaultAttribute> unsortedDefaultValueAttributes, BuildSystemContext? buildSystemContext)
		{
			WarningLevel defaultWarningLevel = WarningLevel.Default;

			{
				try
				{
					object? result = _method?.Invoke(null, [unsortedDefaultValueAttributes, buildSystemContext]);
					defaultWarningLevel = result != null ? (WarningLevel)result : WarningLevel.Default;
				}
				catch (Exception e)
				{
					_logger?.LogWarning("Have failed to reflectively invoke (name:{MethodName}) in the CanApply context. Exception: {Exception}.", _method?.Name, e.Message);
				}

				return defaultWarningLevel;
			}
		}
	}

	/// <summary>
	/// Attribute used to denote methods that should participate in <see cref="IWarningLevelResolver"/> usage.
	/// </summary>
	/// <remarks>Primarily invoked by <see cref="CppCompileWarnings.CoalesceDefaultWarningFromMemberInfo"/>.</remarks>
	[AttributeUsage(AttributeTargets.Method, AllowMultiple = false)]
	internal sealed class WarningLevelResolverDelegateAttribute : Attribute
	{
		private readonly string _resolverName;
		public string ResolverName => _resolverName;

		public WarningLevelResolverDelegateAttribute(string resolverName)
		{
			_resolverName = resolverName;
		}
	}
}
