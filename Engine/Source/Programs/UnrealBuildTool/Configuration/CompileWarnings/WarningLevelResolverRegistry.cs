// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Reflection;
using EpicGames.Core;
using Microsoft.CodeAnalysis;
using Microsoft.Extensions.Logging;
using UnrealBuildTool.Configuration.CompileWarnings;

namespace UnrealBuildTool
{
	/// <summary>
	/// Interface that participates in resolving a set of <see cref="WarningLevelDefaultAttribute"/> against a member.
	/// </summary>
	internal interface IWarningLevelResolver
	{
		/// <summary>
		/// Resolves a <see cref="WarningLevel"/> given a set of <see cref="WarningLevelDefaultAttribute"/>.
		/// </summary>
		/// <param name="unsortedDefaultValueAttributes">The set of attributes to consider in resolution.</param>
		/// <param name="buildSystemContext">The build system context to evaluate these under.</param>
		/// <returns>The resolved warning level.</returns>
		WarningLevel ResolveWarningLevelDefault(IList<WarningLevelDefaultAttribute> unsortedDefaultValueAttributes, BuildSystemContext? buildSystemContext);
	}

	/// <summary>
	/// WarningLevelResolverRegistry is the container for all of the resolvers applicable within the <see cref="CppCompileWarnings.ApplyTargetDefaults"/> and <see cref="CppCompileWarnings.ApplyDefaults"/> contexts.
	/// </summary>
	/// <remarks>The primary entry point for extensibility is <see cref="WarningLevelResolverDelegateAttribute"/>.</remarks>
	internal class WarningLevelResolverRegistry
	{
		private readonly ILogger? _logger = Log.Logger;
		private static readonly Lazy<WarningLevelResolverRegistry> s_resolverRegistry = new(() => new());
		private readonly Dictionary<string, IWarningLevelResolver> _attributeNameToBehaviour = [];

		/// <summary>
		/// The resolver name to resolver representation of the registry.
		/// </summary>
		internal static IReadOnlyDictionary<string, IWarningLevelResolver> Resolvers => s_resolverRegistry.Value._attributeNameToBehaviour;

		/// <summary>
		/// Requests a resolver by name.
		/// </summary>
		/// <param name="resolverName">The name of the resolver to obtain.</param>
		/// <returns>The IWarningLevelResolver if it has been registered, null if it was not found.</returns>
		internal static IWarningLevelResolver? RequestResolver(string resolverName)
		{
			IWarningLevelResolver? returnResolver;
			Resolvers.TryGetValue(resolverName, out returnResolver);

			return returnResolver;
		}

		private WarningLevelResolverRegistry()
		{
			// Process all of the static methods in the assembly that are annotated wtih WarningLevelResolverDelegateAttribute, and evaluate given they match
			// the execution delegate, which is effectively IWarningLevelResolver.ResolveWarningLevelDefault
			MethodInfo? reflectiveInterfaceMethodInfo = typeof(IWarningLevelResolver).GetMethod(nameof(IWarningLevelResolver.ResolveWarningLevelDefault));

			if (reflectiveInterfaceMethodInfo != null)
			{
				Assembly assembly = Assembly.GetExecutingAssembly();
				Type[] allTypes = assembly.GetTypes();
				List<MethodInfo> delegatesForConsideration = new List<MethodInfo>();

				foreach (Type type in allTypes)
				{
					IEnumerable<MethodInfo> methodsWithAttribute = type
						.GetMethods(BindingFlags.Static | BindingFlags.Public | BindingFlags.NonPublic)
						.Where(m => m.GetCustomAttributes(typeof(WarningLevelResolverDelegateAttribute), false).Length > 0);

					delegatesForConsideration.AddRange(methodsWithAttribute);
				}

				foreach (MethodInfo mi in delegatesForConsideration)
				{
					if (mi.ReturnType != reflectiveInterfaceMethodInfo.ReturnType)
					{
						_logger?.LogWarning("Skipping MethodInfo (name:{MethodName}) annotated with WarningLevelResolverDelegateAttribute as it doesn't return {ReturnType}.", mi.Name, reflectiveInterfaceMethodInfo.ReturnType.Name);
						continue;
					}

					if (mi.GetParameters().Length != reflectiveInterfaceMethodInfo.GetParameters().Length)
					{
						_logger?.LogWarning("Skipping MethodInfo (name:{MethodName}) annotated with WarningLevelResolverDelegateAttribute as it doesn't contain the same number of parameters as the base method interface ({Count})", mi.Name, reflectiveInterfaceMethodInfo.GetParameters().Length);
						continue;
					}
					if (!mi.GetParameters().SequenceEqual(reflectiveInterfaceMethodInfo.GetParameters(), EqualityComparer<ParameterInfo>.Create((a, b) => a?.ParameterType == b?.ParameterType)))
					{
						_logger?.LogWarning("Skipping MethodInfo (name:{MethodName}) annotated with WarningLevelResolverDelegateAttribute it's parameters don't match the base method interface.", mi.Name);
						continue;
					}

					WarningLevelResolverDelegateAttribute? attributeInstance = mi.GetCustomAttribute<WarningLevelResolverDelegateAttribute>();
					if (attributeInstance == null)
					{
						_logger?.LogWarning("Skipping MethodInfo (name:{MethodName}) annotated with WarningLevelResolverDelegateAttribute as reflection has failed to obtain the attribute instance.", mi.Name);
						continue;
					}

					if (!_attributeNameToBehaviour.TryAdd(attributeInstance.ResolverName, new WarningLevelResolverWrapper(attributeInstance, mi)))
					{
						_logger?.LogWarning("Duplicate resolver names ({DuplicateName}) encountered when building the WarningLevelResolverRegistry.", attributeInstance.ResolverName);
					}
				}
			}
			else
			{
				_logger?.LogWarning("Base method interface was null. Ignoring all method reflective methods used for resolvers.");
			}
		}
	}
}