// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics;
using System.Linq;
using EpicGames.Core;

namespace UnrealBuildTool
{
	/// <summary>
	/// 
	/// </summary>
	public enum LocalizationTargetDescriptorLoadingPolicy
	{
		/// <summary>
		/// 
		/// </summary>
		Never,

		/// <summary>
		/// 
		/// </summary>
		Always,

		/// <summary>
		/// 
		/// </summary>
		Editor,

		/// <summary>
		/// 
		/// </summary>
		Game,

		/// <summary>
		/// 
		/// </summary>
		PropertyNames,

		/// <summary>
		/// 
		/// </summary>
		ToolTips,
	};

	/// <summary>
	/// How this localization target should be generated during the localization gather pipeline
	/// </summary>
	public enum LocalizationConfigGenerationPolicy
	{
		/// <summary>
		/// This localization target should never have localization config files associated with it during the localization gather pipeline.
		/// </summary>
		Never,
		/// <summary>
		/// This localization target should only use user generated localization config files during the localization gather pipeline.
		/// </summary>
		User,
		/// <summary>
		/// Default auto-generated localization config files will be used to generate the localization target and localization content files during the localization gather pipeline
		/// </summary>
		Auto,
	}

	/// <summary>
	/// Description of a localization target.
	/// </summary>
	[DebuggerDisplay("Name={Name}")]
	public class LocalizationTargetDescriptor
	{
		/// <summary>
		/// Name of this target
		/// </summary>
		public readonly string Name;

		/// <summary>
		/// When should the localization data associated with a target should be loaded?
		/// </summary>
		public LocalizationTargetDescriptorLoadingPolicy LoadingPolicy;

		/// <summary>
		/// How should this localization target's localization config files be generated during a localization gather.
		/// </summary>
		public LocalizationConfigGenerationPolicy ConfigGenerationPolicy;
		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="InName">Name of the target</param>
		/// <param name="InLoadingPolicy">When should the localization data associated with a target should be loaded?</param>
		public LocalizationTargetDescriptor(string InName, LocalizationTargetDescriptorLoadingPolicy InLoadingPolicy)
		{
			Name = InName;
			LoadingPolicy = InLoadingPolicy;
			// Older plugins and localization target descriptors won't have the generation policy. We default it to Never.
			ConfigGenerationPolicy = LocalizationConfigGenerationPolicy.Never;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="InName">The name of the localization target.</param>
		/// <param name="InLoadingPolicy"> When the localization data associated with this localization target should be loaded.</param>
		/// <param name="InGenerationPolicy">How the localization config files should be generated during a localization gather to create the localization data files.</param>
		public LocalizationTargetDescriptor(string InName, LocalizationTargetDescriptorLoadingPolicy InLoadingPolicy, LocalizationConfigGenerationPolicy InGenerationPolicy)
		{
			Name = InName;
			LoadingPolicy = InLoadingPolicy;
			ConfigGenerationPolicy = InGenerationPolicy;
		}

		/// <summary>
		/// Constructs a LocalizationTargetDescriptor from a Json object
		/// </summary>
		/// <param name="InObject"></param>
		/// <returns>The new localization target descriptor</returns>
		public static LocalizationTargetDescriptor FromJsonObject(JsonObject InObject)
		{
			LocalizationTargetDescriptor descriptor = new LocalizationTargetDescriptor(InObject.GetStringField("Name"), InObject.GetEnumField<LocalizationTargetDescriptorLoadingPolicy>("LoadingPolicy"));
			LocalizationConfigGenerationPolicy policy;
			if (InObject.TryGetEnumField<LocalizationConfigGenerationPolicy>("ConfigGenerationPolicy", out policy))
			{
				descriptor.ConfigGenerationPolicy = policy;
			}
			return descriptor;
		}

		/// <summary>
		/// Write this target to a JsonWriter
		/// </summary>
		/// <param name="Writer">Writer to output to</param>
		void Write(JsonWriter Writer)
		{
			Writer.WriteObjectStart();
			Writer.WriteValue("Name", Name);
			Writer.WriteValue("LoadingPolicy", LoadingPolicy.ToString());
			Writer.WriteValue("ConfigGenerationPolicy", ConfigGenerationPolicy.ToString());
			Writer.WriteObjectEnd();
		}

		JsonObject ToJsonObject()
		{
			JsonObject localizationTargetObject = new JsonObject();
			localizationTargetObject.AddOrSetFieldValue("Name", Name);
			localizationTargetObject.AddOrSetFieldValue("LoadingPolicy", LoadingPolicy.ToString());
			localizationTargetObject.AddOrSetFieldValue("ConfigGenerationPolicy", ConfigGenerationPolicy.ToString());

			return localizationTargetObject;
		}

		/// <summary>
		/// Write an array of target descriptors
		/// </summary>
		/// <param name="Writer">The Json writer to output to</param>
		/// <param name="Name">Name of the array</param>
		/// <param name="Targets">Array of targets</param>
		public static void WriteArray(JsonWriter Writer, string Name, LocalizationTargetDescriptor[]? Targets)
		{
			if (Targets != null && Targets.Length > 0)
			{
				Writer.WriteArrayStart(Name);
				foreach (LocalizationTargetDescriptor Target in Targets)
				{
					Target.Write(Writer);
				}
				Writer.WriteArrayEnd();
			}
		}

		/// <summary>
		/// Updates a JsonObject with an array of localization target descriptors.
		/// </summary>
		/// <param name="InObject">The Json object to update.</param>
		/// <param name="Name">Name of the array</param>
		/// <param name="Targets">Array of targets</param>
		public static void UpdateJson(JsonObject InObject, string Name, LocalizationTargetDescriptor[]? Targets)
		{
			if (Targets != null && Targets.Length > 0)
			{
				JsonObject[] JsonObjects = Targets.Select(X => X.ToJsonObject()).ToArray();
				InObject.AddOrSetFieldValue(Name, JsonObjects);
			}
			else
			{
				// The loaded plugin descriptor could have an existing localization target descriptor 
				// If we remove it in memory, we should try and also remove it from the cached json in this case 
				InObject.RemoveField(Name);
			}
		}
	}
}
