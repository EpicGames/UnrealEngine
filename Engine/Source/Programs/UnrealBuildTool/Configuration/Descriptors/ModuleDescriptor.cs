// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.Diagnostics.CodeAnalysis;
using System.Linq;
using EpicGames.Core;
using JsonExtensions;

namespace UnrealBuildTool
{
	/// <summary>
	/// The type of host that can load a module
	/// </summary>
	public enum ModuleHostType
	{
		/// <summary>
		/// 
		/// </summary>
		Default,

		/// <summary>
		/// Any target using the UE runtime
		/// </summary>
		Runtime,

		/// <summary>
		/// Any target except for commandlet
		/// </summary>
		RuntimeNoCommandlet,

		/// <summary>
		/// Any target or program
		/// </summary>
		RuntimeAndProgram,

		/// <summary>
		/// Loaded only in cooked builds
		/// </summary>
		CookedOnly,

		/// <summary>
		/// Loaded only in uncooked builds
		/// </summary>
		UncookedOnly,

		/// <summary>
		/// Loaded only when the engine has support for developer tools enabled
		/// </summary>
		Developer,

		/// <summary>
		/// Loads on any targets where bBuildDeveloperTools is enabled
		/// </summary>
		DeveloperTool,

		/// <summary>
		/// Loaded only by the editor
		/// </summary>
		Editor,

		/// <summary>
		/// Loaded only by the editor, except when running commandlets
		/// </summary>
		EditorNoCommandlet,

		/// <summary>
		/// Loaded by the editor or program targets
		/// </summary>
		EditorAndProgram,

		/// <summary>
		/// Loaded only by programs
		/// </summary>
		Program,

		/// <summary>
		/// Loaded only by servers
		/// </summary>
		ServerOnly,

		/// <summary>
		/// Loaded only by clients, and commandlets, and editor....
		/// </summary>
		ClientOnly,

		/// <summary>
		/// Loaded only by clients and editor (editor can run PIE which is kinda a commandlet)
		/// </summary>
		ClientOnlyNoCommandlet,

		/// <summary>
		/// External module, should never be loaded automatically only referenced
		/// </summary>
		External,
	}

	/// <summary>
	/// Indicates when the engine should attempt to load this module
	/// </summary>
	public enum ModuleLoadingPhase
	{
		/// <summary>
		/// Loaded at the default loading point during startup (during engine init, after game modules are loaded.)
		/// </summary>
		Default,

		/// <summary>
		/// Right after the default phase
		/// </summary>
		PostDefault,

		/// <summary>
		/// Right before the default phase
		/// </summary>
		PreDefault,

		/// <summary>
		/// Loaded as soon as plugins can possibly be loaded (need GConfig)
		/// </summary>
		EarliestPossible,

		/// <summary>
		/// Loaded before the engine is fully initialized, immediately after the config system has been initialized.  Necessary only for very low-level hooks
		/// </summary>
		PostConfigInit,

		/// <summary>
		/// The first screen to be rendered after system splash screen
		/// </summary>
		PostSplashScreen,

		/// <summary>
		/// After PostConfigInit and before coreUobject initialized. used for early boot loading screens before the uobjects are initialized
		/// </summary>
		PreEarlyLoadingScreen,

		/// <summary>
		/// Loaded before the engine is fully initialized for modules that need to hook into the loading screen before it triggers
		/// </summary>
		PreLoadingScreen,

		/// <summary>
		/// After the engine has been initialized
		/// </summary>
		PostEngineInit,

		/// <summary>
		/// Do not automatically load this module
		/// </summary>
		None,
	}

	/// <summary>
	/// Class containing information about a code module
	/// </summary>
	[DebuggerDisplay("Name={Name}")]
	public class ModuleDescriptor
	{
		/// <summary>
		/// Name of this module
		/// </summary>
		public readonly string Name;

		/// <summary>
		/// Usage type of module
		/// </summary>
		public ModuleHostType Type;

		/// <summary>
		/// When should the module be loaded during the startup sequence?  This is sort of an advanced setting.
		/// </summary>
		public ModuleLoadingPhase LoadingPhase = ModuleLoadingPhase.Default;

		/// <summary>
		/// List of allowed platforms
		/// </summary>
		public List<UnrealTargetPlatform>? PlatformAllowList;

		/// <summary>
		/// List of disallowed platforms
		/// </summary>
		public List<UnrealTargetPlatform>? PlatformDenyList;
		
		/// <summary>
		/// Collection of allowed architectures per platform
		/// </summary>
		public Dictionary<UnrealTargetPlatform,List<UnrealArch>>? PlatformArchitectureAllowList;

		/// <summary>
		/// Collection of disallowed architectures per platform
		/// </summary>
		public Dictionary<UnrealTargetPlatform,List<UnrealArch>>? PlatformArchitectureDenyList;

		/// <summary>
		/// List of allowed targets
		/// </summary>
		public TargetType[]? TargetAllowList;

		/// <summary>
		/// List of disallowed targets
		/// </summary>
		public TargetType[]? TargetDenyList;

		/// <summary>
		/// List of allowed target configurations
		/// </summary>
		public UnrealTargetConfiguration[]? TargetConfigurationAllowList;

		/// <summary>
		/// List of disallowed target configurations
		/// </summary>
		public UnrealTargetConfiguration[]? TargetConfigurationDenyList;

		/// <summary>
		/// List of allowed programs
		/// </summary>
		public string[]? ProgramAllowList;

		/// <summary>
		/// List of disallowed programs
		/// </summary>
		public string[]? ProgramDenyList;
		
		/// <summary>
		/// List of allowed game targets
		/// </summary>
		public string[]? GameTargetAllowList;

		/// <summary>
		/// List of disallowed game targets
		/// </summary>
		public string[]? GameTargetDenyList;

		/// <summary>
		/// List of additional dependencies for building this module.
		/// </summary>
		public string[]? AdditionalDependencies;

		/// <summary>
		/// When true, an empty PlatformAllowList is interpreted as 'no platforms' with the expectation that explicit platforms will be added in plugin extensions */
		/// </summary>
		public bool bHasExplicitPlatforms;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="InName">Name of the module</param>
		/// <param name="InType">Type of target that can host this module</param>
		public ModuleDescriptor(string InName, ModuleHostType InType)
		{
			Name = InName;
			Type = InType;
		}

		/// <summary>
		/// Constructs a ModuleDescriptor from a Json object
		/// </summary>
		/// <param name="InObject"></param>
		/// <param name="JsonFilePath"></param>
		/// <returns>The new module descriptor</returns>
		public static ModuleDescriptor FromJsonObject(JsonObject InObject, FileReference JsonFilePath)
		{
			ModuleDescriptor Module = new ModuleDescriptor(InObject.GetStringField("Name"), InObject.GetEnumField<ModuleHostType>("Type"));

			ModuleLoadingPhase LoadingPhase;
			if (InObject.TryGetEnumField<ModuleLoadingPhase>("LoadingPhase", out LoadingPhase))
			{
				Module.LoadingPhase = LoadingPhase;
			}

			try
			{
				string[]? PlatformAllowList;
				// it's important we default to null, and don't have an empty allow list by default, because that will indicate that no
				// platforms should be compiled (see IsCompiledInConfiguration(), it only checks for null, not length)
				Module.PlatformAllowList = null;
				if (InObject.TryGetStringArrayFieldWithDeprecatedFallback("PlatformAllowList", "WhitelistPlatforms", out PlatformAllowList))
				{
					Module.PlatformAllowList = new List<UnrealTargetPlatform>();
					foreach (string TargetPlatformName in PlatformAllowList)
					{
						UnrealTargetPlatform Platform;
						if (UnrealTargetPlatform.TryParse(TargetPlatformName, out Platform))
						{
							Module.PlatformAllowList.Add(Platform);
						}
						else if ( !PluginDescriptor.IsAllowableMissingPlatform(TargetPlatformName, JsonFilePath.Directory) )
						{
							Log.TraceWarningTask(JsonFilePath, $"Unknown platform {TargetPlatformName} while parsing allow list for module descriptor {Module.Name}");
						}
					}
				}

				string[]? PlatformDenyList;
				if (InObject.TryGetStringArrayFieldWithDeprecatedFallback("PlatformDenyList", "BlacklistPlatforms", out PlatformDenyList))
				{
					Module.PlatformDenyList = new List<UnrealTargetPlatform>();
					foreach (string TargetPlatformName in PlatformDenyList)
					{
						UnrealTargetPlatform Platform;
						if (UnrealTargetPlatform.TryParse(TargetPlatformName, out Platform))
						{
							Module.PlatformDenyList.Add(Platform);
						}
						else if ( !PluginDescriptor.IsAllowableMissingPlatform(TargetPlatformName, JsonFilePath.Directory) )
						{
							Log.TraceWarningTask(JsonFilePath, $"Unknown platform {TargetPlatformName} while parsing deny list for module descriptor {Module.Name}");
						}
					}
				}

				
				Dictionary<UnrealTargetPlatform,List<UnrealArch>> ParsePlatformArchitectureList( string[] PlatformArchitectureList )
				{
					Dictionary<UnrealTargetPlatform,List<UnrealArch>> Result = [];

					foreach (string PlatformArchitecture in PlatformArchitectureList)
					{
						string[] Pair = PlatformArchitecture.Split(':');
						if (Pair.Length != 2)
						{
							Log.TraceWarningTask(JsonFilePath, $"Malformed Platform:Architecture pair : {PlatformArchitecture} while parsing platform architecture list for module descriptor {Module.Name}");
							continue;
						}

						UnrealTargetPlatform Platform;
						if (!UnrealTargetPlatform.TryParse(Pair[0], out Platform))
						{
							if ( !PluginDescriptor.IsAllowableMissingPlatform(Pair[0], JsonFilePath.Directory) )
							{
								Log.TraceWarningTask(JsonFilePath, $"Unknown platform {Pair[0]} while parsing platform architecture list for module descriptor {Module.Name}");
							}
							continue;
						}

						UnrealArch Architecture;
						if (!UnrealArch.TryParse(Pair[1], out Architecture))
						{
							Log.TraceWarningTask(JsonFilePath, $"Unknown architecture {Pair[1]} while parsing platform architecture list for module descriptor {Module.Name}");
							continue;
						}

						if (Result.ContainsKey(Platform))
						{
							Result[Platform].Add(Architecture);
						}
						else
						{
							Result.Add(Platform, [Architecture]);
						}
					}

					return Result;

				}

				if (InObject.TryGetStringArrayField("PlatformArchitectureAllowList", out string[]? PlatformArchitectureAllowList ))
				{
					Module.PlatformArchitectureAllowList = ParsePlatformArchitectureList(PlatformArchitectureAllowList);
				}

				if (InObject.TryGetStringArrayField("PlatformArchitectureDenyList", out string[]? PlatformArchitectureDenyList ))
				{
					Module.PlatformArchitectureDenyList = ParsePlatformArchitectureList(PlatformArchitectureDenyList);
				}

				Module.TargetAllowList = InObject.GetEnumArrayFieldWithDeprecatedFallback<TargetType>("TargetAllowList", "WhitelistTargets");
				Module.TargetDenyList = InObject.GetEnumArrayFieldWithDeprecatedFallback<TargetType>("TargetDenyList", "BlacklistTargets");
				Module.TargetConfigurationAllowList = InObject.GetEnumArrayFieldWithDeprecatedFallback<UnrealTargetConfiguration>("TargetConfigurationAllowList", "WhitelistTargetConfigurations");
				Module.TargetConfigurationDenyList = InObject.GetEnumArrayFieldWithDeprecatedFallback<UnrealTargetConfiguration>("TargetConfigurationDenyList", "BlacklistTargetConfigurations");
			}
			catch (BuildException Ex)
			{
				ExceptionUtils.AddContext(Ex, "while parsing module descriptor '{0}'", Module.Name);
				throw;
			}

			string[]? ProgramAllowList;
			if (InObject.TryGetStringArrayFieldWithDeprecatedFallback("ProgramAllowList", "WhitelistPrograms", out ProgramAllowList))
			{
				Module.ProgramAllowList = ProgramAllowList;
			}

			string[]? ProgramDenyList;
			if (InObject.TryGetStringArrayFieldWithDeprecatedFallback("ProgramDenyList", "BlacklistPrograms", out ProgramDenyList))
			{
				Module.ProgramDenyList = ProgramDenyList;
			}
			
			string[]? GameTargetAllowList;
			if (InObject.TryGetStringArrayField("GameTargetAllowList", out GameTargetAllowList))
			{
				Module.GameTargetAllowList = GameTargetAllowList;
			}

			string[]? GameTargetDenyList;
			if (InObject.TryGetStringArrayField("GameTargetDenyList", out GameTargetDenyList))
			{
				Module.GameTargetDenyList = GameTargetDenyList;
			}
			
			string[]? AdditionalDependencies;
			if (InObject.TryGetStringArrayField("AdditionalDependencies", out AdditionalDependencies))
			{
				Module.AdditionalDependencies = AdditionalDependencies;
			}

			bool bHasExplicitPlatforms;
			if (InObject.TryGetBoolField("HasExplicitPlatforms", out bHasExplicitPlatforms))
			{
				Module.bHasExplicitPlatforms = bHasExplicitPlatforms;
			}

			return Module;
		}

		/// <summary>
		/// Write this module to a JsonWriter
		/// </summary>
		/// <param name="Writer">Writer to output to</param>
		void Write(JsonWriter Writer)
		{
			Writer.WriteObjectStart();
			Writer.WriteValue("Name", Name);
			Writer.WriteValue("Type", Type.ToString());
			Writer.WriteValue("LoadingPhase", LoadingPhase.ToString());
			// important note: we don't check the length of the platform allow list, because if an unknown platform was read in, but was not valid, the 
			// list will exist but be empty. We don't want to remove the allow list completely, because that would allow this module on all platforms,
			// which will not be the desired effect
			if (PlatformAllowList != null)
			{
				Writer.WriteArrayStart("PlatformAllowList");
				foreach (UnrealTargetPlatform Platform in PlatformAllowList)
				{
					Writer.WriteValue(Platform.ToString());
				}
				Writer.WriteArrayEnd();
			}
			if (PlatformDenyList != null && PlatformDenyList.Count > 0)
			{
				Writer.WriteArrayStart("PlatformDenyList");
				foreach (UnrealTargetPlatform Platform in PlatformDenyList)
				{
					Writer.WriteValue(Platform.ToString());
				}
				Writer.WriteArrayEnd();
			}
			if (PlatformArchitectureAllowList != null && PlatformArchitectureAllowList.Count > 0)
			{
				Writer.WriteArrayStart("PlatformArchitectureAllowList");
				foreach (string PlatformArchitecture in PlatformArchitectureAllowList.SelectMany( Pair => Pair.Value.Select( Arch => $"{Pair.Key}:{Arch}" ) ))
				{
					Writer.WriteValue(PlatformArchitecture);
				}
				Writer.WriteArrayEnd();
			}
			if (PlatformArchitectureDenyList != null && PlatformArchitectureDenyList.Count > 0)
			{
				Writer.WriteArrayStart("PlatformArchitectureDenyList");
				foreach (string PlatformArchitecture in PlatformArchitectureDenyList.SelectMany( Pair => Pair.Value.Select( Arch => $"{Pair.Key}:{Arch}" ) ))
				{
					Writer.WriteValue(PlatformArchitecture);
				}
				Writer.WriteArrayEnd();
			}
			if (TargetAllowList != null && TargetAllowList.Length > 0)
			{
				Writer.WriteArrayStart("TargetAllowList");
				foreach (TargetType Target in TargetAllowList)
				{
					Writer.WriteValue(Target.ToString());
				}
				Writer.WriteArrayEnd();
			}
			if (TargetDenyList != null && TargetDenyList.Length > 0)
			{
				Writer.WriteArrayStart("TargetDenyList");
				foreach (TargetType Target in TargetDenyList)
				{
					Writer.WriteValue(Target.ToString());
				}
				Writer.WriteArrayEnd();
			}
			if (TargetConfigurationAllowList != null && TargetConfigurationAllowList.Length > 0)
			{
				Writer.WriteArrayStart("TargetConfigurationAllowList");
				foreach (UnrealTargetConfiguration Config in TargetConfigurationAllowList)
				{
					Writer.WriteValue(Config.ToString());
				}
				Writer.WriteArrayEnd();
			}
			if (TargetConfigurationDenyList != null && TargetConfigurationDenyList.Length > 0)
			{
				Writer.WriteArrayStart("TargetConfigurationDenyList");
				foreach (UnrealTargetConfiguration Config in TargetConfigurationDenyList)
				{
					Writer.WriteValue(Config.ToString());
				}
				Writer.WriteArrayEnd();
			}
			if (ProgramAllowList != null && ProgramAllowList.Length > 0)
			{
				Writer.WriteStringArrayField("ProgramAllowList", ProgramAllowList);
			}
			if (ProgramDenyList != null && ProgramDenyList.Length > 0)
			{
				Writer.WriteStringArrayField("ProgramDenyList", ProgramDenyList);
			}
			if (GameTargetAllowList != null && GameTargetAllowList.Length > 0)
			{
				Writer.WriteStringArrayField("GameTargetAllowList", GameTargetAllowList);
			}
			if (GameTargetDenyList != null && GameTargetDenyList.Length > 0)
			{
				Writer.WriteStringArrayField("GameTargetDenyList", GameTargetDenyList);
			}	
			if (AdditionalDependencies != null && AdditionalDependencies.Length > 0)
			{
				Writer.WriteArrayStart("AdditionalDependencies");
				foreach (string AdditionalDependency in AdditionalDependencies)
				{
					Writer.WriteValue(AdditionalDependency);
				}
				Writer.WriteArrayEnd();
			}
			if (bHasExplicitPlatforms)
			{
				Writer.WriteValue("HasExplicitPlatforms", bHasExplicitPlatforms);
			}
			Writer.WriteObjectEnd();
		}

		JsonObject ToJsonObject()
		{
			JsonObject ModuleObject = new JsonObject();
			ModuleObject.AddOrSetFieldValue("Name", Name);
			ModuleObject.AddOrSetFieldValue("Type", Type.ToString());
			ModuleObject.AddOrSetFieldValue("LoadingPhase", LoadingPhase.ToString());
			// important note: we don't check the length of the platform allow list, because if an unknown platform was read in, but was not valid, the 
			// list will exist but be empty. We don't want to remove the allow list completely, because that would allow this module on all platforms,
			// which will not be the desired effect
			if (PlatformAllowList != null)
			{
				string[] PlatformAllowListStringArray = PlatformAllowList.Select(X => X.ToString()).ToArray();
				ModuleObject.AddOrSetFieldValue("PlatformAllowList", PlatformAllowListStringArray);
			}
			if (PlatformDenyList != null && PlatformDenyList.Count > 0)
			{
				string[] PlatformDenyListStringArray = PlatformDenyList.Select(X => X.ToString()).ToArray();
				ModuleObject.AddOrSetFieldValue("PlatformDenyList", PlatformDenyListStringArray);
			}
			if (PlatformArchitectureAllowList != null && PlatformArchitectureAllowList.Count > 0)
			{
				string[] PlatformArchitectureStringArray = PlatformArchitectureAllowList.SelectMany( Pair => Pair.Value.Select( Arch => $"{Pair.Key}:{Arch}" ) ).ToArray();
				ModuleObject.AddOrSetFieldValue("PlatformArchitectureAllowList", PlatformArchitectureStringArray);
			}
			if (PlatformArchitectureDenyList != null)
			{
				string[] PlatformArchitectureStringArray = PlatformArchitectureDenyList.SelectMany( Pair => Pair.Value.Select( Arch => $"{Pair.Key}:{Arch}" ) ).ToArray();
				ModuleObject.AddOrSetFieldValue("PlatformArchitectureDenyList", PlatformArchitectureStringArray);
			}
			if (TargetAllowList != null && TargetAllowList.Length > 0)
			{
				string[] TargetAllowListStringArray = TargetAllowList.Select(X => X.ToString()).ToArray();
				ModuleObject.AddOrSetFieldValue("TargetAllowList", TargetAllowListStringArray);
			}
			if (TargetDenyList != null && TargetDenyList.Length > 0)
			{
				string[] TargetDenyListStringArray = TargetDenyList.Select(X => X.ToString()).ToArray();
				ModuleObject.AddOrSetFieldValue("TargetDenyList", TargetDenyListStringArray);
			}
			if (TargetConfigurationAllowList != null && TargetConfigurationAllowList.Length > 0)
			{
				string[] TargetConfigurationAllowListStringArray = TargetConfigurationAllowList.Select(X => X.ToString()).ToArray();
				ModuleObject.AddOrSetFieldValue("TargetConfigurationAllowList", TargetConfigurationAllowListStringArray);
			}
			if (TargetConfigurationDenyList != null && TargetConfigurationDenyList.Length > 0)
			{
				string[] TargetConfigurationDenyListStringArray = TargetConfigurationDenyList.Select(X => X.ToString()).ToArray();
				ModuleObject.AddOrSetFieldValue("TargetConfigurationDenyList", TargetConfigurationDenyListStringArray);
			}
			if (ProgramAllowList != null && ProgramAllowList.Length > 0)
			{
				ModuleObject.AddOrSetFieldValue("ProgramAllowList", ProgramAllowList);
			}
			if (ProgramDenyList != null && ProgramDenyList.Length > 0)
			{
				ModuleObject.AddOrSetFieldValue("ProgramDenyList", ProgramDenyList);
			}
			if (AdditionalDependencies != null && AdditionalDependencies.Length > 0)
			{
				ModuleObject.AddOrSetFieldValue("AdditionalDependencies", AdditionalDependencies);
			}
			if (bHasExplicitPlatforms)
			{
				ModuleObject.AddOrSetFieldValue("HasExplicitPlatforms", bHasExplicitPlatforms);
			}
			return ModuleObject;
		}

		/// <summary>
		/// Write an array of module descriptors
		/// </summary>
		/// <param name="Writer">The Json writer to output to</param>
		/// <param name="Name">Name of the array</param>
		/// <param name="Modules">Array of modules</param>
		public static void WriteArray(JsonWriter Writer, string Name, ModuleDescriptor[]? Modules)
		{
			if (Modules != null && Modules.Length > 0)
			{
				Writer.WriteArrayStart(Name);
				foreach (ModuleDescriptor Module in Modules)
				{
					Module.Write(Writer);
				}
				Writer.WriteArrayEnd();
			}
		}

		/// <summary>
		/// Updates a JsonObject with an array of module descriptors 
		/// </summary>
		/// <param name="InObject">The JsonObject to update.</param>
		/// <param name="Name">Name of the array</param>
		/// <param name="Modules">Array of modules</param>
		public static void UpdateJson(JsonObject InObject, string Name, ModuleDescriptor[]? Modules)
		{
			if (Modules != null && Modules.Length > 0)
			{
				JsonObject[] JsonObjects = Modules.Select(X => X.ToJsonObject()).ToArray();
				InObject.AddOrSetFieldValue(Name, JsonObjects);
			}
		}

		/// <summary>
		/// Produces any warnings and errors for the module settings
		/// </summary>
		/// <param name="File">File containing the module declaration</param>
		public void Validate(FileReference File)
		{
			if (Type == ModuleHostType.Developer)
			{
				Log.TraceWarningOnce("The 'Developer' module type has been deprecated in 4.24. Use 'DeveloperTool' for modules that can be loaded by game/client/server targets in non-shipping configurations, or 'UncookedOnly' for modules that should only be loaded by uncooked editor and program targets (eg. modules containing blueprint nodes)");
				Log.TraceWarningOnce(File, "The 'Developer' module type has been deprecated in 4.24.");
			}
		}

		/// <summary>
		/// Determines whether the given plugin module is part of the current build.
		/// </summary>
		/// <param name="Platform">The platform being compiled for</param>
		/// <param name="Configuration">The target configuration being compiled for</param>
		/// <param name="TargetName">Name of the target being built</param>
		/// <param name="TargetType">The type of the target being compiled</param>
		/// <param name="bBuildDeveloperTools">Whether the configuration includes developer tools (typically UEBuildConfiguration.bBuildDeveloperTools for UBT callers)</param>
		/// <param name="bBuildRequiresCookedData">Whether the configuration requires cooked content (typically UEBuildConfiguration.bBuildRequiresCookedData for UBT callers)</param>
		/// <param name="Architectures">The architectures being compiled</param>
		public bool IsCompiledInConfiguration(UnrealTargetPlatform Platform, UnrealTargetConfiguration Configuration, string TargetName, TargetType TargetType, bool bBuildDeveloperTools, bool bBuildRequiresCookedData, UnrealArchitectures? Architectures = null)
		{
			return IsCompiledInConfiguration(Platform, Configuration, TargetName, TargetType, bBuildDeveloperTools, bBuildRequiresCookedData, Architectures, out string? _);
		}

		/// <summary>
		/// Determines whether the given plugin module is part of the current build.
		/// </summary>
		/// <param name="Platform">The platform being compiled for</param>
		/// <param name="Configuration">The target configuration being compiled for</param>
		/// <param name="TargetName">Name of the target being built</param>
		/// <param name="TargetType">The type of the target being compiled</param>
		/// <param name="bBuildDeveloperTools">Whether the configuration includes developer tools (typically UEBuildConfiguration.bBuildDeveloperTools for UBT callers)</param>
		/// <param name="bBuildRequiresCookedData">Whether the configuration requires cooked content (typically UEBuildConfiguration.bBuildRequiresCookedData for UBT callers)</param>
		/// <param name="Architectures">The architectures being compiled</param>
		/// <param name="invalidReason">Out parameter, reason why this plugin module is invalid</param>
		public bool IsCompiledInConfiguration(UnrealTargetPlatform Platform, UnrealTargetConfiguration Configuration, string TargetName, TargetType TargetType, bool bBuildDeveloperTools, bool bBuildRequiresCookedData, UnrealArchitectures? Architectures, [NotNullWhen(false)] out string? invalidReason)
		{
			// Check the platform is allowed
			// important note: we don't check the length of the platform allow list, because if an unknown platform was read in, but was not valid, the 
			// list will exist but be empty. In this case, we need to disallow all platforms from building, otherwise, build errors will occur when
			// it starts compiling for _all_ platforms. This means we don't need to check bHasExplicitPlatforms either

			if (PlatformAllowList != null && !PlatformAllowList.Contains(Platform))
			{
				invalidReason = $"PlatformAllowList does not include {Platform}";
				return false;
			}

			// Check the platform is not denied
			if (PlatformDenyList != null && PlatformDenyList.Contains(Platform))
			{
				invalidReason = $"PlatformDenyList includes {Platform}";
				return false;
			}

			// Check the architecture is allowed
			if (Architectures != null && !Architectures.bIsMultiArch && PlatformArchitectureAllowList != null && PlatformArchitectureAllowList.TryGetValue(Platform, out List<UnrealArch>? AllowedArchitectures) && !AllowedArchitectures.Contains(Architectures.SingleArchitecture))
			{
				invalidReason = $"PlatformArchitectureAllowList does not include {Platform}:{Architectures.SingleArchitecture}";
				return false;
			}

			// check the architecture is not denied
			if (Architectures != null && Architectures.bIsMultiArch && PlatformArchitectureDenyList != null && PlatformArchitectureDenyList.ContainsKey(Platform))
			{
				throw new Exception($"PlatformArchitectureDenyList does not support {Platform} Multi-architecture builds ({Name})");
			}
			if (Architectures != null && !Architectures.bIsMultiArch && PlatformArchitectureDenyList != null && PlatformArchitectureDenyList.TryGetValue(Platform, out List<UnrealArch>? DeniedArchitectures) && DeniedArchitectures.Contains(Architectures.SingleArchitecture))
			{
				invalidReason = $"PlatformArchitectureDenyList includes {Platform}:{Architectures.SingleArchitecture}";
				return false;
			}

			// Check the target is allowed
			if (TargetAllowList != null && TargetAllowList.Length > 0 && !TargetAllowList.Contains(TargetType))
			{
				invalidReason = $"TargetAllowList does not include {TargetType}";
				return false;
			}

			// Check the target is not denied
			if (TargetDenyList != null && TargetDenyList.Contains(TargetType))
			{
				invalidReason = $"PlatformDenyList includes {TargetType}";
				return false;
			}

			// Check the target configuration is allowed
			if (TargetConfigurationAllowList != null && TargetConfigurationAllowList.Length > 0 && !TargetConfigurationAllowList.Contains(Configuration))
			{
				invalidReason = $"TargetConfigurationAllowList does not include {Configuration}";
				return false;
			}

			// Check the target configuration is not denied
			if (TargetConfigurationDenyList != null && TargetConfigurationDenyList.Contains(Configuration))
			{
				invalidReason = $"TargetConfigurationDenyList includes {Configuration}";
				return false;
			}

			if (TargetType == TargetType.Program)
			{
				// Check the program name is on the allow list. Note that this behavior is slightly different to other allow/deny checks; we will allow a module of any type if it's explicitly allowed for this program.
				if (ProgramAllowList != null && ProgramAllowList.Length > 0)
				{
					if (!ProgramAllowList.Contains(TargetName))
					{
						invalidReason = $"ProgramAllowList does not include {TargetName}";
						return false;
					}
					invalidReason = null;
					return true;
				}

				// Check the program name is not denied
				if (ProgramDenyList != null && ProgramDenyList.Contains(TargetName))
				{
					invalidReason = $"ProgramDenyList includes {TargetName}";
					return false;
				}
			}
			else 
			{
				// Check that the TargetName is allowed
				if (GameTargetAllowList != null && GameTargetAllowList.Length > 0 && !GameTargetAllowList.Contains(TargetName))
				{
					invalidReason = $"GameTargetAllowList does not include {TargetName}";
					return false;
				}
				
				// Check that the TargetName is not denied
				if (GameTargetDenyList != null && GameTargetDenyList.Contains(TargetName))
				{
					invalidReason = $"GameTargetDenyList includes {TargetName}";
					return false;
				}
			}

			// Check the module is compatible with this target.
			invalidReason = $"Module Type {Type} not supported for Target Type {TargetType}";
			switch (Type)
			{
				case ModuleHostType.Runtime:
				case ModuleHostType.RuntimeNoCommandlet:
					return TargetType != TargetType.Program;
				case ModuleHostType.RuntimeAndProgram:
					return true;
				case ModuleHostType.CookedOnly:
					return bBuildRequiresCookedData;
				case ModuleHostType.UncookedOnly:
					return !bBuildRequiresCookedData;
				case ModuleHostType.Developer:
					return TargetType == TargetType.Editor || TargetType == TargetType.Program;
				case ModuleHostType.DeveloperTool:
					return bBuildDeveloperTools;
				case ModuleHostType.Editor:
				case ModuleHostType.EditorNoCommandlet:
					return TargetType == TargetType.Editor;
				case ModuleHostType.EditorAndProgram:
					return TargetType == TargetType.Editor || TargetType == TargetType.Program;
				case ModuleHostType.Program:
					return TargetType == TargetType.Program;
				case ModuleHostType.ServerOnly:
					return TargetType != TargetType.Program && TargetType != TargetType.Client;
				case ModuleHostType.ClientOnly:
				case ModuleHostType.ClientOnlyNoCommandlet:
					return TargetType != TargetType.Program && TargetType != TargetType.Server;
			}

			return false;
		}
	}
}
