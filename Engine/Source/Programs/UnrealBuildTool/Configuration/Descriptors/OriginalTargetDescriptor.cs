// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Immutable;
using System.Runtime.InteropServices;
using System.Text.Json.Serialization;
using EpicGames.Core;

namespace UnrealBuildTool;

/// <summary>
/// A target descriptor that is read-only and suitable for serialization and equality comparison.
/// This should be kept up-to-date with <inheritdoc cref="TargetDescriptor"/>.
/// "Original" because it stores the state of the descriptor when it was created.
/// </summary>
internal record class OriginalTargetDescriptor
{
	/// <inheritdoc cref="TargetDescriptor.ProjectFile"/>
	public FileReference? ProjectFile { get; init; }

	/// <inheritdoc cref="TargetDescriptor.Name"/>
	public required string Name { get; init; }

	/// <inheritdoc cref="TargetDescriptor.Platform"/>
	public UnrealTargetPlatform Platform { get; init; }

	/// <inheritdoc cref="TargetDescriptor.Configuration"/>
	public UnrealTargetConfiguration Configuration { get; init; }

	/// <inheritdoc cref="TargetDescriptor.Architectures"/>
	public required UnrealArchitectures Architectures { get; init; }

	/// <inheritdoc cref="TargetDescriptor.AdditionalArguments"/>
	public required CommandLineArguments AdditionalArguments { get; init; }

	/// <inheritdoc cref="TargetDescriptor.IsTestsTarget"/>
	public bool IsTestsTarget { get; init; }

	/// <inheritdoc cref="TargetDescriptor.ForeignPlugin"/>
	public FileReference? ForeignPlugin { get; init; }

	/// <inheritdoc cref="TargetDescriptor.bBuildPluginAsLocal"/>
	public bool bBuildPluginAsLocal { get; init; }

	/// <inheritdoc cref="TargetDescriptor.bBuildDependantPlugins"/>
	public bool bBuildDependantPlugins { get; init; }

	/// <inheritdoc cref="TargetDescriptor.OnlyModuleNames"/>
	public required ImmutableHashSet<string> OnlyModuleNames { get; init; }

	/// <inheritdoc cref="TargetDescriptor.FileLists"/>
	public ImmutableArray<FileReference> FileLists { get; init; }

	/// <inheritdoc cref="TargetDescriptor.SpecificFilesToCompile"/>
	public ImmutableArray<FileReference> SpecificFilesToCompile { get; init; }

	/// <inheritdoc cref="TargetDescriptor.RelativePathsToSpecificFilesToCompile"/>
	public ImmutableArray<string> RelativePathsToSpecificFilesToCompile { get; init; }

	/// <inheritdoc cref="TargetDescriptor.WorkingDir"/>
	public string? WorkingDir { get; init; }

	/// <inheritdoc cref="TargetDescriptor.bSingleFileBuildDependents"/>
	public bool bSingleFileBuildDependents { get; init; }

	/// <inheritdoc cref="TargetDescriptor.HotReloadMode"/>
	public HotReloadMode HotReloadMode { get; set; }

	/// <inheritdoc cref="TargetDescriptor.HotReloadModuleNameToSuffix"/>
	public required ImmutableDictionary<string, int> HotReloadModuleNameToSuffix { get; init; }

	/// <inheritdoc cref="TargetDescriptor.LiveCodingModules"/>
	public FileReference? LiveCodingModules { get; init; }

	/// <inheritdoc cref="TargetDescriptor.LiveCodingManifest"/>
	public FileReference? LiveCodingManifest { get; init; }

	/// <inheritdoc cref="TargetDescriptor.LiveCodingLimit"/>
	public uint LiveCodingLimit { get; init; }

	/// <inheritdoc cref="TargetDescriptor.bQuiet"/>
	[JsonIgnore]
	public bool bQuiet { get; init; }

	/// <inheritdoc cref="TargetDescriptor.bRebuild"/>
	[JsonIgnore]
	public bool bRebuild { get; init; }

	/// <inheritdoc cref="TargetDescriptor.bUseUnityBuild"/>
	public bool bUseUnityBuild { get; init; }

	/// <inheritdoc cref="TargetDescriptor.bForceUnityBuild"/>
	public bool bForceUnityBuild { get; init; }

	/// <inheritdoc cref="TargetDescriptor.bIWYU"/>
	public bool bIWYU { get; init; }

	/// <inheritdoc cref="TargetDescriptor.bProfilingCompile"/>
	public bool bProfilingCompile { get; init; }

	/// <inheritdoc cref="TargetDescriptor.IntermediateEnvironment"/>
	public UnrealIntermediateEnvironment IntermediateEnvironment { get; init; }

	public static OriginalTargetDescriptor From(TargetDescriptor targetDescriptor)
	{
		Span<FileReference> fileLists = CollectionsMarshal.AsSpan(targetDescriptor.FileLists);
		Span<FileReference> specificFilesToCompile = CollectionsMarshal.AsSpan(targetDescriptor.SpecificFilesToCompile);
		Span<string> relativePathsToSpecificFiles = CollectionsMarshal.AsSpan(targetDescriptor.RelativePathsToSpecificFilesToCompile);
		return new()
		{
			ProjectFile = targetDescriptor.ProjectFile,
			Name = targetDescriptor.Name,
			Platform = targetDescriptor.Platform,
			Configuration = targetDescriptor.Configuration,
			Architectures = new(targetDescriptor.Architectures),
			AdditionalArguments = targetDescriptor.AdditionalArguments,
			IsTestsTarget = targetDescriptor.IsTestsTarget,
			ForeignPlugin = targetDescriptor.ForeignPlugin,
			bBuildPluginAsLocal = targetDescriptor.bBuildPluginAsLocal,
			bBuildDependantPlugins = targetDescriptor.bBuildDependantPlugins,
			OnlyModuleNames = ImmutableHashSet.Create(targetDescriptor.OnlyModuleNames.Comparer, [.. targetDescriptor.OnlyModuleNames]),
			FileLists = ImmutableArray.Create(fileLists),
			SpecificFilesToCompile = ImmutableArray.Create(specificFilesToCompile),
			RelativePathsToSpecificFilesToCompile = ImmutableArray.Create(relativePathsToSpecificFiles),
			WorkingDir = targetDescriptor.WorkingDir,
			bSingleFileBuildDependents = targetDescriptor.bSingleFileBuildDependents,
			HotReloadMode = targetDescriptor.HotReloadMode,
			HotReloadModuleNameToSuffix = ImmutableDictionary.CreateRange<string, int>(
				targetDescriptor.HotReloadModuleNameToSuffix.Comparer,
				[.. targetDescriptor.HotReloadModuleNameToSuffix]),
			LiveCodingModules = targetDescriptor.LiveCodingModules,
			LiveCodingManifest = targetDescriptor.LiveCodingManifest,
			LiveCodingLimit = targetDescriptor.LiveCodingLimit,
			bQuiet = targetDescriptor.bQuiet,
			bRebuild = targetDescriptor.bRebuild,
			bUseUnityBuild = targetDescriptor.bUseUnityBuild,
			bForceUnityBuild = targetDescriptor.bForceUnityBuild,
			bIWYU = targetDescriptor.bIWYU,
			bProfilingCompile = targetDescriptor.bProfilingCompile,
			IntermediateEnvironment = targetDescriptor.IntermediateEnvironment,
		};
	}
}
