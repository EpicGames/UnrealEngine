// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using System.Linq;
using System.Text.Json;
using EpicGames.Core;

namespace UnrealBuildTool
{
	/// <summary>
	/// The version format for .uplugin files. This rarely changes now; plugin descriptors should maintain backwards compatibility automatically.
	/// </summary>
#pragma warning disable CA1027 // Mark enums with FlagsAttribute
	public enum PluginDescriptorVersion
#pragma warning restore CA1027 // Mark enums with FlagsAttribute
	{
		/// <summary>
		/// Invalid
		/// </summary>
		Invalid = 0,

		/// <summary>
		/// Initial version
		/// </summary>
		Initial = 1,

		/// <summary>
		/// Adding SampleNameHash
		/// </summary>
		NameHash = 2,

		/// <summary>
		/// Unifying plugin/project files (since abandoned, but backwards compatibility maintained)
		/// </summary>
		ProjectPluginUnification = 3,

		/// <summary>
		/// This needs to be the last line, so we can calculate the value of Latest below
		/// </summary>
		LatestPlusOne,

#pragma warning disable CA1069 // Enums should not have duplicate values - having access to Latest is useful
		/// <summary>
		/// The latest plugin descriptor version
		/// </summary>
		Latest = LatestPlusOne - 1
#pragma warning restore CA1069
	}

	/// <summary>
	/// In-memory representation of a .uplugin file
	/// </summary>
	public class PluginDescriptor
	{
		/// <summary>
		/// Descriptor version number
		/// </summary>
		public int FileVersion;

		/// <summary>
		/// Version number for the plugin.  The version number must increase with every version of the plugin, so that the system 
		/// can determine whether one version of a plugin is newer than another, or to enforce other requirements.  This version
		/// number is not displayed in front-facing UI.  Use the VersionName for that.
		/// </summary>
		public int Version;

		/// <summary>
		/// Name of the version for this plugin.  This is the front-facing part of the version number.  It doesn't need to match
		/// the version number numerically, but should be updated when the version number is increased accordingly.
		/// </summary>
		public string? VersionName;

		/// <summary>
		/// Friendly name of the plugin
		/// </summary>
		public string? FriendlyName;

		/// <summary>
		/// Description of the plugin
		/// </summary>
		public string? Description;

		/// <summary>
		/// The name of the category this plugin
		/// </summary>
		public string? Category;

		/// <summary>
		/// The company or individual who created this plugin.  This is an optional field that may be displayed in the user interface.
		/// </summary>
		public string? CreatedBy;

		/// <summary>
		/// Hyperlink URL string for the company or individual who created this plugin.  This is optional.
		/// </summary>
		public string? CreatedByURL;

		/// <summary>
		/// Documentation URL string.
		/// </summary>
		public string? DocsURL;

		/// <summary>
		/// Marketplace URL for this plugin. This URL will be embedded into projects that enable this plugin, so we can redirect to the marketplace if a user doesn't have it installed.
		/// </summary>
		public string? MarketplaceURL;

		/// <summary>
		/// Support URL/email for this plugin.
		/// </summary>
		public string? SupportURL;

		/// <summary>
		/// Sets the version of the engine that this plugin is compatible with.
		/// </summary>
		public string? EngineVersion;
		
		/// <summary>
		/// Sets the version of the engine at which this plugin has been deprecated.
		/// </summary>
		public string? DeprecatedEngineVersion;

		/// <summary>4
		/// If true, this plugin from a platform extension extending another plugin */
		/// </summary>
		public bool bIsPluginExtension;

		/// <summary>
		/// List of platforms supported by this plugin. This list will be copied to any plugin reference from a project file, to allow filtering entire plugins from staged builds.
		/// </summary>
		public List<UnrealTargetPlatform>? SupportedTargetPlatforms;

		/// <summary>
		/// List of programs supported by this plugin.
		/// </summary>
		public string[]? SupportedPrograms;

		/// <summary>
		/// List of all modules associated with this plugin
		/// </summary>
		public List<ModuleDescriptor>? Modules;

		/// <summary>
		/// List of all localization targets associated with this plugin
		/// </summary>
		public LocalizationTargetDescriptor[]? LocalizationTargets;

		/// <summary>
		/// The Verse path to the root of this plugin's content directory
		/// </summary>
		public string? VersePath;

		/// <summary>
		/// Origin/visibility of Verse code in this plugin's Content/Verse folder
		/// </summary>
		public VerseScope VerseScope = VerseScope.PublicUser;

		/// <summary>
		/// The version of the Verse language that this plugin targets.
		/// If no value is specified, the latest stable version is used.
		/// </summary>
		public uint? VerseVersion;

		/// <summary>
		/// Whether this plugin has SceneGraph enabled, which impacts the generated Verse Asset Digest.
		/// </summary>
		public bool? bEnableSceneGraph = false;

		/// <summary>
		/// Whether this plugin has SceneGraph Animation enabled, which impacts the generated Verse Asset Digest.
		/// </summary>
		public bool? bEnableSceneGraphAnimation = false;

		/// <summary>
		/// If to generate Verse source code definitions from assets contained in this plugin
		/// </summary>
		public bool? bEnableVerseAssetReflection = false;

		/// <summary>
		/// Whether this plugin should be enabled by default for all projects
		/// </summary>
		public Nullable<bool> bEnabledByDefault;

		/// <summary>
		/// Can this plugin contain content?
		/// </summary>
		public bool bCanContainContent;

		/// <summary>
		/// Can this plugin contain Verse code (either in content directory or in any of its modules)?
		/// </summary>
		public bool bCanContainVerse;

		/// <summary>
		/// Marks the plugin as beta in the UI
		/// </summary>
		public bool bIsBetaVersion;

		/// <summary>
		/// Marks the plugin as experimental in the UI
		/// </summary>
		public bool bIsExperimentalVersion;

		/// <summary>
		/// Set for plugins which are installed
		/// </summary>
		public bool bInstalled;

		/// <summary>
		/// For plugins that are under a platform folder (eg. /IOS/), determines whether compiling the plugin requires the build platform and/or SDK to be available
		/// </summary>
		public bool bRequiresBuildPlatform;

		/// <summary>
		/// When true, prevents other plugins from depending on this plugin
		/// </summary>
		public bool bIsSealed;

		/// <summary>
		/// When true, this plugin should not contain any code or modules.
		/// </summary>
		public bool bNoCode;

		/// <summary>
		/// If true, the GameFeaturePlugin this is on should have no content other than the GameFeatureData asset.
		/// </summary>
		public bool bNoGameFeatureContent;

		/// <summary>
		/// When true, this plugin's modules will not be loaded automatically nor will it's content be mounted automatically. It will load/mount when explicitly requested and LoadingPhases will be ignored
		/// </summary>
		public bool bExplicitlyLoaded;

		/// <summary>
		/// When true, an empty SupportedTargetPlatforms is interpreted as 'no platforms' with the expectation that explicit platforms will be added in plugin platform extensions
		/// </summary>
		public bool bHasExplicitPlatforms;

		/// <summary>
		/// Set of pre-build steps to execute, keyed by host platform name.
		/// </summary>
		public CustomBuildSteps? PreBuildSteps;

		/// <summary>
		/// Set of post-build steps to execute, keyed by host platform name.
		/// </summary>
		public CustomBuildSteps? PostBuildSteps;

		/// <summary>
		/// Additional plugins that this plugin depends on
		/// </summary>
		public List<PluginReferenceDescriptor>? Plugins;

		/// <summary>
		/// Plugins that this plugin should never depend on
		/// </summary>
		public string[]? DisallowedPlugins;

		/// <summary>
		/// The JsonObject created from reading a .uplugin on disk or from parsing a json text
		/// This preserves the order of all the fields from the source json as well as account for any custom fields.
		/// </summary>
		private readonly JsonObject CachedJson;

		/// <summary>
		/// Read-only access to the raw JSON object for this plugin descriptor.
		/// Callers can use this to read non-standard fields that are not modelled by this class.
		/// </summary>
		public JsonObject RawJson => CachedJson;

		/// <summary>
		/// Reads a plugin descriptor from a json object
		/// </summary>
		/// <param name="RawObject">The object to read from</param>
		/// <param name="PluginPath"></param>
		/// <returns>New plugin descriptor</returns>
		public PluginDescriptor(JsonObject RawObject, FileReference PluginPath)
		{
			CachedJson = RawObject;
			// Read the version
			if (!RawObject.TryGetIntegerField("FileVersion", out FileVersion))
			{
				if (!RawObject.TryGetIntegerField("PluginFileVersion", out FileVersion))
				{
					throw new BuildException("Plugin descriptor does not contain a valid FileVersion entry");
				}
			}

			// Check it's not newer than the latest version we can parse
			if (FileVersion > (int)PluginDescriptorVersion.Latest)
			{
				throw new BuildException("Plugin descriptor appears to be in a newer version ({0}) of the file format that we can load (max version: {1}).", FileVersion, (int)PluginDescriptorVersion.Latest);
			}

			// Read the other fields
			RawObject.TryGetIntegerField("Version", out Version);
			RawObject.TryGetStringField("VersionName", out VersionName);
			RawObject.TryGetStringField("FriendlyName", out FriendlyName);
			RawObject.TryGetStringField("Description", out Description);

			if (!RawObject.TryGetStringField("Category", out Category))
			{
				// Category used to be called CategoryPath in .uplugin files
				RawObject.TryGetStringField("CategoryPath", out Category);
			}

			// Due to a difference in command line parsing between Windows and Mac, we shipped a few Mac samples containing
			// a category name with escaped quotes. Remove them here to make sure we can list them in the right category.
			if (Category != null && Category.Length >= 2 && Category.StartsWith("\"", StringComparison.Ordinal) && Category.EndsWith("\"", StringComparison.Ordinal))
			{
				Category = Category.Substring(1, Category.Length - 2);
			}

			RawObject.TryGetStringField("CreatedBy", out CreatedBy);
			RawObject.TryGetStringField("CreatedByURL", out CreatedByURL);
			RawObject.TryGetStringField("DocsURL", out DocsURL);
			RawObject.TryGetStringField("MarketplaceURL", out MarketplaceURL);
			RawObject.TryGetStringField("SupportURL", out SupportURL);
			RawObject.TryGetStringField("EngineVersion", out EngineVersion);
			RawObject.TryGetStringField("DeprecatedEngineVersion", out DeprecatedEngineVersion);
			RawObject.TryGetStringArrayField("SupportedPrograms", out SupportedPrograms);
			RawObject.TryGetBoolField("bIsPluginExtension", out bIsPluginExtension);

			string[]? SupportedTargetPlatformNames;
			if (RawObject.TryGetStringArrayField("SupportedTargetPlatforms", out SupportedTargetPlatformNames))
			{
				SupportedTargetPlatforms = new List<UnrealTargetPlatform>();
				foreach (string TargetPlatformName in SupportedTargetPlatformNames)
				{
					UnrealTargetPlatform Platform;
					if (UnrealTargetPlatform.TryParse(TargetPlatformName, out Platform))
					{
						SupportedTargetPlatforms.Add(Platform);
					}
					else if ( !IsAllowableMissingPlatform(TargetPlatformName, PluginPath.Directory) )
					{
						Log.TraceWarningTask(PluginPath, $"Unknown platform {TargetPlatformName} listed in plugin with FriendlyName \"{FriendlyName}\"");
					}
				}
			}

			JsonObject[]? ModulesArray;
			if (RawObject.TryGetObjectArrayField("Modules", out ModulesArray))
			{
				Modules = Array.ConvertAll(ModulesArray, x => ModuleDescriptor.FromJsonObject(x, PluginPath)).ToList();
			}

			JsonObject[]? LocalizationTargetsArray;
			if (RawObject.TryGetObjectArrayField("LocalizationTargets", out LocalizationTargetsArray))
			{
				LocalizationTargets = Array.ConvertAll(LocalizationTargetsArray, x => LocalizationTargetDescriptor.FromJsonObject(x));
			}

			RawObject.TryGetStringField("VersePath", out VersePath);

			VerseScope PluginVerseScope;
			if (RawObject.TryGetEnumField<VerseScope>("VerseScope", out PluginVerseScope))
			{
				VerseScope = PluginVerseScope;
			}

			uint PluginVerseVersion;
			if (RawObject.TryGetUnsignedIntegerField("VerseVersion", out PluginVerseVersion))
			{
				VerseVersion = PluginVerseVersion;
			}

			bool bPluginEnableSceneGraph;
			if (RawObject.TryGetBoolField("EnableSceneGraph", out bPluginEnableSceneGraph))
			{
				bEnableSceneGraph = bPluginEnableSceneGraph;
			}

			bool bPluginEnableSceneGraphAnimation;
			if (RawObject.TryGetBoolField("EnableSceneGraphAnimation", out bPluginEnableSceneGraphAnimation))
			{
				bEnableSceneGraphAnimation = bPluginEnableSceneGraphAnimation;
			}

			bool bPluginEnableVerseAssetReflection;
			if (RawObject.TryGetBoolField("EnableVerseAssetReflection", out bPluginEnableVerseAssetReflection))
			{
				bEnableVerseAssetReflection = bPluginEnableVerseAssetReflection;
			}

			bool bEnabledByDefaultValue;
			if (RawObject.TryGetBoolField("EnabledByDefault", out bEnabledByDefaultValue))
			{
				bEnabledByDefault = bEnabledByDefaultValue;
			}

			RawObject.TryGetBoolField("CanContainContent", out bCanContainContent);
			RawObject.TryGetBoolField("CanContainVerse", out bCanContainVerse);
			RawObject.TryGetBoolField("IsBetaVersion", out bIsBetaVersion);
			RawObject.TryGetBoolField("IsExperimentalVersion", out bIsExperimentalVersion);
			RawObject.TryGetBoolField("Installed", out bInstalled);

			bool bCanBeUsedWithUnrealHeaderTool;
			if (RawObject.TryGetBoolField("CanBeUsedWithUnrealHeaderTool", out bCanBeUsedWithUnrealHeaderTool) && bCanBeUsedWithUnrealHeaderTool)
			{
				Array.Resize(ref SupportedPrograms, (SupportedPrograms == null) ? 1 : SupportedPrograms.Length + 1);
				SupportedPrograms[^1] = "UnrealHeaderTool";
			}

			RawObject.TryGetBoolField("RequiresBuildPlatform", out bRequiresBuildPlatform);
			RawObject.TryGetBoolField("Sealed", out bIsSealed);
			RawObject.TryGetBoolField("NoCode", out bNoCode);
			RawObject.TryGetBoolField("NoGameFeatureContent", out bNoGameFeatureContent);
			RawObject.TryGetBoolField("ExplicitlyLoaded", out bExplicitlyLoaded);
			RawObject.TryGetBoolField("HasExplicitPlatforms", out bHasExplicitPlatforms);

			CustomBuildSteps.TryRead(RawObject, "PreBuildSteps", out PreBuildSteps);
			CustomBuildSteps.TryRead(RawObject, "PostBuildSteps", out PostBuildSteps);

			JsonObject[]? PluginsArray;
			if (RawObject.TryGetObjectArrayField("Plugins", out PluginsArray))
			{
				Plugins = Array.ConvertAll(PluginsArray, x => PluginReferenceDescriptor.FromJsonObject(x)).ToList();
			}

			JsonObject[]? DisallowedPluginsArray;
			if (RawObject.TryGetObjectArrayField("DisallowedPlugins", out DisallowedPluginsArray))
			{
				DisallowedPlugins = Array.ConvertAll(DisallowedPluginsArray, x => x.GetStringField("Name"));
			}
			else
			{
				// Backwards compatibility still check for a simple array.
				RawObject.TryGetStringArrayField("DisallowedPlugins", out DisallowedPlugins);
			}
		}

		/// <summary>
		/// Creates a plugin descriptor from a file on disk preserving all custom fields in the file.
		/// </summary>
		/// <param name="FileName">The filename to read</param>
		/// <returns>New plugin descriptor</returns>
		public static PluginDescriptor FromFile(FileReference FileName)
		{
			try
			{
				JsonObject RawObject = JsonObject.Read(FileName);

				PluginDescriptor Descriptor = new PluginDescriptor(RawObject, FileName);
				Descriptor.Validate(FileName);
				return Descriptor;
			}
			catch (JsonException ex)
			{
				throw new JsonException($"{ex.Message} (in {FileName})", FileName.FullName, ex.LineNumber, ex.BytePositionInLine, ex);
			}
		}

		/// <summary>
		/// Saves the descriptor to disk. This only saves the default fields in a .uplugin and does not account for cusotm fields.
		/// </summary>
		/// <param name="FileName">The filename to write to</param>
		public void Save(string FileName)
		{
			using (JsonWriter Writer = new JsonWriter(FileName))
			{
				Writer.WriteObjectStart();
				Write(Writer);
				Writer.WriteObjectEnd();
			}
		}

		/// <summary>
		/// Saves the descriptor to disk preserving all custom fields that were read in.
		/// </summary>
		/// <param name="fileName">The filename to write to</param>
		public void Save2(string fileName)
		{
			// @TODO: This should replace all instances of Save() at some point in the future. There's just still a lot of references to test and refactor that needs to be verified. 
			UpdateJson();
			string jsonString = CachedJson.ToJsonString();
			File.WriteAllText(fileName, jsonString);
		}

		/// <summary>
		/// Writes the plugin descriptor to an existing Json writer
		/// </summary>
		/// <param name="Writer">The writer to receive plugin data</param>
		public void Write(JsonWriter Writer)
		{
			Writer.WriteValue("FileVersion", (int)ProjectDescriptorVersion.Latest);
			Writer.WriteValue("Version", Version);
			Writer.WriteValue("VersionName", VersionName);
			Writer.WriteValue("FriendlyName", FriendlyName);
			Writer.WriteValue("Description", Description);
			Writer.WriteValue("Category", Category);
			Writer.WriteValue("CreatedBy", CreatedBy);
			Writer.WriteValue("CreatedByURL", CreatedByURL);
			Writer.WriteValue("DocsURL", DocsURL);
			Writer.WriteValue("MarketplaceURL", MarketplaceURL);
			Writer.WriteValue("SupportURL", SupportURL);
			if (!String.IsNullOrEmpty(EngineVersion))
			{
				Writer.WriteValue("EngineVersion", EngineVersion);
			}
			if (!String.IsNullOrEmpty(DeprecatedEngineVersion))
			{
				Writer.WriteValue("DeprecatedEngineVersion", DeprecatedEngineVersion);
			}
			if (!String.IsNullOrEmpty(VersePath))
			{
				Writer.WriteValue("VersePath", VersePath);
			}
			if (VerseScope != VerseScope.PublicUser)
			{
				Writer.WriteValue("VerseScope", VerseScope.ToString());
			}
			if (VerseVersion.HasValue)
			{
				Writer.WriteValue("VerseVersion", VerseVersion.Value);
			}
			if (bEnabledByDefault.HasValue)
			{
				Writer.WriteValue("EnabledByDefault", bEnabledByDefault.Value);
			}
			Writer.WriteValue("CanContainContent", bCanContainContent);
			if (bCanContainVerse)
			{
				Writer.WriteValue("CanContainVerse", bCanContainVerse);
			}
			if (bIsBetaVersion)
			{
				Writer.WriteValue("IsBetaVersion", bIsBetaVersion);
			}
			if (bIsExperimentalVersion)
			{
				Writer.WriteValue("IsExperimentalVersion", bIsExperimentalVersion);
			}
			if (bInstalled)
			{
				Writer.WriteValue("Installed", bInstalled);
			}

			if (bRequiresBuildPlatform)
			{
				Writer.WriteValue("RequiresBuildPlatform", bRequiresBuildPlatform);
			}

			if (bIsSealed)
			{
				Writer.WriteValue("Sealed", bIsSealed);
			}

			if (bNoCode)
			{
				Writer.WriteValue("NoCode", bNoCode);
			}

			if (bNoGameFeatureContent)
			{
				Writer.WriteValue("NoGameFeatureContent", bNoGameFeatureContent);
			}

			if (bExplicitlyLoaded)
			{
				Writer.WriteValue("ExplicitlyLoaded", bExplicitlyLoaded);
			}

			if (bHasExplicitPlatforms)
			{
				Writer.WriteValue("HasExplicitPlatforms", bHasExplicitPlatforms);
			}

			if (SupportedTargetPlatforms != null && SupportedTargetPlatforms.Count > 0)
			{
				Writer.WriteStringArrayField("SupportedTargetPlatforms", SupportedTargetPlatforms.Select<UnrealTargetPlatform, string>(x => x.ToString()).ToArray());
			}

			if (SupportedPrograms != null && SupportedPrograms.Length > 0)
			{
				Writer.WriteStringArrayField("SupportedPrograms", SupportedPrograms);
			}
			if (bIsPluginExtension)
			{
				Writer.WriteValue("bIsPluginExtension", bIsPluginExtension);
			}

			if (Modules != null && Modules.Count > 0)
			{
				ModuleDescriptor.WriteArray(Writer, "Modules", Modules.ToArray());
			}

			LocalizationTargetDescriptor.WriteArray(Writer, "LocalizationTargets", LocalizationTargets);

			PreBuildSteps?.Write(Writer, "PreBuildSteps");

			PostBuildSteps?.Write(Writer, "PostBuildSteps");

			if (Plugins != null && Plugins.Count > 0)
			{
				PluginReferenceDescriptor.WriteArray(Writer, "Plugins", Plugins.ToArray());
			}

			if (DisallowedPlugins != null && DisallowedPlugins.Length > 0)
			{
				Writer.WriteStringArrayField("DisallowedPlugins", DisallowedPlugins);
			}
		}

		private void UpdateJson()
		{
			CachedJson.AddOrSetFieldValue("FileVersion", (int)ProjectDescriptorVersion.Latest);
			CachedJson.AddOrSetFieldValue("Version", Version);
			CachedJson.AddOrSetFieldValue("VersionName", VersionName);
			CachedJson.AddOrSetFieldValue("FriendlyName", FriendlyName);
			CachedJson.AddOrSetFieldValue("Description", Description);
			CachedJson.AddOrSetFieldValue("Category", Category);
			CachedJson.AddOrSetFieldValue("CreatedBy", CreatedBy);
			CachedJson.AddOrSetFieldValue("CreatedByURL", CreatedByURL);
			CachedJson.AddOrSetFieldValue("DocsURL", DocsURL);
			CachedJson.AddOrSetFieldValue("MarketplaceURL", MarketplaceURL);
			CachedJson.AddOrSetFieldValue("SupportURL", SupportURL);
			if (!String.IsNullOrEmpty(EngineVersion))
			{
				CachedJson.AddOrSetFieldValue("EngineVersion", EngineVersion);
			}
			if (!String.IsNullOrEmpty(DeprecatedEngineVersion))
			{
				CachedJson.AddOrSetFieldValue("DeprecatedEngineVersion", DeprecatedEngineVersion);
			}
			if (!String.IsNullOrEmpty(VersePath))
			{
				CachedJson.AddOrSetFieldValue("VersePath", VersePath);
			}
			if (VerseScope != VerseScope.PublicUser)
			{
				CachedJson.AddOrSetFieldValue("VerseScope", VerseScope.ToString());
			}
			if (bEnabledByDefault.HasValue)
			{
				CachedJson.AddOrSetFieldValue("EnabledByDefault", bEnabledByDefault.Value);
			}
			CachedJson.AddOrSetFieldValue("CanContainContent", bCanContainContent);
			if (bCanContainVerse)
			{
				CachedJson.AddOrSetFieldValue("CanContainVerse", bCanContainVerse);
			}
			if (bIsBetaVersion)
			{
				CachedJson.AddOrSetFieldValue("IsBetaVersion", bIsBetaVersion);
			}
			if (bIsExperimentalVersion)
			{
				CachedJson.AddOrSetFieldValue("IsExperimentalVersion", bIsExperimentalVersion);
			}
			if (bInstalled)
			{
				CachedJson.AddOrSetFieldValue("Installed", bInstalled);
			}

			if (bRequiresBuildPlatform)
			{
				CachedJson.AddOrSetFieldValue("RequiresBuildPlatform", bRequiresBuildPlatform);
			}

			if (bIsSealed)
			{
				CachedJson.AddOrSetFieldValue("Sealed", bIsSealed);
			}

			if (bExplicitlyLoaded)
			{
				CachedJson.AddOrSetFieldValue("ExplicitlyLoaded", bExplicitlyLoaded);
			}

			if (bHasExplicitPlatforms)
			{
				CachedJson.AddOrSetFieldValue("HasExplicitPlatforms", bHasExplicitPlatforms);
			}

			if (SupportedTargetPlatforms != null && SupportedTargetPlatforms.Count > 0)
			{
				CachedJson.AddOrSetFieldValue("SupportedTargetPlatforms", SupportedTargetPlatforms.Select<UnrealTargetPlatform, string>(x => x.ToString()).ToArray());
			}

			if (SupportedPrograms != null && SupportedPrograms.Length > 0)
			{
				CachedJson.AddOrSetFieldValue("SupportedPrograms", SupportedPrograms);
			}
			if (bIsPluginExtension)
			{
				CachedJson.AddOrSetFieldValue("bIsPluginExtension", bIsPluginExtension);
			}

			if (Modules != null && Modules.Count > 0)
			{
				ModuleDescriptor.UpdateJson(CachedJson, "Modules", Modules.ToArray());
			}

			LocalizationTargetDescriptor.UpdateJson(CachedJson, "LocalizationTargets", LocalizationTargets);

			if (PreBuildSteps != null)
			{
				CachedJson.AddOrSetFieldValue("PreBuildSteps", PreBuildSteps.ToJsonObject());
			}

			if (PostBuildSteps != null)
			{
				CachedJson.AddOrSetFieldValue("PostBuildSteps", PostBuildSteps.ToJsonObject());
			}

			if (Plugins != null && Plugins.Count > 0)
			{
				PluginReferenceDescriptor.UpdateJson(CachedJson, "Plugins", Plugins.ToArray());
			}
			if (DisallowedPlugins != null && DisallowedPlugins.Length > 0)
			{
				CachedJson.AddOrSetFieldValue("DisallowedPlugins", DisallowedPlugins);
			}
		}

		/// <summary>
		/// Produces any warnings and errors for the plugin descriptor
		/// </summary>
		/// <param name="FileName">File containing the plugin</param>
		public void Validate(FileReference FileName)
		{
			if (Modules != null)
			{
				foreach (ModuleDescriptor Module in Modules)
				{
					Module.Validate(FileName);
				}

				if (bIsPluginExtension)
				{
					foreach (ModuleDescriptor ChildModule in Modules)
					{
						if (ChildModule.bHasExplicitPlatforms && ChildModule.PlatformAllowList != null && ChildModule.PlatformAllowList.Count == 0)
						{
							// The order that child plugins are merged into the parent is undefined - there is no hierarchy. Only the PlatformAllowList and PlatformDenyList are currently merged into the parent
							// Having an explicity-empty list here suggests someone is trying to create a hierarchy and may get caught out if their module declaration also includes TargetAllowList, ProgramAllowList etc. as these properties will be ignored
							Log.TraceWarningOnce(FileName, $"Plugin extensions should not declare HasExplicitPlatforms with an empty PlatformAllowList. (module {ChildModule.Name})");
						}
					}
				}
			}
		}

		/// <summary>
		/// Determines if this reference enables the plugin for a given platform
		/// </summary>
		/// <param name="Platform">The platform to check</param>
		/// <returns>True if the plugin should be enabled</returns>
		public bool SupportsTargetPlatform(UnrealTargetPlatform Platform)
		{
			if (bHasExplicitPlatforms)
			{
				return SupportedTargetPlatforms != null && SupportedTargetPlatforms.Contains(Platform);
			}
			else
			{
				return SupportedTargetPlatforms == null || SupportedTargetPlatforms.Count == 0 || SupportedTargetPlatforms.Contains(Platform);
			}
		}

		/// <summary>
		/// Retrieve the list of supported target platforms as a string list
		/// </summary>
		/// <returns>String list of supported target platforms</returns>
		public string[]? GetSupportedTargetPlatformNames()
		{
			return SupportedTargetPlatforms?.Select(P => P.ToString()).ToArray();
		}

		/// <summary>
		/// Indicates whether it is acceptable for the given platform to be missing
		/// Typically for platform extensions that are not covered by NDA in third party plugins
		/// </summary>
		/// <param name="Platform"></param>
		/// <param name="PluginFolder"></param>
		internal static bool IsAllowableMissingPlatform(string Platform, DirectoryReference PluginFolder)
		{
			// certain directories should always show "best practice" and enforce plugin extensions
			DirectoryReference[] EnforcedDirectories = new DirectoryReference[]
			{
				DirectoryReference.Combine(UnrealBuildBase.Unreal.EngineDirectory, "Plugins"),
				DirectoryReference.Combine(UnrealBuildBase.Unreal.RootDirectory, "Samples"),
			};
			if (EnforcedDirectories.Any( X => PluginFolder.IsUnderDirectory(X) ) )
			{
				return false;
			}

			// all other plugin locations allow some unknown platforms (internal code, game projects etc)
			string[] AllowedMissingPlatforms = new string[] 
			{ 
				"WinGDK",
			};
			return AllowedMissingPlatforms.Contains(Platform);
		}

		/// <summary>
		/// Determines whether the given plugin module is part of the current build.
		/// </summary>
		/// <param name="Name">The name of the module</param>
		/// <param name="Platform">The platform being compiled for</param>
		/// <param name="Configuration">The target configuration being compiled for</param>
		/// <param name="TargetName">Name of the target being built</param>
		/// <param name="TargetType">The type of the target being compiled</param>
		/// <param name="bBuildDeveloperTools">Whether the configuration includes developer tools (typically UEBuildConfiguration.bBuildDeveloperTools for UBT callers)</param>
		/// <param name="bBuildRequiresCookedData">Whether the configuration requires cooked content (typically UEBuildConfiguration.bBuildRequiresCookedData for UBT callers)</param>
		/// <param name="Architectures">The architectures being built</param>
		/// <param name="invalidReason">Out parameter, reason why this plugin module is invalid</param>
		public bool IsModuleCompiledInConfiguration(string Name, UnrealTargetPlatform Platform, UnrealTargetConfiguration Configuration, string TargetName, TargetType TargetType, bool bBuildDeveloperTools, bool bBuildRequiresCookedData, UnrealArchitectures? Architectures, [NotNullWhen(false)] out string? invalidReason)
		{
			IEnumerable<ModuleDescriptor>? Descriptors = Modules?.Where(x => x.Name == Name);
			if (Descriptors == null)
			{
				invalidReason = null;
				return true;
			}

			HashSet<string> invalidReasons = new();

			foreach (ModuleDescriptor ModuleInfo in Descriptors)
			{
				if (!ModuleInfo.IsCompiledInConfiguration(Platform, Configuration, TargetName, TargetType, bBuildDeveloperTools, bBuildRequiresCookedData, Architectures, out string? InvalidReason))
				{
					invalidReasons.Add(InvalidReason);
				}
				else
				{
					invalidReason = null;
					return true;
				}
			}

			invalidReason = invalidReasons.Any() ? String.Join(", ", invalidReasons.Order()) : null;
			return invalidReason == null;
		}
	}
}
