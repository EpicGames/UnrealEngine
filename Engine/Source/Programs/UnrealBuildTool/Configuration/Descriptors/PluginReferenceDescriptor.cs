// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics;
using System.Linq;
using EpicGames.Core;
using JsonExtensions;

namespace UnrealBuildTool
{
	/// <summary>
	/// Representation of a reference to a plugin from a project file
	/// </summary>
	[DebuggerDisplay("Name={Name}")]
	public class PluginReferenceDescriptor
	{
		/// <summary>
		/// Name of the plugin
		/// </summary>
		public string Name;

		/// <summary>
		/// Whether it should be enabled by default
		/// </summary>
		public bool bEnabled;

		/// <summary>
		/// Whether this plugin is optional, and the game should silently ignore it not being present
		/// </summary>
		public bool bOptional;

		/// <summary>
		/// Whether this plugin should be activated, if it is a Game Feature Plugin
		/// </summary>
		public bool bActivate;

		/// <summary>
		/// Description of the plugin for users that do not have it installed.
		/// </summary>
		public string? Description;

		/// <summary>
		/// URL for this plugin on the marketplace, if the user doesn't have it installed.
		/// </summary>
		public string? MarketplaceURL;

		/// <summary>
		/// If enabled, list of platforms for which the plugin should be enabled (or all platforms if blank).
		/// </summary>
		public string[]? PlatformAllowList;

		/// <summary>
		/// If enabled, list of platforms for which the plugin should be disabled.
		/// </summary>
		public string[]? PlatformDenyList;

		/// <summary>
		/// If enabled, list of target configurations for which the plugin should be enabled (or all target configurations if blank).
		/// </summary>
		public UnrealTargetConfiguration[]? TargetConfigurationAllowList;

		/// <summary>
		/// If enabled, list of target configurations for which the plugin should be disabled.
		/// </summary>
		public UnrealTargetConfiguration[]? TargetConfigurationDenyList;

		/// <summary>
		/// If enabled, list of targets for which the plugin should be enabled (or all targets if blank).
		/// </summary>
		public TargetType[]? TargetAllowList;

		/// <summary>
		/// If enabled, list of targets for which the plugin should be disabled.
		/// </summary>
		public TargetType[]? TargetDenyList;

		/// <summary>
		/// Overrides the loading phase in uplugin so that a plugin can be explicitly loaded for specified targets.
		/// </summary>
		public TargetType[]? TargetsToExplicitlyLoad;

		/// <summary>
		/// The list of supported platforms for this plugin. This field is copied from the plugin descriptor, and supplements the user's allowed/denied platforms.
		/// </summary>
		public string[]? SupportedTargetPlatforms;

		/// <summary>
		/// When true, empty SupportedTargetPlatforms and PlatformAllowList are interpreted as 'no platforms' with the expectation that explicit platforms will be added in plugin platform extensions
		/// </summary>
		public bool bHasExplicitPlatforms;

		/// <summary>
		/// When set, specifies a specific version of the plugin that this references.
		/// </summary>
		public int? RequestedVersion;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="InName">Name of the plugin</param>
		/// <param name="InMarketplaceURL">The marketplace URL for plugins which are not installed</param>
		/// <param name="bInEnabled">Whether the plugin is enabled</param>
#pragma warning disable CA1054 // URI parameters should not be strings - URL is not processed, so does not need to be modelled or validated
		public PluginReferenceDescriptor(string InName, string? InMarketplaceURL, bool bInEnabled)
#pragma warning restore CA1054
		{
			Name = InName;
			MarketplaceURL = InMarketplaceURL;
			bEnabled = bInEnabled;
		}

		/// <summary>
		/// Construct a PluginReferenceDescriptor from a Json object
		/// </summary>
		/// <param name="Writer">The writer for output fields</param>
		public void Write(JsonWriter Writer)
		{
			Writer.WriteObjectStart();
			Writer.WriteValue("Name", Name);
			Writer.WriteValue("Enabled", bEnabled);
			if (bEnabled && bOptional)
			{
				Writer.WriteValue("Optional", bOptional);
			}
			if (bEnabled && bActivate)
			{
				Writer.WriteValue("Activate", bActivate);
			}
			if (!String.IsNullOrEmpty(Description))
			{
				Writer.WriteValue("Description", Description);
			}
			if (!String.IsNullOrEmpty(MarketplaceURL))
			{
				Writer.WriteValue("MarketplaceURL", MarketplaceURL);
			}
			if (PlatformAllowList != null && PlatformAllowList.Length > 0)
			{
				Writer.WriteStringArrayField("PlatformAllowList", PlatformAllowList.Select(x => x.ToString()).ToArray());
			}
			if (PlatformDenyList != null && PlatformDenyList.Length > 0)
			{
				Writer.WriteStringArrayField("PlatformDenyList", PlatformDenyList.Select(x => x.ToString()).ToArray());
			}
			if (TargetConfigurationAllowList != null && TargetConfigurationAllowList.Length > 0)
			{
				Writer.WriteEnumArrayField("TargetConfigurationAllowList", TargetConfigurationAllowList);
			}
			if (TargetConfigurationDenyList != null && TargetConfigurationDenyList.Length > 0)
			{
				Writer.WriteEnumArrayField("TargetConfigurationDenyList", TargetConfigurationDenyList);
			}
			if (TargetAllowList != null && TargetAllowList.Length > 0)
			{
				Writer.WriteEnumArrayField("TargetAllowList", TargetAllowList);
			}
			if (TargetDenyList != null && TargetDenyList.Length > 0)
			{
				Writer.WriteEnumArrayField("TargetDenyList", TargetDenyList);
			}
			if (TargetsToExplicitlyLoad != null && TargetsToExplicitlyLoad.Length > 0)
			{
				Writer.WriteEnumArrayField("TargetsToExplicitlyLoad", TargetsToExplicitlyLoad);
			}
			if (SupportedTargetPlatforms != null && SupportedTargetPlatforms.Length > 0)
			{
				Writer.WriteStringArrayField("SupportedTargetPlatforms", SupportedTargetPlatforms.Select(x => x.ToString()).ToArray());
			}
			if (bHasExplicitPlatforms)
			{
				Writer.WriteValue("HasExplicitPlatforms", bHasExplicitPlatforms);
			}
			if (bEnabled && RequestedVersion != null)
			{
				Writer.WriteValue("Version", RequestedVersion.Value);
			}
			Writer.WriteObjectEnd();
		}

		private JsonObject ToJsonObject()
		{
			JsonObject PluginReferenceObject = new JsonObject();
			PluginReferenceObject.AddOrSetFieldValue("Name", Name);
			PluginReferenceObject.AddOrSetFieldValue("Enabled", bEnabled);
			if (bEnabled && bOptional)
			{
				PluginReferenceObject.AddOrSetFieldValue("Optional", bOptional);
			}
			if (bEnabled && bActivate)
			{
				PluginReferenceObject.AddOrSetFieldValue("Activate", bActivate);
			}
			if (!String.IsNullOrEmpty(Description))
			{
				PluginReferenceObject.AddOrSetFieldValue("Description", Description);
			}
			if (!String.IsNullOrEmpty(MarketplaceURL))
			{
				PluginReferenceObject.AddOrSetFieldValue("MarketplaceURL", MarketplaceURL);
			}
			if (PlatformAllowList != null && PlatformAllowList.Length > 0)
			{
				PluginReferenceObject.AddOrSetFieldValue("PlatformAllowList", PlatformAllowList.Select(x => x.ToString()).ToArray());
			}
			if (PlatformDenyList != null && PlatformDenyList.Length > 0)
			{
				PluginReferenceObject.AddOrSetFieldValue("PlatformDenyList", PlatformDenyList.Select(x => x.ToString()).ToArray());
			}
			if (TargetConfigurationAllowList != null && TargetConfigurationAllowList.Length > 0)
			{
				PluginReferenceObject.AddOrSetFieldValue("TargetConfigurationAllowList", TargetConfigurationAllowList);
			}
			if (TargetConfigurationDenyList != null && TargetConfigurationDenyList.Length > 0)
			{
				PluginReferenceObject.AddOrSetFieldValue("TargetConfigurationDenyList", TargetConfigurationDenyList);
			}
			if (TargetAllowList != null && TargetAllowList.Length > 0)
			{
				PluginReferenceObject.AddOrSetFieldValue("TargetAllowList", TargetAllowList);
			}
			if (TargetDenyList != null && TargetDenyList.Length > 0)
			{
				PluginReferenceObject.AddOrSetFieldValue("TargetDenyList", TargetDenyList);
			}
			if (TargetsToExplicitlyLoad != null && TargetsToExplicitlyLoad.Length > 0)
			{
				PluginReferenceObject.AddOrSetFieldValue("TargetsToExplicitlyLoad", TargetsToExplicitlyLoad);
			}
			if (SupportedTargetPlatforms != null && SupportedTargetPlatforms.Length > 0)
			{
				PluginReferenceObject.AddOrSetFieldValue("SupportedTargetPlatforms", SupportedTargetPlatforms.Select(x => x.ToString()).ToArray());
			}
			if (bHasExplicitPlatforms)
			{
				PluginReferenceObject.AddOrSetFieldValue("HasExplicitPlatforms", bHasExplicitPlatforms);
			}
			if (bEnabled && RequestedVersion != null)
			{
				PluginReferenceObject.AddOrSetFieldValue("Version", RequestedVersion.Value);
			}
			return PluginReferenceObject;
		}

		/// <summary>
		/// Write an array of module descriptors
		/// </summary>
		/// <param name="Writer">The Json writer to output to</param>
		/// <param name="Name">Name of the array</param>
		/// <param name="Plugins">Array of plugins</param>
		public static void WriteArray(JsonWriter Writer, string Name, PluginReferenceDescriptor[]? Plugins)
		{
			if (Plugins != null && Plugins.Length > 0)
			{
				Writer.WriteArrayStart(Name);
				foreach (PluginReferenceDescriptor Plugin in Plugins)
				{
					Plugin.Write(Writer);
				}
				Writer.WriteArrayEnd();
			}
		}

		/// <summary>
		/// Updates the json object with an array of plugin descriptors.
		/// </summary>
		/// <param name="InObject">The Json object to update.</param>
		/// <param name="Name">Name of the array</param>
		/// <param name="Plugins">Array of plugins</param>
		public static void UpdateJson(JsonObject InObject, string Name, PluginReferenceDescriptor[]? Plugins)
		{
			if (Plugins != null && Plugins.Length > 0)
			{
				JsonObject[] JsonObjects = Plugins.Select(X => X.ToJsonObject()).ToArray();
				InObject.AddOrSetFieldValue(Name, JsonObjects);
			}
		}

		/// <summary>
		/// Checks the given platform names array and logs a message about any that are not known. This may simply be due to not having the platform synced in Engine/Platforms/[PlatformName].
		/// </summary>
		/// <param name="PlatformNames"></param>
		private void VerifyPlatformNames(string[]? PlatformNames)
		{
			if (PlatformNames != null)
			{
				foreach (string PlatformName in PlatformNames)
				{
					if (!UnrealTargetPlatform.TryParse(PlatformName, out _))
					{
						Log.TraceLogOnce("Ignoring unknown platform '{0}' (referenced via a project's plugin descriptor for '{1}')", PlatformName, Name);
					}
				}
			}
		}

		/// <summary>
		/// Construct a PluginReferenceDescriptor from a Json object
		/// </summary>
		/// <param name="RawObject">The Json object containing a plugin reference descriptor</param>
		/// <returns>New PluginReferenceDescriptor object</returns>
		public static PluginReferenceDescriptor FromJsonObject(JsonObject RawObject)
		{
			string? PluginReferenceName = null;
			try
			{
				PluginReferenceName = RawObject.GetStringField("Name");

				PluginReferenceDescriptor Descriptor = new PluginReferenceDescriptor(PluginReferenceName, null, RawObject.GetBoolField("Enabled"));
				RawObject.TryGetBoolField("Optional", out Descriptor.bOptional);
				RawObject.TryGetBoolField("Activate", out Descriptor.bActivate);
				RawObject.TryGetStringField("Description", out Descriptor.Description);
				RawObject.TryGetStringField("MarketplaceURL", out Descriptor.MarketplaceURL);

				// Only parse platform information if enabled
				if (Descriptor.bEnabled)
				{
					RawObject.TryGetStringArrayFieldWithDeprecatedFallback("PlatformAllowList", "WhitelistPlatforms", out Descriptor.PlatformAllowList);
					RawObject.TryGetStringArrayFieldWithDeprecatedFallback("PlatformDenyList", "BlacklistPlatforms", out Descriptor.PlatformDenyList);

					Descriptor.TargetAllowList = RawObject.GetEnumArrayFieldWithDeprecatedFallback<TargetType>("TargetAllowList", "WhitelistTargets");
					Descriptor.TargetDenyList = RawObject.GetEnumArrayFieldWithDeprecatedFallback<TargetType>("TargetDenyList", "BlacklistTargets");
					Descriptor.TargetConfigurationAllowList = RawObject.GetEnumArrayFieldWithDeprecatedFallback<UnrealTargetConfiguration>("TargetConfigurationAllowList", "WhitelistTargetConfigurations");
					Descriptor.TargetConfigurationDenyList = RawObject.GetEnumArrayFieldWithDeprecatedFallback<UnrealTargetConfiguration>("TargetConfigurationDenyList", "BlacklistTargetConfigurations");

					RawObject.TryGetEnumArrayField<TargetType>("TargetsToExplicitlyLoad", out Descriptor.TargetsToExplicitlyLoad);
					RawObject.TryGetStringArrayField("SupportedTargetPlatforms", out Descriptor.SupportedTargetPlatforms);
					RawObject.TryGetBoolField("HasExplicitPlatforms", out Descriptor.bHasExplicitPlatforms);

					int RequestedVersion = -1;
					if (RawObject.TryGetIntegerField("Version", out RequestedVersion))
					{
						Descriptor.RequestedVersion = RequestedVersion;
					}

					Descriptor.VerifyPlatformNames(Descriptor.PlatformAllowList);
					Descriptor.VerifyPlatformNames(Descriptor.PlatformDenyList);
					Descriptor.VerifyPlatformNames(Descriptor.SupportedTargetPlatforms);
				}

				return Descriptor;
			}
			catch (Exception Ex)
			{
				ExceptionUtils.AddContext(Ex, "while parsing plugin reference descriptor '{0}'", PluginReferenceName ?? "<unknown>");
				throw;
			}
		}

		/// <summary>
		/// Determines if this reference enables the plugin for a given platform
		/// </summary>
		/// <param name="Platform">The platform to check</param>
		/// <returns>True if the plugin should be enabled</returns>
		public bool IsEnabledForPlatform(UnrealTargetPlatform Platform)
		{
			if (!bEnabled)
			{
				return false;
			}
			if (bHasExplicitPlatforms)
			{
				if (PlatformAllowList == null || !PlatformAllowList.Contains(Platform.ToString()))
				{
					return false;
				}
			}
			else if (PlatformAllowList != null && PlatformAllowList.Length > 0 && !PlatformAllowList.Contains(Platform.ToString()))
			{
				return false;
			}
			if (PlatformDenyList != null && PlatformDenyList.Contains(Platform.ToString()))
			{
				return false;
			}
			return true;
		}

		/// <summary>
		/// Determines if this reference enables the plugin for a given target configuration
		/// </summary>
		/// <param name="TargetConfiguration">The target configuration to check</param>
		/// <returns>True if the plugin should be enabled</returns>
		public bool IsEnabledForTargetConfiguration(UnrealTargetConfiguration TargetConfiguration)
		{
			if (!bEnabled)
			{
				return false;
			}
			if (TargetConfigurationAllowList != null && TargetConfigurationAllowList.Length > 0 && !TargetConfigurationAllowList.Contains(TargetConfiguration))
			{
				return false;
			}
			if (TargetConfigurationDenyList != null && TargetConfigurationDenyList.Contains(TargetConfiguration))
			{
				return false;
			}
			return true;
		}

		/// <summary>
		/// Determines if this reference enables the plugin for a given target
		/// </summary>
		/// <param name="Target">The target to check</param>
		/// <returns>True if the plugin should be enabled</returns>
		public bool IsEnabledForTarget(TargetType Target)
		{
			if (!bEnabled)
			{
				return false;
			}
			if (TargetAllowList != null && TargetAllowList.Length > 0 && !TargetAllowList.Contains(Target))
			{
				return false;
			}
			if (TargetDenyList != null && TargetDenyList.Contains(Target))
			{
				return false;
			}
			return true;
		}

		/// <summary>
		/// Determines if this reference is valid for the given target platform.
		/// </summary>
		/// <param name="Platform">The platform to check</param>
		/// <returns>True if the plugin for this target platform</returns>
		public bool IsSupportedTargetPlatform(UnrealTargetPlatform Platform)
		{
			if (bHasExplicitPlatforms)
			{
				return SupportedTargetPlatforms != null && SupportedTargetPlatforms.Contains(Platform.ToString());
			}
			else
			{
				return SupportedTargetPlatforms == null || SupportedTargetPlatforms.Length == 0 || SupportedTargetPlatforms.Contains(Platform.ToString());
			}
		}
	}
}
