// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text.Json;
using EpicGames.Core;

namespace UnrealBuildTool
{
	/// <summary>
	/// The version format for .uproject files. This rarely changes now; project descriptors should maintain backwards compatibility automatically.
	/// </summary>
	enum ProjectDescriptorVersion
	{
		/// <summary>
		/// Invalid
		/// </summary>
		Invalid = 0,

		/// <summary>
		/// Initial version
		/// </summary>
		Initial = 1,

		/// <summary>
		/// Adding SampleNameHash
		/// </summary>
		NameHash = 2,

		/// <summary>
		/// Unifying plugin/project files (since abandoned, but backwards compatibility maintained)
		/// </summary>
		ProjectPluginUnification = 3,

		/// <summary>
		/// This needs to be the last line, so we can calculate the value of Latest below
		/// </summary>
		LatestPlusOne,

#pragma warning disable CA1069 // Enums should not have duplicate values - having access to Latest is useful
		/// <summary>
		/// The latest plugin descriptor version
		/// </summary>
		Latest = LatestPlusOne - 1
#pragma warning restore CA1069
	}

	/// <summary>
	/// In-memory representation of a .uproject file
	/// </summary>
	public class ProjectDescriptor
	{
		/// <summary>
		/// Descriptor version number.
		/// </summary>
		public int FileVersion;

		/// <summary>
		/// The engine to open this project with.
		/// </summary>
		public string? EngineAssociation;

		/// <summary>
		/// Category to show under the project browser
		/// </summary>
		public string? Category;

		/// <summary>
		/// Description to show in the project browser
		/// </summary>
		public string? Description;

		/// <summary>
		/// List of all modules associated with this project
		/// </summary>
		public ModuleDescriptor[]? Modules;

		/// <summary>
		/// List of plugins for this project (may be enabled/disabled)
		/// </summary>
		public PluginReferenceDescriptor[]? Plugins;

		/// <summary>
		/// Array of additional root directories
		/// </summary>
		public List<DirectoryReference> AdditionalRootDirectories = new List<DirectoryReference>();

		/// <summary>
		/// List of additional plugin directories to scan for available plugins
		/// </summary>
		public List<DirectoryReference> AdditionalPluginDirectories = new List<DirectoryReference>();

		/// <summary>
		/// Array of platforms that this project is targeting
		/// </summary>
		public string[]? TargetPlatforms;

		/// <summary>
		/// A hash that is used to determine if the project was forked from a sample
		/// </summary>
		public uint EpicSampleNameHash;

		/// <summary>
		/// Steps to execute before creating rules assemblies in this project
		/// </summary>
		public CustomBuildSteps? InitSteps;

		/// <summary>
		/// Steps to execute before building targets in this project
		/// </summary>
		public CustomBuildSteps? PreBuildSteps;

		/// <summary>
		/// Steps to execute before building targets in this project
		/// </summary>
		public CustomBuildSteps? PostBuildSteps;

		/// <summary>
		/// Indicates if this project is an Enterprise project
		/// </summary>
		public bool IsEnterpriseProject;

		/// <summary>
		/// Indicates that enabled by default engine plugins should not be enabled unless explicitly enabled by the project or target files.
		/// </summary>
		public bool DisableEnginePluginsByDefault;

		/// <summary>
		/// Constructor.
		/// </summary>
		public ProjectDescriptor()
		{
			FileVersion = (int)ProjectDescriptorVersion.Latest;
			IsEnterpriseProject = false;
			DisableEnginePluginsByDefault = false;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="RawObject">Raw JSON object to parse</param>
		/// <param name="BaseDir">Base directory for resolving relative paths</param>
		/// <param name="JsonFilePath"></param>
		public ProjectDescriptor(JsonObject RawObject, DirectoryReference BaseDir, FileReference JsonFilePath)
		{
			// Read the version
			if (!RawObject.TryGetIntegerField("FileVersion", out FileVersion))
			{
				if (!RawObject.TryGetIntegerField("ProjectFileVersion", out FileVersion))
				{
					throw new BuildException("Project does not contain a valid FileVersion entry");
				}
			}

			// Check it's not newer than the latest version we can parse
			if (FileVersion > (int)PluginDescriptorVersion.Latest)
			{
				throw new BuildException("Project descriptor appears to be in a newer version ({0}) of the file format that we can load (max version: {1}).", FileVersion, (int)ProjectDescriptorVersion.Latest);
			}

			// Read simple fields
			RawObject.TryGetStringField("EngineAssociation", out EngineAssociation);
			RawObject.TryGetStringField("Category", out Category);
			RawObject.TryGetStringField("Description", out Description);
			RawObject.TryGetBoolField("Enterprise", out IsEnterpriseProject);
			RawObject.TryGetBoolField("DisableEnginePluginsByDefault", out DisableEnginePluginsByDefault);

			// Read the modules
			JsonObject[]? ModulesArray;
			if (RawObject.TryGetObjectArrayField("Modules", out ModulesArray))
			{
				Modules = Array.ConvertAll(ModulesArray, x => ModuleDescriptor.FromJsonObject(x, JsonFilePath));
			}

			// Read the plugins
			JsonObject[]? PluginsArray;
			if (RawObject.TryGetObjectArrayField("Plugins", out PluginsArray))
			{
				Plugins = Array.ConvertAll(PluginsArray, x => PluginReferenceDescriptor.FromJsonObject(x));
			}

			// Read the additional root directories
			string[]? RootDirectoryStrings;
			if (RawObject.TryGetStringArrayField("AdditionalRootDirectories", out RootDirectoryStrings))
			{
				AdditionalRootDirectories.AddRange(RootDirectoryStrings.Select(x => DirectoryReference.Combine(BaseDir, x)));
			}

			// Read the additional plugin directories
			string[]? PluginDirectoryStrings;
			if (RawObject.TryGetStringArrayField("AdditionalPluginDirectories", out PluginDirectoryStrings))
			{
				AdditionalPluginDirectories.AddRange(PluginDirectoryStrings.Select(x => DirectoryReference.Combine(BaseDir, x)));
			}

			// Read the target platforms
			RawObject.TryGetStringArrayField("TargetPlatforms", out TargetPlatforms);

			// Get the sample name hash
			RawObject.TryGetUnsignedIntegerField("EpicSampleNameHash", out EpicSampleNameHash);

			// Read the init, pre and post-build steps
			CustomBuildSteps.TryRead(RawObject, "InitSteps", out InitSteps);
			CustomBuildSteps.TryRead(RawObject, "PreBuildSteps", out PreBuildSteps);
			CustomBuildSteps.TryRead(RawObject, "PostBuildSteps", out PostBuildSteps);
		}

		/// <summary>
		/// Creates a plugin descriptor from a file on disk
		/// </summary>
		/// <param name="FileName">The filename to read</param>
		/// <returns>New plugin descriptor</returns>
		public static ProjectDescriptor FromFile(FileReference FileName)
		{
			try
			{
				JsonObject RawObject = JsonObject.Read(FileName);

				ProjectDescriptor Descriptor = new ProjectDescriptor(RawObject, FileName.Directory, FileName);
				if (Descriptor.Modules != null)
				{
					foreach (ModuleDescriptor Module in Descriptor.Modules)
					{
						Module.Validate(FileName);
					}
				}
				return Descriptor;
			}
			catch (JsonException ex)
			{
				throw new JsonException($"{ex.Message} (in {FileName})", FileName.FullName, ex.LineNumber, ex.BytePositionInLine, ex);
			}
			catch (Exception ex)
			{
				throw new Exception($"{ex.Message} (while reading project file {FileName})", ex);
			}
		}

		/// <summary>
		/// Creates a plugin descriptor based on a directory, searching for best uproject
		/// </summary>
		/// <param name="DirectoryName">The directory to search for a uproject</param>
		/// <returns>New plugin descriptor</returns>
		public static ProjectDescriptor FromDirectory(DirectoryReference DirectoryName)
		{
			FileReference ProjectFile = FileReference.Combine(DirectoryName, DirectoryName.GetDirectoryName() + ".uproject");
			if (FileReference.Exists(ProjectFile))
			{
				return ProjectDescriptor.FromFile(ProjectFile);
			}

			// find any uproject file, in case the name doesn't match
			List<FileReference> FoundProjects = [.. DirectoryReference.EnumerateFiles(DirectoryName, "*.uproject", SearchOption.TopDirectoryOnly)];
			if (FoundProjects.Count == 0)
			{
				throw new FileNotFoundException($"Unable to find a .uproject file in {DirectoryName}");
			}
			if (FoundProjects.Count > 1)
			{
				Log.TraceWarningOnce("Found multiple uproject files in {0}, choosing {1}", DirectoryName, FoundProjects[0].GetFileName());
			}
			return ProjectDescriptor.FromFile(FoundProjects[0]);
		}

		/// <summary>
		/// If the descriptor has either additional plugin directories or additional root directories
		/// then it is considered to have additional paths.  The additional paths will be relative
		/// to the provided directory.
		/// </summary>
		/// <param name="RootDirectories">The directory set to add the paths too.</param>
		/// <param name="ProjectDir">The directory which is used to setup the additional paths</param>
		public void AddAdditionalPaths(List<DirectoryReference> RootDirectories, DirectoryReference ProjectDir)
		{
			RootDirectories.AddRange(AdditionalRootDirectories);
			RootDirectories.AddRange(AdditionalPluginDirectories);
		}

		/// <summary>
		/// Saves the descriptor to disk
		/// </summary>
		/// <param name="FileName">The filename to write to</param>
		public void Save(FileReference FileName)
		{
			using (JsonWriter Writer = new JsonWriter(FileName))
			{
				Writer.WriteObjectStart();
				Write(Writer, FileName.Directory);
				Writer.WriteObjectEnd();
			}
		}

		/// <summary>
		/// Writes the plugin descriptor to an existing Json writer
		/// </summary>
		/// <param name="Writer">The writer to receive plugin data</param>
		/// <param name="BaseDir">Base directory to save paths relative to</param>
		public void Write(JsonWriter Writer, DirectoryReference BaseDir)
		{
			Writer.WriteValue("FileVersion", (int)ProjectDescriptorVersion.Latest);
			Writer.WriteValue("EngineAssociation", EngineAssociation);
			Writer.WriteValue("Category", Category);
			Writer.WriteValue("Description", Description);

			if (DisableEnginePluginsByDefault)
			{
				Writer.WriteValue("DisableEnginePluginsByDefault", DisableEnginePluginsByDefault);
			}

			// Write the enterprise flag
			if (IsEnterpriseProject)
			{
				Writer.WriteValue("Enterprise", IsEnterpriseProject);
			}

			// Write the module list
			ModuleDescriptor.WriteArray(Writer, "Modules", Modules);

			// Write the plugin list
			PluginReferenceDescriptor.WriteArray(Writer, "Plugins", Plugins);

			// Write the custom module roots
			if (AdditionalRootDirectories.Count > 0)
			{
				Writer.WriteStringArrayField("AdditionalRootDirectories", AdditionalRootDirectories.Select(x => x.MakeRelativeTo(BaseDir).Replace(Path.DirectorySeparatorChar, '/')));
			}

			// Write out the additional plugin directories to scan
			if (AdditionalPluginDirectories.Count > 0)
			{
				Writer.WriteStringArrayField("AdditionalPluginDirectories", AdditionalPluginDirectories.Select(x => x.MakeRelativeTo(BaseDir).Replace(Path.DirectorySeparatorChar, '/')));
			}

			// Write the target platforms
			if (TargetPlatforms != null && TargetPlatforms.Length > 0)
			{
				Writer.WriteStringArrayField("TargetPlatforms", TargetPlatforms);
			}

			// If it's a signed sample, write the name hash
			if (EpicSampleNameHash != 0)
			{
				Writer.WriteValue("EpicSampleNameHash", (uint)EpicSampleNameHash);
			}

			// Write the custom build steps
			InitSteps?.Write(Writer, "InitSteps");
			PreBuildSteps?.Write(Writer, "PreBuildSteps");
			PostBuildSteps?.Write(Writer, "PostBuildSteps");
		}
	}
}
