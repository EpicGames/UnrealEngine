// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using System.Linq;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// Describes all of the information needed to initialize a UEBuildTarget object
	/// </summary>
	internal sealed record class TargetDescriptor
	{
		public const string TEST_TARGETS_SUFFIX = "Tests";

		public FileReference? ProjectFile { get; set; }
		public required string Name { get; set; }
		public UnrealTargetPlatform Platform { get; set; }
		public UnrealTargetConfiguration Configuration { get; set; }
		public required UnrealArchitectures Architectures { get; set; }
		public required CommandLineArguments AdditionalArguments { get; set; }
		public bool IsTestsTarget { get; set; }

		public static string GetTestedName(string Name)
		{
			if (Name.EndsWith(TEST_TARGETS_SUFFIX, StringComparison.Ordinal))
			{
				return Name.Substring(0, Name.Length - TEST_TARGETS_SUFFIX.Length);
			}
			return Name;
		}

		/// <summary>
		/// Foreign plugin to compile against this target
		/// </summary>
		[CommandLine("-Plugin=")]
		public FileReference? ForeignPlugin { get; set; }

		/// <summary>
		/// Whether we should treat the ForeignPlugin argument as a local plugin for building purposes
		/// </summary>
		[CommandLine("-BuildPluginAsLocal")]
		public bool bBuildPluginAsLocal { get; set; }

		/// <summary>
		/// When building a foreign plugin, whether to build plugins it depends on as well.
		/// </summary>
		[CommandLine("-BuildDependantPlugins")]
		public bool bBuildDependantPlugins { get; set; }

		/// <summary>
		/// Set of module names to compile.
		/// </summary>
		[CommandLine("-Module=", ListSeparator = '+')]
		public HashSet<string> OnlyModuleNames { get; } = new(StringComparer.OrdinalIgnoreCase);

		/// <summary>
		/// Lists of files to compile
		/// </summary>
		[CommandLine("-FileList=")]
		public List<FileReference> FileLists { get; } = [];

		/// <summary>
		/// Individual file(s) to compile
		/// </summary>
		[CommandLine("-File=")]
		[CommandLine("-SingleFile=")]
		public List<FileReference> SpecificFilesToCompile { get; } = [];

		/// <summary>
		/// Relative path to file(s) to compile
		/// </summary>
		[CommandLine("-Files=", ListSeparator = ';')]
		public List<string> RelativePathsToSpecificFilesToCompile { get; } = [];

		/// <summary>
		/// Working directory when compiling with RelativePathsToSpecificFilesToCompile
		/// </summary>
		[CommandLine("-WorkingDir=")]
		public string? WorkingDir { get; set; }

		/// <summary>
		/// Will build all files that directly include any of the files provided in -SingleFile, not including generated code
		/// </summary>
		[CommandLine("-SingleFileBuildDependents")]
		public bool bSingleFileBuildDependents { get; set; }

		/// <summary>
		/// Whether to perform hot reload for this target
		/// </summary>
		[CommandLine("-NoHotReload", Value = nameof(HotReloadMode.Disabled))]
		[CommandLine("-ForceHotReload", Value = nameof(HotReloadMode.FromIDE))]
		[CommandLine("-LiveCoding", Value = nameof(HotReloadMode.LiveCoding))]
		public HotReloadMode HotReloadMode { get; set; }

		/// <summary>
		/// Map of module name to suffix for hot reloading from the editor
		/// </summary>
		public Dictionary<string, int> HotReloadModuleNameToSuffix { get; } = new(StringComparer.OrdinalIgnoreCase);

		/// <summary>
		/// Path to a file containing a list of modules that may be modified for live coding.
		/// </summary>
		[CommandLine("-LiveCodingModules=")]
		public FileReference? LiveCodingModules { get; set; }

		/// <summary>
		/// Path to the manifest for passing info about the output to live coding
		/// </summary>
		[CommandLine("-LiveCodingManifest=")]
		public FileReference? LiveCodingManifest { get; set; }

		/// <summary>
		/// If a non-zero value, a live coding request will be terminated if more than the given number of actions are required.
		/// </summary>
		[CommandLine("-LiveCodingLimit=")]
		public uint LiveCodingLimit { get; set; }

		/// <summary>
		/// Suppress messages about building this target
		/// </summary>
		[CommandLine("-Quiet")]
		public bool bQuiet { get; set; }

		/// <summary>
		/// Clean the target before trying to build it
		/// </summary>
		[CommandLine("-Rebuild")]
		public bool bRebuild { get; set; }

		/// <summary>
		/// Whether to unify C++ code into larger files for faster compilation.
		/// </summary>
		[CommandLine("-DisableUnity", Value = "false")]
		public bool bUseUnityBuild { get; set; } = true;

		/// <summary>
		/// Whether to force C++ source files to be combined into larger files for faster compilation.
		/// </summary>
		[CommandLine("-ForceUnity")]
		public bool bForceUnityBuild { get; set; }

		/// <summary>
		/// Enables "include what you use" mode.
		/// </summary>
		[CommandLine("-IWYU")]
		public bool bIWYU { get; set; }

		/// <summary>
		/// Set when trying to profile a compile to make sure the environment is setup correctly.
		/// </summary>
		[CommandLine("-ProfilingCompile")]
		public bool bProfilingCompile { get; set; }

		/// <summary>
		/// Intermediate environment. Determines if the intermediates end up in a different folder than normal.
		/// </summary>
		public UnrealIntermediateEnvironment IntermediateEnvironment
		{
			get
			{
				if (IntermediateEnvironmentOverride.HasValue)
				{
					return IntermediateEnvironmentOverride.Value;
				}
				if (bIWYU)
				{
					return UnrealIntermediateEnvironment.IWYU;
				}
				if (!bUseUnityBuild && !bForceUnityBuild)
				{
					return UnrealIntermediateEnvironment.NonUnity;
				}
				if (bProfilingCompile)
				{
					return UnrealIntermediateEnvironment.ProfilingCompile;
				}
				return UnrealIntermediateEnvironment.Default;
			}
			set => IntermediateEnvironmentOverride = value;
		}
		private UnrealIntermediateEnvironment? IntermediateEnvironmentOverride;

		public OriginalTargetDescriptor Original { get; private set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="ProjectFile">Path to the project file</param>
		/// <param name="TargetName">Name of the target to build</param>
		/// <param name="Platform">Platform to build for</param>
		/// <param name="Configuration">Configuration to build</param>
		/// <param name="Architectures">Architectures to build for</param>
		/// <param name="Arguments">Other command-line arguments for the target</param>
		[SetsRequiredMembers]
		public TargetDescriptor(FileReference? ProjectFile, string TargetName, UnrealTargetPlatform Platform, UnrealTargetConfiguration Configuration, UnrealArchitectures Architectures, CommandLineArguments? Arguments)
		{
			this.ProjectFile = ProjectFile;
			Name = TargetName;
			this.Platform = Platform;
			this.Configuration = Configuration;
			if (Architectures == null)
			{
				this.Architectures = UnrealArchitectureConfig.ForPlatform(Platform).ActiveArchitectures(ProjectFile, TargetName);
			}
			else
			{
				this.Architectures = Architectures;
			}

			// If there are any additional command line arguments
			List<string> AdditionalArguments = new List<string>();
			if (Arguments != null)
			{
				// Apply the arguments to this object
				Arguments.ApplyTo(this);

				// Read the file lists
				foreach (FileReference FileList in FileLists)
				{
					string[] Files = FileReference.ReadAllLines(FileList);
					foreach (string File in Files.Where(x => !String.IsNullOrWhiteSpace(x)))
					{
						SpecificFilesToCompile.Add(FileReference.Combine(Unreal.RootDirectory, File));
					}
				}

				// Create the full path for the files specified in RelativePathsToSpecificFilesToCompile
				DirectoryReference CurrentWorkingDir = Unreal.RootDirectory;
				if (WorkingDir != null)
				{
					CurrentWorkingDir = new DirectoryReference(WorkingDir);
				}

				foreach (string RelativeFilePath in RelativePathsToSpecificFilesToCompile)
				{
					SpecificFilesToCompile.Add(FileReference.Combine(CurrentWorkingDir, RelativeFilePath));
				}

				// Parse all the hot-reload module names
				foreach (string ModuleWithSuffix in Arguments.GetValues("-ModuleWithSuffix="))
				{
					int SuffixIdx = ModuleWithSuffix.LastIndexOf(',');
					if (SuffixIdx == -1)
					{
						throw new BuildException("Missing suffix argument from -ModuleWithSuffix=Name,Suffix");
					}

					string ModuleName = ModuleWithSuffix.Substring(0, SuffixIdx);

					int Suffix;
					if (!Int32.TryParse(ModuleWithSuffix.Substring(SuffixIdx + 1), out Suffix))
					{
						throw new BuildException("Suffix for modules must be an integer");
					}

					HotReloadModuleNameToSuffix[ModuleName] = Suffix;
				}

				// Pull out all the arguments that haven't been used so far
				for (int Idx = 0; Idx < Arguments.Count; Idx++)
				{
					if (!Arguments.HasBeenUsed(Idx))
					{
						AdditionalArguments.Add(Arguments[Idx]);
					}
				}
			}
			this.AdditionalArguments = new CommandLineArguments(AdditionalArguments.ToArray());

			Original = OriginalTargetDescriptor.From(this);
		}

		public static TargetDescriptor FromTargetInfo(TargetInfo Info)
		{
			return new TargetDescriptor(Info.ProjectFile, Info.Name, Info.Platform, Info.Configuration, Info.Architectures, Info.Arguments);
		}

		/// <summary>
		/// Parse a list of target descriptors from the command line
		/// </summary>
		/// <param name="Arguments">Command-line arguments</param>
		/// <param name="BuildConfiguration">Build configuration to get common flags from</param>
		/// <param name="Logger">Logger for output</param>
		/// <returns>List of target descriptors</returns>
		public static List<TargetDescriptor> ParseCommandLine(CommandLineArguments Arguments, BuildConfiguration BuildConfiguration, ILogger Logger)
		{
			List<TargetDescriptor> TargetDescriptors = new List<TargetDescriptor>();
			ParseCommandLine(Arguments, BuildConfiguration.bUsePrecompiled, BuildConfiguration.bSkipRulesCompile, BuildConfiguration.bForceRulesCompile, TargetDescriptors, Logger);

			// apply the intermediate environment from the build configuration
			if (BuildConfiguration.IntermediateEnvironment.HasValue)
			{
				foreach (TargetDescriptor TargetDescriptor in TargetDescriptors)
				{
					TargetDescriptor.IntermediateEnvironment = BuildConfiguration.IntermediateEnvironment.Value;
					// We have to remake the original to include the corrected IntermediateEnvironment.
					// This should be included in the original as it's still before it's been handed over to the user.
					TargetDescriptor.Original = OriginalTargetDescriptor.From(TargetDescriptor);
				}
			}

			return TargetDescriptors;
		}

		/// <summary>
		/// Parse a list of target descriptors from the command line
		/// </summary>
		/// <param name="Arguments">Command-line arguments</param>
		/// <param name="bUsePrecompiled">Whether to use a precompiled engine distribution</param>
		/// <param name="bSkipRulesCompile">Whether to skip compiling rules assemblies</param>
		/// <param name="bForceRulesCompile">Whether to always compile all rules assemblies</param>
		/// <param name="Logger">Logger for output</param>
		/// <returns>List of target descriptors</returns>
		public static List<TargetDescriptor> ParseCommandLine(CommandLineArguments Arguments, bool bUsePrecompiled, bool bSkipRulesCompile, bool bForceRulesCompile, ILogger Logger)
		{
			List<TargetDescriptor> TargetDescriptors = new List<TargetDescriptor>();
			ParseCommandLine(Arguments, bUsePrecompiled, bSkipRulesCompile, bForceRulesCompile, TargetDescriptors, Logger);
			return TargetDescriptors;
		}

		public static IEnumerable<FileReference?> ParseCommandLineForProjects(CommandLineArguments Arguments, ILogger Logger)
		{
			List<TargetDescriptor> SimplifiedTargetDescriptors = new List<TargetDescriptor>();
			ParseCommandLine(Arguments, false, true, false, bCreateSimplifiedDescriptors: true, SimplifiedTargetDescriptors, Logger);
			return SimplifiedTargetDescriptors.Select(x => x.ProjectFile);
		}

		/// <summary>
		/// Parse a list of target descriptors from the command line
		/// </summary>
		/// <param name="Arguments">Command-line arguments</param>
		/// <param name="bUsePrecompiled">Whether to use a precompiled engine distribution</param>
		/// <param name="bSkipRulesCompile">Whether to skip compiling rules assemblies</param>
		/// <param name="bForceRulesCompile">Whether to always compile rules assemblies</param>
		/// <param name="TargetDescriptors">Receives the list of parsed target descriptors</param>
		/// <param name="Logger">Logger for output</param>
		public static void ParseCommandLine(CommandLineArguments Arguments, bool bUsePrecompiled, bool bSkipRulesCompile, bool bForceRulesCompile, List<TargetDescriptor> TargetDescriptors, ILogger Logger)
		{
			// call the internal one
			ParseCommandLine(Arguments, bUsePrecompiled, bSkipRulesCompile, bForceRulesCompile, bCreateSimplifiedDescriptors: false, TargetDescriptors, Logger);
		}

		private static void ParseCommandLine(CommandLineArguments Arguments, bool bUsePrecompiled, bool bSkipRulesCompile, bool bForceRulesCompile, bool bCreateSimplifiedDescriptors, List<TargetDescriptor> TargetDescriptors, ILogger Logger)
		{
			List<string> TargetLists;
			Arguments = Arguments.Remove("-TargetList=", out TargetLists);

			List<string> Targets;
			Arguments = Arguments.Remove("-Target=", out Targets);

			// We don't have defined equality semantics to compare target descriptors, so catch any textual duplicates passed to us.
			// This is a safeguard, as duplicate targets are not handled 100% correctly by the rest of UBT.
			TargetLists = [.. TargetLists.Distinct(StringComparer.OrdinalIgnoreCase)];
			Targets = [.. Targets.Distinct(StringComparer.OrdinalIgnoreCase)];

			if (TargetLists.Count > 0 || Targets.Count > 0)
			{
				// Try to parse multiple arguments from a single command line
				foreach (string TargetList in TargetLists)
				{
					string[] Lines = File.ReadAllLines(TargetList);
					foreach (string Line in Lines)
					{
						string TrimLine = Line.Trim();
						if (TrimLine.Length > 0 && TrimLine[0] != ';')
						{
							CommandLineArguments NewArguments = Arguments.Append(CommandLineArguments.Split(TrimLine));
							ParseCommandLine(NewArguments, bUsePrecompiled, bSkipRulesCompile, bForceRulesCompile, bCreateSimplifiedDescriptors, TargetDescriptors, Logger);
						}
					}
				}

				foreach (string Target in Targets)
				{
					CommandLineArguments NewArguments = Arguments.Append(CommandLineArguments.Split(Target));
					ParseCommandLine(NewArguments, bUsePrecompiled, bSkipRulesCompile, bForceRulesCompile, bCreateSimplifiedDescriptors, TargetDescriptors, Logger);
				}
			}
			else
			{
				// Otherwise just process the whole command line together
				ParseSingleCommandLine(Arguments, bUsePrecompiled, bSkipRulesCompile, bForceRulesCompile, bCreateSimplifiedDescriptors, TargetDescriptors, Logger);
			}
		}

		/// <summary>
		/// Parse a list of target descriptors from the command line
		/// </summary>
		/// <param name="Arguments">Command-line arguments</param>
		/// <param name="bUsePrecompiled">Whether to use a precompiled engine distribution</param>
		/// <param name="bSkipRulesCompile">Whether to skip compiling rules assemblies</param>
		/// <param name="bForceRulesCompile">Whether to always compile all rules assemblies</param>
		/// <param name="TargetDescriptors">List of target descriptors</param>
		/// <param name="Logger">Logger for output</param>
		public static void ParseSingleCommandLine(CommandLineArguments Arguments, bool bUsePrecompiled, bool bSkipRulesCompile, bool bForceRulesCompile, List<TargetDescriptor> TargetDescriptors, ILogger Logger)
		{
			// call the internal one
			ParseSingleCommandLine(Arguments, bUsePrecompiled, bSkipRulesCompile, bForceRulesCompile, bCreateSimplifiedDescriptors: false, TargetDescriptors, Logger);
		}

		private static void ParseSingleCommandLine(CommandLineArguments Arguments, bool bUsePrecompiled, bool bSkipRulesCompile, bool bForceRulesCompile, bool bCreateSimplifiedDescriptors, List<TargetDescriptor> TargetDescriptors, ILogger Logger)
		{
			List<UnrealTargetPlatform> Platforms = new List<UnrealTargetPlatform>();
			List<UnrealTargetConfiguration> Configurations = new List<UnrealTargetConfiguration>();
			List<string> TargetNames = new List<string>();
			FileReference? ProjectFile = Arguments.GetFileReferenceOrDefault("-Project=", null);

			// Settings for creating/using static libraries for the engine
			for (int ArgumentIndex = 0; ArgumentIndex < Arguments.Count; ArgumentIndex++)
			{
				string Argument = Arguments[ArgumentIndex];
				if (Argument.Length > 0 && Argument[0] != '-')
				{
					// Mark this argument as used. We'll interpret it as one thing or another.
					Arguments.MarkAsUsed(ArgumentIndex);

					// Check if it's a project file argument
					if (Argument.EndsWith(".uproject", StringComparison.OrdinalIgnoreCase))
					{
						FileReference NewProjectFile = new FileReference(Argument);
						if (ProjectFile != null && ProjectFile != NewProjectFile)
						{
							throw new BuildException("Multiple project files specified on command line (first {0}, then {1})", ProjectFile, NewProjectFile);
						}
						ProjectFile = new FileReference(Argument);
						continue;
					}

					// Split it into separate arguments
					string[] InlineArguments = Argument.Split('+');

					// Try to parse them as platforms
					UnrealTargetPlatform ParsedPlatform;
					if (UnrealTargetPlatform.TryParse(InlineArguments[0], out ParsedPlatform))
					{
						Platforms.Add(ParsedPlatform);
						for (int InlineArgumentIdx = 1; InlineArgumentIdx < InlineArguments.Length; InlineArgumentIdx++)
						{
							Platforms.Add(UnrealTargetPlatform.Parse(InlineArguments[InlineArgumentIdx]));
						}
						continue;
					}

					// Try to parse them as configurations
					UnrealTargetConfiguration ParsedConfiguration;
					if (Enum.TryParse(InlineArguments[0], true, out ParsedConfiguration))
					{
						Configurations.Add(ParsedConfiguration);
						for (int InlineArgumentIdx = 1; InlineArgumentIdx < InlineArguments.Length; InlineArgumentIdx++)
						{
							string InlineArgument = InlineArguments[InlineArgumentIdx];
							if (!Enum.TryParse(InlineArgument, true, out ParsedConfiguration))
							{
								throw new BuildException("Invalid configuration '{0}'", InlineArgument);
							}
							Configurations.Add(ParsedConfiguration);
						}
						continue;
					}

					// Otherwise assume they are target names
					TargetNames.AddRange(InlineArguments);
				}
			}

			if (Platforms.Count == 0)
			{
				throw new BuildException("No platforms specified for target");
			}
			if (Configurations.Count == 0)
			{
				throw new BuildException("No configurations specified for target");
			}

			// make a single simple descriptor with out a lot of processing or rules assembly creation
			if (bCreateSimplifiedDescriptors)
			{
				// we expect either a targetname from the above loop, or -targetype combined with -project, so we should have -project or a TargetName already
				string? TargetName = TargetNames.FirstOrDefault();
				if (TargetName == null)
				{
					if (ProjectFile == null)
					{
						Logger.LogInformation("When looking for per-project SDK overrides, we got a commandline that could not be used to find a target project ({CmdLine})", Arguments.ToString());
						return;
					}
					// just assume Game target type, it doesn't matter
					TargetName = ProjectFile.GetFileNameWithoutAnyExtensions() + "Game";
				}
				if (ProjectFile == null)
				{
					NativeProjects.TryGetProjectForTarget(TargetName, Logger, out ProjectFile);
				}

				TargetDescriptors.Add(new TargetDescriptor(ProjectFile, TargetName, Platforms.First(), Configurations.First(), new UnrealArchitectures(UnrealArch.X64), Arguments));

				// we are done now with simple descriptors
				return;
			}

			// Make sure the project file exists, and make sure we're using the correct case.
			if (ProjectFile != null)
			{
				FileInfo ProjectFileInfo = FileUtils.FindCorrectCase(ProjectFile.ToFileInfo());
				if (!ProjectFileInfo.Exists)
				{
					throw new BuildException("Unable to find project '{0}'.", ProjectFile);
				}
				ProjectFile = new FileReference(ProjectFileInfo);
			}

			// Expand all the platforms, architectures and configurations
			foreach (UnrealTargetPlatform Platform in Platforms.Distinct())
			{
				// Make sure the platform is valid
				if (!InstalledPlatformInfo.IsValid(null, Platform, null, EProjectType.Code, InstalledPlatformState.Downloaded))
				{
					if (!InstalledPlatformInfo.IsValid(null, Platform, null, EProjectType.Code, InstalledPlatformState.Supported))
					{
						throw new BuildException("The {0} platform is not supported from this engine distribution.", Platform);
					}
					else
					{
						throw new BuildException("Missing files required to build {0} targets. Enable {0} as an optional download component in the Epic Games Launcher.", Platform);
					}
				}

				UEBuildPlatform BuildPlatform = UEBuildPlatform.GetBuildPlatform(Platform);

				// Parse the architecture parameter, or use null to look up platform defaults later
				string ParamArchitectureList = Arguments.GetStringOrDefault("-Architecture=", "") + Arguments.GetStringOrDefault("-Architectures=", "");
				UnrealArchitectures? ParamArchitectures = UnrealArchitectures.FromString(ParamArchitectureList, Platform);

				foreach (UnrealTargetConfiguration Configuration in Configurations.Distinct())
				{
					// Create all the target descriptors for targets specified by type
					foreach (string TargetTypeString in Arguments.GetValues("-TargetType="))
					{
						TargetType TargetType;
						if (!Enum.TryParse(TargetTypeString, out TargetType))
						{
							throw new BuildException("Invalid target type '{0}'", TargetTypeString);
						}

						if (ProjectFile == null)
						{
							TargetNames.Add(RulesCompiler.CreateEngineRulesAssembly(bUsePrecompiled, bSkipRulesCompile, bForceRulesCompile, Logger).GetTargetNameByType(TargetType, Platform, Configuration, null, Logger));
						}
						else
						{
							TargetNames.Add(RulesCompiler.CreateProjectRulesAssembly(ProjectFile, bUsePrecompiled, bSkipRulesCompile, bForceRulesCompile, Logger).GetTargetNameByType(TargetType, Platform, Configuration, ProjectFile, Logger));
						}
					}

					// Make sure we could parse something
					if (TargetNames.Count == 0)
					{
						throw new BuildException("No target name was specified on the command-line.");
					}

					// Create all the target descriptors
					foreach (string TargetName in TargetNames)
					{
						FileReference? TargetProjectFile = ProjectFile;
						UnrealArchitectures Architectures;

						// If a project file was not specified see if we can find one
						if (TargetProjectFile == null && NativeProjects.TryGetProjectForTarget(TargetName, Logger, out TargetProjectFile))
						{
							Logger.LogDebug("Found project file for {TargetName} - {ProjectFile}", TargetName, TargetProjectFile);
						}
						// Programs can have a .uproject without finding a matching .Target (since the source and metadata directories are split up)
						if (TargetProjectFile == null)
						{
							// find one with a matching name
							TargetProjectFile = NativeProjects.EnumerateProjectFiles(Logger)
								.Where(x => x.GetFileNameWithoutAnyExtensions().Equals(TargetName, StringComparison.OrdinalIgnoreCase))
								.FirstOrDefault();
						}

						// make a temp target for hybrid content-as-code projects
						if (TargetProjectFile != null)
						{
							NativeProjects.ConditionalMakeTempTargetForHybridProject(TargetProjectFile, Logger);
						}

						if (ParamArchitectures != null)
						{
							Architectures = ParamArchitectures;
						}
						else
						{
							// ask the platform what achitectures it wants for this project
							Architectures = BuildPlatform.ArchitectureConfig.ActiveArchitectures(TargetProjectFile, TargetName);
						}

						// If the platform wants a target for each architecture, make a target descriptor for each architecture, otherwise one target for all architectures
						if (BuildPlatform.ArchitectureConfig.Mode == UnrealArchitectureMode.OneTargetPerArchitecture)
						{
							foreach (UnrealArch Architecture in Architectures.Architectures)
							{
								TargetDescriptors.Add(new TargetDescriptor(TargetProjectFile, TargetName, Platform, Configuration, new UnrealArchitectures(Architecture), Arguments));
							}
						}
						else
						{
							TargetDescriptors.Add(new TargetDescriptor(TargetProjectFile, TargetName, Platform, Configuration, Architectures, Arguments));
						}
					}
				}
			}

			// Register any found descriptors for telemetry, using the first as the primary target
			TelemetryService.Get().SetPrimaryTargetDetails(TargetDescriptors.FirstOrDefault());
			TargetDescriptors.ForEach(x => TelemetryService.Get().AddEndpointsFromConfig(x.ProjectFile?.Directory, Logger));
		}
	}
}
