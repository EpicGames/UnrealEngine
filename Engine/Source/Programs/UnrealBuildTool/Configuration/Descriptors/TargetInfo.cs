// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Linq;
using EpicGames.Core;

namespace UnrealBuildTool
{
	/// <summary>
	/// Information about a target, passed along when creating a module descriptor
	/// </summary>
	public class TargetInfo
	{
		/// <summary>
		/// Name of the target
		/// </summary>
		public readonly string Name;

		/// <summary>
		/// The platform that the target is being built for
		/// </summary>
		public readonly UnrealTargetPlatform Platform;

		/// <summary>
		/// The configuration being built
		/// </summary>
		public readonly UnrealTargetConfiguration Configuration;

		/// <summary>
		/// Architecture that the target is being built for
		/// </summary>
		public readonly UnrealArchitectures Architectures;

		/// <summary>
		/// Intermediate environment. Determines if the intermediates end up in a different folder than normal.
		/// </summary>
		public UnrealIntermediateEnvironment IntermediateEnvironment;

		/// <summary>
		/// The project containing the target
		/// </summary>
		public readonly FileReference? ProjectFile;

		/// <summary>
		/// The current build version
		/// </summary>
#pragma warning disable CA1822 // Mark members as static - used in module build files, leave as-is
		public ReadOnlyBuildVersion Version => ReadOnlyBuildVersion.Current;
#pragma warning restore CA1822

		/// <summary>
		/// Additional command line arguments for this target
		/// </summary>
		public CommandLineArguments? Arguments;

		/// <summary>
		/// Constructs a TargetInfo for passing to the TargetRules constructor.
		/// </summary>
		/// <param name="Name">Name of the target being built</param>
		/// <param name="Platform">The platform that the target is being built for</param>
		/// <param name="Configuration">The configuration being built</param>
		/// <param name="Architectures">The architectures being built for</param>
		/// <param name="ProjectFile">Path to the project file containing the target</param>
		/// <param name="Arguments">Additional command line arguments for this target</param>
		/// <param name="IntermediateEnvironment">Intermediate environment to use</param>
		public TargetInfo(string Name, UnrealTargetPlatform Platform, UnrealTargetConfiguration Configuration, UnrealArchitectures? Architectures, FileReference? ProjectFile, CommandLineArguments? Arguments, UnrealIntermediateEnvironment IntermediateEnvironment = UnrealIntermediateEnvironment.Default)
		{
			this.Name = Name;
			this.Platform = Platform;
			this.Configuration = Configuration;
			this.IntermediateEnvironment = IntermediateEnvironment;
			this.ProjectFile = ProjectFile;
			this.Arguments = Arguments;

			if (Architectures == null)
			{
				this.Architectures = UnrealArchitectureConfig.ForPlatform(Platform).ActiveArchitectures(ProjectFile, Name);
			}
			else
			{
				this.Architectures = Architectures;
			}
		}

		/// <summary>
		/// Construct a TargetInfo from an archive on disk
		/// </summary>
		/// <param name="Reader">Archive to read from</param>
		public TargetInfo(BinaryArchiveReader Reader)
		{
			Name = Reader.ReadString()!;
			Platform = UnrealTargetPlatform.Parse(Reader.ReadString()!);
			string ConfigurationStr = Reader.ReadString()!;
			Architectures = new UnrealArchitectures(Reader.ReadArray(() => Reader.ReadString()!)!);
			ProjectFile = Reader.ReadFileReferenceOrNull();
			string[]? ArgumentStrs = Reader.ReadArray(() => Reader.ReadString()!);

			if (!UnrealTargetConfiguration.TryParse(ConfigurationStr, out Configuration))
			{
				throw new BuildException(String.Format("The configration name {0} is not a valid configration name. Valid names are ({1})", Name,
					String.Join(",", Enum.GetValues(typeof(UnrealTargetConfiguration)).Cast<UnrealTargetConfiguration>().Select(x => x.ToString()))));
			}

			string? IntermediateEnvironmentStr = Reader.ReadString();
			if (IntermediateEnvironmentStr != null)
			{
				// @TODO Should this log or throw on failure?
				_ = UnrealIntermediateEnvironment.TryParse(IntermediateEnvironmentStr, out IntermediateEnvironment);
			}

			Arguments = ArgumentStrs == null ? null : new CommandLineArguments(ArgumentStrs);
		}

		/// <summary>
		/// Write a TargetInfo to an archive on disk
		/// </summary>
		/// <param name="Writer">Archive to write to</param>
		public void Write(BinaryArchiveWriter Writer)
		{
			Writer.WriteString(Name);
			Writer.WriteString(Platform.ToString());
			Writer.WriteString(Configuration.ToString());
			Writer.WriteArray(Architectures.Architectures.ToArray(), Item => Writer.WriteString(Item.ToString()));
			Writer.WriteFileReference(ProjectFile);
			Writer.WriteArray(Arguments?.GetRawArray(), Item => Writer.WriteString(Item));
			Writer.WriteString(IntermediateEnvironment.ToString());
		}
	}
}
