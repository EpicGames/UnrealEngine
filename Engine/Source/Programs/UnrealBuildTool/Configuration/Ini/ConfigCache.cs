// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using System.Linq;
using System.Reflection;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Logging.Abstractions;

namespace UnrealBuildTool
{
	/// <summary>
	/// Caches config files and config file hierarchies
	/// </summary>
	public static class ConfigCache
	{
		/// <summary>
		/// Delegate to add a value to an ICollection in a target object
		/// </summary>
		/// <param name="TargetObject">The object containing the field to be modified</param>
		/// <param name="ValueObject">The value to add</param>
		delegate void AddElementDelegate(object TargetObject, object? ValueObject);

		/// <summary>
		/// Caches information about a member with a [ConfigFile] attribute in a type
		/// </summary>
		abstract class ConfigMember
		{
			/// <summary>
			/// The attribute instance
			/// </summary>
			public ConfigFileAttribute Attribute;

			/// <summary>
			/// For fields implementing ICollection, specifies the element type
			/// </summary>
			public Type? ElementType;

			/// <summary>
			/// For fields implementing ICollection, a callback to add an element type.
			/// </summary>
			public AddElementDelegate? AddElement;

			/// <summary>
			/// Constructor
			/// </summary>
			/// <param name="Attribute"></param>
			public ConfigMember(ConfigFileAttribute Attribute)
			{
				this.Attribute = Attribute;
			}

			/// <summary>
			/// Returns Reflection.MemberInfo describing the target class member.
			/// </summary>
			public abstract MemberInfo MemberInfo { get; }

			/// <summary>
			/// Returns Reflection.Type of the target class member.
			/// </summary>
			public abstract Type Type { get; }

			/// <summary>
			/// Returns the value setter of the target class member.
			/// </summary>
			public abstract Action<object?, object?> SetValue { get; }

			/// <summary>
			/// Returns the value getter of the target class member.
			/// </summary>
			public abstract Func<object?, object?> GetValue { get; }
		}

		/// <summary>
		/// Caches information about a field with a [ConfigFile] attribute in a type
		/// </summary>
		class ConfigField : ConfigMember
		{
			/// <summary>
			/// Reflection description of the field with the config attribute.
			/// </summary>
			private readonly FieldInfo FieldInfo;

			/// <summary>
			/// Constructor
			/// </summary>
			/// <param name="FieldInfo"></param>
			/// <param name="Attribute"></param>
			public ConfigField(FieldInfo FieldInfo, ConfigFileAttribute Attribute)
				: base(Attribute)
			{
				this.FieldInfo = FieldInfo;
			}

			public override MemberInfo MemberInfo => FieldInfo;
			public override Type Type => FieldInfo.FieldType;
			public override Action<object?, object?> SetValue => FieldInfo.SetValue;
			public override Func<object?, object?> GetValue => FieldInfo.GetValue;
		}

		/// <summary>
		/// Caches information about a property with a [ConfigFile] attribute in a type
		/// </summary>
		class ConfigProperty : ConfigMember
		{
			/// <summary>
			/// Reflection description of the property with the config attribute.
			/// </summary>
			private readonly PropertyInfo PropertyInfo;

			/// <summary>
			/// Constructor
			/// </summary>
			/// <param name="PropertyInfo"></param>
			/// <param name="Attribute"></param>
			public ConfigProperty(PropertyInfo PropertyInfo, ConfigFileAttribute Attribute)
				: base(Attribute)
			{
				this.PropertyInfo = PropertyInfo;
			}

			public override MemberInfo MemberInfo => PropertyInfo;
			public override Type Type => PropertyInfo.PropertyType;
			public override Action<object?, object?> SetValue => PropertyInfo.SetValue;
			public override Func<object?, object?> GetValue => PropertyInfo.GetValue;
		}

		/// <summary>
		/// Allowed modification types allowed for default config files
		/// </summary>
		public enum ConfigDefaultUpdateType
		{
			/// <summary>
			/// Used for non-array types, this will replace a setting, or it will add a setting if it doesn't exist
			/// </summary>
			SetValue,
			/// <summary>
			/// Used to add an array entry to the end of any existing array entries, or will add to the end of the section
			/// </summary>
			AddArrayEntry,
		}

		/// <summary>
		/// Stores information identifying a unique config hierarchy
		/// </summary>
		class ConfigHierarchyKey
		{
			/// <summary>
			/// The hierarchy type
			/// </summary>
			public ConfigHierarchyType Type;

			/// <summary>
			/// The project directory to read from
			/// </summary>
			public DirectoryReference? ProjectDir;

			/// <summary>
			/// Which platform-specific files to read
			/// </summary>
			public UnrealTargetPlatform Platform;

			/// <summary>
			/// Custom config subdirectory to load
			/// </summary>
			public string CustomConfig;

			/// <summary>
			/// Custom override strings
			/// </summary>
			public List<string> OverrideStrings;

			/// <summary>
			/// A hotfix directory where modifications/additions to config files can be read from
			/// </summary>
			public DirectoryReference? HotfixDir;

			/// <summary>
			/// If specified, this hierarchy will include plugins enabled for the given target type
			/// </summary>
			public TargetType? IncludePluginsForTargetType;

			/// <summary>
			/// Constructor
			/// </summary>
			/// <param name="Type">The hierarchy type</param>
			/// <param name="ProjectDir">The project directory to read from</param>
			/// <param name="Platform">Which platform-specific files to read</param>
			/// <param name="CustomConfig">Custom config subdirectory to load</param>
			/// <param name="OverrideStrings">Custom override strings</param>
			/// <param name="HotfixDir">A hotfix directory where modifications/additions to config files can be read from</param>
			/// <param name="IncludePluginsForTargetType">If specified, this hierarchy will unclude plugins enabled for the given target type</param>
			public ConfigHierarchyKey(ConfigHierarchyType Type, DirectoryReference? ProjectDir, UnrealTargetPlatform Platform, string CustomConfig, List<string> OverrideStrings, DirectoryReference? HotfixDir, TargetType? IncludePluginsForTargetType)
			{
				this.Type = Type;
				this.ProjectDir = ProjectDir;
				this.Platform = Platform;
				this.CustomConfig = CustomConfig;
				this.OverrideStrings = OverrideStrings;
				this.HotfixDir = HotfixDir;
				this.IncludePluginsForTargetType = IncludePluginsForTargetType;
			}

			/// <summary>
			/// Test whether this key is equal to another object
			/// </summary>
			/// <param name="Other">The object to compare against</param>
			/// <returns>True if the objects match, false otherwise</returns>
			public override bool Equals(object? Other)
			{
				ConfigHierarchyKey? OtherKey = Other as ConfigHierarchyKey;
				return OtherKey != null &&
					OtherKey.Type == Type &&
					OtherKey.ProjectDir == ProjectDir &&
					OtherKey.Platform == Platform &&
					OtherKey.CustomConfig == CustomConfig &&
					OtherKey.OverrideStrings.SequenceEqual(OverrideStrings) &&
					OtherKey.HotfixDir == HotfixDir &&
					OtherKey.IncludePluginsForTargetType == IncludePluginsForTargetType;
			}

			/// <summary>
			/// Returns a stable hash of this object
			/// </summary>
			/// <returns>Hash value for this object</returns>
			public override int GetHashCode()
			{
				int Hash = 17;
				Hash = (Hash * 31) + Type.GetHashCode();
				Hash = (Hash * 31) + ((ProjectDir == null) ? 0 : ProjectDir.GetHashCode());
				Hash = (Hash * 31) + Platform.GetHashCode();
				Hash = (Hash * 31) + CustomConfig.GetHashCode(StringComparison.Ordinal);
				foreach (string OverrideString in OverrideStrings)
				{
					Hash = (Hash * 31) + OverrideString.GetHashCode(StringComparison.Ordinal);
				}
				Hash = (Hash * 31) + ((HotfixDir == null) ? 0 : HotfixDir.GetHashCode());
				Hash = (Hash * 31) + ((IncludePluginsForTargetType == null) ? 0 : IncludePluginsForTargetType.GetHashCode());

				return Hash;
			}
		}

		/// <summary>
		/// Cache of individual config files
		/// </summary>
		static readonly ConcurrentDictionary<FileReference, Lazy<ConfigFile?>> LocationToConfigFile = [];

		/// <summary>
		/// Cache of config hierarchies by project
		/// </summary>
		static readonly ConcurrentDictionary<ConfigHierarchyKey, Lazy<ConfigHierarchy>> HierarchyKeyToHierarchy = [];

		/// <summary>
		/// Cache of config fields by type
		/// </summary>
		static readonly ConcurrentDictionary<Type, Lazy<List<ConfigMember>>> TypeToConfigMembers = [];

		/// <summary>
		/// Attempts to read a config file (or retrieve it from the cache)
		/// </summary>
		/// <param name="Location">Location of the file to read</param>
		/// <param name="ConfigFile">On success, receives the parsed config file</param>
		/// <returns>True if the file exists and was read, false otherwise</returns>
		internal static bool TryReadFile(FileReference Location, [NotNullWhen(true)] out ConfigFile? ConfigFile)
		{
			ConfigFile = LocationToConfigFile.GetOrAdd(Location, static Location => new Lazy<ConfigFile?>(() =>
			{
				if (FileReference.Exists(Location))
				{
					return new ConfigFile(Location);
				}

				// Note: it's important to keep the null value here so we don't keep checking for files that don't exist
				return null;
			})).Value;
			return ConfigFile != null;
		}

		/// <summary>
		/// Reads a config hierarchy (or retrieve it from the cache)
		/// </summary>
		/// <param name="Type">The type of hierarchy to read</param>
		/// <param name="ProjectDir">The project directory to read the hierarchy for</param>
		/// <param name="Platform">Which platform to read platform-specific config files for</param>
		/// <param name="CustomConfig">Optional override config directory to search, for support of multiple target types</param>
		/// <param name="CustomArgs">Optional list of command line arguments added to the existing command line arguments</param>
		/// <param name="HotfixDir">A hotfix directory where modifications/additions to config files can be read from</param>
		/// <param name="IncludePluginsForTargetType">If specified, this hierarchy will unclude plugins enabled for the given target type</param>
		/// <returns>The requested config hierarchy</returns>
		public static ConfigHierarchy ReadHierarchy(ConfigHierarchyType Type, DirectoryReference? ProjectDir, UnrealTargetPlatform Platform, string CustomConfig = "", string[]? CustomArgs = null, DirectoryReference? HotfixDir = null, TargetType? IncludePluginsForTargetType = null)
		{
			CommandLineArguments CombinedArgs;
			if (CustomArgs != null)
			{
				List<string> CommandLineArgs = new List<string>(Environment.GetCommandLineArgs());
				foreach (string CustomArg in CustomArgs)
				{
					if (CustomArg.StartsWith("-", StringComparison.InvariantCultureIgnoreCase))
					{
						CommandLineArgs.Add(CustomArg);
					}
					else
					{
						CommandLineArgs.Add("-" + CustomArg);
					}
				}
				CombinedArgs = new CommandLineArguments(CommandLineArgs.ToArray());
			}
			else
			{
				CombinedArgs = new CommandLineArguments(Environment.GetCommandLineArgs());
			}

			return ReadHierarchy(Type, ProjectDir, Platform, CustomConfig, CombinedArgs, HotfixDir, IncludePluginsForTargetType);
		}

		/// <summary>
		/// Reads a config hierarchy (or retrieve it from the cache)
		/// </summary>
		/// <param name="Type">The type of hierarchy to read</param>
		/// <param name="ProjectDir">The project directory to read the hierarchy for</param>
		/// <param name="Platform">Which platform to read platform-specific config files for</param>
		/// <param name="CustomConfig">Optional override config directory to search, for support of multiple target types</param>
		/// <param name="CmdLineArgs">The command line arguments to parse</param>
		/// <param name="HotfixDir">A hotfix directory where modifications/additions to config files can be read from</param>
		/// <param name="IncludePluginsForTargetType">If specified, this hierarchy will unclude plugins enabled for the given target type</param>
		/// <returns>The requested config hierarchy</returns>
		public static ConfigHierarchy ReadHierarchy(ConfigHierarchyType Type, DirectoryReference? ProjectDir, UnrealTargetPlatform Platform,
			string CustomConfig, CommandLineArguments CmdLineArgs, DirectoryReference? HotfixDir = null, TargetType? IncludePluginsForTargetType = null)
		{
			// Handle command line overrides
			List<string> OverrideStrings = new List<string>();

			string IniConfigArgPrefix = "-ini:" + Enum.GetName(typeof(ConfigHierarchyType), Type) + ":";
			string CustomConfigPrefix = "-CustomConfig=";
			bool bDumpIniLoads = false;
			foreach (string CmdLineArg in CmdLineArgs)
			{
				if (CmdLineArg.StartsWith(IniConfigArgPrefix, StringComparison.OrdinalIgnoreCase))
				{
					OverrideStrings.Add(CmdLineArg.Substring(IniConfigArgPrefix.Length));
				}
				if (CmdLineArg.StartsWith(CustomConfigPrefix, StringComparison.OrdinalIgnoreCase))
				{
					CustomConfig = CmdLineArg.Substring(CustomConfigPrefix.Length);
				}
				if (CmdLineArg.Equals("-dumpiniloads", StringComparison.OrdinalIgnoreCase))
				{
					bDumpIniLoads = true;
				}
			}

			CustomConfig ??= String.Empty;

			// Get the key to use for the cache. It cannot be null, so we use the engine directory if a project directory is not given.
			ConfigHierarchyKey Key = new ConfigHierarchyKey(Type, ProjectDir, Platform, CustomConfig, OverrideStrings, HotfixDir, IncludePluginsForTargetType);

			ILogger Logger = NullLogger.Instance;

			if (bDumpIniLoads)
			{
				Logger = Log.Logger;
			}

			// Try to get the cached hierarchy with this key
			Logger.LogInformation("Requested Hierarchy: {Type},{ProjectDir},{Platform},{CustomConfig}", Type, ProjectDir, Platform, CustomConfig);
			return HierarchyKeyToHierarchy.GetOrAdd(Key, new Lazy<ConfigHierarchy>(() => InternalReadHierarchy(Logger, Type, ProjectDir, Platform, CustomConfig, HotfixDir, IncludePluginsForTargetType, OverrideStrings))).Value;
		}

		static ConfigHierarchy InternalReadHierarchy(ILogger Logger, ConfigHierarchyType Type, DirectoryReference? ProjectDir, UnrealTargetPlatform Platform, string CustomConfig
			, DirectoryReference? HotfixDir, TargetType? IncludePluginsForTargetType, List<string> OverrideStrings)
		{
			// Find all the input files
			List<ConfigFile> Files = new List<ConfigFile>();
			IEnumerable<FileReference> IniFileNames = ConfigHierarchy.EnumerateConfigFileLocations(Type, ProjectDir, Platform, CustomConfig, IncludePluginsForTargetType);
			List<FileReference> ReadFiles = [];
			foreach (FileReference IniFileName in IniFileNames)
			{
				ConfigFile? File;
				Logger.LogInformation("Trying to load: {IniPath}", IniFileName);
				if (TryReadFile(IniFileName, out File))
				{
					Logger.LogInformation($"    Found!");
					Files.Add(File);
					ReadFiles.Add(IniFileName);
				}

				if (HotfixDir != null)
				{
					FileReference HotfixFile = FileReference.Combine(HotfixDir, IniFileName.GetFileName());

					if (TryReadFile(HotfixFile, out File))
					{
						Logger.LogInformation("    Hotfixed with '{HotfixPath}'", HotfixFile);
						Files.Add(File);
						ReadFiles.Add(HotfixFile);
					}
				}
			}

			foreach (string OverrideString in OverrideStrings)
			{
				ConfigFile OverrideFile = new ConfigFile(OverrideString);
				Files.Add(OverrideFile);
				Logger.LogInformation("Trying to load override: {OverridePath}", OverrideFile);
			}

			// Create the hierarchy
			return new ConfigHierarchy(Files, ReadFiles);
		}

		/// <summary>
		/// Gets a list of ConfigFields for the given type
		/// </summary>
		/// <param name="TargetObjectType">Type to get configurable fields for</param>
		/// <returns>List of config fields for the given type</returns>
		static List<ConfigMember> FindConfigMembersForType(Type TargetObjectType)
		{
			return TypeToConfigMembers.GetOrAdd(TargetObjectType, static TargetObjectType => new Lazy<List<ConfigMember>>(() =>
			{
				List<ConfigMember> Members = new List<ConfigMember>();
				if (TargetObjectType.BaseType != null)
				{
					Members.AddRange(FindConfigMembersForType(TargetObjectType.BaseType));
				}
				foreach (FieldInfo FieldInfo in TargetObjectType.GetFields(BindingFlags.Instance | BindingFlags.GetField | BindingFlags.GetProperty | BindingFlags.Public | BindingFlags.NonPublic | BindingFlags.DeclaredOnly))
				{
					ProcessConfigTypeMember<FieldInfo>(FieldInfo, Members, (FieldInfo, Attribute) => new ConfigField(FieldInfo, Attribute));
				}
				foreach (PropertyInfo PropertyInfo in TargetObjectType.GetProperties(BindingFlags.Instance | BindingFlags.GetField | BindingFlags.GetProperty | BindingFlags.Public | BindingFlags.NonPublic | BindingFlags.DeclaredOnly))
				{
					ProcessConfigTypeMember<PropertyInfo>(PropertyInfo, Members, (PropertyInfo, Attribute) => new ConfigProperty(PropertyInfo, Attribute));
				}
				return Members;
			})).Value;
		}

		static void ProcessConfigTypeMember<MEMBER>(MEMBER MemberInfo, List<ConfigMember> Members, Func<MEMBER, ConfigFileAttribute, ConfigMember> CreateConfigMember)
			where MEMBER : System.Reflection.MemberInfo
		{
			IEnumerable<ConfigFileAttribute> Attributes = MemberInfo.GetCustomAttributes<ConfigFileAttribute>();
			foreach (ConfigFileAttribute Attribute in Attributes)
			{
				// Copy the field 
				ConfigMember Setter = CreateConfigMember(MemberInfo, Attribute);

				// Check if the field type implements ICollection<>. If so, we can take multiple values.
				foreach (Type InterfaceType in Setter.Type.GetInterfaces())
				{
					if (InterfaceType.IsGenericType && InterfaceType.GetGenericTypeDefinition() == typeof(ICollection<>))
					{
						MethodInfo MethodInfo = InterfaceType.GetRuntimeMethod("Add", new Type[] { InterfaceType.GenericTypeArguments[0] })!;
						Setter.AddElement = (Target, Value) => { MethodInfo.Invoke(Setter.GetValue(Target), new object?[] { Value }); };
						Setter.ElementType = InterfaceType.GenericTypeArguments[0];
						break;
					}
				}

				// Add it to the output list
				Members.Add(Setter);
			}
		}

		/// <summary>
		/// Read config settings for the given object
		/// </summary>
		/// <param name="ProjectDir">Path to the project directory</param>
		/// <param name="Platform">The platform being built</param>
		/// <param name="TargetObject">Object to receive the settings</param>
		/// <param name="CustomConfig">Optional override config directory to search, for support of multiple target types</param>
		public static void ReadSettings(DirectoryReference? ProjectDir, UnrealTargetPlatform Platform, object TargetObject, string CustomConfig = "")
		{
			ReadSettings(ProjectDir, Platform, TargetObject, null, null, CustomConfig);
		}

		/// <summary>
		/// Read config settings for the given object
		/// </summary>
		/// <param name="ProjectDir">Path to the project directory</param>
		/// <param name="Platform">The platform being built</param>
		/// <param name="TargetObject">Object to receive the settings</param>
		/// <param name="ConfigValues">Will be populated with config values that were retrieved. May be null.</param>
		/// <param name="CmdLineArgs">Command line arguments, if null the application's command line arguments will be used</param>
		/// <param name="CustomConfig">Optional override config directory to search, for support of multiple target types</param>
		internal static void ReadSettings(DirectoryReference? ProjectDir, UnrealTargetPlatform Platform, object TargetObject, Dictionary<ConfigDependencyKey, IReadOnlyList<string>?>? ConfigValues, CommandLineArguments? CmdLineArgs, string CustomConfig = "")
		{
			List<ConfigMember> Members = FindConfigMembersForType(TargetObject.GetType());

			foreach (ConfigMember Member in Members)
			{
				// Read the hierarchy listed
				ConfigHierarchy Hierarchy;

				if (CmdLineArgs == null)
				{
					Hierarchy = ReadHierarchy(Member.Attribute.ConfigType, ProjectDir, Platform, CustomConfig);
				}
				else
				{
					Hierarchy = ReadHierarchy(Member.Attribute.ConfigType, ProjectDir, Platform, CustomConfig, CmdLineArgs);
				}

				// Get the key name
				string KeyName = Member.Attribute.KeyName ?? Member.MemberInfo.Name;

				// Get the value(s) associated with this key
				IReadOnlyList<string>? Values;
				Hierarchy.TryGetValues(Member.Attribute.SectionName, KeyName, out Values);

				// Parse the values from the config files and update the target object
				if (Member.AddElement == null)
				{
					if (Values != null && Values.Count == 1)
					{
						object? Value;
						if (TryParseValue(Values[0], Member.Type, out Value))
						{
							Member.SetValue(TargetObject, Value);
						}
					}
				}
				else
				{
					if (Values != null)
					{
						foreach (string Item in Values)
						{
							object? Value;
							if (TryParseValue(Item, Member.ElementType!, out Value))
							{
								Member.AddElement(TargetObject, Value);
							}
						}
					}
				}

				// Save the dependency
				if (ConfigValues != null)
				{
					ConfigDependencyKey Key = new ConfigDependencyKey(Member.Attribute.ConfigType, ProjectDir, Platform, Member.Attribute.SectionName, KeyName);
					ConfigValues[Key] = Values;
				}
			}
		}

		/// <summary>
		/// Attempts to parse the given text into an object which matches a specific field type
		/// </summary>
		/// <param name="Text">The text to parse</param>
		/// <param name="FieldType">The type of field to parse</param>
		/// <param name="Value">If successful, a value of type 'FieldType'</param>
		/// <returns>True if the value could be parsed, false otherwise</returns>
		public static bool TryParseValue(string Text, Type FieldType, out object? Value)
		{
			if (FieldType == typeof(string))
			{
				Value = Text;
				return true;
			}
			else if (FieldType == typeof(bool))
			{
				bool BoolValue;
				if (ConfigHierarchy.TryParse(Text, out BoolValue))
				{
					Value = BoolValue;
					return true;
				}
				else
				{
					Value = null;
					return false;
				}
			}
			else if (FieldType == typeof(int))
			{
				int IntValue;
				if (ConfigHierarchy.TryParse(Text, out IntValue))
				{
					Value = IntValue;
					return true;
				}
				else
				{
					Value = null;
					return false;
				}
			}
			else if (FieldType == typeof(float))
			{
				float FloatValue;
				if (ConfigHierarchy.TryParse(Text, out FloatValue))
				{
					Value = FloatValue;
					return true;
				}
				else
				{
					Value = null;
					return false;
				}
			}
			else if (FieldType == typeof(double))
			{
				double DoubleValue;
				if (ConfigHierarchy.TryParse(Text, out DoubleValue))
				{
					Value = DoubleValue;
					return true;
				}
				else
				{
					Value = null;
					return false;
				}
			}
			else if (FieldType == typeof(Guid))
			{
				Guid GuidValue;
				if (ConfigHierarchy.TryParse(Text, out GuidValue))
				{
					Value = GuidValue;
					return true;
				}
				else
				{
					Value = null;
					return false;
				}
			}
			else if (FieldType.IsEnum)
			{
				try
				{
					Value = Enum.Parse(FieldType, Text);
					return true;
				}
				catch
				{
					Value = null;
					return false;
				}
			}
			else if (FieldType.GetGenericTypeDefinition() == typeof(Nullable<>))
			{
				return TryParseValue(Text, FieldType.GetGenericArguments()[0], out Value);
			}
			else
			{
				throw new Exception("Unsupported type for [ConfigFile] attribute");
			}
		}

		#region Updating Default Config file support

		/// <summary>
		/// Calculates the path to where the project's Default config of the type given (ie DefaultEngine.ini)
		/// </summary>
		/// <param name="ConfigType">Game, Engine, etc</param>
		/// <param name="ProjectDir">Project directory, used to find Config/Default[Type].ini</param>
		public static FileReference GetDefaultConfigFileReference(ConfigHierarchyType ConfigType, DirectoryReference ProjectDir)
		{
			return FileReference.Combine(ProjectDir, "Config", $"Default{ConfigType}.ini");
		}

		/// <summary>
		/// Calculates the path to where the project's platform specific config of the type given (ie DefaultEngine.ini)
		/// </summary>
		/// <param name="ConfigType">Game, Engine, etc</param>
		/// <param name="ProjectDir">Project directory, used to find Config/Default[Type].ini</param>
		/// <param name="Platform">Platform name</param>
		public static FileReference GetPlatformConfigFileReference(ConfigHierarchyType ConfigType, DirectoryReference ProjectDir, string Platform)
		{
			return FileReference.Combine(ProjectDir, "Platforms", Platform, "Config", $"{Platform}{ConfigType}.ini");
		}

		/// <summary>
		/// Updates a section in a Default***.ini, and will write it out. If the file is not writable, p4 can attempt to check it out.
		/// </summary>
		/// <param name="ConfigType">Game, Engine, etc</param>
		/// <param name="ProjectDir">Project directory, used to find Config/Default[Type].ini</param>
		/// <param name="UpdateType">How to modify the secion</param>
		/// <param name="Section">Name of the section with the Key in it</param>
		/// <param name="Key">Key to update</param>
		/// <param name="Value">Value to write for te Key</param>
		/// <param name="Logger">Logger for output</param>
		/// <returns></returns>		
		public static bool WriteSettingToDefaultConfig(ConfigHierarchyType ConfigType, DirectoryReference ProjectDir, ConfigDefaultUpdateType UpdateType, string Section, string Key, string Value, ILogger Logger)
		{
			FileReference DefaultConfigFile = GetDefaultConfigFileReference(ConfigType, ProjectDir);
			return WriteSettingToConfigFile(DefaultConfigFile, UpdateType, Section, Key, Value, Logger);
		}

		/// <summary>
		/// Updates a section in a Default***.ini, and will write it out. If the file is not writable, p4 can attempt to check it out.
		/// </summary>
		/// <param name="ConfigType">Game, Engine, etc</param>
		/// <param name="ProjectDir">Project directory, used to find Config/Default[Type].ini</param>
		/// <param name="Platform">Platform name. Must have a directory under projects Platforms subdirectory</param>
		/// <param name="UpdateType">How to modify the secion</param>
		/// <param name="Section">Name of the section with the Key in it</param>
		/// <param name="Key">Key to update</param>
		/// <param name="Value">Value to write for te Key</param>
		/// <param name="Logger">Logger for output</param>
		/// <returns></returns>		
		public static bool WriteSettingToPlatformConfigFile(ConfigHierarchyType ConfigType, DirectoryReference ProjectDir, string Platform, ConfigDefaultUpdateType UpdateType, string Section, string Key, string Value, ILogger Logger)
		{
			FileReference PlatformConfigFile = GetPlatformConfigFileReference(ConfigType, ProjectDir, Platform);
			return WriteSettingToConfigFile(PlatformConfigFile, UpdateType, Section, Key, Value, Logger);
		}

		/// <summary>
		/// Updates a section in config file, and will write it out. If the file is not writable, p4 can attempt to check it out.
		/// </summary>
		/// <param name="ConfigFile">Config file (.ini) name</param>
		/// <param name="UpdateType">How to modify the secion</param>
		/// <param name="Section">Name of the section with the Key in it</param>
		/// <param name="Key">Key to update</param>
		/// <param name="Value">Value to write for te Key</param>
		/// <param name="Logger">Logger for output</param>
		/// <returns></returns>		
		public static bool WriteSettingToConfigFile(FileReference ConfigFile, ConfigDefaultUpdateType UpdateType, string Section, string Key, string Value, ILogger Logger)
		{

			if (!FileReference.Exists(ConfigFile))
			{
				Logger.LogWarning("Failed to find config file '{DefaultConfigFile}' to update", ConfigFile);
				return false;
			}

			if (File.GetAttributes(ConfigFile.FullName).HasFlag(FileAttributes.ReadOnly))
			{
				Logger.LogWarning("Config file '{ConfigFile}' is read-only, unable to write setting {Key}", ConfigFile.FullName, Key);
				return false;
			}

			// generate the section header
			string SectionString = $"[{Section}]";

			// genrate the line we are going to write to the config
			string KeyWithEquals = $"{Key}=";
			string LineToWrite = (UpdateType == ConfigDefaultUpdateType.AddArrayEntry ? "+" : "");
			LineToWrite += KeyWithEquals + Value;

			// read in all the lines so we can insert or replace one
			List<string> Lines = File.ReadAllLines(ConfigFile.FullName).ToList();

			// look for the section
			int SectionIndex = -1;
			for (int Index = 0; Index < Lines.Count; Index++)
			{
				if (Lines[Index].Trim().Equals(SectionString, StringComparison.OrdinalIgnoreCase))
				{
					SectionIndex = Index;
					break;
				}
			}

			// if section not found, just append to the end
			if (SectionIndex == -1)
			{
				Lines.Add(SectionString);
				Lines.Add(LineToWrite);

				File.WriteAllLines(ConfigFile.FullName, Lines);
				return true;
			}

			// find the last line in the section with the prefix
			int LastIndexOfPrefix = -1;
			int NextSectionIndex = -1;
			for (int Index = SectionIndex + 1; Index < Lines.Count; Index++)
			{
				string Line = Lines[Index];
				if (Line.StartsWith('+') || Line.StartsWith('-') || Line.StartsWith('.') || Line.StartsWith('!'))
				{
					Line = Line.Substring(1);
				}

				// look for last array entry in case of multiples (or the only line for non-array type)
				if (Line.StartsWith(KeyWithEquals, StringComparison.InvariantCultureIgnoreCase))
				{
					LastIndexOfPrefix = Index;
				}
				else if (Lines[Index].StartsWith("[", StringComparison.Ordinal))
				{
					NextSectionIndex = Index;
					// we found another section, so break out
					break;
				}
			}

			// now we know enough to either insert or replace a line

			// if we never found the key, we will insert at the end of the section
			if (LastIndexOfPrefix == -1)
			{
				// if we didn't find a next section, thjen we will insert at the end of the file
				if (NextSectionIndex == -1)
				{
					NextSectionIndex = Lines.Count;
				}

				// move past blank lines between sections
				while (String.IsNullOrWhiteSpace(Lines[NextSectionIndex - 1]))
				{
					NextSectionIndex--;
				}
				// insert before the next section (or end of file)
				Lines.Insert(NextSectionIndex, LineToWrite);
			}
			// otherwise, insert after, or replace, a line, depending on type
			else
			{
				if (UpdateType == ConfigDefaultUpdateType.AddArrayEntry)
				{
					Lines.Insert(LastIndexOfPrefix + 1, LineToWrite);
				}
				else
				{
					Lines[LastIndexOfPrefix] = LineToWrite;
				}
			}

			// now the lines are updated, we can overwrite the file
			File.WriteAllLines(ConfigFile.FullName, Lines);
			return true;
		}

		/// <summary>
		/// Invalidates the hierarchy internal caches so that next call to ReadHierarchy will re-read from disk
		/// but existing ones will still be valid with old values
		/// </summary>
		public static void InvalidateCaches()
		{
			lock (LocationToConfigFile)
			{
				LocationToConfigFile.Clear();
			}

			lock (HierarchyKeyToHierarchy)
			{
				HierarchyKeyToHierarchy.Clear();
			}
		}

		#endregion
	}
}
