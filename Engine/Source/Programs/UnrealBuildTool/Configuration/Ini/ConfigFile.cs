// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace UnrealBuildTool
{
	/// <summary>
	/// Specifies the action to take for a config line, as denoted by its prefix.
	/// </summary>
	public enum ConfigLineAction
	{
		/// <summary>
		/// Assign the value to the key
		/// </summary>
		Set,

		/// <summary>
		/// Add the value to the key (denoted with +X=Y in config files)
		/// </summary>
		Add,

		/// <summary>
		/// Remove the key without having to match value (denoted with !X in config files)
		/// </summary>
		RemoveKey,

		/// <summary>
		/// Remove the matching key and value (denoted with -X=Y in config files)
		/// </summary>
		RemoveKeyValue

	}

	/// <summary>
	/// Contains a pre-parsed raw config line, consisting of action, key and value components.
	/// </summary>
	public class ConfigLine
	{
		/// <summary>
		/// The action to take when merging this key/value pair with an existing value
		/// </summary>
		public ConfigLineAction Action;

		/// <summary>
		/// Name of the key to modify
		/// </summary>
		public string Key;

		/// <summary>
		/// Value to assign to the key
		/// </summary>
		public string Value;

		/// <summary>
		/// Whether to preserve enclosing quotations around <see cref="Value"/>
		/// </summary>
		internal bool ShouldPreserveValueQuotations = false;

		/// <summary>
		/// Constructor.
		/// </summary>
		/// <param name="Action">Action to take when merging this key/value pair with an existing value</param>
		/// <param name="Key">Name of the key to modify</param>
		/// <param name="Value">Value to assign</param>
		public ConfigLine(ConfigLineAction Action, string Key, string Value)
		{
			this.Action = Action;
			this.Key = Key;
			this.Value = Value;
		}

		/// <summary>
		/// Constructor.
		/// </summary>
		/// <param name="Action">Action to take when merging this key/value pair with an existing value</param>
		/// <param name="Key">Name of the key to modify</param>
		/// <param name="Value">Value to assign</param>
		/// <param name="ShouldPreserveValueQuotations">Whether to preserve enclosing quotations around <see cref="Value"/></param>
		public ConfigLine(ConfigLineAction Action, string Key, string Value, bool ShouldPreserveValueQuotations) : this(Action, Key, Value)
		{
			this.ShouldPreserveValueQuotations = ShouldPreserveValueQuotations;
		}

		/// <summary>
		/// Formats this object for the debugger
		/// </summary>
		/// <returns>The original config line</returns>
		public override string ToString()
		{
			string Prefix = (Action == ConfigLineAction.Add) ? "+" : (Action == ConfigLineAction.RemoveKey) ? "!" : (Action == ConfigLineAction.RemoveKeyValue) ? "-" : "";
			return String.Format("{0}{1}={2}", Prefix, Key, Value);
		}
	}

	/// <summary>
	/// Contains the lines which appeared in a config section, with all comments and whitespace removed
	/// </summary>
	public class ConfigFileSection
	{
		/// <summary>
		/// Name of the section
		/// </summary>
		public string Name;

		/// <summary>
		/// Lines which appeared in the config section
		/// </summary>
		public List<ConfigLine> Lines = new List<ConfigLine>();

		/// <summary>
		/// Construct an empty config section with the given name
		/// </summary>
		/// <param name="Name">Name of the config section</param>
		public ConfigFileSection(string Name)
		{
			this.Name = Name;
		}

		/// <summary>
		/// try to get a line using it's name, if the line doesn't exist returns false
		/// </summary>
		/// <param name="Name">Name of the line you want to get</param>
		/// <param name="OutLine">The result of the operation</param>
		/// <returns>return true if the line is retrieved return false and null OutLine if Name isn't found in this section</returns>
		public bool TryGetLine(string Name, [NotNullWhen(true)] out ConfigLine? OutLine)
		{
			foreach (ConfigLine Line in Lines)
			{
				if (Line.Key.Equals(Name, StringComparison.Ordinal))
				{
					OutLine = Line;
					return true;
				}
			}
			OutLine = null;
			return false;
		}
	}

	/// <summary>
	/// Represents a single config file as stored on disk. 
	/// </summary>
	public class ConfigFile
	{
		/// <summary>
		/// Maps names to config sections
		/// </summary>
		readonly Dictionary<string, ConfigFileSection> Sections = new Dictionary<string, ConfigFileSection>(StringComparer.InvariantCultureIgnoreCase);

		// Remap of config names/sections
		private static readonly Dictionary<string, string> SectionNameRemap = new();
		private static readonly Dictionary<string, Dictionary<string, string>> SectionKeyRemap = new();
		private static readonly HashSet<string> WarnedKeys = new(StringComparer.InvariantCultureIgnoreCase);

		private static string RemapSectionOrKey(Dictionary<string, string>? Remap, string Key, string Context)
		{
			if (Remap != null)
			{
				string? Remapped;
				if (Remap.TryGetValue(Key, out Remapped))
				{
					if (!WarnedKeys.Contains(Key))
					{
						WarnedKeys.Add(Key);
						Log.Logger.LogWarning(
							"DEPRECATION: '{Key}', {Context}, has been deprecated. Using '{Remapped}' instead. It is recommended you update your .ini files as soon as possible, and replace {Key} with {Remapped}",
							Key,
							Context,
							Remapped,
							Key,
							Remapped);
					}
					return Remapped;
				}
			}
			return Key;
		}

		/// <summary>
		/// Static (class) constructor which is called before any ConfigFile instance is created
		/// </summary>
		static ConfigFile()
		{
			Dictionary<string, ConfigFileSection> Sections = new(StringComparer.InvariantCultureIgnoreCase);
			try
			{
				// read the special ConfigRemap.ini file into sections
				FileReference ConfigRemapFile = FileReference.Combine(UnrealBuildBase.Unreal.EngineDirectory, "Config", "ConfigRedirects.ini");
				if (FileReference.Exists(ConfigRemapFile))
				{
					ReadIntoSections(ConfigRemapFile, Sections, ConfigLineAction.Set);
				}
			}
			catch (Exception)
			{
				// Make ConfigFile when EngineDirectory is unknown a warning since ConfigRemapFile cannot be read in this case; e.g. Assemblies outside Engine that depend on ConfigFile
				Log.Logger.LogWarning("Failed to read ConfigRemapFile into Sections");
			}

			// walk over the sections, where all but the special SectionNameRemap section is a section of keys to remap in that same section
			foreach (KeyValuePair<string, ConfigFileSection> Pair in Sections)
			{
				// remember a remap for section names
				if (Pair.Key.Equals("SectionNameRemap", StringComparison.OrdinalIgnoreCase))
				{
					foreach (ConfigLine Line in Pair.Value.Lines)
					{
						SectionNameRemap.Add(Line.Key, Line.Value);
					}
				}
				else
				{
					// any other section is rmembered by the section name here, and each key/value pair is a remap for the given section
					Dictionary<string, string> KeyRemap = new(StringComparer.OrdinalIgnoreCase);
					SectionKeyRemap.Add(Pair.Key, KeyRemap);
					foreach (ConfigLine Line in Pair.Value.Lines)
					{
						KeyRemap.Add(Line.Key, Line.Value);
					}
				}
			}
		}

		/// <summary>
		/// Constructs a new, empty config file
		/// </summary>
		/// <param name="DefaultAction">The default action to take when encountering arrays without a '+' prefix</param>
		public ConfigFile(ConfigLineAction DefaultAction = ConfigLineAction.Set)
		{
		}

		/// <summary>
		/// Reads config data from the given file.
		/// </summary>
		/// <param name="Location">File to read from</param>
		/// <param name="DefaultAction">The default action to take when encountering arrays without a '+' prefix</param>
		public ConfigFile(FileReference Location, ConfigLineAction DefaultAction = ConfigLineAction.Set)
		{
			ReadIntoSections(Location, Sections, DefaultAction);
		}

		private static void ReadIntoSections(FileReference Location, Dictionary<string, ConfigFileSection> Sections, ConfigLineAction DefaultAction)
		{
			using (StreamReader Reader = new StreamReader(Location.FullName))
			{
				ConfigFileSection? CurrentSection = null;
				Dictionary<string, string>? CurrentRemap = null;

				for (; ; )
				{
					string? Line = Reader.ReadLine();
					if (Line == null)
					{
						break;
					}

					// Find the first non-whitespace character
					for (int StartIdx = 0; StartIdx < Line.Length; StartIdx++)
					{
						if (Line[StartIdx] != ' ' && Line[StartIdx] != '\t')
						{
							// Find the last non-whitespace character. If it's an escaped newline, merge the following line with it.
							int EndIdx = Line.Length;
							while (EndIdx > StartIdx)
							{
								if (Line[EndIdx - 1] == '\\')
								{
									string? NextLine = Reader.ReadLine();
									if (NextLine == null)
									{
										break;
									}
									Line += NextLine;
									EndIdx = Line.Length;
									continue;
								}
								if (Line[EndIdx - 1] != ' ' && Line[EndIdx - 1] != '\t')
								{
									break;
								}

								EndIdx--;
							}

							// Break out if we've got a comment
							if (Line[StartIdx] == ';')
							{
								break;
							}
							if (Line[StartIdx] == '/' && StartIdx + 1 < Line.Length && Line[StartIdx + 1] == '/')
							{
								break;
							}

							// Check if it's the start of a new section
							if (Line[StartIdx] == '[')
							{
								CurrentSection = null;
								if (Line[EndIdx - 1] == ']')
								{
									string SectionName = Line.Substring(StartIdx + 1, EndIdx - StartIdx - 2);

									// lookup remaps
									SectionName = RemapSectionOrKey(SectionNameRemap, SectionName, $"which is a config section in '{Location.FullName}'");
									SectionKeyRemap.TryGetValue(SectionName, out CurrentRemap);

									if (!Sections.TryGetValue(SectionName, out CurrentSection))
									{
										CurrentSection = new ConfigFileSection(SectionName);
										Sections.Add(SectionName, CurrentSection);
									}
								}
								break;
							}

							// Otherwise add it to the current section or add a new one
							if (CurrentSection != null)
							{
								TryAddConfigLine(CurrentSection, CurrentRemap, Location.FullName, Line, StartIdx, EndIdx, DefaultAction, Sections);
								break;
							}

							// Otherwise just ignore it
							break;
						}
					}
				}
			}
		}

		/// <summary>
		/// Reads config data from the given string.
		/// </summary>
		/// <param name="IniText">Single line string of config settings in the format [Section1]:Key1=Value1,[Section2]:Key2=Value2</param>
		/// <param name="DefaultAction">The default action to take when encountering arrays without a '+' prefix</param>
		public ConfigFile(string IniText, ConfigLineAction DefaultAction = ConfigLineAction.Set)
		{

			// Break into individual settings of the form [Section]:Key=Value
			string[] SettingLines = IniText.Split(new char[] { ',' });
			Dictionary<string, string>? CurrentRemap;
			foreach (string Setting in SettingLines)
			{
				// Locate and break off the section name
				string SectionName = Setting.Remove(Setting.IndexOf(':', StringComparison.Ordinal)).Trim(new char[] { '[', ']' });
				// lookup remaps
				RemapSectionOrKey(SectionNameRemap, SectionName, $"which is a config section (found in a string in code - search your .ini files for it, the source file is unknown)");
				SectionKeyRemap.TryGetValue(SectionName, out CurrentRemap);

				if (SectionName.Length > 0)
				{
					if (!Sections.TryGetValue(SectionName, out ConfigFileSection? CurrentSection))
					{
						CurrentSection = new ConfigFileSection(SectionName);
						Sections.Add(SectionName, CurrentSection);
					}

					if (CurrentSection != null)
					{
						string IniKeyValue = Setting.Substring(Setting.IndexOf(':', StringComparison.Ordinal) + 1);
						TryAddConfigLine(CurrentSection, CurrentRemap, "unknown source file", IniKeyValue, 0, IniKeyValue.Length, DefaultAction, Sections);
					}
				}
			}
		}

		/// <summary>
		/// Try to parse a key/value pair from the given line, and add it to a config section
		/// </summary>
		/// <param name="Section">The section to receive the parsed config line</param>
		/// <param name="KeyRemap">Optional map to change names of keys on read</param> 
		/// <param name="Filename">Optional map to change names of keys on read</param> 
		/// <param name="Line">Text to parse</param>
		/// <param name="StartIdx">Index of the first non-whitespace character in this line</param>
		/// <param name="EndIdx">Index of the last (exclusive) non-whitespace character in this line</param>
		/// <param name="DefaultAction">The default action to take if '+' or '-' is not specified on a config line</param>
		/// <param name="Sections">The sections to find the redirected section for receiving the config line</param>
		/// <returns>True if the line was parsed correctly, false otherwise</returns>
		static bool TryAddConfigLine(ConfigFileSection Section, Dictionary<string, string>? KeyRemap, string Filename, string Line, int StartIdx, int EndIdx, ConfigLineAction DefaultAction, Dictionary<string, ConfigFileSection> Sections)
		{
			// Find the '=' character separating key and value
			int EqualsIdx = Line.IndexOf('=', StartIdx, EndIdx - StartIdx);
			if (EqualsIdx == -1 && Line[StartIdx] != '!')
			{
				return false;
			}

			// Keep track of the start of the key name
			int KeyStartIdx = StartIdx;

			// Remove the +/-/! prefix, if present
			ConfigLineAction Action = DefaultAction;
			if (Line[KeyStartIdx] == '+' || Line[KeyStartIdx] == '-' || Line[KeyStartIdx] == '!')
			{
				Action = (Line[KeyStartIdx] == '+') ? ConfigLineAction.Add : (Line[KeyStartIdx] == '!') ? ConfigLineAction.RemoveKey : ConfigLineAction.RemoveKeyValue;
				KeyStartIdx++;
				while (Line[KeyStartIdx] == ' ' || Line[KeyStartIdx] == '\t')
				{
					KeyStartIdx++;
				}
			}

			// RemoveKey actions do not require a value
			if (Action == ConfigLineAction.RemoveKey && EqualsIdx == -1)
			{
				Section.Lines.Add(new ConfigLine(Action, Line.Substring(KeyStartIdx).Trim(), ""));
				return true;
			}

			// Remove trailing spaces after the name of the key
			int KeyEndIdx = EqualsIdx;
			for (; KeyEndIdx > KeyStartIdx; KeyEndIdx--)
			{
				if (Line[KeyEndIdx - 1] != ' ' && Line[KeyEndIdx - 1] != '\t')
				{
					break;
				}
			}

			// Make sure there's a non-empty key name
			if (KeyStartIdx == EqualsIdx)
			{
				return false;
			}

			// Skip whitespace between the '=' and the start of the value
			int ValueStartIdx = EqualsIdx + 1;
			for (; ValueStartIdx < EndIdx; ValueStartIdx++)
			{
				if (Line[ValueStartIdx] != ' ' && Line[ValueStartIdx] != '\t')
				{
					break;
				}
			}

			bool HasStrippedValueQuotes = false;

			// Strip quotes around the value if present
			int ValueEndIdx = EndIdx;
			if (ValueEndIdx >= ValueStartIdx + 2 && Line[ValueStartIdx] == '"' && Line[ValueEndIdx - 1] == '"')
			{
				HasStrippedValueQuotes = true;
				ValueStartIdx++;
				ValueEndIdx--;
			}

			// Add it to the config section
			string Key = Line.Substring(KeyStartIdx, KeyEndIdx - KeyStartIdx);
			string Value = Line.Substring(ValueStartIdx, ValueEndIdx - ValueStartIdx);

			// remap the key if needed
			string NewKey = RemapSectionOrKey(KeyRemap, Key, $"which is a config key in section [{Section.Name}], in '{Filename}'");

			// look for a section:name remap
			if (!NewKey.Equals(Key, StringComparison.Ordinal) && NewKey.Contains(':', StringComparison.Ordinal))
			{
				string SectionName = NewKey.Substring(0, NewKey.IndexOf(':', StringComparison.Ordinal));
				ConfigFileSection? CurrentSection;
				if (!Sections.TryGetValue(SectionName, out CurrentSection))
				{
					CurrentSection = new ConfigFileSection(SectionName);
					Sections.Add(SectionName, CurrentSection);
				}

				string KeyName = NewKey.Substring(NewKey.IndexOf(':', StringComparison.Ordinal) + 1);

				CurrentSection.Lines.Add(new ConfigLine(Action, KeyName, Value, HasStrippedValueQuotes));

				return true;
			}

			Section.Lines.Add(new ConfigLine(Action, NewKey, Value, HasStrippedValueQuotes));
			return true;
		}

		/// <summary>
		/// Names of sections in this file
		/// </summary>
		public IEnumerable<string> SectionNames => Sections.Keys;

		/// <summary>
		/// Tries to get a config section by name, or creates one if it doesn't exist
		/// </summary>
		/// <param name="SectionName">Name of the section to look for</param>
		/// <returns>The config section</returns>
		public ConfigFileSection FindOrAddSection(string SectionName)
		{
			ConfigFileSection? Section;
			if (!Sections.TryGetValue(SectionName, out Section))
			{
				Section = new ConfigFileSection(SectionName);
				Sections.Add(SectionName, Section);
			}
			return Section;
		}

		/// <summary>
		/// Tries to get a config section by name
		/// </summary>
		/// <param name="SectionName">Name of the section to look for</param>
		/// <param name="RawSection">On success, the config section that was found</param>
		/// <returns>True if the section was found, false otherwise</returns>
		public bool TryGetSection(string SectionName, [NotNullWhen(true)] out ConfigFileSection? RawSection)
		{
			return Sections.TryGetValue(SectionName, out RawSection);
		}

		/// <summary>
		/// Write the config file out to the given location. Useful for debugging.
		/// </summary>
		/// <param name="Location">The file to write</param>
		public void Write(FileReference Location)
		{
			using (StreamWriter Writer = new StreamWriter(Location.FullName))
			{
				foreach (ConfigFileSection Section in Sections.Values)
				{
					Writer.WriteLine("[{0}]", Section.Name);
					foreach (ConfigLine Line in Section.Lines)
					{
						if (Line.ShouldPreserveValueQuotations)
						{
							Line.Value = $"\"{Line.Value}\"";
						}
						Writer.WriteLine("{0}", Line.ToString());
					}
					Writer.WriteLine();
				}
			}
		}
	}
}
