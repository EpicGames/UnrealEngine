// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace UnrealBuildTool
{
	/// <summary>
	/// Attribute indicating a value which should be populated from a UE .ini config file
	/// </summary>
	[AttributeUsage(AttributeTargets.Field | AttributeTargets.Property)]
	public sealed class ConfigFileAttribute : Attribute
	{
		/// <summary>
		/// Name of the config hierarchy to read from
		/// </summary>
		public ConfigHierarchyType ConfigType { get; }

		/// <summary>
		/// Section containing the setting
		/// </summary>
		public string SectionName { get; }

		/// <summary>
		/// Key name to search for
		/// </summary>
		public string? KeyName { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="configType">Type of the config hierarchy to read from</param>
		/// <param name="sectionName">Section containing the setting</param>
		/// <param name="keyName">Key name to search for. Optional; uses the name of the field if not set.</param>
		public ConfigFileAttribute(ConfigHierarchyType configType, string sectionName, string? keyName = null)
		{
			ConfigType = configType;
			SectionName = sectionName;
			KeyName = keyName;
		}
	}
}
