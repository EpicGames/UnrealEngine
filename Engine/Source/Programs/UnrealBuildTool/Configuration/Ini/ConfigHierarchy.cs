// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.Linq;
using System.Text;
using System.Text.RegularExpressions;
using EpicGames.Core;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// Types of config file hierarchy
	/// </summary>
	public enum ConfigHierarchyType
	{
		/// <summary>
		/// BaseGame.ini, DefaultGame.ini, etc...
		/// </summary>
		Game,

		/// <summary>
		/// BaseEngine.ini, DefaultEngine.ini, etc...
		/// </summary>
		Engine,

		/// <summary>
		/// BaseEditor.ini, DefaultEditor.ini, etc...
		/// </summary>
		Editor,

		/// <summary>
		/// BaseEditorPerProjectUserSettings.ini, DefaultEditorPerProjectUserSettings.ini, etc..
		/// </summary>
		EditorPerProjectUserSettings,

		/// <summary>
		/// BaseEncryption.ini, DefaultEncryption.ini, etc..
		/// </summary>
		Encryption,

		/// <summary>
		/// BaseCrypto.ini, DefaultCrypto.ini, etc..
		/// </summary>
		Crypto,

		/// <summary>
		/// BaseEditorSettings.ini, DefaultEditorSettings.ini, etc...
		/// </summary>
		EditorSettings,

		/// <summary>
		/// BaseInstallBundle.ini, DefaultInstallBundle.ini, etc...
		/// </summary>
		InstallBundle,

		/// <summary>
		/// BasePakFileRules.ini, DefaultPakFileRules.ini, etc, etc....
		/// </summary>
		PakFileRules,
	}

	/// <summary>
	/// Stores a set of merged key/value pairs for a config section
	/// </summary>
	public class ConfigHierarchySection
	{
		/// <summary>
		/// Map of key names to their values
		/// </summary>
		readonly Dictionary<string, List<string>> KeyToValue = new Dictionary<string, List<string>>(StringComparer.InvariantCultureIgnoreCase);

		/// <summary>
		/// List of key names in the order they were first added (preserves INI file order)
		/// </summary>
		readonly List<string> KeyOrder = new List<string>();

		/// <summary>
		/// List of (key, value) pairs in the exact order they were added from INI file
		/// Preserves interleaved order of array elements with different keys (e.g., +Steps= and +Steps[Variant]=)
		/// </summary>
		readonly List<(string Key, string Value)> ArrayElementOrder = new List<(string, string)>();

		/// <summary>
		/// Construct a merged config section from the given per-file config sections
		/// </summary>
		/// <param name="FileSections">Config sections from individual files</param>
		public ConfigHierarchySection(IEnumerable<ConfigFileSection> FileSections)
		{
			foreach (ConfigFileSection FileSection in FileSections)
			{
				foreach (ConfigLine Line in FileSection.Lines)
				{
					if (Line.Action == ConfigLineAction.RemoveKey)
					{
						KeyToValue.Remove(Line.Key);
						KeyOrder.RemoveAll(k => k.Equals(Line.Key, StringComparison.OrdinalIgnoreCase));
						ArrayElementOrder.RemoveAll(pair => pair.Key.Equals(Line.Key, StringComparison.OrdinalIgnoreCase));
						continue;
					}

					// Find or create the values for this key
					List<string>? Values;

					if (KeyToValue.TryGetValue(Line.Key, out Values))
					{
						// Update the existing list
						if (Line.Action == ConfigLineAction.Set)
						{
							Values.Clear();
							Values.Add(Line.Value);
							// Track order: when Set clears the array, also remove all previous entries for this key
							ArrayElementOrder.RemoveAll(pair => pair.Key.Equals(Line.Key, StringComparison.OrdinalIgnoreCase));
							ArrayElementOrder.Add((Line.Key, Line.Value));
						}
						else if (Line.Action == ConfigLineAction.Add)
						{
							Values.Add(Line.Value);
							ArrayElementOrder.Add((Line.Key, Line.Value));
						}
						else if (Line.Action == ConfigLineAction.RemoveKeyValue)
						{
							Values.RemoveAll(x => x.Equals(Line.Value, StringComparison.OrdinalIgnoreCase));
							// Also remove from order tracking
							ArrayElementOrder.RemoveAll(pair => pair.Key.Equals(Line.Key, StringComparison.OrdinalIgnoreCase) && pair.Value.Equals(Line.Value, StringComparison.OrdinalIgnoreCase));
						}
					}
					else
					{
						// If it's a set or add action, create and add a new list
						if (Line.Action == ConfigLineAction.Set || Line.Action == ConfigLineAction.Add)
						{
							Values = new List<string>();
							Values.Add(Line.Value);
							KeyToValue.Add(Line.Key, Values);
							KeyOrder.Add(Line.Key);
							ArrayElementOrder.Add((Line.Key, Line.Value));
						}
					}
				}
			}
		}

		/// <summary>
		/// Returns a list of key names
		/// </summary>
		public IEnumerable<string> KeyNames => KeyOrder;

		/// <summary>
		/// Tries to find the value for a given key
		/// </summary>
		/// <param name="KeyName">The key name to search for</param>
		/// <param name="Value">On success, receives the corresponding value</param>
		/// <returns>True if the key was found, false otherwise</returns>
		public bool TryGetValue(string KeyName, [NotNullWhen(true)] out string? Value)
		{
			List<string>? ValuesList;
			if (KeyToValue.TryGetValue(KeyName, out ValuesList) && ValuesList.Count > 0)
			{
				Value = ValuesList[0];
				return true;
			}
			else
			{
				Value = null;
				return false;
			}
		}

		/// <summary>
		/// Tries to find the values for a given key
		/// </summary>
		/// <param name="KeyName">The key name to search for</param>
		/// <param name="Values">On success, receives a list of the corresponding values</param>
		/// <returns>True if the key was found, false otherwise</returns>
		public bool TryGetValues(string KeyName, [NotNullWhen(true)] out IReadOnlyList<string>? Values)
		{
			List<string>? ValuesList;
			if (KeyToValue.TryGetValue(KeyName, out ValuesList))
			{
				Values = ValuesList;
				return true;
			}
			else
			{
				Values = null;
				return false;
			}
		}

		/// <summary>
		/// Returns array elements in the exact order they were added from INI files.
		/// Preserves interleaving of elements with different keys (e.g., +Steps= and +Steps[Variant]=)
		/// </summary>
		public IReadOnlyList<(string Key, string Value)> GetArrayElementsInOrder()
		{
			return ArrayElementOrder;
		}
	}

	/// <summary>
	/// Encapsulates a hierarchy of config files, merging sections from them together on request 
	/// </summary>
	public sealed class ConfigHierarchy : IDisposable
	{
		/// <summary>
		/// Array of 
		/// </summary>
		readonly ConfigFile[] Files;

		/// <summary>
		/// Set of file references
		/// </summary>
		public IEnumerable<FileReference> FileReferences { get; }

		/// <summary>
		/// Cache of requested config sections
		/// </summary>
		readonly ConcurrentDictionary<string, Lazy<ConfigHierarchySection>> NameToSection = new(StringComparer.InvariantCultureIgnoreCase);

		/// <summary>
		/// Construct a config hierarchy from the given files
		/// </summary>
		/// <param name="Files">Set of files to include (in order)</param>
		/// <param name="FileReferences">Set of file paths, for tracking</param>
		public ConfigHierarchy(IEnumerable<ConfigFile> Files, IEnumerable<FileReference>? FileReferences = null)
		{
			this.Files = [.. Files];
			this.FileReferences = FileReferences ?? [];
		}

		#region IDisposible
		/// <inheritdoc/>
		public void Dispose()
		{
		}
		#endregion

		/// <summary>
		/// Names of all sections in all config files
		/// </summary>
		/// <returns></returns>
		public HashSet<string> SectionNames => [.. Files.SelectMany(x => x.SectionNames)];

		/// <summary>
		/// Finds a config section with the given name
		/// </summary>
		/// <param name="SectionName">Name of the section to look for</param>
		/// <returns>The merged config section</returns>
		public ConfigHierarchySection FindSection(string SectionName)
		{
			return NameToSection.GetOrAdd(SectionName, new Lazy<ConfigHierarchySection>(() =>
			{
				// Find all the raw sections from the file hierarchy
				List<ConfigFileSection> RawSections = new List<ConfigFileSection>();
				foreach (ConfigFile File in Files)
				{
					ConfigFileSection? RawSection;
					if (File.TryGetSection(SectionName, out RawSection))
					{
						RawSections.Add(RawSection);
					}
				}

				// Merge them together and add it to the cache
				return new ConfigHierarchySection(RawSections);
			})).Value;
		}

		/// <summary>
		/// Legacy function for ease of transition from ConfigCacheIni to ConfigHierarchy. Gets a bool with the given key name.
		/// </summary>
		/// <param name="SectionName">Section name</param>
		/// <param name="KeyName">Key name</param>
		/// <param name="Value">Value associated with the specified key. If the key has more than one value, only the first one is returned</param>
		/// <returns>True if the key exists</returns>
		public bool GetBool(string SectionName, string KeyName, out bool Value)
		{
			return TryGetValue(SectionName, KeyName, out Value);
		}

		/// <summary>
		/// Legacy function for ease of transition from ConfigCacheIni to ConfigHierarchy. Gets an array with the given key name, returning null on failure.
		/// </summary>
		/// <param name="SectionName">Section name</param>
		/// <param name="KeyName">Key name</param>
		/// <param name="Values">Value associated with the specified key. If the key has more than one value, only the first one is returned</param>
		/// <returns>True if the key exists</returns>
		public bool GetArray(string SectionName, string KeyName, [NotNullWhen(true)] out List<string>? Values)
		{
			IReadOnlyList<string>? ValuesEnumerable;
			if (TryGetValues(SectionName, KeyName, out ValuesEnumerable))
			{
				Values = ValuesEnumerable.ToList();
				return true;
			}
			else
			{
				Values = null;
				return false;
			}
		}

		/// <summary>
		/// Legacy function for ease of transition from ConfigCacheIni to ConfigHierarchy. Gets a string with the given key name, returning an empty string on failure.
		/// </summary>
		/// <param name="SectionName">Section name</param>
		/// <param name="KeyName">Key name</param>
		/// <param name="Value">Value associated with the specified key. If the key has more than one value, only the first one is returned</param>
		/// <returns>True if the key exists</returns>
		public bool GetString(string SectionName, string KeyName, out string Value)
		{
			string? RetrievedValue;
			if (TryGetValue(SectionName, KeyName, out RetrievedValue))
			{
				Value = RetrievedValue;
				return true;
			}
			else
			{
				Value = "";
				return false;
			}
		}

		/// <summary>
		/// Legacy function for ease of transition from ConfigCacheIni to ConfigHierarchy. Gets an int with the given key name.
		/// </summary>
		/// <param name="SectionName">Section name</param>
		/// <param name="KeyName">Key name</param>
		/// <param name="Value">Value associated with the specified key. If the key has more than one value, only the first one is returned</param>
		/// <returns>True if the key exists</returns>
		public bool GetInt32(string SectionName, string KeyName, out int Value)
		{
			return TryGetValue(SectionName, KeyName, out Value);
		}

		/// <summary>
		/// Gets a single string value associated with the specified key.
		/// </summary>
		/// <param name="SectionName">Section name</param>
		/// <param name="KeyName">Key name</param>
		/// <param name="Value">Value associated with the specified key. If the key has more than one value, only the first one is returned</param>
		/// <returns>True if the key exists</returns>
		public bool TryGetValue(string SectionName, string KeyName, [NotNullWhen(true)] out string? Value)
		{
			return FindSection(SectionName).TryGetValue(KeyName, out Value);
		}

		/// <summary>
		/// Gets a single bool value associated with the specified key.
		/// </summary>
		/// <param name="SectionName">Section name</param>
		/// <param name="KeyName">Key name</param>
		/// <param name="Value">Value associated with the specified key. If the key has more than one value, only the first one is returned</param>
		/// <returns>True if the key exists</returns>
		public bool TryGetValue(string SectionName, string KeyName, out bool Value)
		{
			string? Text;
			if (!TryGetValue(SectionName, KeyName, out Text))
			{
				Value = false;
				return false;
			}
			return TryParse(Text, out Value);
		}

		/// <summary>
		/// Gets a single Int32 value associated with the specified key.
		/// </summary>
		/// <param name="SectionName">Section name</param>
		/// <param name="KeyName">Key name</param>
		/// <param name="Value">Value associated with the specified key. If the key has more than one value, only the first one is returned</param>
		/// <returns>True if the key exists</returns>
		public bool TryGetValue(string SectionName, string KeyName, out int Value)
		{
			string? Text;
			if (!TryGetValue(SectionName, KeyName, out Text))
			{
				Value = 0;
				return false;
			}
			return TryParse(Text, out Value);
		}

		/// <summary>
		/// Gets a single GUID value associated with the specified key.
		/// </summary>
		/// <param name="SectionName">Section name</param>
		/// <param name="KeyName">Key name</param>
		/// <param name="Value">Value associated with the specified key. If the key has more than one value, only the first one is returned</param>
		/// <returns>True if the key exists</returns>
		public bool TryGetValue(string SectionName, string KeyName, out Guid Value)
		{
			string? Text;
			if (!TryGetValue(SectionName, KeyName, out Text))
			{
				Value = Guid.Empty;
				return false;
			}
			return TryParse(Text, out Value);
		}

		/// <summary>
		/// Gets a single-precision floating point value associated with the specified key.
		/// </summary>
		/// <param name="SectionName">Section name</param>
		/// <param name="KeyName">Key name</param>
		/// <param name="Value">Value associated with the specified key. If the key has more than one value, only the first one is returned</param>
		/// <returns>True if the key exists</returns>
		public bool TryGetValue(string SectionName, string KeyName, out float Value)
		{
			string? Text;
			if (!TryGetValue(SectionName, KeyName, out Text))
			{
				Value = 0;
				return false;
			}
			return TryParse(Text, out Value);
		}

		/// <summary>
		/// Gets a double-precision floating point value associated with the specified key.
		/// </summary>
		/// <param name="SectionName">Section name</param>
		/// <param name="KeyName">Key name</param>
		/// <param name="Value">Value associated with the specified key. If the key has more than one value, only the first one is returned</param>
		/// <returns>True if the key exists</returns>
		public bool TryGetValue(string SectionName, string KeyName, out double Value)
		{
			string? Text;
			if (!TryGetValue(SectionName, KeyName, out Text))
			{
				Value = 0;
				return false;
			}
			return TryParse(Text, out Value);
		}

		/// <summary>
		/// Gets an enumeration value associated with the specified key.
		/// </summary>
		/// <typeparam name="T"></typeparam>
		/// <param name="SectionName">Section name</param>
		/// <param name="KeyName">Key name</param>
		/// <param name="Value">Value associated with the specified key. If the key has more than one value, only the first one is returned</param>
		/// <returns>True if the key exists</returns>
		public bool TryGetValue<T>(string SectionName, string KeyName, out T Value) where T : struct
		{
			string? Text;
			if (!TryGetValue(SectionName, KeyName, out Text))
			{
				Value = default;
				return false;
			}
			return Enum.TryParse<T>(Text, out Value);
		}

		/// <summary>
		/// Gets all values associated with the specified key
		/// </summary>
		/// <param name="SectionName">Section where the key is located</param>
		/// <param name="KeyName">Key name</param>
		/// <param name="Values">Copy of the list containing all values associated with the specified key</param>
		/// <returns>True if the key exists</returns>
		public bool TryGetValues(string SectionName, string KeyName, [NotNullWhen(true)] out IReadOnlyList<string>? Values)
		{
			return FindSection(SectionName).TryGetValues(KeyName, out Values);
		}

		/// <summary>
		/// Gets the value for the given type. Can return a full struct hierarchy.
		/// </summary>
		/// <param name="SectionName">Section where the key is located</param>
		/// <param name="KeyName">Key name</param>
		/// <param name="Value">Value hierarchy associated with the specified key. All field names must exist</param>
		/// <returns>True if the key exists and could be parsed</returns>
		public bool TryGetValueGeneric<T>(string SectionName, string KeyName, [NotNullWhen(true)] out T? Value) where T : new()
		{
			if (TryGetValue(SectionName, KeyName, out string? Line))
			{
				return ConfigValueParser.TryParseGeneric(Line, out Value);
			}

			Value = default;
			return false;
		}

		/// <summary>
		/// Gets the array of values for the given type. Can return a full struct hierarchy.
		/// </summary>
		/// <param name="SectionName">Section where the key is located</param>
		/// <param name="KeyName">Key name</param>
		/// <param name="Values">Array of values associated with the specified key. All field names must exist</param>
		/// <returns>True if the key exists and could be parsed</returns>
		public bool TryGetValuesGeneric<T>(string SectionName, string KeyName, [NotNullWhen(true)] out T[]? Values) where T : new()
		{
			if (TryGetValues(SectionName, KeyName, out IReadOnlyList<string>? Lines))
			{
				return ConfigValueParser.TryParseArrayGeneric(Lines.ToArray(), out Values);
			}

			Values = null;
			return false;
		}

		/// <summary>
		/// Parse a string as a boolean value
		/// </summary>
		/// <param name="Text">The text to parse</param>
		/// <param name="Value">The parsed value, if successful</param>
		/// <returns>True if the text was parsed, false otherwise</returns>
		public static bool TryParse(string Text, out bool Value)
		{
			// C# Boolean type expects "False" or "True" but since we're not case sensitive, we need to suppor that manually
			if (Text == "1" || Text.Equals("true", StringComparison.OrdinalIgnoreCase))
			{
				Value = true;
				return true;
			}
			else if (Text == "0" || Text.Equals("false", StringComparison.OrdinalIgnoreCase))
			{
				Value = false;
				return true;
			}
			else
			{
				Value = false;
				return false;
			}
		}

		/// <summary>
		/// Parse a string as an integer value
		/// </summary>
		/// <param name="Text">The text to parse</param>
		/// <param name="Value">The parsed value, if successful</param>
		/// <returns>True if the text was parsed, false otherwise</returns>
		public static bool TryParse(string Text, out int Value)
		{
			return Int32.TryParse(Text, out Value);
		}

		/// <summary>
		/// Parse a string as a GUID value
		/// </summary>
		/// <param name="Text">The text to parse</param>
		/// <param name="Value">The parsed value, if successful</param>
		/// <returns>True if the text was parsed, false otherwise</returns>
		public static bool TryParse(string Text, out Guid Value)
		{
			bool textContainsAbcd =
				Text.Contains("A=", StringComparison.Ordinal) &&
				Text.Contains("B=", StringComparison.Ordinal) &&
				Text.Contains("C=", StringComparison.Ordinal) &&
				Text.Contains("D=", StringComparison.Ordinal);
			if (textContainsAbcd)
			{
				char[] Separators = new char[] { '(', ')', '=', ',', ' ', 'A', 'B', 'C', 'D' };
				string[] ComponentValues = Text.Split(Separators, StringSplitOptions.RemoveEmptyEntries);
				if (ComponentValues.Length == 4)
				{
					StringBuilder HexString = new StringBuilder();
					for (int ComponentIndex = 0; ComponentIndex < 4; ComponentIndex++)
					{
						int IntegerValue;
						if (!Int32.TryParse(ComponentValues[ComponentIndex], out IntegerValue))
						{
							Value = Guid.Empty;
							return false;
						}
						HexString.Append(IntegerValue.ToString("X8"));
					}
					Text = HexString.ToString();
				}
			}
			return Guid.TryParseExact(Text, "N", out Value);
		}

		/// <summary>
		/// Parse a string as a single-precision floating point value
		/// </summary>
		/// <param name="Text">The text to parse</param>
		/// <param name="Value">The parsed value, if successful</param>
		/// <returns>True if the text was parsed, false otherwise</returns>
		public static bool TryParse(string Text, out float Value)
		{
			if (Text.EndsWith("f", StringComparison.Ordinal))
			{
				return Single.TryParse(Text.Substring(0, Text.Length - 1), out Value);
			}
			else
			{
				return Single.TryParse(Text, out Value);
			}
		}

		/// <summary>
		/// Parse a string as a double-precision floating point value
		/// </summary>
		/// <param name="Text">The text to parse</param>
		/// <param name="Value">The parsed value, if successful</param>
		/// <returns>True if the text was parsed, false otherwise</returns>
		public static bool TryParse(string Text, out double Value)
		{
			if (Text.EndsWith("f", StringComparison.Ordinal))
			{
				return Double.TryParse(Text.Substring(0, Text.Length - 1), out Value);
			}
			else
			{
				return Double.TryParse(Text, out Value);
			}
		}

		/// <summary>
		/// Attempts to parse the given line as a UE config object (eg. (Name="Foo",Number=1234)).
		/// </summary>
		/// <param name="Line">Line of text to parse</param>
		/// <param name="Properties">Receives key/value pairs for the config object</param>
		/// <returns>True if an object was parsed, false otherwise</returns>
		public static bool TryParse(string Line, [NotNullWhen(true)] out Dictionary<string, string>? Properties)
		{
			// Convert the string to a zero-terminated array, to make parsing easier.
			char[] Chars = new char[Line.Length + 1];
			Line.CopyTo(0, Chars, 0, Line.Length);

			// Get the opening paren
			int Idx = 0;
			while (Char.IsWhiteSpace(Chars[Idx]))
			{
				Idx++;
			}
			if (Chars[Idx] != '(')
			{
				Properties = null;
				return false;
			}

			// Read to the next token
			Idx++;
			while (Char.IsWhiteSpace(Chars[Idx]))
			{
				Idx++;
			}

			// Create the dictionary to receive the new properties
			Dictionary<string, string> NewProperties = new Dictionary<string, string>();

			// Read a sequence of key/value pairs
			StringBuilder Value = new StringBuilder();
			if (Chars[Idx] != ')')
			{
				for (; ; )
				{
					// Find the end of the name
					int NameIdx = Idx;
					while (Char.IsLetterOrDigit(Chars[Idx]) || Chars[Idx] == '_')
					{
						Idx++;
					}
					if (Idx == NameIdx)
					{
						Properties = null;
						return false;
					}

					// Extract the key string, and make sure it hasn't already been added
					string Key = new string(Chars, NameIdx, Idx - NameIdx);
					if (NewProperties.ContainsKey(Key))
					{
						Properties = null;
						return false;
					}

					// Consume the equals character
					while (Char.IsWhiteSpace(Chars[Idx]))
					{
						Idx++;
					}
					if (Chars[Idx] != '=')
					{
						Properties = null;
						return false;
					}

					// Move to the value
					Idx++;
					while (Char.IsWhiteSpace(Chars[Idx]))
					{
						Idx++;
					}

					// Parse the value
					Value.Clear();
					if (Char.IsLetterOrDigit(Chars[Idx]) || Chars[Idx] == '_' || Chars[Idx] == '-')
					{
						while (Char.IsLetterOrDigit(Chars[Idx]) || Chars[Idx] == '_' || Chars[Idx] == '-' || Chars[Idx] == '.')
						{
							Value.Append(Chars[Idx]);
							Idx++;
						}
					}
					else if (Chars[Idx] == '\"')
					{
						Idx++;
						for (; Chars[Idx] != '\"'; Idx++)
						{
							if (Chars[Idx] == '\0')
							{
								Properties = null;
								return false;
							}
							else
							{
								Value.Append(Chars[Idx]);
							}
						}
						Idx++;
					}
					else if (Chars[Idx] == '(')
					{
						Value.Append(Chars[Idx++]);

						bool bInQuotes = false;
						for (int Nesting = 1; Nesting > 0; Idx++)
						{
							if (Chars[Idx] == '\0')
							{
								Properties = null;
								return false;
							}
							else if (Chars[Idx] == '(' && !bInQuotes)
							{
								Nesting++;
							}
							else if (Chars[Idx] == ')' && !bInQuotes)
							{
								Nesting--;
							}
							else if (Chars[Idx] == '\"' || Chars[Idx] == '\'')
							{
								bInQuotes ^= true;
							}
							Value.Append(Chars[Idx]);
						}
					}
					else if (Chars[Idx] != ')' && Chars[Idx] != ',')
					{
						Properties = null;
						return false;
					}

					// Extract the value string
					NewProperties[Key] = Value.ToString();

					// Move to the separator
					while (Char.IsWhiteSpace(Chars[Idx]))
					{
						Idx++;
					}
					if (Chars[Idx] == ')')
					{
						break;
					}
					if (Chars[Idx] != ',')
					{
						Properties = null;
						return false;
					}

					// Move to the next field
					Idx++;
					while (Char.IsWhiteSpace(Chars[Idx]))
					{
						Idx++;
					}
				}
			}

			// Make sure we're at the end of the string
			Idx++;
			while (Char.IsWhiteSpace(Chars[Idx]))
			{
				Idx++;
			}
			if (Chars[Idx] != '\0')
			{
				Properties = null;
				return false;
			}

			Properties = NewProperties;
			return true;
		}

		/// <summary>
		/// Attempts to parse the given line as a UE config array (eg. ("one", "two", "three") ).
		/// </summary>
		/// <param name="Line">Line of text to parse</param>
		/// <param name="Array">Receives array for the config array</param>
		/// <returns>True if an array was parsed, false otherwise</returns>
		public static bool TryParse(string Line, [NotNullWhen(true)] out string[]? Array)
		{
			// Convert the string to a zero-terminated array, to make parsing easier.
			char[] Chars = new char[Line.Length + 1];
			Line.CopyTo(0, Chars, 0, Line.Length);

			// Get the opening paren
			int Idx = 0;
			while (Char.IsWhiteSpace(Chars[Idx]))
			{
				Idx++;
			}
			if (Chars[Idx] != '(')
			{
				Array = null;
				return false;
			}

			// Read to the next token
			Idx++;
			while (Char.IsWhiteSpace(Chars[Idx]))
			{
				Idx++;
			}

			// Create the list to receive the new items
			List<string> NewArray = new List<string>();

			// Read a sequence items
			StringBuilder Value = new StringBuilder();
			if (Chars[Idx] != ')')
			{
				for (; ; )
				{
					// Skip whitespace
					while (Char.IsWhiteSpace(Chars[Idx]))
					{
						Idx++;
					}

					// Parse the value
					Value.Clear();
					if (Char.IsLetterOrDigit(Chars[Idx]) || Chars[Idx] == '_' || Chars[Idx] == '-')
					{
						while (Char.IsLetterOrDigit(Chars[Idx]) || Chars[Idx] == '_' || Chars[Idx] == '-' || Chars[Idx] == '.')
						{
							Value.Append(Chars[Idx]);
							Idx++;
						}
					}
					else if (Chars[Idx] == '\"')
					{
						Idx++;
						for (; Chars[Idx] != '\"'; Idx++)
						{
							if (Chars[Idx] == '\0')
							{
								Array = null;
								return false;
							}
							else
							{
								Value.Append(Chars[Idx]);
							}
						}
						Idx++;
					}
					else if (Chars[Idx] == '(')
					{
						Value.Append(Chars[Idx++]);

						bool bInQuotes = false;
						for (int Nesting = 1; Nesting > 0; Idx++)
						{
							if (Chars[Idx] == '\0')
							{
								Array = null;
								return false;
							}
							else if (Chars[Idx] == '(' && !bInQuotes)
							{
								Nesting++;
							}
							else if (Chars[Idx] == ')' && !bInQuotes)
							{
								Nesting--;
							}
							else if (Chars[Idx] == '\"' || Chars[Idx] == '\'')
							{
								bInQuotes ^= true;
							}
							Value.Append(Chars[Idx]);
						}
					}
					else
					{
						Array = null;
						return false;
					}

					// Store the item
					NewArray.Add(Value.ToString());

					// Move to the separator
					while (Char.IsWhiteSpace(Chars[Idx]))
					{
						Idx++;
					}
					if (Chars[Idx] == ')')
					{
						break;
					}
					if (Chars[Idx] != ',')
					{
						Array = null;
						return false;
					}

					// Move to the next field
					Idx++;
					while (Char.IsWhiteSpace(Chars[Idx]))
					{
						Idx++;
					}
				}
			}

			// Make sure we're at the end of the string
			Idx++;
			while (Char.IsWhiteSpace(Chars[Idx]))
			{
				Idx++;
			}
			if (Chars[Idx] != '\0')
			{
				Array = null;
				return false;
			}

			Array = NewArray.ToArray();
			return true;
		}

		/// <summary>
		/// Attempts to parse the given line as a UE map (eg. (("key1","value1"), ("key2","value2")).
		/// </summary>
		/// <param name="Line">Line of text to parse</param>
		/// <param name="Map">Receives dictionary for the config map</param>
		/// <returns>True if a map was parsed, false otherwise</returns>
		public static bool TryParseAsMap(string Line, [NotNullWhen(true)] out Dictionary<string, string>? Map)
		{
			// read outer array
			if (!TryParse(Line, out string[]? Array))
			{
				Map = null;
				return false;
			}

			// read each pair - they're stored in the same way as an array of 2
			Dictionary<string, string> NewMap = new Dictionary<string, string>();
			foreach (string ArrayItem in Array)
			{
				if (!TryParse(ArrayItem, out string[]? Pairs) || Pairs.Length != 2)
				{
					Map = null;
					return false;
				}

				NewMap[Pairs[0]] = Pairs[1];
			}

			Map = NewMap;
			return true;
		}

		class ConfigLayerExpansion
		{
			// a set of replacements from the source file to possible other files
			public string? Before1 = null;
			public string? After1 = null;
			public string? Before2 = null;
			public string? After2 = null;
		};

		static readonly string[] ConfigLayers =
		{
			// Engine/Base.ini
			"{ENGINE}/Config/Base.ini",
			// Engine/Base*.ini
 			"{ENGINE}/Config/Base{TYPE}.ini",
			// Engine/Platform/BasePlatform*.ini
			"{ENGINE}/Config/{PLATFORM}/Base{PLATFORM}{TYPE}.ini",
			// Project/Default*.ini
			"{PROJECT}/Config/Default{TYPE}.ini",
			// Project/Generated*.ini  this is reserved for files which are generated by buildmachine processes (i.e. should never be checked in)
			"{PROJECT}/Config/Generated{TYPE}.ini",
			// Project/Config/Custom/CustomConfig/Default*.ini only if CustomConfig is defined
			"{PROJECT}/Config/Custom/{CUSTOMCONFIG}/Default{TYPE}.ini",
			// Engine/Platform/Platform*.ini
			"{ENGINE}/Config/{PLATFORM}/{PLATFORM}{TYPE}.ini",
			// Project/Platform/Platform*.ini
			"{PROJECT}/Config/{PLATFORM}/{PLATFORM}{TYPE}.ini",
			// Project/Platform/GeneratedPlatform*.ini  this is reserved for files which are generated by buildmachine processes (i.e. should never be checked in)
			"{PROJECT}/Config/{PLATFORM}/Generated{PLATFORM}{TYPE}.ini",
			// Project/Platform/Custom/CustomConfig/Platform*.ini only if CustomConfig is defined
			"{PROJECT}/Config/{PLATFORM}/Custom/{CUSTOMCONFIG}/{PLATFORM}{TYPE}.ini",
			// AppSettings/.../System*.ini
			"{APPSETTINGS}/Unreal Engine/Engine/Config/System{TYPE}.ini",
			"{APPSETTINGS}/UnrealEngine/Engine/Config/System{TYPE}.ini",
			// UserSettings/.../User*.ini
			"{USERSETTINGS}/Unreal Engine/Engine/Config/User{TYPE}.ini",
			"{USERSETTINGS}/UnrealEngine/Engine/Config/User{TYPE}.ini",
			// UserDir/.../User*.ini
			"{USER}/Unreal Engine/Engine/Config/User{TYPE}.ini",
			"{USER}/UnrealEngine/Engine/Config/User{TYPE}.ini",
			// Project/User*.ini
			"{PROJECT}/Config/User{TYPE}.ini",
		};

		static readonly ConfigLayerExpansion[] ConfigLayerExpansions =
		{
			// The base expansion (ie, no expansion)
			new ConfigLayerExpansion { }, 
			// Restricted Locations
			new ConfigLayerExpansion { Before1 = "{ENGINE}/", After1 = "{ENGINE}/Restricted/NotForLicensees/", Before2 = "{PROJECT}/Config/", After2 = "{RESTRICTEDPROJECT_NFL}/Config/" },
			new ConfigLayerExpansion { Before1 = "{ENGINE}/", After1 = "{ENGINE}/Restricted/NoRedist/",         Before2 = "{PROJECT}/Config/", After2 = "{RESTRICTEDPROJECT_NR}/Config/" },
			new ConfigLayerExpansion { Before1 = "{ENGINE}/", After1 = "{ENGINE}/Restricted/LimitedAccess/",    Before2 = "{PROJECT}/Config/", After2 = "{RESTRICTEDPROJECT_LA}/Config/" },
			// Platform Extensions
			new ConfigLayerExpansion { Before1 = "{ENGINE}/Config/{PLATFORM}/", After1 = "{EXTENGINE}/Config/",    Before2 = "{PROJECT}/Config/{PLATFORM}/", After2 = "{EXTPROJECT}/Config/" },
			// Platform Extensions in Restricted Locations
			new ConfigLayerExpansion { Before1 = "{ENGINE}/Config/{PLATFORM}/", After1 = "{ENGINE}/Restricted/NotForLicensees/Platforms/{PLATFORM}/Config/",   Before2 = "{PROJECT}/Config/{PLATFORM}/", After2 = "{RESTRICTEDPROJECT_NFL}/Platforms/{PLATFORM}/{OPT_SUBDIR}Config/" },
			new ConfigLayerExpansion { Before1 = "{ENGINE}/Config/{PLATFORM}/", After1 = "{ENGINE}/Restricted/NoRedist/Platforms/{PLATFORM}/Config/",          Before2 = "{PROJECT}/Config/{PLATFORM}/", After2 = "{RESTRICTEDPROJECT_NR}/Platforms/{PLATFORM}/{OPT_SUBDIR}Config/" },
			new ConfigLayerExpansion { Before1 = "{ENGINE}/Config/{PLATFORM}/", After1 = "{ENGINE}/Restricted/LimitedAccess/Platforms/{PLATFORM}/Config/",     Before2 = "{PROJECT}/Config/{PLATFORM}/", After2 = "{RESTRICTEDPROJECT_LA}/Platforms/{PLATFORM}/{OPT_SUBDIR}Config/" },
		};

		private static string PerformBasicReplacements(string InString, string BaseIniName, string CustomConfig)
		{
			string OutString = InString.Replace("{TYPE}", BaseIniName, StringComparison.Ordinal);

			OutString = OutString.Replace("{APPSETTINGS}", Unreal.ApplicationSettingDirectory.FullName, StringComparison.Ordinal);
			OutString = OutString.Replace("{USERSETTINGS}", Unreal.UserSettingDirectory.FullName, StringComparison.Ordinal);

			DirectoryReference? UserDir = Unreal.UserDirectory;
			if (UserDir != null)
			{
				OutString = OutString.Replace("{USER}", UserDir.FullName, StringComparison.Ordinal);
			}

			OutString = OutString.Replace("{CUSTOMCONFIG}", CustomConfig, StringComparison.Ordinal);

			return OutString;
		}

		private static string? PerformExpansionReplacements(ConfigLayerExpansion Expansion, string InString)
		{
			// if there's replacement to do, the output is just the output
			if (Expansion.Before1 == null)
			{
				return InString;
			}

			// if nothing to replace, then skip it entirely
			if (!InString.Contains(Expansion.Before1, StringComparison.Ordinal) && (Expansion.Before2 == null || !InString.Contains(Expansion.Before2, StringComparison.Ordinal)))
			{
				return null;
			}

			// replace the directory bits
			string OutString = InString.Replace(Expansion.Before1, Expansion.After1, StringComparison.Ordinal);
			if (Expansion.Before2 != null)
			{
				OutString = OutString.Replace(Expansion.Before2, Expansion.After2, StringComparison.Ordinal);
			}
			return OutString;
		}

		private static string PerformFinalExpansions(string InString, string PlatformName, DirectoryReference? ProjectDir)
		{
			string PlatformExtensionEngineConfigDir = DirectoryReference.Combine(Unreal.EngineDirectory, "Platforms", PlatformName).FullName;

			string OutString = InString.Replace("{ENGINE}", Unreal.EngineDirectory.FullName, StringComparison.Ordinal);
			OutString = OutString.Replace("{EXTENGINE}", PlatformExtensionEngineConfigDir, StringComparison.Ordinal);
			OutString = OutString.Replace("{PLATFORM}", PlatformName, StringComparison.Ordinal);

			if (ProjectDir != null)
			{
				DirectoryReference NFLDir;
				DirectoryReference NRDir;
				DirectoryReference LADir;
				string OptionalSubDir = "";

				if (ProjectDir.IsUnderDirectory(Unreal.EngineDirectory))
				{
					OptionalSubDir = ProjectDir.MakeRelativeTo(Unreal.EngineDirectory) + "/";
					NFLDir = DirectoryReference.Combine(Unreal.EngineDirectory, "Restricted/NotForLicensees");
					NRDir = DirectoryReference.Combine(Unreal.EngineDirectory, "Restricted/NoRedist");
					LADir = DirectoryReference.Combine(Unreal.EngineDirectory, "Restricted/LimitedAccess");
				}
				else
				{
					NFLDir = DirectoryReference.Combine(ProjectDir, "Restricted/NotForLicensees");
					NRDir = DirectoryReference.Combine(ProjectDir, "Restricted/NoRedist");
					LADir = DirectoryReference.Combine(ProjectDir, "Restricted/LimitedAccess");
				}

				if (ProjectDir.IsUnderDirectory(NFLDir))
				{
					OptionalSubDir = ProjectDir.MakeRelativeTo(NFLDir) + "/";
				}
				else if (ProjectDir.IsUnderDirectory(NRDir))
				{
					OptionalSubDir = ProjectDir.MakeRelativeTo(NRDir) + "/";
				}
				else if (ProjectDir.IsUnderDirectory(LADir))
				{
					OptionalSubDir = ProjectDir.MakeRelativeTo(LADir) + "/";
				}

				string PlatformExtensionProjectConfigDir = DirectoryReference.Combine(ProjectDir, "Platforms", PlatformName).FullName;

				OutString = OutString.Replace("{PROJECT}", ProjectDir.FullName, StringComparison.Ordinal);
				OutString = OutString.Replace("{EXTPROJECT}", PlatformExtensionProjectConfigDir, StringComparison.Ordinal);
				OutString = OutString.Replace("{RESTRICTEDPROJECT_NFL}", NFLDir.FullName, StringComparison.Ordinal);
				OutString = OutString.Replace("{RESTRICTEDPROJECT_NR}", NRDir.FullName, StringComparison.Ordinal);
				OutString = OutString.Replace("{RESTRICTEDPROJECT_LA}", LADir.FullName, StringComparison.Ordinal);
				OutString = OutString.Replace("{OPT_SUBDIR}", OptionalSubDir, StringComparison.Ordinal);

			}

			return OutString;
		}

		/// <summary>
		/// Returns a list of INI filenames for the given project
		/// </summary>
		public static IEnumerable<FileReference> EnumerateConfigFileLocations(ConfigHierarchyType Type, DirectoryReference? ProjectDir, UnrealTargetPlatform Platform, string CustomConfig, TargetType? IncludePluginsForTargetType)
		{
			string BaseIniName = Enum.GetName(typeof(ConfigHierarchyType), Type) ?? String.Empty;
			string PlatformName = GetIniPlatformName(Platform);
			List<DirectoryReference> PluginDirs = new();
			List<DirectoryReference> PluginPlatformExtDirs = new();

			if (IncludePluginsForTargetType.HasValue)
			{
				ProjectDescriptor? Project = null;
				if (ProjectDir != null)
				{
					Project = ProjectDescriptor.FromDirectory(ProjectDir);
				}
				List<PluginInfo> AllPLugins = Plugins.ReadAvailablePlugins(Unreal.EngineDirectory, ProjectDir, null);
				foreach (PluginInfo Plugin in AllPLugins)
				{
					if (Plugins.IsPluginEnabledForTarget(Plugin, Project, Platform, UnrealTargetConfiguration.Development, IncludePluginsForTargetType.Value))
					{
						PluginDirs.Add(Plugin.Directory);
						foreach (FileReference ChildPlugin in Plugin.ChildFiles)
						{
							// only look at a child plugin for a platform we are enumerating (ie, it is in a /PlatformName/ subdir), but always look in Restricted folders
							if (ChildPlugin.ContainsName("Platforms", 0))
							{
								if (ChildPlugin.ContainsName(PlatformName, 0))
								{
									PluginPlatformExtDirs.Add(ChildPlugin.Directory);
								}
							}
							else
							{
								PluginDirs.Add(ChildPlugin.Directory);
							}
						}
					}
				}
			}

			foreach (string Layer in ConfigLayers)
			{
				bool bHasPlatformTag = Layer.Contains("{PLATFORM}", StringComparison.Ordinal);
				bool bHasProjectTag = Layer.Contains("{PROJECT}", StringComparison.Ordinal);
				bool bHasUserTag = Layer.Contains("{USER}", StringComparison.Ordinal);
				bool bHasCustomConfigTag = Layer.Contains("{CUSTOMCONFIG}", StringComparison.Ordinal);

				// skip certain layers if we are platform-less, project-less, or userdir-less
				if ((bHasPlatformTag && PlatformName == "None") ||
					(bHasProjectTag && ProjectDir == null) ||
					(bHasUserTag && Unreal.UserSettingDirectory == null) ||
					(bHasCustomConfigTag && String.IsNullOrEmpty(CustomConfig)))
				{
					continue;
				}

				string LayerPath = PerformBasicReplacements(Layer, BaseIniName, CustomConfig);

				// we only expand engine/project inis
				if (Layer.Contains("{ENGINE}", StringComparison.Ordinal) || Layer.Contains("{PROJECT}", StringComparison.Ordinal))
				{
					foreach (ConfigLayerExpansion Expansion in ConfigLayerExpansions)
					{
						// expansion replacements
						string? ExpandedPath = PerformExpansionReplacements(Expansion, LayerPath);

						// if nothing was replaced, then skip it, as it won't change anything
						if (ExpandedPath == null)
						{
							continue;
						}

						// now go up the ini parent chain
						if (bHasPlatformTag)
						{
							DataDrivenPlatformInfo.ConfigDataDrivenPlatformInfo? Info = DataDrivenPlatformInfo.GetDataDrivenInfoForPlatform(PlatformName);
							if (Info != null && Info.IniParentChain != null)
							{
								// the IniParentChain
								foreach (string ParentPlatform in Info.IniParentChain)
								{
									// @note: We are using the ParentPlatform as both PlatformExtensionName _and_ IniPlatformName. This is because the parent
									// may not even exist as a UnrealTargetPlatform, and all we have is a string to look up, and it would just get the same
									// string back, if we did look it up. This could become an issue if Win64 becomes a PlatformExtension, and wants to have 
									// a parent Platform, of ... something. This is likely to never be an issue, but leaving this note here just in case.
									yield return new FileReference(PerformFinalExpansions(ExpandedPath, ParentPlatform, ProjectDir));
								}
							}
							// always yield the active platform last 
							yield return new FileReference(PerformFinalExpansions(ExpandedPath, PlatformName, ProjectDir));
						}
						else
						{
							yield return new FileReference(PerformFinalExpansions(ExpandedPath, "", ProjectDir));
						}
					}
				}
				else
				{
					yield return new FileReference(LayerPath);
				}
			}

			foreach (DirectoryReference PluginDir in PluginDirs)
			{
				yield return FileReference.Combine(PluginDir, "Config", $"{Type}.ini");
				yield return FileReference.Combine(PluginDir, "Config", PlatformName, $"{PlatformName}{Type}.ini");
			}
			foreach (DirectoryReference PluginDir in PluginPlatformExtDirs)
			{
				yield return FileReference.Combine(PluginDir, "Config", $"{PlatformName}{Type}.ini");
			}

			// Find all the generated config files
			foreach (FileReference GeneratedConfigFile in EnumerateGeneratedConfigFileLocations(Type, ProjectDir, Platform))
			{
				yield return GeneratedConfigFile;
			}
		}

		/// <summary>
		/// Returns a list of INI filenames for the given project
		/// </summary>
		public static IEnumerable<FileReference> EnumerateGeneratedConfigFileLocations(ConfigHierarchyType Type, DirectoryReference? ProjectDir, UnrealTargetPlatform Platform)
		{
			string BaseIniName = Enum.GetName(typeof(ConfigHierarchyType), Type)!;
			string PlatformName = GetIniPlatformName(Platform);

			// Get the generated config file too. EditorSettings overrides this from 
			if (Type == ConfigHierarchyType.EditorSettings)
			{
				yield return FileReference.Combine(GetGameAgnosticSavedDir(), "Config", PlatformName + "Editor", BaseIniName + ".ini");
			}
			else if (Type == ConfigHierarchyType.EditorPerProjectUserSettings)
			{
				yield return FileReference.Combine(GetGeneratedConfigDir(ProjectDir), PlatformName + "Editor", BaseIniName + ".ini");
			}
			else
			{
				yield return FileReference.Combine(GetGeneratedConfigDir(ProjectDir), PlatformName, BaseIniName + ".ini");
			}
		}

		/// <summary>
		/// Determines the path to the generated config directory (same as FPaths::GeneratedConfigDir())
		/// </summary>
		/// <returns></returns>
		public static DirectoryReference GetGeneratedConfigDir(DirectoryReference? ProjectDir)
		{
			if (ProjectDir == null)
			{
				return DirectoryReference.Combine(Unreal.EngineDirectory, "Saved", "Config");
			}
			else
			{
				return DirectoryReference.Combine(ProjectDir, "Saved", "Config");
			}
		}

		/// <summary>
		/// Determes the path to the game-agnostic saved directory (same as FPaths::GameAgnosticSavedDir())
		/// </summary>
		/// <returns></returns>
		public static DirectoryReference GetGameAgnosticSavedDir()
		{
			if (Unreal.IsEngineInstalled())
			{
				DirectoryReference? UserSettingDir = Unreal.UserSettingDirectory;
				if (UserSettingDir != null)
				{
					return DirectoryReference.Combine(UserSettingDir, "UnrealEngine", String.Format("{0}.{1}", ReadOnlyBuildVersion.Current.MajorVersion, ReadOnlyBuildVersion.Current.MinorVersion), "Saved");
				}
			}
			return DirectoryReference.Combine(Unreal.EngineDirectory, "Saved");
		}

		/// <summary>
		/// Returns the platform name to use as part of platform-specific config files
		/// </summary>
		public static string GetIniPlatformName(UnrealTargetPlatform TargetPlatform)
		{
			if (TargetPlatform == UnrealTargetPlatform.Win64)
			{
				return "Windows";
			}
			else
			{
				return TargetPlatform.ToString();
			}
		}

		#region Unreal struct/map parsing helpers

		/// <summary>
		/// Gets an ini setting, and then pulls the value for a property out of a struct, in the format:
		///   [SomeSection]
		///   SomeStruct=(Foo=Bar,Prop="My Value")
		/// </summary>
		/// <param name="Section">Ini section ('SomeSection' in this example)</param>
		/// <param name="Setting">Name of the struct setting ('SomeStruct' in this example)</param>
		/// <param name="Property">Name of the property inside the struct ('Prop' in this example)</param>
		/// <returns>The value retrieved from the struct ('My Value' in this example), or null if anything was not found</returns>
		public string? GetStructEntryForSetting(string Section, string Setting, string Property)
		{
			string ConfigEntry;
			if (GetString(Section, Setting, out ConfigEntry))
			{
				return GetStructEntry(ConfigEntry, Property, false);
			}
			return null;
		}

		/// <summary>
		/// Pulls the value for a property out of a struct in the given input, in the format:
		///   (Foo=Bar,Prop="My Value")
		/// </summary>
		/// <param name="Input">The entire struct as retrieved from the ini via GetString()</param>
		/// <param name="Property">Name of the property inside the struct ('Prop' in this example)</param>
		/// <param name="bIsArrayProperty">Pass true when the value pulled is an array, like (Foo=(X=1,Y=2)), this would return X=1,Y=2</param>
		/// <returns>The value retrieved from the struct ('My Value' in this example), or null if anything was not found</returns>
		public static string? GetStructEntry(string Input, string Property, bool bIsArrayProperty)
		{
			string PrimaryRegex;
			string? AltRegex = null;
			if (bIsArrayProperty)
			{
				PrimaryRegex = String.Format("{0}\\s*=\\s*\\((.*?)\\)", Property);
			}
			else
			{
				// handle quoted strings, allowing for escaped quotation marks (basically doing " followed by whatever, until we see a quote that was not proceeded by a \, and gather the whole mess in an outer group)
				PrimaryRegex = String.Format("{0}\\s*=\\s*\"((.*?)[^\\\\])\"", Property);
				// when no quotes, we skip over whitespace, and we end when we see whitespace, a comma or a ). This will handle (Ip = 192.168.0.1 , Name=....) , and return only '192.168.0.1'
				AltRegex = String.Format("{0}\\s*=\\s*(.*?)[\\s,\\)]", Property);
			}

			// attempt to match it!
			Match Result = Regex.Match(Input, PrimaryRegex);
			if (!Result.Success && AltRegex != null)
			{
				Result = Regex.Match(Input, AltRegex);
			}

			// if we got a success, return the main match value
			if (Result.Success)
			{
				return Result.Groups[1].Value.ToString();
			}

			return null;
		}

		/// <summary>
		/// Gets an ini setting, and then pulls the value for a property out of a map, in the format:
		///   [SomeSection]
		///   SomeMap=((Foo=Bar),(SomeKey="My Value"))
		/// </summary>
		/// <param name="Section">Ini section ('SomeSection' in this example)</param>
		/// <param name="Setting">Name of the struct setting ('SomeMap' in this example)</param>
		/// <param name="Key">Name of the key inside the struct ('SomeKey' in this example)</param>
		/// <returns>The value retrieved from the map ('My Value' in this example), or null if anything was not found</returns>
		public string? GetMapValueForSetting(string Section, string Setting, string Key)
		{
			string ConfigEntry;
			if (GetString(Section, Setting, out ConfigEntry))
			{
				return GetMapValue(ConfigEntry, Key);
			}
			return null;
		}

		/// <summary>
		/// Pulls the value for a property out of a struct in the given input, in the format:
		///   ((Foo=Bar),(SomeKey="My Value"))
		/// </summary>
		/// <param name="Input">The entire struct as retrieved from the ini via GetString()</param>
		/// <param name="Key">Name of the key inside the struct ('SomeKey' in this example). Key cannot have escaped quotes or commas</param>
		/// <returns>The value retrieved from the map ('My Value' in this example), or null if anything was not found</returns>
		public static string? GetMapValue(string Input, string Key)
		{
			// handle quoted strings, allowing for escaped quotation marks (and possibly the key in quotes as well)
			string PrimaryRegex = String.Format("{0}\"?\\s*,\\s*\"((.*?)[^\\\\])\"", Key);
			string AltRegex = String.Format("{0}\"?\\s*,\\s*(.*?)[\\s,\\)]", Key);

			// attempt to match it!
			Match Result = Regex.Match(Input, PrimaryRegex);
			if (!Result.Success && AltRegex != null)
			{
				Result = Regex.Match(Input, AltRegex);
			}

			// if we got a success, return the main match value
			if (Result.Success)
			{
				return Result.Groups[1].Value.ToString();
			}

			return null;
		}

		/// <summary>
		/// Load a ini value for the given key, then use GetStructKeyValuePairs to return the key/value pairs of the struct
		/// </summary>
		/// <param name="Section">Ini section to read from</param>
		/// <param name="Setting">Name of the ini key to read from</param>
		/// <returns>Dictionary of struct vars/values</returns>
		public Dictionary<string, string>? GetStructKeyValuePairsForSetting(string Section, string Setting)
		{
			string ConfigEntry;
			if (GetString(Section, Setting, out ConfigEntry))
			{
				return GetStructKeyValuePairs(ConfigEntry);
			}
			return null;
		}

		/// <summary>
		/// Given a string input (a struct loaded from a .ini file usually), like (Foo=A, Bar="Hello world"), this will return a dictionary of all values, with quotes trimmed off
		/// In this example, { { Foo, A } , { Bar, Hello world } }
		/// </summary>
		/// <param name="Input">String containing a struct representation</param>
		/// <returns>Dictionary of struct vars/values</returns>
		public static Dictionary<string, string> GetStructKeyValuePairs(string Input)
		{
			// we expect parens around a properly encoded struct
			if (!Input.StartsWith("(", StringComparison.Ordinal) || !Input.EndsWith(")", StringComparison.Ordinal))
			{
				return new Dictionary<string, string>();
			}
			// strip ()
			Input = Input.Substring(1, Input.Length - 2);

			List<string> Props = new List<string>();

			int TokenStart = 0;
			int StrLen = Input.Length;
			while (TokenStart < StrLen)
			{
				// get the next location of each special character
				int NextComma = Input.IndexOf(',', TokenStart);
				int NextQuote = Input.IndexOf('\"', TokenStart);
				// comma first? easy
				if (NextComma != -1 && NextComma < NextQuote)
				{
					Props.Add(Input.Substring(TokenStart, NextComma - TokenStart));
					TokenStart = NextComma + 1;
				}
				// comma but no quotes
				else if (NextComma != -1 && NextQuote == -1)
				{
					Props.Add(Input.Substring(TokenStart, NextComma - TokenStart));
					TokenStart = NextComma + 1;
				}
				// neither found, use the rest
				else if (NextComma == -1 && NextQuote == -1)
				{
					Props.Add(Input.Substring(TokenStart));
					break;
				}
				// quote first? look for quote after
				else
				{
					NextQuote = Input.IndexOf('\"', NextQuote + 1);
					// are we at the end?
					if (NextQuote + 1 == StrLen)
					{
						// use the rest of the string
						Props.Add(Input.Substring(TokenStart));
						break;
					}
					// it's expected that the following character is a comma, if not, give up
					if (Input[NextQuote + 1] != ',')
					{
						break;
					}
					// if next is comma, we are done this token
					Props.Add(Input.Substring(TokenStart, (NextQuote - TokenStart) + 1));
					// skip over the quote and following commma
					TokenStart = NextQuote + 2;
				}
			}

			// now make a dictionary from the properties
			Dictionary<string, string> KeyValues = new Dictionary<string, string>();
			foreach (string AProp in Props)
			{
				string Prop = AProp.Trim(" \t".ToCharArray());
				// find the first = (UE properties can't have an equal sign, so it's valid to do)
				int Equals = Prop.IndexOf('=', StringComparison.Ordinal);
				// we must have one
				if (Equals == -1)
				{
					continue;
				}

				string Key = Prop.Substring(0, Equals);
				string Value = Prop.Substring(Equals + 1);
				// trim off any quotes around the entire value
				Value = Value.Trim(" \"".ToCharArray());
				Key = Key.Trim(" ".ToCharArray());
				KeyValues.Add(Key, Value);
			}

			// convert to array type
			return KeyValues;
		}

		#endregion
	}
}
