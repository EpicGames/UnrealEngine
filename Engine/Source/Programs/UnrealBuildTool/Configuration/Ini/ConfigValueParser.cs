// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.Linq;
using EpicGames.Core;
using UnrealBuildBase;
using static UnrealBuildTool.ConfigHierarchy;

namespace UnrealBuildTool
{
	/// <summary>
	/// Functionality for parsing values from a config value
	/// </summary>
	public static class ConfigValueParser
	{
		/// <summary>
		/// Attempts to parse the given line as the given type. Struct member variables are expected to match
		/// </summary>
		/// <param name="T">Type to attempt to parse</param>
		/// <param name="Line">Line of text to parse</param>
		/// <param name="Result">Receives parse result</param>
		/// <returns>True if the parse was successful, false otherwise</returns>
		public static bool TryParseTyped(Type T, string Line, [NotNullWhen(true)] out object? Result)
		{
			try
			{
				if (T == typeof(string))
				{
					Result = Line;
					return true;
				}
				else if (T.IsPrimitive)
				{
					if (T == typeof(bool) && TryParse(Line, out bool BoolValue))
					{
						Result = BoolValue;
						return true;
					}
					else if (T == typeof(int) && TryParse(Line, out int IntValue))
					{
						Result = IntValue;
						return true;
					}
					else if (T == typeof(float) && TryParse(Line, out float FloatValue))
					{
						Result = FloatValue;
						return true;
					}
					else if (T == typeof(double) && TryParse(Line, out double DoubleValue))
					{
						Result = DoubleValue;
						return true;
					}

					throw new Exception($"Cannot parse \"{Line}\" for primitive type {T}");
				}
				else if (T.IsEnum)
				{
					if (Enum.TryParse(T, Line, out Result) && Result != null)
					{
						return true;
					}

					throw new Exception($"Cannot parse \"{Line}\" for enum {T}");
				}
				else if (T.IsArray)
				{
					if (TryParse(Line, out string[]? ArrayValues))
					{
						// check type & early out if we can
						Type ItemType = T.GetElementType()!;
						if (ItemType == typeof(string))
						{
							Result = ArrayValues;
							return true;
						}

						// make a generic array
						Array ResultArray = Array.CreateInstance(ItemType, ArrayValues.Length);
						for (int i = 0; i < ArrayValues.Length; i++)
						{
							if (TryParseTyped(ItemType, ArrayValues[i], out object? ItemValue))
							{
								ResultArray.SetValue(ItemValue, i);
							}
							else
							{
								Result = null;
								return false;
							}
						}
						Result = ResultArray;
						return true;
					}
				}
				else if (T == typeof(Guid))
				{
					if (TryParse(Line, out Guid GuidValue))
					{
						Result = GuidValue;
						return true;
					}

					throw new Exception($"Cannot parse Guid from {Line}");
				}
				else if (T.IsGenericType)
				{
					if (T.GetGenericTypeDefinition() == typeof(Dictionary<,>) && TryParseAsMap(Line, out Dictionary<string, string>? DictValues))
					{
						// check types & early out if we can
						Type KeyType = T.GetGenericArguments()[0];
						Type ValType = T.GetGenericArguments()[1];
						if (KeyType == typeof(string) && ValType == typeof(string))
						{
							Result = DictValues;
							return true;
						}

						// make a generic dictionary
						IDictionary ResultDict = (IDictionary)Activator.CreateInstance(T)!;
						foreach (KeyValuePair<string, string> DictValue in DictValues)
						{
							if (TryParseTyped(KeyType, DictValue.Key, out object? Key) && Key != null &&
								TryParseTyped(ValType, DictValue.Value, out object? Value) && Value != null)
							{
								ResultDict.Add(Key, Value);
							}
							else
							{
								Result = null;
								return false;
							}
						}

						Result = ResultDict;
						return true;
					}
					else if (T.GetGenericTypeDefinition() == typeof(List<>) && TryParse(Line, out string[]? ListValues))
					{
						// check types & early out if we can
						Type ItemType = T.GetGenericArguments()[0];
						if (ItemType == typeof(string))
						{
							Result = ListValues.ToList();
							return true;
						}

						// make a generic list
						IList ResultList = (IList)Activator.CreateInstance(T)!;
						foreach (string Value in ListValues)
						{
							if (TryParseTyped(ItemType, Value, out object? ItemValue))
							{
								ResultList.Add(ItemValue);
							}
							else
							{
								Result = null;
								return false;
							}
						}
						Result = ResultList;
						return true;
					}

					throw new Exception($"Cannot parse \"{Line}\" for generic type {T}");
				}

				// try to extract fields
				if (!TryParse(Line, out Dictionary<string, string>? Properties))
				{
					Properties = null;
				}

				// some special cases that may or may not use Properties
				if (T == typeof(FileReference))
				{
					if (Properties != null && Properties.ContainsKey("FilePath")) // FFilePath
					{
						Result = FileReference.Combine(Unreal.EngineDirectory, "Binaries", BuildHostPlatform.Current.Platform.ToString(), Properties["FilePath"]);
						return true;
					}
					else if (Properties == null)
					{
						Result = new FileReference(Line);
						return true;
					}
					else
					{
						throw new Exception($"Cannot parse FileReference from {Line}");
					}
				}
				else if (T == typeof(DirectoryReference))
				{
					if (Properties != null && Properties.ContainsKey("Path")) // FDirectoryPath
					{
						Result = DirectoryReference.Combine(Unreal.EngineDirectory, "Binaries", BuildHostPlatform.Current.Platform.ToString(), Properties["Path"]);
						return true;
					}
					else if (Properties == null)
					{
						Result = new DirectoryReference(Line);
						return true;
					}
					else
					{
						throw new Exception($"Cannot parse DirectoryReference from {Line}");
					}
				}

				// must have Properties from this point onwards
				if (Properties == null)
				{
					Result = null;
					return false;
				}

				// parse out structure
				Result = Activator.CreateInstance(T);
				if (Result == null)
				{
					return false;
				}
				const System.Reflection.FieldAttributes IgnoreFlags = System.Reflection.FieldAttributes.Static | System.Reflection.FieldAttributes.InitOnly;
				foreach (System.Reflection.FieldInfo Field in T.GetFields().Where(X => !X.Attributes.HasFlag(IgnoreFlags)))
				{
					if (Properties.ContainsKey(Field.Name))
					{
						if (TryParseTyped(Field.FieldType, Properties[Field.Name], out object? FieldValue))
						{
							Field.SetValue(Result, FieldValue);
							continue;
						}
						else
						{
							Result = null;
							return false;
						}
					}

					// NB. We allow uninitialized values in structs/classes - it may be expected & ini saving omits empty strings etc.
				}
				return true;
			}
			catch (Exception)
			{
				//Log.TraceInformation(e.Message);
				Result = null;
				return false;
			}
		}

		/// <summary>
		/// Attempts to parse the given line as the given template type
		/// </summary>
		/// <param name="Line">Line of text to parse</param>
		/// <param name="Result">Receives parse result</param>
		/// <returns>True if the parse was successful, false otherwise</returns>
		public static bool TryParseGeneric<T>(string Line, [NotNullWhen(true)] out T? Result) where T : new()
		{
			if (TryParseTyped(typeof(T), Line, out object? ObjResult) && ObjResult != null)
			{
				Result = (T)ObjResult;
				return true;
			}

			Result = default;
			return false;
		}

		/// <summary>
		/// Attempts to parse the given array of lines as array of the given template type
		/// </summary>
		/// <param name="Lines">Line of text to parse</param>
		/// <param name="Result">Receives parse result</param>
		/// <returns>True if the parse was successful, false otherwise</returns>
		public static bool TryParseArrayGeneric<T>(string[] Lines, [NotNullWhen(true)] out T[]? Result) where T : new()
		{
			if (Lines.Length == 0)
			{
				Result = Array.Empty<T>();
				return true;
			}

			Result = new T[Lines.Length];
			for (int i = 0; i < Lines.Length; i++)
			{
				if (!TryParseGeneric(Lines[i], out Result[i]!))
				{
					Result = null;
					return false;
				}
			}
			return true;
		}
	}
}
