// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.Linq;
using EpicGames.Core;

namespace UnrealBuildTool
{
	/// <summary>
	/// Identifier for a config file key, including information about the hierarchy used to read it
	/// </summary>
	[DebuggerDisplay("{Name}")]
	class ConfigDependencyKey : IEquatable<ConfigDependencyKey>
	{
		/// <summary>
		/// The config hierarchy type
		/// </summary>
		public ConfigHierarchyType Type;

		/// <summary>
		/// Project directory to read config files from
		/// </summary>
		public DirectoryReference? ProjectDir;

		/// <summary>
		/// The platform being built
		/// </summary>
		public UnrealTargetPlatform Platform;

		/// <summary>
		/// The section name
		/// </summary>
		public string SectionName;

		/// <summary>
		/// The key name
		/// </summary>
		public string KeyName;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="Type">The config hierarchy type</param>
		/// <param name="ProjectDir">Project directory to read config files from</param>
		/// <param name="Platform">The platform being built</param>
		/// <param name="SectionName">The section name</param>
		/// <param name="KeyName">The key name</param>
		public ConfigDependencyKey(ConfigHierarchyType Type, DirectoryReference? ProjectDir, UnrealTargetPlatform Platform, string SectionName, string KeyName)
		{
			this.Type = Type;
			this.ProjectDir = ProjectDir;
			this.Platform = Platform;
			this.SectionName = SectionName;
			this.KeyName = KeyName;
		}

		/// <summary>
		/// Construct a key from an archive
		/// </summary>
		/// <param name="Reader">Archive to read from</param>
		public ConfigDependencyKey(BinaryArchiveReader Reader)
		{
			Type = (ConfigHierarchyType)Reader.ReadInt();
			ProjectDir = Reader.ReadDirectoryReference();
			Platform = Reader.ReadUnrealTargetPlatform();
			SectionName = Reader.ReadString()!;
			KeyName = Reader.ReadString()!;
		}

		/// <summary>
		/// Writes this key to an archive
		/// </summary>
		/// <param name="Writer">Archive to write to</param>
		public void Write(BinaryArchiveWriter Writer)
		{
			Writer.WriteInt((int)Type);
			Writer.WriteDirectoryReference(ProjectDir);
			Writer.WriteUnrealTargetPlatform(Platform);
			Writer.WriteString(SectionName);
			Writer.WriteString(KeyName);
		}

		/// <summary>
		/// Tests whether this key is equal to another object
		/// </summary>
		/// <param name="Other">The object to compare to</param>
		/// <returns>True if the keys are equal, false otherwise</returns>
		public override bool Equals(object? Other)
		{
			return (Other is ConfigDependencyKey key) && Equals(key);
		}

		/// <summary>
		/// Tests whether this key is equal to another key
		/// </summary>
		/// <param name="Other">The key to compare to</param>
		/// <returns>True if the keys are equal, false otherwise</returns>
		public bool Equals(ConfigDependencyKey? Other)
		{
			return Other is not null && Type == Other.Type && ProjectDir == Other.ProjectDir && Platform == Other.Platform && SectionName == Other.SectionName && KeyName == Other.KeyName;
		}

		/// <summary>
		/// Gets a hash code for this object
		/// </summary>
		/// <returns>Hash code for the object</returns>
		public override int GetHashCode()
		{
			int Hash = 17;
			Hash = (Hash * 31) + Type.GetHashCode();
			Hash = (Hash * 31) + ((ProjectDir == null) ? 0 : ProjectDir.GetHashCode());
			Hash = (Hash * 31) + Platform.GetHashCode();
			Hash = (Hash * 31) + SectionName.GetHashCode(StringComparison.Ordinal);
			Hash = (Hash * 31) + KeyName.GetHashCode(StringComparison.Ordinal);
			return Hash;
		}
	}

	/// <summary>
	/// Stores a list of config key/value pairs that have been read
	/// </summary>
	class ConfigValueTracker
	{
		/// <summary>
		/// The dependencies list
		/// </summary>
		readonly IReadOnlyDictionary<ConfigDependencyKey, IReadOnlyList<string>?> Dependencies;

		/// <summary>
		/// Constructor
		/// </summary>
		public ConfigValueTracker(IReadOnlyDictionary<ConfigDependencyKey, IReadOnlyList<string>?> ConfigValues)
		{
			Dependencies = new Dictionary<ConfigDependencyKey, IReadOnlyList<string>?>(ConfigValues);
		}

		/// <summary>
		/// Construct an object from an archive on disk
		/// </summary>
		/// <param name="Reader">Archive to read from</param>
		public ConfigValueTracker(BinaryArchiveReader Reader)
		{
			Dependencies = Reader.ReadDictionary(() => new ConfigDependencyKey(Reader), () => (IReadOnlyList<string>?)Reader.ReadList(() => Reader.ReadString()))!;
		}

		/// <summary>
		/// Write the dependencies object to disk
		/// </summary>
		/// <param name="Writer">Archive to write to</param>
		public void Write(BinaryArchiveWriter Writer)
		{
			Writer.WriteDictionary(Dependencies, Key => Key.Write(Writer), Value => Writer.WriteList(Value, x => Writer.WriteString(x)));
		}

		/// <summary>
		/// Checks whether the list of dependencies is still valid
		/// </summary>
		/// <returns></returns>
		public bool IsValid()
		{
			foreach (KeyValuePair<ConfigDependencyKey, IReadOnlyList<string>?> Pair in Dependencies)
			{
				// Read the appropriate hierarchy
				ConfigHierarchy Hierarchy = ConfigCache.ReadHierarchy(Pair.Key.Type, Pair.Key.ProjectDir, Pair.Key.Platform);

				// Get the value(s) associated with this key
				IReadOnlyList<string>? NewValues;
				Hierarchy.TryGetValues(Pair.Key.SectionName, Pair.Key.KeyName, out NewValues);

				// Check if they're different
				if (Pair.Value == null)
				{
					if (NewValues != null)
					{
						return false;
					}
				}
				else
				{
					if (NewValues == null || !Enumerable.SequenceEqual(Pair.Value, NewValues, StringComparer.Ordinal))
					{
						return false;
					}
				}
			}
			return true;
		}
	}
}

