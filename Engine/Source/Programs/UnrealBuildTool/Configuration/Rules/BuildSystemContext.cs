// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool
{
	/// <summary>
	/// Interface that defines standard build context parameters that can influence the resolution of build warnings.
	/// </summary>
	public interface IBuildContextProvider
	{
		/// <summary>
		/// Obtains the <see cref="BuildSettingsVersion"/> associated with the provided context provider.
		/// </summary>
		/// <returns>The <see cref="BuildSettingsVersion"/>.</returns>
		public BuildSettingsVersion GetBuildSettings();
	}

	/// <summary>
	/// The default provider that will just specify the <see cref="BuildSettingsVersion.V5"/> as a default.
	/// </summary>
	public class DefaultBuildContextProvider : IBuildContextProvider
	{
		/// <inheritdoc/>
		public BuildSettingsVersion GetBuildSettings()
		{
			return BuildSettingsVersion.V5;
		}
	}

	#region -- Adapters for IBuildContextProvider -- 

	internal class ModuleRulesBuildSettingsProvider : IBuildContextProvider
	{
		private readonly ModuleRules _moduleRules;

		public static implicit operator ModuleRulesBuildSettingsProvider(ModuleRules moduleRules)
		{
			return new ModuleRulesBuildSettingsProvider(moduleRules);
		}

		public ModuleRulesBuildSettingsProvider(ModuleRules moduleRules)
		{
			_moduleRules = moduleRules;
		}

		public BuildSettingsVersion GetBuildSettings()
		{
			return _moduleRules != null ? _moduleRules.DefaultBuildSettings : (default);
		}
	}

	internal class TargetRulesBuildSettingsProvider : IBuildContextProvider
	{
		private readonly TargetRules _targetRules;

		public static implicit operator TargetRulesBuildSettingsProvider(TargetRules targetRules)
		{
			return new TargetRulesBuildSettingsProvider(targetRules);
		}

		public TargetRulesBuildSettingsProvider(TargetRules targetRules)
		{
			_targetRules = targetRules;
		}

		public BuildSettingsVersion GetBuildSettings()
		{
			return _targetRules.DefaultBuildSettings;
		}
	}

	#endregion -- Adapters for IBuildContextProvider --

	internal class BuildSystemContext
	{
		internal BuildSystemContext(IBuildContextProvider context, TargetRules? targetRules = null, ModuleRules? moduleRules = null)
		{
			_targetRulesPrivate = targetRules;
			_moduleRulesPrivate = moduleRules;
			_buildContext = context;
		}

		/// <summary>
		/// Helper method to obtain a readonly target rules from the given context, if at all possible.
		/// </summary>
		/// <returns>The resolved ReadOnlyTargetRules from the current build context.</returns>
		internal ReadOnlyTargetRules? GetReadOnlyTargetRules()
		{
			return _targetRulesPrivate != null ? new ReadOnlyTargetRules(_targetRulesPrivate) : _moduleRulesPrivate?.Target;
		}

		internal TargetRules? _targetRulesPrivate;
		internal ModuleRules? _moduleRulesPrivate;
		internal IBuildContextProvider _buildContext;
	}
}