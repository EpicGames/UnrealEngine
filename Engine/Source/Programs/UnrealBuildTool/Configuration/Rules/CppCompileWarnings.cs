// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Linq;
using System.Reflection;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildTool.Configuration.CompileWarnings;

namespace UnrealBuildTool
{
	internal static class WarningLevelExtensions
	{
		/// <summary>
		/// Applies a warning on top of another, given the new <paramref name="appliedWarningLevel"/> has a value, and it is not the <see cref="WarningLevel.Default"/>.
		/// </summary>
		/// <param name="inSourceWarningLevel">The source warning level to override.</param>
		/// <param name="appliedWarningLevel">The warning level to apply to the source.</param>
		/// <param name ="overwriteSourceValue">Flag to overwrite the source warning with new value. If false, will only apply setting if value is unset.</param>
		/// <remarks>If <paramref name="appliedWarningLevel"/> has a value of null or <see cref="WarningLevel.Default"/>, it will be ignored as an override.</remarks>
		internal static void ApplyWarning(this ref WarningLevel inSourceWarningLevel, WarningLevel? appliedWarningLevel, bool overwriteSourceValue = true)
		{
			inSourceWarningLevel = GetAppliedWarningLevel(inSourceWarningLevel, appliedWarningLevel, overwriteSourceValue);
		}

		/// <summary>
		/// Gets the applied warning level on top of another,  given the new <paramref name="appliedWarningLevel"/> has a value, and it is not the <see cref="WarningLevel.Default"/>.
		/// </summary>
		/// <param name="sourceWarningLevel">The source warning level to override.</param>
		/// <param name="appliedWarningLevel">The warning level to apply to the source.</param>
		/// <param name ="overwriteSourceValue">Flag to overwrite the source warning with new value. If false, will only apply setting if value is unset.</param>
		/// <returns>The applied warning level to use.</returns>
		/// <remarks>If <paramref name="appliedWarningLevel"/> has a value of null or <see cref="WarningLevel.Default"/>, it will be ignored as an override.</remarks>
		internal static WarningLevel GetAppliedWarningLevel(WarningLevel sourceWarningLevel, WarningLevel? appliedWarningLevel, bool overwriteSourceValue)
		{
			if ((appliedWarningLevel.HasValue && appliedWarningLevel.Value != WarningLevel.Default) && (overwriteSourceValue || (!overwriteSourceValue && sourceWarningLevel == WarningLevel.Default)))
			{
				sourceWarningLevel = appliedWarningLevel.Value;
			}

			return sourceWarningLevel;
		}
	}

	/// <summary>
	/// Container class used for C++ compiler warning settings.
	/// </summary>
	public partial class CppCompileWarnings
	{
		/*
		 * To add a new compiler warning, you need to do the following:
		 * 1. Copy the following stub code:
		 *		public WarningLevel NEW_WARNING_LEVEL_NAME
		 *		{
		 *			get => ResolveWarning(ParentCppCompileWarnings?.NEW_WARNING_LEVEL_NAME, _NEW_WARNING_LEVEL_NAME);
		 *			set => _NEW_WARNING_LEVEL_NAME = value;
		 *		}
		 *		private WarningLevel _NEW_WARNING_LEVEL_NAME;
		 * 2. Update the ReadOnlyCppCompileWarnings class to include a forwarding property:
		 *		public WarningLevel NEW_WARNING_LEVEL_NAME => _inner.NEW_WARNING_LEVEL_NAME;
		 * 3. Set your default WarningLevel by using WarningLevelDefault attribute (if WarningLevelDefault is not applied to the warning,it will be Default)
		 *		For Target defaults - specify it's initialization context to Target 
		 *		For regular constructor - specify it's initialization context to Constructor
		 *		For both contexts - specify it's initialization context to Any
		 * 4. Access the new property through the (TargetRules|ModuleRules|CppCompileEnvironment).CppCompileWarnings as needed
		 * 5. (Optional) For compiler line support, utilize the ApplyWarningsAttribute system to participate in invocations to GenerateWarningCommandLineAgs.
		 * 6. (Optional) For the ApplyWarningsAttribute, pass in the ArgPosition if the argument must be placed after another set.
		 * 
		 * Note: https://github.com/llvm-mirror/clang/tree/master/test/SemaCXX contains practical examples for various clang warnings.
		 */

		#region -- Compiler Warning Settings --

		/// <summary>
		/// Forces shadow variable warnings to be treated as errors on platforms that support it.
		/// MSVC - 
		///		https://learn.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-4-c4456
		///		4456 - declaration of 'LocalVariable' hides previous local declaration
		///		https://learn.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-4-c4458
		///		4458 - declaration of 'parameter' hides class member
		///		https://learn.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-4-c4459
		///		4459 - declaration of 'LocalVariable' hides global declaration
		///	Clang - 
		///		 https://clang.llvm.org/docs/DiagnosticsReference.html#wshadow
		/// </summary>
		[CommandLine("-ShadowVariableErrors", Value = nameof(WarningLevel.Error))]
		[WarningsMSCV(["/wd4456", "/wd4458", "/wd4459"], null, ["/we4456", "/we4458", "/we4459"])]
		[ShadowVariableWarningsClangToolChain(["-Wno-shadow"], ["-Wshadow", "-Wno-error=shadow"], ["-Wshadow"])]
		[VersionWarningLevelDefault(WarningLevel.Warning, BuildSettingsVersion.V1, BuildSettingsVersion.Latest, InitializationContext.Constructor)]
		[VersionWarningLevelDefault(WarningLevel.Warning, BuildSettingsVersion.V1, BuildSettingsVersion.V1, InitializationContext.Target)]
		[VersionWarningLevelDefault(WarningLevel.Error, BuildSettingsVersion.V2, BuildSettingsVersion.Latest, InitializationContext.Target)]
		public WarningLevel ShadowVariableWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.ShadowVariableWarningLevel, _shadowVariableWarningLevel);
			set => _shadowVariableWarningLevel = value;
		}
		private WarningLevel _shadowVariableWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat unsafe type casts as on platforms that support it (e.g., double->float or int64->int32)
		/// MSVC - 
		///		https://learn.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-levels-3-and-4-c4244
		///		4244 - conversion from 'type1' to 'type2', possible loss of data
		///		44244 - Note: The extra 4 is not a typo, /wLXXXX sets warning XXXX to level L
		///		https://learn.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-1-c4838
		///		4838 - conversion from 'type1' to 'type2' requires a narrowing conversion
		///		44838 - Note: The extra 4 is not a typo, /wLXXXX sets warning XXXX to level L
		///	Clang - 
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wfloat-conversion
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wimplicit-float-conversion
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wimplicit-int-conversion
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wc-11-narrowing
		///		
		///		To enable (too many hits right now)
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wshorten-64-to-32
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wsign-conversion
		/// </summary>
		/// <remarks>This should be kept in sync with PRAGMA_DISABLE_UNSAFE_TYPECAST_WARNINGS in ClangPlatformCompilerPreSetup.h</remarks>
		/// <remarks>Clang: "shorten-64-to-32"; too many hits right now, probably want it *soon*; "sign-conversion"; too many hits right now, probably want it eventually</remarks>
		[XmlConfigFile(Category = "BuildConfiguration")]
		[UnsafeTypeCastWarningsVCToolChain(["/wd4244", "/wd4838"], ["/w44244", "/w44838"], ["/we4244", "/we4838"])]
		[WarningsClangToolChain(
			["-Wno-float-conversion", "-Wno-implicit-float-conversion", "-Wno-implicit-int-conversion", "-Wno-c++11-narrowing"],
			["-Wfloat-conversion", "-Wno-error=float-conversion", "-Wimplicit-float-conversion", "-Wno-error=implicit-float-conversion", "-Wimplicit-int-conversion", "-Wno-error=implicit-int-conversion", "-Wc++11-narrowing", "-Wno-error=c++11-narrowing"],
			["-Wfloat-conversion", "-Wimplicit-float-conversion", "-Wimplicit-int-conversion", "-Wc++11-narrowing"])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel UnsafeTypeCastWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.UnsafeTypeCastWarningLevel, _unsafeTypeCastWarningLevel);
			set => _unsafeTypeCastWarningLevel = value;
		}
		private WarningLevel _unsafeTypeCastWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat undefined identifiers in conditional expressions.
		/// MSVC - 
		///		https://learn.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-4-c4668 
		///		4668 - 'symbol' is not defined as a preprocessor macro, replacing with '0' for 'directives'
		///		44668 - Note: The extra 4 is not a typo, /wLXXXX sets warning XXXX to level L
		/// Clang - 
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wundef
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		[CommandLine("-UndefinedIdentifierWarningLevel=")]
		[UndefinedIdentifierWarningsVCToolChain(null, ["/w44668"], ["/we4668"])]
		[WarningsClangToolChain(null, ["-Wundef", "-Wno-error=undef"], ["-Wundef"])]
		[VersionWarningLevelDefault(WarningLevel.Off, BuildSettingsVersion.V1, BuildSettingsVersion.V5, InitializationContext.Any)]
		[VersionWarningLevelDefault(WarningLevel.Error, BuildSettingsVersion.V6, BuildSettingsVersion.Latest, InitializationContext.Any)]
		public WarningLevel UndefinedIdentifierWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.UndefinedIdentifierWarningLevel, _undefinedIdentifierWarningLevel);
			set => _undefinedIdentifierWarningLevel = value;
		}
		private WarningLevel _undefinedIdentifierWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat unreachable code.
		/// MSVC - 
		///		https://learn.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-4-c4702 
		///		4702 - unreachable code
		///		44702 - Note: The extra 4 is not a typo, /wLXXXX sets warning XXXX to level L
		/// Clang - 
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wunreachable-code-aggressive
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		[CommandLine("-UnreachableCodeWarningLevel=")]
		[WarningsVCToolChainAttribute(null, ["/w44702"], ["/we4702"])]
		[WarningsClangToolChain(null, ["-Wunreachable-code-aggressive", "-Wno-error=unreachable-code-aggressive"], ["-Wunreachable-code-aggressive"])]
		[VersionWarningLevelDefault(WarningLevel.Off, BuildSettingsVersion.V1, BuildSettingsVersion.V6, InitializationContext.Any)]
		[VersionWarningLevelDefault(WarningLevel.Error, BuildSettingsVersion.V7, BuildSettingsVersion.Latest, InitializationContext.Any)]
			public WarningLevel UnreachableCodeWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.UnreachableCodeWarningLevel, _unreachableCodeWarningLevel);
			set => _unreachableCodeWarningLevel = value;
		}
		private WarningLevel _unreachableCodeWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat unhandled enumerators in switches on enumeration-typed values.
		/// MSVC - 
		///		4061 - https://learn.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-4-c4061 
		///		44061 - Note: The extra 4 is not a typo, /wLXXXX sets warning XXXX to level L
		///	Clang - 
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wswitch-enum
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		[CommandLine("-SwitchUnhandledEnumeratorWarningLevel=")]
		[WarningsVCToolChain(null, ["/w44061"], ["/we4061"])]
		[WarningsClangToolChain(null, ["-Wswitch-enum", "-Wno-error=switch-enum"], ["-Wswitch-enum"])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel SwitchUnhandledEnumeratorWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.SwitchUnhandledEnumeratorWarningLevel, _switchUnhandledEnumeratorWarningLevel);
			set => _switchUnhandledEnumeratorWarningLevel = value;
		}
		private WarningLevel _switchUnhandledEnumeratorWarningLevel;

		/// <summary>
		/// The compiler detected a conversion from size_t to a smaller type.
		/// MSVC - 
		///		https://learn.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-3-c4267
		///		4267 - 'var' : conversion from 'size_t' to 'type', possible loss of data
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		[WarningsVCToolChain(["/wd4267"], ["/w4267"], ["/we4267"])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel ShortenSizeTToIntWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.UnsafeTypeCastWarningLevel, _shortenSizeTToIntWarningLevel);
			set => _shortenSizeTToIntWarningLevel = value;
		}
		private WarningLevel _shortenSizeTToIntWarningLevel;

		#region -- Clang Compiler --

		/// <summary>
		/// Indicates what warning/error level to treat invocations to destructors that aren't virtual.
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wdelete-non-virtual-dtor
		/// </summary>
		[NonVCClangWarningsClangToolChain(["-Wno-delete-non-virtual-dtor"], ["-Wdelete-non-virtual-dtor", "-Wno-error=delete-non-virtual-dtor"], ["-Wdelete-non-virtual-dtor"])]
		[BasicWarningLevelDefault(WarningLevel.Error)]
		public WarningLevel DeleteNonVirtualDtorWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.DeleteNonVirtualDtorWarningLevel, _deleteNonVirtualDtorWarningLevel);
			set => _deleteNonVirtualDtorWarningLevel = value;
		}
		private WarningLevel _deleteNonVirtualDtorWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat implicit conversions betwen enums and integers.
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wenum-conversion
		/// </summary>
		[NonVCClangWarningsClangToolChain(["-Wno-enum-conversion"], ["-Wenum-conversion", "-Wno-error=enum-conversion"], ["-Wenum-conversion"], null, ArgPosition.Beginning)]
		[BasicWarningLevelDefault(WarningLevel.Error)]
		public WarningLevel EnumConversionWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.EnumConversionWarningLevel, _enumConversionWarningLevel);
			set => _enumConversionWarningLevel = value;
		}
		private WarningLevel _enumConversionWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat assignments of enum types to a bit=field, which can lead to implicit conversations (causing unexpected results).
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wbitfield-enum-conversion
		/// </summary>
		[NonVCClangWarningsClangToolChain(["-Wno-bitfield-enum-conversion"], ["-Wbitfield-enum-conversion", "-Wno-error=bitfield-enum-conversion"], ["-Wbitfield-enum-conversion"])]
		[BasicWarningLevelDefault(WarningLevel.Error)]
		public WarningLevel ClangBitfieldEnumConversion
		{
			get => ResolveWarning(ParentCppCompileWarnings?.ClangBitfieldEnumConversion, _clangBitfieldEnumConversion);
			set => _clangBitfieldEnumConversion = value;
		}
		private WarningLevel _clangBitfieldEnumConversion;

		/// <summary>
		/// Indicates what warning/error level to treat conversions between enum types.
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wenum-enum-conversion
		/// </summary>
		[WarningsClangToolChain(["-Wno-enum-enum-conversion"], ["-Wenum-enum-conversion", "-Wno-error=enum-enum-conversion"], ["-Wenum-enum-conversion"])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel EnumEnumConversionWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.EnumEnumConversionWarningLevel, _enumEnumConversionWarningLevel);
			set => _enumEnumConversionWarningLevel = value;
		}
		private WarningLevel _enumEnumConversionWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat conversions between an enumeration and a floating-point type.
		/// Clang - 
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wenum-float-conversion
		/// </summary>
		[WarningsClangToolChain(["-Wno-enum-float-conversion"], ["-Wenum-float-conversion", "-Wno-error=enum-float-conversion"], ["-Wenum-float-conversion"])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel EnumFloatConversionWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.EnumFloatConversionWarningLevel, _enumFloatConversionWarningLevel);
			set => _enumFloatConversionWarningLevel = value;
		}
		private WarningLevel _enumFloatConversionWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat ambiguous use of reversed operator overloads.
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wambiguous-reversed-operator
		/// </summary>
		[WarningsClangToolChain(["-Wno-ambiguous-reversed-operator"], ["-Wambiguous-reversed-operator", "-Wno-error=ambiguous-reversed-operator"], ["-Wambiguous-reversed-operator"], [nameof(FilterID.Cpp20Min)])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel AmbiguousReversedOperatorWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.AmbiguousReversedOperatorWarningLevel, _ambiguousReversedOperatorWarningLevel);
			set => _ambiguousReversedOperatorWarningLevel = value;
		}
		private WarningLevel _ambiguousReversedOperatorWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat implicit conversions between two different anonymous enumerations
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wdeprecated-anon-enum-enum-conversion
		/// </summary>
		[WarningsClangToolChain(["-Wno-deprecated-anon-enum-enum-conversion"], ["-Wdeprecated-anon-enum-enum-conversion", "-Wno-error=deprecated-anon-enum-enum-conversion"], ["-Wdeprecated-anon-enum-enum-conversion"], [nameof(FilterID.Cpp20Min)])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel DeprecatedAnonEnumEnumConversionWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.DeprecatedAnonEnumEnumConversionWarningLevel, _deprecatedAnonEnumEnumConversionWarningLevel);
			set => _deprecatedAnonEnumEnumConversionWarningLevel = value;
		}
		private WarningLevel _deprecatedAnonEnumEnumConversionWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat deprecated usages of volatile (due to better alternatives like atomic operations).
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wdeprecated-volatile
		/// </summary>
		[WarningsClangToolChain(["-Wno-deprecated-volatile"], ["-Wdeprecated-volatile", "-Wno-error=deprecated-volatile"], ["-Wdeprecated-volatile"], [nameof(FilterID.Cpp20Min)])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel DeprecatedVolatileWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.DeprecatedVolatileWarningLevel, _deprecatedVolatileWarningLevel);
			set => _deprecatedVolatileWarningLevel = value;
		}
		private WarningLevel _deprecatedVolatileWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat compairing function pointers with relational operators (&lt;,&gt;, etc), which is undefined behavior.
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wordered-compare-function-pointers
		/// </summary>
		[WarningsClangToolChain(["-Wno-ordered-compare-function-pointers"], ["-Wordered-compare-function-pointers", "-Wno-error=ordered-compare-function-pointers"], ["-Wordered-compare-function-pointers"])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel OrderedCompareFunctionPointers
		{
			get => ResolveWarning(ParentCppCompileWarnings?.OrderedCompareFunctionPointers, _orderedCompareFunctionPointers);
			set => _orderedCompareFunctionPointers = value;
		}
		private WarningLevel _orderedCompareFunctionPointers;

		/// <summary>
		/// Indicates what warning/error level to treat bitwise operations are used where logical operators are likely intended.
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wbitwise-instead-of-logical
		/// </summary>
		[WarningsClangToolChain(["-Wno-bitwise-instead-of-logical"], ["-Wbitwise-instead-of-logical", "-Wno-error=bitwise-instead-of-logical"], ["-Wbitwise-instead-of-logical"], [nameof(FilterID.Version14Min)])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel BitwiseInsteadOfLogicalWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.BitwiseInsteadOfLogicalWarningLevel, _bitwiseInsteadOfLogicalWarningLevel);
			set => _bitwiseInsteadOfLogicalWarningLevel = value;
		}
		private WarningLevel _bitwiseInsteadOfLogicalWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat usages of a class where a deprecated copy constructor or copy assignment operator exists.
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wdeprecated-copy
		/// </summary>
		[WarningsClangToolChain(["-Wno-deprecated-copy"], ["-Wdeprecated-copy", "-Wno-error=deprecated-copy"], ["-Wdeprecated-copy"], [nameof(FilterID.Version16Min)])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel DeprecatedCopyWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.DeprecatedCopyWarningLevel, _deprecatedCopyWarningLevel);
			set => _deprecatedCopyWarningLevel = value;
		}
		private WarningLevel _deprecatedCopyWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat usages of a class where a deprecated implicitly declared copy constructor or copy assignment operator exists.
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wdeprecated-copy-with-user-provided-copy
		/// </summary>
		/// <remarks><see cref="ArgPosition.End"/> due to it's relation to <see cref="DeprecatedCopyWarningLevel"/>.</remarks>
		[WarningsClangToolChain(["-Wno-deprecated-copy-with-user-provided-copy"], ["-Wdeprecated-copy-with-user-provided-copy", "-Wno-error=deprecated-copy-with-user-provided-copy"], ["-Wdeprecated-copy-with-user-provided-copy"], [nameof(FilterID.Version16Min)], ArgPosition.End)]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel DeprecatedCopyWithUserProvidedCopyWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.DeprecatedCopyWithUserProvidedCopyWarningLevel, _deprecatedCopyWithUserProvidedCopyWarningLevel);
			set => _deprecatedCopyWithUserProvidedCopyWarningLevel = value;
		}
		private WarningLevel _deprecatedCopyWithUserProvidedCopyWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat invalid, unevaluated strings.
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#winvalid-unevaluated-string
		/// </summary>
		[WarningsClangToolChain(["-Wno-invalid-unevaluated-string"], ["-Winvalid-unevaluated-string", "-Wno-error=invalid-unevaluated-string"], ["-Winvalid-unevaluated-string"], [nameof(SpecializedFilters.AndroidNDKR26ToolChainVersionExclusion)])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel InvalidUnevaluatedStringWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.InvalidUnevaluatedStringWarningLevel, _invalidUnevaluatedStringWarningLevel);
			set => _invalidUnevaluatedStringWarningLevel = value;
		}
		private WarningLevel _invalidUnevaluatedStringWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat infinity macros, and subsequent undefined behavior.
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wnan-infinity-disabled
		/// </summary>
		/// <remarks>
		/// We use the NAN macro in a few places to initialize floats while we also set -ffast-math, which disables NaN support. 
		/// It could be easily fixed in the engine, but would create the risk of Win64 building fine and the code failing just on Clang-based platforms. 
		/// We tend to use NAN just as a bit pattern, and not rely on it in calculations, so it should be reasonably safe to disable.
		/// </remarks>
		[WarningsClangToolChain(["-Wno-nan-infinity-disabled"], ["-Wnan-infinity-disabled", "-Wno-error=nan-infinity-disabled"], ["-Wnan-infinity-disabled"], [nameof(FilterID.Version18Point1Min)])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel NaNInfinityDisabledWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.NaNInfinityDisabledWarningLevel, _nanInfinityDisabledWarningLevel);
			set => _nanInfinityDisabledWarningLevel = value;
		}
		private WarningLevel _nanInfinityDisabledWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat extra qualifications.
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wextra-qualification
		/// </summary>
		[WarningsClangToolChain(["-Wno-extra-qualification"], ["-Wextra-qualification", "-Wno-error=extra-qualification"], ["-Wextra-qualification"], [nameof(FilterID.Version19Min)])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel LevelExtraQualificationWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.LevelExtraQualificationWarningLevel, _levelExtraQualificationWarningLevel);
			set => _levelExtraQualificationWarningLevel = value;
		}
		private WarningLevel _levelExtraQualificationWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat function pointer casts to an incompatible function type, potentially leading to undefined behavior.
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wcast-function-type-mismatch
		/// </summary>
		[WarningsClangToolChain(["-Wno-cast-function-type-mismatch"], ["-Wcast-function-type-mismatch", "-Wno-error=cast-function-type-mismatch"], ["-Wcast-function-type-mismatch"], [nameof(SpecializedFilters.AndroidNDKR28ToolChainVersionExclusion)])]
		[WarningsVCToolChain(null, ["/w44191"], ["/we4191"])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel CastFunctionTypeMismatchWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.CastFunctionTypeMismatchWarningLevel, _castFunctionTypeMismatchWarningLevel);
			set => _castFunctionTypeMismatchWarningLevel = value;
		}
		private WarningLevel _castFunctionTypeMismatchWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat templates used without specifying the required template arguments, such as missing parameter packs (&lt;...&gt;) after the template keyword.
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wmissing-template-arg-list-after-template-kw
		/// </summary>
		[WarningsClangToolChain(["-Wno-missing-template-arg-list-after-template-kw"], ["-Wmissing-template-arg-list-after-template-kw", "-Wno-error=missing-template-arg-list-after-template-kw"], ["-Wmissing-template-arg-list-after-template-kw"], [nameof(SpecializedFilters.AndroidNDKR28ToolChainVersionExclusion)])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel MissingTemplateArgListAfterTemplateWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.MissingTemplateArgListAfterTemplateWarningLevel, _missingTemplateArgListAfterTemplateWarningLevel);
			set => _missingTemplateArgListAfterTemplateWarningLevel = value;
		}
		private WarningLevel _missingTemplateArgListAfterTemplateWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat gnu string literal operator templates.
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wgnu-string-literal-operator-template
		/// </summary>
		/// <remarks>We use this feature to allow static FNames.</remarks>
		[WarningsClangToolChain(["-Wno-gnu-string-literal-operator-template"], ["-Wgnu-string-literal-operator-template", "-Wno-error=gnu-string-literal-operator-template"], ["-Wgnu-string-literal-operator-template"])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel GNUStringLiteralOperatorTemplateWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.GNUStringLiteralOperatorTemplateWarningLevel, _gnuStringLiteralOperatorTemplateWarningLevel);
			set => _gnuStringLiteralOperatorTemplateWarningLevel = value;
		}
		private WarningLevel _gnuStringLiteralOperatorTemplateWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat inconsistent missing overrides.
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#winconsistent-missing-override
		/// </summary>
		[WarningsClangToolChain(["-Wno-inconsistent-missing-override"], ["-Winconsistent-missing-override", "-Wno-error=inconsistent-missing-override"], ["-Winconsistent-missing-override"])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel InconsistentMissingOverrideWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.InconsistentMissingOverrideWarningLevel, _inconsistentMissingOverrideWarningLevel);
			set => _inconsistentMissingOverrideWarningLevel = value;
		}
		private WarningLevel _inconsistentMissingOverrideWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat incorrect usages of std::offsetof, such as with non-POD types, where behavior is undefined.
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#winvalid-offsetof
		/// </summary>
		/// <remarks>Needed to suppress warnings about using offsetof on non-POD types.</remarks>
		[WarningsClangToolChain(["-Wno-invalid-offsetof"], ["-Winvalid-offsetof", "-Wno-error=invalid-offsetof"], ["-Winvalid-offsetof"])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel InvalidOffsetWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.InvalidOffsetWarningLevel, _invalidOffsetOfWarningLevel);
			set => _invalidOffsetOfWarningLevel = value;
		}
		private WarningLevel _invalidOffsetOfWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat switch related warnings.
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wswitch
		/// </summary>
		/// <remarks>This hides the "enumeration value 'XXXXX' not handled in switch [-Wswitch]" warnings - we should maybe remove this at some point and add UE_LOG(, Fatal, ) to default cases.</remarks>
		[WarningsClangToolChain(["-Wno-switch"], ["-Wswitch", "-Wno-error=switch"], ["-Wswitch"])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel SwitchWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.SwitchWarningLevel, _switchWarningLevel);
			set => _switchWarningLevel = value;
		}
		private WarningLevel _switchWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat tautological comparisons (i.e. obvious comparisons that are always guaranteed).
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wtautological-compare
		/// </summary>
		/// <remarks>This hides the "warning : comparison of unsigned expression &lt; 0 is always false" type warnings due to constant comparisons, which are possible with template arguments/</remarks>
		[WarningsClangToolChain(["-Wno-tautological-compare"], ["-Wtautological-compare", "Wno-error=tautological-compare"], ["-Wtautological-compare"])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel TautologicalCompareWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.TautologicalCompareWarningLevel, _tautologicalCompareWarningLevel);
			set => _tautologicalCompareWarningLevel = value;
		}
		private WarningLevel _tautologicalCompareWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat unknown pragmas with.
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wunknown-pragmas
		/// </summary>
		/// <remarks>Slate triggers this (with its optimize on/off pragmas).</remarks>
		[WarningsClangToolChain(["-Wno-unknown-pragmas"], ["-Wunknown-pragmas", "-Wno-error=unknown-pragmas"], ["-Wunknown-pragmas"])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel UnknownPragmasWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.UnknownPragmasWarningLevel, _unknownPragmasWarningLevel);
			set => _unknownPragmasWarningLevel = value;
		}
		private WarningLevel _unknownPragmasWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat unused symbols.
		/// Clang - 
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wunused-but-set-variable
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wunused-but-set-parameter
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wunused-function
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wunused-lambda-capture
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wunused-local-typedef
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wunused-private-field
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wunused-variable
		/// </summary>
		/// <remarks>
		/// unused-function: This will hide the warnings about static functions in headers that aren't used in every single .cpp file.
		/// unused-lambda-capture: Suppressed because capturing of compile-time constants is seemingly inconsistent. And MSVC doesn't do that.
		/// unused-local-typedef: Clang is being overly strict here? PhysX headers trigger this.
		/// unused-private-field: This will prevent the issue of warnings for unused private variables. MultichannelTcpSocket.h triggers this, possibly more.
		/// </remarks>
		[WarningsClangToolChain(
		["-Wno-unused-but-set-variable", "-Wno-unused-but-set-parameter", "-Wno-unused-function", "-Wno-unused-lambda-capture", "-Wno-unused-local-typedef", "-Wno-unused-private-field", "-Wno-unused-variable"],
		["-Wunused-but-set-variable", "-Wno-error=unused-but-set-variable", "-Wunused-but-set-parameter", "-Wno-error=unused-but-set-parameter", "-Wunused-function", "-Wno-error=unused-function", "-Wunused-lambda-capture", "-Wno-error=unused-lambda-capture", "-Wunused-local-typedef", "-Wno-error=unused-local-typedef", "-Wunused-private-field", "-Wno-error=unused-private-field", "-Wunused-variable", "-Wno-error=unused-variable"],
		["-Wunused-but-set-variable", "-Wunused-but-set-parameter", "-Wunused-function", "-Wunused-lambda-capture", "-Wunused-local-typedef", "-Wunused-private-field", "-Wunused-variable"])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel UnusedWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.UnusedWarningLevel, _unusedWarningLevel);
			set => _unusedWarningLevel = value;
		}
		private WarningLevel _unusedWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat undefined templates for variables.
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wundefined-var-template
		/// </summary>
		/// <remarks>Not a good warning to be disabling.</remarks>
		[WarningsClangToolChain(["-Wno-undefined-var-template"], ["-Wundefined-var-template", "-Wno-error=undefined-var-template"], ["-Wundefined-var-template"])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel UndefinedVarTemplateWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.UndefinedVarTemplateWarningLevel, _undefinedVarTemplateWarningLevel);
			set => _undefinedVarTemplateWarningLevel = value;
		}
		private WarningLevel _undefinedVarTemplateWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat profile instructions.
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wprofile-instr-out-of-date
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wprofile-instr-unprofiled
		/// </summary>
		/// <remarks> Clang emits warnings for each compiled object file that doesn't have a matching entry in the profile data. This can happen when the profile data is older than the binaries we're compiling. Disable these warnings.</remarks>
		[WarningsClangToolChain(["-Wno-profile-instr-out-of-date", "-Wno-profile-instr-unprofiled"], ["-Wprofile-instr-out-of-date", "-Wno-error=profile-instr-out-of-date", "-Wprofile-instr-unprofiled", "-Wno-error=profile-instr-unprofiled"], ["-Wprofile-instr-out-of-date", "-Wprofile-instr-unprofiled"], [nameof(StandardFilters.PGOOptimizedFilter)])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel ProfileInstructWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.ProfileInstructWarningLevel, _profileInstructWarningLevel);
			set => _profileInstructWarningLevel = value;
		}
		private WarningLevel _profileInstructWarningLevel;

		/// <summary>
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wbackend-plugin
		/// </summary>
		/// <remarks>Apparently there can be hashing conflicts with PGO which can result in: 'Function control flow change detected (hash mismatch)' warnings. </remarks>
		[WarningsClangToolChain(["-Wno-backend-plugin"], ["-Wbackend-plugin", "-Wno-error=backend-plugin"], ["-Wbackend-plugin"], [nameof(StandardFilters.PGOOptimizedFilter)])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel BackendPluginWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.BackendPluginWarningLevel, _backendPluginWarningLevel);
			set => _backendPluginWarningLevel = value;
		}
		private WarningLevel _backendPluginWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat values that are computed, but not used (i.e.unnecessary evaluation of an rvalue or expression).
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wunused-value
		/// </summary>
		/// <remarks>
		/// Shipping builds will cause -Wno-unused-value to warn with "ensure", so disable only in those case.
		/// -Wunused-result - we will always re-enable this as it's controlled by the same set. This will be moved out to a separate property. It is on by default, so it only must be re-enabled in the disable context.
		/// </remarks>
		[UnusedValueClangToolChain(["-Wno-unused-value", "-Wunused-result"], ["-Wunused-value", "-Wno-error=unused-value", "-Wunused-result"], ["-Wunused-value"])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel UnusedValueWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.UnusedValueWarningLevel, _unusedValueWarningLevel);
			set => _unusedValueWarningLevel = value;
		}
		private WarningLevel _unusedValueWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat implicit conversaion losses for integer precision.
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wshorten-64-to-32
		/// </summary>
		/// <remarks>-Wimplicit-int-conversion also controls -Wshorten-64-to-32 as of clang 19.</remarks>
		/// <remarks><see cref="ArgPosition.End"/> due to it's relation to <see cref="UnsafeTypeCastWarningLevel"/>.</remarks>
		[WarningsClangToolChain(["-Wno-shorten-64-to-32"], ["-Wshorten-64-to-32", "-Wno-error=shorten-64-to-32"], ["-Wshorten-64-to-32"], [nameof(FilterID.Version19Min)], ArgPosition.End)]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel Shorten64To32WarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.Shorten64To32WarningLevel, _shorten64To32WarningLevel);
			set => _shorten64To32WarningLevel = value;
		}
		private WarningLevel _shorten64To32WarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat incorrect extern template declarations.
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wdllexport-explicit-instantiation-decl
		/// </summary>
		/// <remarks>The code base contains lots of places where we do "extern template X_API class ..." and we want to keep doing that to reduce compile times.</remarks>
		[WarningsClangToolChain(["-Wno-dllexport-explicit-instantiation-decl"], ["-Wdllexport-explicit-instantiation-decl", "-Wno-error=dllexport-explicit-instantiation-decl"], ["-Wdllexport-explicit-instantiation-decl"])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel DllExportExplicitInstantiationDeclWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.DllExportExplicitInstantiationDeclWarningLevel, _dllExportExplicitInstantiationDeclWarningLevel);
			set => _dllExportExplicitInstantiationDeclWarningLevel = value;
		}
		private WarningLevel _dllExportExplicitInstantiationDeclWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat nontrivial memory access.
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wnontrivial-memaccess
		/// </summary>
		[WarningsClangToolChain(["-Wno-nontrivial-memaccess"], ["-Wnontrivial-memaccess", "-Wno-error=nontrivial-memaccess"], ["-Wnontrivial-memaccess"], [nameof(FilterID.Version20Min)])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel NonTrivialMemAccessWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.NonTrivialMemAccessWarningLevel, _nonTrivialMemAccessWarningLevel);
			set => _nonTrivialMemAccessWarningLevel = value;
		}
		private WarningLevel _nonTrivialMemAccessWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat warnings associated with Microsoft-specific language extensions and compatibility issues.
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wmicrosoft
		/// </summary>
		/// <remarks>Allow Microsoft-specific syntax to slide, even though it may be non-standard.  Needed for Windows headers.</remarks>
		[WarningsVCClang(["-Wno-microsoft"], ["-Wmicrosoft", "-Wno-error=microsoft"], ["-Wmicrosoft"])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel MicrosoftGroupWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.MicrosoftGroupWarningLevel, _microsoftGroupWarningLevel);
			set => _microsoftGroupWarningLevel = value;
		}
		private WarningLevel _microsoftGroupWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat compatibility mods for clang, and how it searches for system headers.
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wmsvc-include
		/// </summary>
		/// <remarks> Hack due to how we have our 'DummyPCH' wrappers setup when using unity builds.  This warning should not be disabled!</remarks>
		[WarningsVCClang(["-Wno-msvc-include"], ["-Wmsvc-include", "-Wno-error=msvc-include"], ["-Wmsvc-include"])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel MSVCIncludeWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.MSVCIncludeWarningLevel, _msvcIncludeWarningLevel);
			set => _msvcIncludeWarningLevel = value;
		}
		private WarningLevel _msvcIncludeWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat improper or inconsistent usage of #pragma pack, which can lead to ABI/unexpected struct layouts.
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wpragma-pack
		/// </summary>
		/// <remarks>
		/// This is disabled because clang explicitly warns about changing pack alignment in a header and not restoring it afterwards, which is something we do with the Pre/PostWindowsApi.h headers.
		/// @TODO clang: push/pop this in  Pre/PostWindowsApi.h headers instead?
		/// </remarks>
		[WarningsVCClang(["-Wno-pragma-pack"], ["-Wpragma-pack", "-Wno-error=pragma-pack"], ["-Wpragma-pack"])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel PragmaPackWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.PragmaPackWarningLevel, _pragmaPackWarningLevel);
			set => _pragmaPackWarningLevel = value;
		}
		private WarningLevel _pragmaPackWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat cases where operator new or operator delete is declared inline, leading to issues with memory allocdealloc across different translation units.
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#winline-new-delete
		/// </summary>
		/// <remarks>@todo clang: We declare operator new as inline.  Clang doesn't seem to like that.</remarks>
		[WarningsVCClang(["-Wno-inline-new-delete"], ["-Winline-new-delete", "-Wno-error=inline-new-delete"], ["-Winline-new-delete"])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel InlineNewDeleteWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.InlineNewDeleteWarningLevel, _inlineNewDeleteWarningLevel);
			set => _inlineNewDeleteWarningLevel = value;
		}
		private WarningLevel _inlineNewDeleteWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat cases where function declarations and definitions have mismatched implicit exception qualifiers.
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wimplicit-exception-spec-mismatch
		/// </summary>

		[WarningsVCClang(["-Wno-implicit-exception-spec-mismatch"], ["-Wimplicit-exception-spec-mismatch", "-Wno-error=implicit-exception-spec-mismatch"], ["-Wimplicit-exception-spec-mismatch"])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel ImplicitExceptionSpecMismatchWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.ImplicitExceptionSpecMismatchWarningLevel, _implicitExceptionSpecMismatchWarningLevel);
			set => _implicitExceptionSpecMismatchWarningLevel = value;
		}
		private WarningLevel _implicitExceptionSpecMismatchWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat cases when an expression that isn't explicitly a boolean is used in a boolean context.
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wundefined-bool-conversion
		/// </summary>
		/// <remarks>Sometimes we compare 'this' pointers against nullptr, which Clang warns about by default.</remarks>
		[WarningsVCClang(["-Wno-undefined-bool-conversion"], ["-Wundefined-bool-conversion", "-Wno-error=undefined-bool-conversion"], ["-Wundefined-bool-conversion"])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel UndefinedBoolConversionWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.UndefinedBoolConversionWarningLevel, _undefinedBoolConversionWarningLevel);
			set => _undefinedBoolConversionWarningLevel = value;
		}
		private WarningLevel _undefinedBoolConversionWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat usages of string literals as writable char* (allowed in older c++), but has since been deprecated.
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wdeprecated-writable-strings
		/// </summary>
		[WarningsVCClang(["-Wno-deprecated-writable-strings"], ["-Wdeprecated-writable-strings", "-Wno-error=deprecated-writable-strings"], ["-Wdeprecated-writable-strings"])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel DeprecatedWriteableStringsWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.DeprecatedWriteableStringsWarningLevel, _deprecatedWriteableStringsWarningLevel);
			set => _deprecatedWriteableStringsWarningLevel = value;
		}
		private WarningLevel _deprecatedWriteableStringsWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat deprecated usages of the register storage class specifier.
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wdeprecated-register
		/// </summary>
		[WarningsVCClang(["-Wno-deprecated-register"], ["-Wdeprecated-register", "-Wno-error=deprecated-register"], ["-Wdeprecated-register"])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel DeprecatedRegisterWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.DeprecatedRegisterWarningLevel, _deprecatedRegisterWarningLevel);
			set => _deprecatedRegisterWarningLevel = value;
		}
		private WarningLevel _deprecatedRegisterWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat usages of logical operations and related parenthesis use. 
		///		e.g.   return false &amp;&amp; false || array[f.get()]; // expected-warning {{'&amp;&amp;' within '||'}} expected-note {{parentheses}}
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wlogical-op-parentheses
		/// </summary>
		/// <remarks>Disable needed for external headers we shan't change.</remarks>
		[WarningsVCClang(["-Wno-logical-op-parentheses"], ["-Wlogical-op-parentheses", "-Wno-error=logical-op-parentheses"], ["-Wlogical-op-parentheses"])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel LogicalOpParenthesesWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.LogicalOpParenthesesWarningLevel, _logicalOpParenthesesWarningLevel);
			set => _logicalOpParenthesesWarningLevel = value;
		}
		private WarningLevel _logicalOpParenthesesWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat occurrences where arithmetic operations are performed on nullptr.
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wnull-arithmetic
		/// </summary>
		/// <remarks>Disable needed for external headers we shan't change.</remarks>
		[WarningsVCClang(["-Wno-null-arithmetic"], ["-Wnull-arithmetic", "-Wno-error=null-arithmetic"], ["-Wnull-arithmetic"])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel NullArithmeticWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.NullArithmeticWarningLevel, _nullArithmeticWarningLevel);
			set => _nullArithmeticWarningLevel = value;
		}
		private WarningLevel _nullArithmeticWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat cases where a function with C linkage has a return type that isn't compatible with C.
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wreturn-type-c-linkage
		/// </summary>
		/// <remarks>Disable needed for PhysX.</remarks>
		[WarningsVCClang(["-Wno-return-type-c-linkage"], ["-Wreturn-type-c-linkage", "-Wno-error=return-type-c-linkage"], ["-Wreturn-type-c-linkage"])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel ReturnTypeCLinkageWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.ReturnTypeCLinkageWarningLevel, _returnTypeCLinkageWarningLevel);
			set => _returnTypeCLinkageWarningLevel = value;
		}
		private WarningLevel _returnTypeCLinkageWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat usages of attributes applied to a function or variable that are ignored.
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wignored-attributes
		/// </summary>
		/// <remarks>Disable needed for nvtesslib.</remarks>
		[WarningsVCClang(["-Wno-ignored-attributes"], ["-Wignored-attributes", "-Wno-error=ignored-attributes"], ["-Wignored-attributes"])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel IgnoredAttributesWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.IgnoredAttributesWarningLevel, _ignoredAttributesWarningLevel);
			set => _ignoredAttributesWarningLevel = value;
		}
		private WarningLevel _ignoredAttributesWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat varaibles that are used before initialization.
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wuninitialized
		/// </summary>
		[WarningsVCClang(["-Wno-uninitialized"], ["-Wuninitialized", "-Wno-error=uninitialized"], ["-Wuninitialized"])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel UninitializedWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.UninitializedWarningLevel, _uninitializedWarningLevel);
			set => _uninitializedWarningLevel = value;
		}
		private WarningLevel _uninitializedWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat functions with a non-void return type, but doesn't return a value in all code paths.
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wreturn-type
		/// </summary>
		/// <remarks>Disable needed for external headers we shan't change.</remarks>
		[WarningsVCClang(["-Wno-return-type"], ["-Wreturn-type", "-Wno-error=return-type"], ["-Wreturn-type"])]
		[VersionWarningLevelDefault(WarningLevel.Off, BuildSettingsVersion.V1, BuildSettingsVersion.V6, InitializationContext.Any)]
		[VersionWarningLevelDefault(WarningLevel.Error, BuildSettingsVersion.V7, BuildSettingsVersion.Latest, InitializationContext.Any)]
		public WarningLevel ReturnTypeWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.ReturnTypeWarningLevel, _returnTypeWarningLevel);
			set => _returnTypeWarningLevel = value;
		}
		private WarningLevel _returnTypeWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat function parameters that are declared, but not used within the function body.
		/// MSVC - 
		///		https://learn.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-4-c4100
		///		4100 - 'identifier' : unreferenced formal parameter
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wunused-parameter
		/// </summary>
		/// <remarks>Unused function parameter. A lot are named 'bUnused'...</remarks>
		[WarningsMSCV(["/wd4100"], ["/w44100"], ["/we4100"])]
		[WarningsVCClang(["-Wno-unused-parameter"], ["-Wunused-parameter", "-Wno-error=unused-parameter"], ["-Wunused-parameter"])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel UnusedParameterWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.UnusedParameterWarningLevel, _unusedParameterWarningLevel);
			set => _unusedParameterWarningLevel = value;
		}
		private WarningLevel _unusedParameterWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat type qualifiers (const/volatile) that are applied to a pointer or ref, but are ignored due to not having an effect.
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wignored-qualifiers
		/// </summary>
		/// <remarks>const ignored when returning by value e.g. 'const int foo() { return 4; }'.</remarks>
		[WarningsVCClang(["-Wno-ignored-qualifiers"], ["-Wignored-qualifiers", "-Wno-error=ignored-qualifiers"], ["-Wignored-qualifiers"])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel IgnoredQualifiersWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.IgnoredQualifiersWarningLevel, _ignoredQualifiersWarningLevel);
			set => _ignoredQualifiersWarningLevel = value;
		}
		private WarningLevel _ignoredQualifiersWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat occurrences of macro expansion resulting in a defined expression being used in an undefined manner.
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wexpansion-to-defined
		/// </summary>
		/// <remarks>Usage of 'defined(X)' in a macro definition. Gives different results under MSVC.</remarks>
		[WarningsVCClang(["-Wno-expansion-to-defined"], ["-Wexpansion-to-defined", "-Wno-error=expansion-to-defined"], ["-Wexpansion-to-defined"])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel ExpansionToDefined
		{
			get => ResolveWarning(ParentCppCompileWarnings?.ExpansionToDefined, _expansionToDefined);
			set => _expansionToDefined = value;
		}
		private WarningLevel _expansionToDefined;

		/// <summary>
		/// Indicates what warning/error level to treat comparisons of signed and unsigned integer types.
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wsign-compare
		/// </summary>
		/// <remarks>Signed/unsigned comparison - millions of these.</remarks>
		[WarningsVCClang(["-Wno-sign-compare"], ["-Wsign-compare", "-Wno-error=sign-compare"], ["-Wsign-compare"])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel SignCompareWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.SignCompareWarningLevel, _signCompareWarningLevel);
			set => _signCompareWarningLevel = value;
		}
		private WarningLevel _signCompareWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat initializer lists used for structor or array missing one or more fields.
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wmissing-field-initializers
		/// </summary>
		/// <remarks>Excessive warning, generated when you initialize with MyStruct A = {0};</remarks>
		[WarningsVCClang(["-Wno-missing-field-initializers"], ["-Wmissing-field-initializers", "-Wno-error=missing-field-initializers"], ["-Wmissing-field-initializers"])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel MissingFieldInitializersWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.MissingFieldInitializersWarningLevel, _missingFieldInitializersWarningLevel);
			set => _missingFieldInitializersWarningLevel = value;
		}
		private WarningLevel _missingFieldInitializersWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat usages of non-portable include paths.
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wnonportable-include-path
		/// </summary>
		[WarningsVCClang(["-Wno-nonportable-include-path"], ["-Wnonportable-include-path", "-Wno-error=nonportable-include-path"], ["-Wnonportable-include-path"])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel NonPortableIncludePathWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.NonPortableIncludePathWarningLevel, _nonPortableIncludePathWarningLevel);
			set => _nonPortableIncludePathWarningLevel = value;
		}
		private WarningLevel _nonPortableIncludePathWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat preprocessor's token-pasting operator (##) is used incorrectly, resulting in ill-formed tokens after concatenation. 
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#winvalid-token-paste
		///	MASVC -
		///		https://learn.microsoft.com/en-us/cpp/preprocessor/token-pasting-operator-hash-hash?view=msvc-170
		/// </summary>
		[WarningsVCClang(["-Wno-invalid-token-paste"], ["-Winvalid-token-paste", "-Wno-error=invalid-token-paste"], ["-Winvalid-token-paste"])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel InvalidTokenPasteWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.InvalidTokenPasteWarningLevel, _invalidTokenPasteWarningLevel);
			set => _invalidTokenPasteWarningLevel = value;
		}
		private WarningLevel _invalidTokenPasteWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat arithmetic operations that are performed with a null pointer.
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wnull-pointer-arithmetic
		/// </summary>
		[WarningsVCClang(["-Wno-null-pointer-arithmetic"], ["-Wnull-pointer-arithmetic", "-Wno-error=null-pointer-arithmetic"], ["-Wnull-pointer-arithmetic"])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel NullPointerArithmeticWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.NullPointerArithmeticWarningLevel, _nullPointerArithmeticWarningLevel);
			set => _nullPointerArithmeticWarningLevel = value;
		}
		private WarningLevel _nullPointerArithmeticWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat usages of constant values being used within a logical operation, which may lead to redundant logic evaluation.
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wconstant-logical-operand
		/// </summary>
		/// <remarks>Triggered by || of two template-derived values inside a static_assert.</remarks>
		[WarningsVCClang(["-Wno-constant-logical-operand"], ["-Wconstant-logical-operand", "-Wno-error=constant-logical-operand"], ["-Wconstant-logical-operand"])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel ConstantLogicalOperandWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.ConstantLogicalOperandWarningLevel, _constantLogicalOperandWarningLevel);
			set => _constantLogicalOperandWarningLevel = value;
		}
		private WarningLevel _constantLogicalOperandWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat assignments of enum types to a bit=field, which can lead to implicit conversations (causing unexpected results).
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wbitfield-enum-conversion
		/// </summary>
		[WarningsVCClang(["-Wno-bitfield-enum-conversion"], ["-Wbitfield-enum-conversion", "-Wno-error=bitfield-enum-conversion"], ["-Wbitfield-enum-conversion"])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel BitfieldEnumConversion
		{
			get => ResolveWarning(ParentCppCompileWarnings?.BitfieldEnumConversion, _bitfieldEnumConversion);
			set => _bitfieldEnumConversion = value;
		}
		private WarningLevel _bitfieldEnumConversion;

		/// <summary>
		/// Indicates what warning/error level to treat instances of pointer subtraction, where a pointer may be null.
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wnull-pointer-subtraction
		/// </summary>
		[WarningsVCClang(["-Wno-null-pointer-subtraction"], ["-Wnull-pointer-subtraction", "-Wno-error=null-pointer-subtraction"], ["-Wnull-pointer-subtraction"])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel NullPointerSubtractionWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.NullPointerSubtractionWarningLevel, _nullPointerSubtractionWarningLevel);
			set => _nullPointerSubtractionWarningLevel = value;
		}
		private WarningLevel _nullPointerSubtractionWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat dangling pointer usage.
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wdangling
		/// </summary>	
		[WarningsVCClang(["-Wno-dangling"], ["-Wdangling", "-Wno-error=dangling"], ["-Wdangling"])]
		[VersionWarningLevelDefault(WarningLevel.Off, BuildSettingsVersion.V1, BuildSettingsVersion.V6, InitializationContext.Any)]
		[VersionWarningLevelDefault(WarningLevel.Error, BuildSettingsVersion.V7, BuildSettingsVersion.Latest, InitializationContext.Any)]
		public WarningLevel DanglingWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.DanglingWarningLevel, _danglingWarningLevel);
			set => _danglingWarningLevel = value;
		}
		private WarningLevel _danglingWarningLevel;

		#region -- Duplicated Clang VC Compiler --

		/// <summary>
		/// Indicates what warning/error level to treat unhandled enumerators in switches on enumeration-typed values.
		/// MSVC - 
		///		4061 - https://learn.microsoft.com/en-us/cpp/error-messages/compiler-warnings/compiler-warning-level-4-c4061 
		///		44061 - Note: The extra 4 is not a typo, /wLXXXX sets warning XXXX to level L
		///	Clang - 
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wswitch-enum
		///	</summary>
		/// <remarks>
		/// This is a special case where we have a duplicate field for clang <see cref="CppCompileWarnings.SwitchUnhandledEnumeratorWarningLevel"/>.
		/// The reason for duplication is that the clang field will only participate in the shipping configuration, whereas the MSVC variant has been disabled in all.
		/// In order to ensure no change of behavior for MSVC toolchain within the clang compilation context, we will keep this in, and deprecate with assignments to the clang setting.
		/// </remarks>
		[WarningsVCClang(["-Wno-switch-enum"])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel MSVCSwitchEnumWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.MSVCSwitchEnumWarningLevel, _msvcSwitchEnumWarningLevel);
			set => _msvcSwitchEnumWarningLevel = value;
		}
		private WarningLevel _msvcSwitchEnumWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat values that are computed, but not used (i.e.unnecessary evaluation of an rvalue or expression).
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wunused-value
		/// </summary>
		/// <remarks>
		/// This is a special case where we have a duplicate field for clang <see cref="CppCompileWarnings.UnusedValueWarningLevel"/>.
		/// The reason for duplication is that the clang field will only participate in the shipping configuration, whereas the MSVC variant has been disabled in all.
		/// In order to ensure no change of behavior for MSVC toolchain within the clang compilation context, we will keep this in, and deprecate with assignments to the clang setting.
		/// -Wunused-result - we will always re-enable this as it's controlled by the same set. This will be moved out to a separate property. It is on by default, so it only must be re-enabled in the disable context.
		/// </remarks>
		[WarningsVCClang(["-Wno-unused-value", "-Wunused-result"], ["-Wunused-value", "-Wno-error=unused-value", "-Wunused-result"], ["-Wunused-value"])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel MSVCUnusedValueWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.MSVCUnusedValueWarningLevel, _msvcUnusedValueWarningLevel);
			set => _msvcUnusedValueWarningLevel = value;
		}
		private WarningLevel _msvcUnusedValueWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat usages of functions, methods, or variables, that are marked as deprecated.
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wdeprecated-declarations
		/// </summary>
		/// <remarks>
		/// This is a special case where we have a duplicate field for clang <see cref="CppCompileWarnings.DeprecationWarningLevel"/>.
		/// The reason for duplication is that the clang field will be defaulted to WARNING in clang, but not for msvc which is disabled.
		/// In order to ensure no change of behavior for MSVC toolchain within the clang compilation context, we will keep this in, and deprecate with assignments to the clang setting.
		/// </remarks>
		[WarningsVCClang(["-Wno-deprecated-declarations"], ["-Wdeprecated-declarations", "-Wno-error=deprecated-declarations"], ["-Wdeprecated-declarations"])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel MSVCDeprecationWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.MSVCDeprecationWarningLevel, _msvcDeprecationWarningLevel);
			set => _msvcDeprecationWarningLevel = value;
		}
		private WarningLevel _msvcDeprecationWarningLevel;

		#endregion -- Duplicated Clang VC Compiler --

		#endregion -- Clang Compiler --

		#region -- Intel Compiler --

		/// <summary>
		/// Indicates what warning/error level to treat instances of mismatched printf style format specifiers.
		/// </summary>
		[WarningsIntelCompiler(["-Wno-format"], ["-Wformat", "-Wno-error=format"], ["-Wformat"])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel FormatWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.FormatWarningLevel, _formatWarningLevel);
			set => _formatWarningLevel = value;
		}
		private WarningLevel _formatWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat instances of implicit int conversion.
		/// Clang - https://clang.llvm.org/docs/DiagnosticsReference.html#wimplicit-int-conversion
		/// </summary>
		[WarningsIntelCompiler(["-Wno-implicit-int-conversion"], ["-Wimplicit-int-conversion", "-Wno-error=implicit-int-conversion"], ["-Wimplicit-int-conversion"])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel ImplicitIntConversionWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.ImplicitIntConversionWarningLevel, _implicitIntConversionWarningLevel);
			set => _implicitIntConversionWarningLevel = value;
		}
		private WarningLevel _implicitIntConversionWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat instances where a command line argument is provided, but not used for anything.
		/// </summary>
		[WarningsIntelCompiler(["-Wno-single-bit-bitfield-constant-conversion"], ["-Wsingle-bit-bitfield-constant-conversion", "-Wno-error=single-bit-bitfield-constant-conversion"], ["-Wsingle-bit-bitfield-constant-conversion"])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel SingleBitfieldConstantConversionWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.SingleBitfieldConstantConversionWarningLevel, _singleBitfieldConstantConversionWarningLevel);
			set => _singleBitfieldConstantConversionWarningLevel = value;
		}
		private WarningLevel _singleBitfieldConstantConversionWarningLevel;

		/// <summary>
		/// 
		/// Clang - https://clang.llvm.org/docs/DiagnosticsReference.html#wunused-command-line-argument
		/// </summary>
		[WarningsIntelCompiler(["-Wno-unused-command-line-argument"], ["-Wunused-command-line-argument", "-Wno-error=unused-command-line-argument"], ["-Wunused-command-line-argument"])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel UnusedCommandLineArgumentWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.UnusedCommandLineArgumentWarningLevel, _unusedCommandLineArgumentWarningLevel);
			set => _unusedCommandLineArgumentWarningLevel = value;
		}
		private WarningLevel _unusedCommandLineArgumentWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat instances of malformed comments.
		///  Clang - https://clang.llvm.org/docs/DiagnosticsReference.html#wcomment
		/// </summary>
		[WarningsIntelCompiler(["-Wno-comment"], ["-Wcomment", "-Wno-error=comment"], ["-Wcomment"])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel CommentWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.CommentWarningLevel, _commentWarningLevel);
			set => _commentWarningLevel = value;
		}
		private WarningLevel _commentWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat instances of copy construct in range loop iteration.
		///  Clang - https://clang.llvm.org/docs/DiagnosticsReference.html#wrange-loop-construct
		/// </summary>
		[WarningsIntelCompiler(["-Wno-range-loop-construct"], ["-Wrange-loop-construct", "-Wno-error=Wrange-loop-construct"], ["-Wrange-loop-construct"])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel RangeLoopConstructWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.RangeLoopConstructWarningLevel, _rangeLoopConstructWarningLevel);
			set => _rangeLoopConstructWarningLevel = value;
		}
		private WarningLevel _rangeLoopConstructWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat instances where a #pragma once is used in a source file instead of a header.
		/// Clang - https://clang.llvm.org/docs/DiagnosticsReference.html#wpragma-once-outside-header
		/// </summary>
		[WarningsIntelCompiler(["-Wno-pragma-once-outside-header"], ["-Wpragma-once-outside-header", "-Wno-error=Wpragma-once-outside-header"], ["-Wpragma-once-outside-header"])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel PragmaOnceOutsideHeaderWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.PragmaOnceOutsideHeaderWarningLevel, _pragmaOnceOutsideHeaderWarningLevel);
			set => _pragmaOnceOutsideHeaderWarningLevel = value;
		}
		private WarningLevel _pragmaOnceOutsideHeaderWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat instances where the logical NOT operator (!) is used in a way that's ambiguous with respect to parentheses.
		/// Clang - https://clang.llvm.org/docs/DiagnosticsReference.html#wlogical-not-parentheses
		/// </summary>
		[WarningsIntelCompiler(["-Wno-logical-not-parentheses"], ["-Wlogical-not-parentheses", "-Wno-error=Wlogical-not-parentheses"], ["-Wlogical-not-parentheses"])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel LogicalNotParenthesesWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.LogicalNotParenthesesWarningLevel, _logicalNotParenthesesWarningLevel);
			set => _logicalNotParenthesesWarningLevel = value;
		}
		private WarningLevel _logicalNotParenthesesWarningLevel;

		/// <summary>
		///  Indicates what warning/error level to treat instances where a C++20 feature is being used, but compiling with an older standard.
		/// Clang - https://clang.llvm.org/docs/DiagnosticsReference.html#wc-20-extensions
		/// </summary>
		[WarningsIntelCompiler(["-Wno-c++20-extensions"], ["-Wc++20-extensions", "-Wno-error=c++20-extensions"], ["-Wc++20-extensions"])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel Cpp20ExtensionsWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.Cpp20ExtensionsWarningLevel, _cpp20ExtensionsWarningLevel);
			set => _cpp20ExtensionsWarningLevel = value;
		}
		private WarningLevel _cpp20ExtensionsWarningLevel;

		#endregion -- Intel Compiler --

		#region -- CppCompileEnvironment Settings

		/// <summary>
		/// Indicates what warning/error level to treat usages of functions, methods, or variables, that are marked as deprecated.
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wdeprecated-declarations
		/// </summary>
		/// <remarks>Disable needed for wxWidgets.</remarks>
		[XmlConfigFile(Category = "BuildConfiguration", Name = nameof(DeprecationWarningLevel))]
		[WarningsClangToolChain(["-Wno-deprecated-declarations"], ["-Wdeprecated-declarations", "-Wno-error=deprecated-declarations"], ["-Wdeprecated-declarations"])]
		[WarningsVCToolChain(["/wd4996"], null, ["/we4996"])]
		[BasicWarningLevelDefault(WarningLevel.Warning)]
		public WarningLevel DeprecationWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.DeprecationWarningLevel, _deprecationWarningLevel);
			set => _deprecationWarningLevel = value;
		}
		private WarningLevel _deprecationWarningLevel;

		/// <summary>
		/// Indicates what warning/error level to treat usages of __DATE__ or __TIME__ as they prevent reproducible builds.
		/// Clang -
		///		https://clang.llvm.org/docs/DiagnosticsReference.html#wdate-time
		/// </summary>
		[WarningsClangToolChain(["-Wno-date-time"], ["-Wdate-time", "-Wno-error=date-time"], ["-Wdate-time"], [nameof(StandardFilters.DeterministcFlagSetFilter)])]
		[WarningsMSCV(["/wd5048"], null, ["/we5048"], [nameof(StandardFilters.DeterministcFlagSetFilter)])]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel DeterministicWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.DeterministicWarningLevel, _deterministicWarningLevel);
			set => _deterministicWarningLevel = value;
		}
		private WarningLevel _deterministicWarningLevel;

		#endregion -- CppCompileEnvironment Settings

		#endregion -- Compiler Warning Settings --

		#region -- Performance Warning Settings --

		/// <summary>
		/// Indicates what warning/error level to treat potential PCH performance issues.
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		[CommandLine("-PCHPerformanceIssueWarningLevel=")]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel PCHPerformanceIssueWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.PCHPerformanceIssueWarningLevel, _pchPerformanceIssueWarningLevel);
			set => _pchPerformanceIssueWarningLevel = value;
		}
		private WarningLevel _pchPerformanceIssueWarningLevel;

		#endregion -- Performance Warning Settings --

		#region -- Rules Validation Warning Settings --

		/// <summary>
		/// How to treat module unsupported validation messages
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		[CommandLine("-ModuleUnsupportedWarningLevel=")]
		[BasicWarningLevelDefault(WarningLevel.Warning)]
		public WarningLevel ModuleUnsupportedWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.ModuleUnsupportedWarningLevel, _moduleUnsupportedWarningLevel);
			set => _moduleUnsupportedWarningLevel = value;
		}
		private WarningLevel _moduleUnsupportedWarningLevel;

		/// <summary>
		/// How to treat plugin specific module unsupported validation messages
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		[CommandLine("-PluginModuleUnsupportedWarningLevel=")]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel PluginModuleUnsupportedWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.PluginModuleUnsupportedWarningLevel, _pluginModuleUnsupportedWarningLevel);
			set => _pluginModuleUnsupportedWarningLevel = value;
		}
		private WarningLevel _pluginModuleUnsupportedWarningLevel;

		/// <summary>
		/// How to treat general module include path validation messages
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		[CommandLine("-ModuleIncludePathWarningLevel=")]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel ModuleIncludePathWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.ModuleIncludePathWarningLevel, _moduleIncludePathWarningLevel);
			set => _moduleIncludePathWarningLevel = value;
		}
		private WarningLevel _moduleIncludePathWarningLevel;

		/// <summary>
		/// How to treat private module include path validation messages, where a module is adding an include path that exposes private headers
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		[CommandLine("-ModuleIncludePrivateWarningLevel=")]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel ModuleIncludePrivateWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.ModuleIncludePrivateWarningLevel, _moduleIncludePrivateWarningLevel);
			set => _moduleIncludePrivateWarningLevel = value;
		}
		private WarningLevel _moduleIncludePrivateWarningLevel;

		/// <summary>
		/// How to treat unnecessary module sub-directory include path validation messages
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		[CommandLine("-ModuleIncludeSubdirectoryWarningLevel=")]
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel ModuleIncludeSubdirectoryWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.ModuleIncludeSubdirectoryWarningLevel, _moduleIncludeSubdirectoryWarningLevel);
			set => _moduleIncludeSubdirectoryWarningLevel = value;
		}
		private WarningLevel _moduleIncludeSubdirectoryWarningLevel;

		/// <summary>
		/// How to treat conflicts when a disabled plugin is being enabled by another plugin referencing it
		/// </summary>
		[BasicWarningLevelDefault(WarningLevel.Default)]
		public WarningLevel DisablePluginsConflictWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.DisablePluginsConflictWarningLevel, _disablePluginsConflictWarningLevel);
			set => _disablePluginsConflictWarningLevel = value;
		}
		private WarningLevel _disablePluginsConflictWarningLevel;

		/// <summary>
		/// Enable warnings for when there are .gen.cpp files that could be inlined in a matching handwritten cpp file
		/// </summary>
		[BasicWarningLevelDefault(WarningLevel.Off)]
		public WarningLevel NonInlinedGenCppWarningLevel
		{
			get => ResolveWarning(ParentCppCompileWarnings?.NonInlinedGenCppWarningLevel, _nonInlinedGenCppWarningLevel);
			set => _nonInlinedGenCppWarningLevel = value;
		}
		private WarningLevel _nonInlinedGenCppWarningLevel;

		#endregion -- Rules Validation Warning Settings --

		#region -- Constructors --

		/// <summary>
		/// CppCompilerWarnings constructor based off of <see cref="TargetRules"/> context.
		/// </summary>
		/// <param name="targetRulesContextObject">The <see cref="TargetRules"/> context to construct the warnings object from.</param>
		/// <param name="logger">Logger for output.</param>
		/// <remarks>The <see cref="TargetRules"/> implicitly has no default parent context.</remarks>
		public CppCompileWarnings(TargetRules targetRulesContextObject, ILogger? logger)
		{
			BuildContextProvider = new BuildSystemContext(new TargetRulesBuildSettingsProvider(targetRulesContextObject), targetRulesContextObject, null);
			ParentCppCompileWarnings = null;
			_logger = logger;
		}

		/// <summary>
		/// CppCompilerWarnings constructor based off of <see cref="ModuleRules"/> context.
		/// </summary>
		/// <param name="moduleRulesContextObject">The <see cref="ModuleRules"/> context to construct the warnings object from.</param>
		/// <param name="logger">Logger for output.</param>
		/// <remarks>The provided module implicitly has a child relationship to it's <see cref="ModuleRules.Target"/>, and will yield to it's warning specifications unless overridden.</remarks>
		public CppCompileWarnings(ModuleRules moduleRulesContextObject, ILogger? logger)
		{
			BuildContextProvider = new BuildSystemContext(new ModuleRulesBuildSettingsProvider(moduleRulesContextObject), null, moduleRulesContextObject);
			ParentCppCompileWarnings = moduleRulesContextObject.Target.CppCompileWarningSettings;
			_logger = logger;
		}

		/// <summary>
		/// CppCompilerWarnings constructor based off of the explicit relationship between the newly constructed <see cref="CppCompileWarnings"/> object, and the provided parent <see cref="IBuildContextProvider"/> object.
		/// </summary>
		/// <param name="contextObject">The parent context which will be used as the default.</param>
		/// <param name="logger">Logger for output.</param>
		/// <param name="parentCompilerWarnings">The parent compiler warnings to associated with this new compiler warnings instance.</param>
		public CppCompileWarnings(IBuildContextProvider contextObject, ILogger? logger, ReadOnlyCppCompileWarnings? parentCompilerWarnings = null)
		{
			BuildContextProvider = new BuildSystemContext(contextObject);
			ParentCppCompileWarnings = parentCompilerWarnings;
			_logger = logger;
		}

		/// <summary>
		/// Default constructor that uses the <see cref="DefaultBuildContextProvider"/>, and uses <see cref="CppCompileEnvironment"/> default settings 
		/// </summary>
		public CppCompileWarnings()
		{
			BuildContextProvider = new BuildSystemContext(new DefaultBuildContextProvider());
			ApplyDefaults();
		}

		/// <summary>
		/// Creates a memberwise shallow-copy of the provided object.
		/// </summary>
		/// <param name="other">The instance to clone.</param>
		/// <returns>A clone of the provided <see cref="CppCompileWarnings"/>.</returns>
		internal static CppCompileWarnings CreateShallowCopy(CppCompileWarnings other)
		{
			return (CppCompileWarnings)other.MemberwiseClone();
		}

		#endregion -- Constructors --

		#region -- Public API --

		/// <summary>
		/// Adds the provided <see cref="CppCompileWarnings"/> as a direct ancestor, promoting previous parent if it exists.
		/// </summary>
		/// <param name="newParentCppCompilerWarnings">The new direct parent.</param>
		public void AddParent(CppCompileWarnings newParentCppCompilerWarnings)
		{
			if (ParentCppCompileWarnings == null)
			{
				ParentCppCompileWarnings = new ReadOnlyCppCompileWarnings(newParentCppCompilerWarnings);
				return;
			}

			_logger?.LogTrace("Promoting existing parent CppCompileWarnings, and inserting new immediate parent.");
			newParentCppCompilerWarnings.ParentCppCompileWarnings = ParentCppCompileWarnings;

			ParentCppCompileWarnings = new ReadOnlyCppCompileWarnings(newParentCppCompilerWarnings);
		}

		/// <summary>
		/// Removes the immediate parent <see cref="CppCompileWarnings"/>, using the next ancestor.
		/// </summary>
		public void RemoveParent()
		{
			if (ParentCppCompileWarnings == null)
			{
				return;
			}

			_logger?.LogTrace("Removing parent CppCompileWarnings.");
			ReadOnlyCppCompileWarnings? ancestor = ParentCppCompileWarnings.ParentCppCompileWarnings;
			ParentCppCompileWarnings = ancestor;
		}

		#endregion -- Public API --

		private static readonly Lazy<List<(MemberInfo, List<ApplyWarningsAttribute>)>> s_membersWithApplyWarningsAttribute = new(() => GetMembersWithAttributes<ApplyWarningsAttribute>(typeof(CppCompileWarnings), typeof(WarningLevel)));
		private static readonly Lazy<List<(MemberInfo, List<WarningLevelDefaultAttribute>)>> s_membersWithWarningLevelDefaultAttribute = new(() => GetMembersWithAttributes<WarningLevelDefaultAttribute>(typeof(CppCompileWarnings), typeof(WarningLevel)).ToList());
		private static readonly ConcurrentDictionary<MemberInfo, Lazy<Dictionary<Type, List<WarningLevelDefaultAttribute>>>>[] s_groupedAttributes = { [], [], [] };

		/// <summary>
		/// Generates the compiler warnings command line arguments for the requested <see cref="UEToolChain"/> type, and version.
		/// </summary>
		/// <param name="compileEnvironment">The compile environment of the invoking tool chain.</param>
		/// <param name="toolChainType">The type of the invoking toolchain.</param>
		/// <param name="toolChainVersion">The version of the tool chain, if applicable.</param>
		/// <param name="analyzer">The analyzer to use when considering arguments, if applicable.</param>
		/// <returns>A list of the compiler command line arguments for compile warnings.</returns>
		/// <remarks>The type of the tool chain must be the specific class within the hierarchy due to how <see cref="ClangToolChain.GetCompileArguments_WarningsAndErrors"/> applies arguments.</remarks>
		internal IEnumerable<string> GenerateWarningCommandLineArgs(CppCompileEnvironment compileEnvironment, System.Type toolChainType, VersionNumber? toolChainVersion = null, StaticAnalyzer analyzer = StaticAnalyzer.None)
		{
			if (!typeof(UEToolChain).IsAssignableFrom(toolChainType))
			{
				_logger?.LogWarning("Requested command line arguments of a toolchain type ({TypeName}) that is not a part of the UEToolChain hierarchy.", toolChainType.Name);

				return [];
			}

			CompilerWarningsToolChainContext toolChainContext = new CompilerWarningsToolChainContext(compileEnvironment, BuildContextProvider, toolChainType, toolChainVersion, analyzer); //pull out into request?
			List<string> warningArguments = [];
			List<ValueTuple<ApplyWarningsAttribute, WarningLevel>> warningAttributesBuffer = [];

			foreach ((MemberInfo mi, List<ApplyWarningsAttribute> miAttr) in s_membersWithApplyWarningsAttribute.Value)
			{
				ApplyWarningsAttribute? applyWarningsAttribute = null;

				bool isFirst = true;

				// We only want to iterate this enumerator once since it does quite a lot of work. Also, this is called a lot so avoid Linq
				foreach (ApplyWarningsAttribute? attr in miAttr)
				{
					if (!attr.CanApplyToContext(toolChainContext))
					{
						continue;
					}

					if (isFirst)
					{
						applyWarningsAttribute = attr;
						isFirst = false;
					}
					else
					{
						applyWarningsAttribute = null;
						_logger?.LogWarning("Ambiguous ApplyWarningsAttributes on the property ({PropertyName}), for the provided tool-chain ({ToolChainName}). Skipping this property.", mi.Name, toolChainContext._toolChainType.Name);
						break;
					}
				}

				if (applyWarningsAttribute != null)
				{
					WarningLevel instanceWarningLevel = GetCurrentWarningLevel(mi, this);
					warningAttributesBuffer.Add(new ValueTuple<ApplyWarningsAttribute, WarningLevel>(applyWarningsAttribute, instanceWarningLevel));
				}
			}

			warningAttributesBuffer.SortBy(x => x.Item1.Position);

			foreach (ValueTuple<ApplyWarningsAttribute, WarningLevel> activeAttribute in warningAttributesBuffer)
			{
				activeAttribute.Item1.ApplyWarningsToArguments(activeAttribute.Item2, toolChainContext, warningArguments);
			}

			return warningArguments;
		}

		#region -- Internal API --

		/// <summary>
		/// Applies the Target context defaults to the provided <see cref="CppCompileWarnings"/>.
		/// </summary>
		/// <param name="cppCompileWarnings">The CppCompileWarnings object to apply target defaults to.</param>
		/// <param name="overwriteSourceValue">Whether to overwrite warning setting values, or set if default.</param>
		/// <remarks>
		/// To have items automatically participate in default setting, use <see cref="BasicWarningLevelDefaultAttribute"/>, with <see cref="InitializationContext.Any"/> or <see cref="InitializationContext.Target"/>.
		/// </remarks>
		internal static void ApplyTargetDefaults(CppCompileWarnings cppCompileWarnings, bool overwriteSourceValue = false)
		{
			cppCompileWarnings._logger?.LogTrace("Applying default C++ Compiler warnings for the Target({TargetName}) context.", cppCompileWarnings.BuildContextProvider?._targetRulesPrivate?.Name);

			foreach ((MemberInfo mi, List<WarningLevelDefaultAttribute> miAttr) in s_membersWithWarningLevelDefaultAttribute.Value)
			{
				WarningLevel defaultWarningLevel = WarningLevel.Default;
				defaultWarningLevel = CoalesceDefaultWarningFromMemberInfo(mi, miAttr, InitializationContext.Target, defaultWarningLevel, cppCompileWarnings.BuildContextProvider, cppCompileWarnings._logger);

				// Obtain instance value, and see if we need to preserve the TargetRules' constructor setting, or apply the default.
				WarningLevel currentWarningLevel = GetCurrentWarningLevel(mi, cppCompileWarnings);

				currentWarningLevel = WarningLevelExtensions.GetAppliedWarningLevel(currentWarningLevel, defaultWarningLevel, overwriteSourceValue);

				SetWarningLevel(mi, cppCompileWarnings, currentWarningLevel);
			}
		}

		internal static WarningLevel CoalesceDefaultWarningFromMemberInfo(MemberInfo mi, List<WarningLevelDefaultAttribute> miAttr, InitializationContext defaultWarningContext, WarningLevel defaultWarningLevel, BuildSystemContext? buildSystemContext, ILogger? logger)
		{
			// This is really slow so needs to be cached.
			Dictionary<Type, List<WarningLevelDefaultAttribute>> groupedAttributes = s_groupedAttributes[(int)defaultWarningContext].GetOrAdd(mi, new Lazy<Dictionary<Type, List<WarningLevelDefaultAttribute>>>(() =>
			{
				IList<WarningLevelDefaultAttribute> attributes = [.. miAttr.Where(prop => prop.Context == InitializationContext.Any || prop.Context == defaultWarningContext)];
				Type fallbackType = typeof(BasicWarningLevelDefaultAttribute);
				return attributes
					.GroupBy(attr => attr.GetType())
					.OrderBy(g => g.Key == fallbackType ? 1 : 0)
					.ToDictionary(g => g.Key, g => g.ToList());
			})).Value;

			IReadOnlyDictionary<string, IWarningLevelResolver> resolvers = WarningLevelResolverRegistry.Resolvers;

			foreach (KeyValuePair<Type, List<WarningLevelDefaultAttribute>> item in groupedAttributes)
			{
				if (resolvers.ContainsKey(item.Key.Name))
				{
					WarningLevel resolvedWarning = resolvers[item.Key.Name].ResolveWarningLevelDefault(item.Value, buildSystemContext);

					defaultWarningLevel = resolvedWarning;
				}
				else if (item.Value.Count == 1)
				{
					logger?.LogInformation("Was unable to find a {Resolver} for the attribute type: {AttributeTypeName}. Falling back to basic retrieval as there's only one attribute.", nameof(IWarningLevelResolver), item.Key.Name);

					WarningLevel resolvedWarning = item.Value[0].GetDefaultLevel(buildSystemContext);

					defaultWarningLevel = resolvedWarning;
				}
				else
				{
					logger?.LogError("Was unable to find a {Resolver} for the attribute type: {AttributeTypeName}, and there was multiple attributes of the same type.", nameof(IWarningLevelResolver), item.Key.Name);
				}

				if (defaultWarningLevel != WarningLevel.Default)
				{
					// This is very spammy. Only comment out when debugging
					//logger?.LogTrace("Obtained default value for ({Property}) via the attribute ({AttributeType}).", mi.Name, item.Key.Name);
					break;
				}
			}

			return defaultWarningLevel;
		}

		#endregion

		#region -- Private API --

		/// <summary>
		/// General purpose resolver that will resolve a warning based off of the precedence of 1. locally set in the <see cref="CppCompileWarnings"/> object; 2. utilize the parent if it exists;
		/// </summary>
		/// <param name="parentWarningLevel">The parent <see cref="CppCompileWarnings"/> <see cref="WarningLevel"/>.</param>
		/// <param name="instanceWarningLevel">The instance <see cref="WarningLevel"/>.</param>
		/// <returns>The resolved warning level.</returns>
		private static WarningLevel ResolveWarning(WarningLevel? parentWarningLevel, WarningLevel instanceWarningLevel)
		{
			WarningLevel resolvedWarning = WarningLevel.Default;

			resolvedWarning.ApplyWarning(parentWarningLevel);
			resolvedWarning.ApplyWarning(instanceWarningLevel);

			return resolvedWarning;
		}

		/// <summary>
		/// Default settings for the no-arg constructor <see cref="CppCompileWarnings()"/>.
		/// </summary>
		/// <remarks>
		/// To have items automatically participate in default setting, use <see cref="BasicWarningLevelDefaultAttribute"/>, with <see cref="InitializationContext.Any"/> or <see cref="InitializationContext.Constructor"/>.
		/// </remarks>
		private void ApplyDefaults()
		{
			_logger?.LogTrace("Applying default C++ Compiler warnings for the default construction context.");

			foreach ((MemberInfo mi, List<WarningLevelDefaultAttribute> miAttr) in s_membersWithWarningLevelDefaultAttribute.Value)
			{
				WarningLevel defaultWarningLevel = WarningLevel.Default;
				defaultWarningLevel = CoalesceDefaultWarningFromMemberInfo(mi, miAttr, InitializationContext.Constructor, defaultWarningLevel, BuildContextProvider, _logger);

				SetWarningLevel(mi, this, defaultWarningLevel);
			}
		}

		#region -- Reflection helpers --

		private static List<(MemberInfo, List<TAttr>)> GetMembersWithAttributes<TAttr>(Type type, Type targetType) where TAttr : Attribute
		{
			List<(MemberInfo, List<TAttr>)> list = [];

			void TryAddMemberInfo(MemberInfo mi, Type memberType)
			{
				if (memberType == targetType)
				{
					IEnumerable<TAttr> attrs = mi.GetCustomAttributes<TAttr>();
					if (attrs.Any())
					{
						list.Add((mi, attrs.ToList()));
					}
				}
			}

			BindingFlags flags = BindingFlags.Instance | BindingFlags.Public | BindingFlags.NonPublic;
			foreach (PropertyInfo prop in type.GetProperties(flags))
			{
				TryAddMemberInfo(prop, prop.PropertyType);
			}

			foreach (FieldInfo field in type.GetFields(flags))
			{
				TryAddMemberInfo(field, field.FieldType);
			}

			return list;
		}

		private static void SetWarningLevel(MemberInfo member, object instance, WarningLevel value)
		{
			switch (member)
			{
				case PropertyInfo pi:
					pi.SetValue(instance, value);
					break;
				case FieldInfo fi:
					fi.SetValue(instance, value);
					break;
			}
		}

		private static WarningLevel GetCurrentWarningLevel(MemberInfo member, object instance)
		{
			object? rawValue = member switch
			{
				PropertyInfo pi => pi.GetValue(instance),
				FieldInfo fi => fi.GetValue(instance),
				_ => null
			};

			return rawValue != null ? (WarningLevel)rawValue : WarningLevel.Default;
		}

		#endregion

		#endregion -- Private API --

		#region -- Internal & Private Instance State --

		private readonly ILogger? _logger;
		private BuildSystemContext BuildContextProvider { get; init; }
		internal ReadOnlyCppCompileWarnings? ParentCppCompileWarnings { get; private set; }

		#endregion -- Internal & Private Instance State --
	}
}