// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using EpicGames.Core;

namespace UnrealBuildTool;

/// <summary>
/// Metadata describing an external (third-party) module.
/// </summary>
/// <param name="Uri">A URI pointing to the homepage of the external software.</param>
/// <param name="Version">
/// The referenced version of the external software.
/// This is modelled as a string as not all external dependencies will follow a specific versioning convention such as SemVer.
/// </param>
/// <param name="TpsFilePath">
/// The path to the `.tps` file for this external software.
/// This can be relative to the module file, or use <c>$(EngineDir)</c> or <c>$(ProjectDir)</c>.
/// </param>
public record ExternalModuleMetadata(
	Uri Uri,
	string Version,
	string TpsFilePath
)
{
	/// <summary>
	/// Used to denote a missing .tps file for third party software when one should exist.
	/// </summary>
	public const string TpsMissing = "<missing>";

	/// <summary>
	/// Used to denote a .tps file not being required - for instance when a first-party component is being linked externally.
	/// </summary>
	public const string TpsNotRequired = "<notrequired>";
}

internal record ProcessedExternalModuleMetadata(
	Uri Uri,
	string Version,
	FileReference TpsFilePath
);
