// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace UnrealBuildTool
{
	public partial class ModuleRules
	{
		/// <summary>
		/// Obsolete: Direct the compiler to generate AVX instructions wherever SSE or AVX intrinsics are used, on the platforms that support it.
		/// Note that by enabling this you are changing the minspec for the PC platform, and the resultant executable will crash on machines without AVX support.
		/// </summary>
		[Obsolete("bUseAVX is obsolete and will be removed in UE5.4, please replace with MinCpuArchX64")]
		public bool bUseAVX
		{
			get => MinCpuArchX64 >= MinimumCpuArchitectureX64.AVX;
			set => MinCpuArchX64 = value ? MinimumCpuArchitectureX64.AVX : MinimumCpuArchitectureX64.None;
		}

		/// <summary>
		/// Enforce "include what you use" rules when PCHUsage is set to ExplicitOrSharedPCH; warns when monolithic headers (Engine.h, UnrealEd.h, etc...) 
		/// are used, and checks that source files include their matching header first.
		/// </summary>
		[Obsolete("Deprecated in UE5.2 - Use IWYUSupport instead.")]
#pragma warning disable CA1044 // Properties should not be write only - ignore for deprecated property on public API
		public bool bEnforceIWYU
		{
			set
			{
				if (!value)
				{
					IWYUSupport = IWYUSupport.None;
				}
			}
		}
#pragma warning restore CA1044

		/// <summary>
		/// How to treat deterministic warnings (experimental).
		/// </summary>
		[Obsolete("Deprecated in UE5.6 - Replace with ModuleRules.CppCompileWarningSettings.DeterministicWarningLevel")]
		public WarningLevel DeterministicWarningLevel
		{
			get => CppCompileWarningSettings.DeterministicWarningLevel;
			set => CppCompileWarningSettings.DeterministicWarningLevel = value;
		}

		/// <summary>
		/// How to treat shadow variable warnings
		/// </summary>
		[Obsolete("Deprecated in UE5.6 - Replace with ModuleRules.CppCompileWarningSettings.ShadowVariableWarningLevel")]
		public WarningLevel ShadowVariableWarningLevel
		{
			get => CppCompileWarningSettings.ShadowVariableWarningLevel;
			set => CppCompileWarningSettings.ShadowVariableWarningLevel = value;
		}

		/// <summary>
		/// How to treat unsafe implicit type cast warnings (e.g., double->float or int64->int32)
		/// </summary>
		[Obsolete("Deprecated in UE5.6 - Replace with ModuleRules.CppCompileWarningSettings.UnsafeTypeCastWarningLevel")]
		public WarningLevel UnsafeTypeCastWarningLevel
		{
			get => CppCompileWarningSettings.UnsafeTypeCastWarningLevel;
			set => CppCompileWarningSettings.UnsafeTypeCastWarningLevel = value;
		}

		/// <summary>
		/// Indicates what warning/error level to treat undefined identifiers in conditional expressions.
		/// </summary>
		[Obsolete("Deprecated in UE5.6 - Replace with ModuleRules.CppCompileWarningSettings.UndefinedIdentifierWarningLevel")]
		public WarningLevel UndefinedIdentifierWarningLevel
		{
			get => CppCompileWarningSettings.UndefinedIdentifierWarningLevel;
			set => CppCompileWarningSettings.UndefinedIdentifierWarningLevel = value;
		}

		/// <summary>
		/// Enable warnings for using undefined identifiers in #if expressions
		/// </summary>
		[Obsolete("Deprecated in UE5.5 - Replace with ModuleRules.UndefinedIdentifierWarningLevel")]
		public bool bEnableUndefinedIdentifierWarnings { get; set; } = true;

		/// <summary>
		/// How to treat general module include path validation messages
		/// </summary>
		[Obsolete("Deprecated in UE5.6 - Replace with ModuleRules.CppCompileWarningSettings.ModuleIncludePathWarningLevel")]
		public WarningLevel ModuleIncludePathWarningLevel
		{
			get => CppCompileWarningSettings.ModuleIncludePathWarningLevel;
			set => CppCompileWarningSettings.ModuleIncludePathWarningLevel = value;
		}

		/// <summary>
		/// How to treat private module include path validation messages, where a module is adding an include path that exposes private headers
		/// </summary>
		[Obsolete("Deprecated in UE5.6 - Replace with ModuleRules.CppCompileWarningSettings.ModuleIncludePrivateWarningLevel")]
		public WarningLevel ModuleIncludePrivateWarningLevel
		{
			get => CppCompileWarningSettings.ModuleIncludePrivateWarningLevel;
			set => CppCompileWarningSettings.ModuleIncludePrivateWarningLevel = value;
		}

		/// <summary>
		/// How to treat unnecessary module sub-directory include path validation messages
		/// </summary>
		[Obsolete("Deprecated in UE5.6 - Replace with ModuleRules.CppCompileWarningSettings.ModuleIncludeSubdirectoryWarningLevel")]
		public WarningLevel ModuleIncludeSubdirectoryWarningLevel
		{
			get => CppCompileWarningSettings.ModuleIncludeSubdirectoryWarningLevel;
			set => CppCompileWarningSettings.ModuleIncludeSubdirectoryWarningLevel = value;
		}

		/// <summary>
		/// Enable warnings for when there are .gen.cpp files that could be inlined in a matching handwritten cpp file
		/// </summary>
		[Obsolete("Deprecated in UE5.6 - Replace with ModuleRules.CppCompileWarningSettings.NonInlinedGenCppWarningLevel")]
		public bool bEnableNonInlinedGenCppWarnings { get; set; }
	}
}