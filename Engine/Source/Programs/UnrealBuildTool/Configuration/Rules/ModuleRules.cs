// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using System.Linq;
using System.Reflection;
using System.Text.Json.Serialization;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// Controls how a particular warning is treated
	/// </summary>
	public enum WarningLevel
	{
		/// <summary>
		/// Use the default behavior
		/// </summary>
		Default,

		/// <summary>
		/// Do not display diagnostics
		/// </summary>
		Off,

		/// <summary>
		/// Output warnings normally
		/// </summary>
		Warning,

		/// <summary>
		/// Output warnings as errors
		/// </summary>
		Error,
	}

	/// <summary>
	/// Describes the origin and visibility of Verse code
	/// </summary>
	public enum VerseScope
	{
		/// <summary>
		/// Created by Epic and only public definitions will be visible to public users
		/// </summary>
		PublicAPI,

		/// <summary>
		/// Created by Epic and is entirely hidden from public users
		/// </summary>
		InternalAPI,

		/// <summary>
		/// Created by a public user
		/// </summary>
		PublicUser,

		/// <summary>
		/// Created by an Epic internal user
		/// </summary>
		InternalUser
	}

	/// <summary>
	/// To what extent a module supports include-what-you-use
	/// </summary>
	public enum IWYUSupport
	{
		/// <summary>
		/// None means code does not even compile. IWYU needs to skip this module entirely
		/// </summary>
		None,

		/// <summary>
		/// Module could be modified with iwyu but we want it to stay the way it is and handle changes manually
		/// </summary>
		KeepAsIs,

		/// <summary>
		/// Module is parsed and processed. This means that from the outside it is stripped for includes
		/// even though the files are not modified. This can be used to defer iwyu work on a module.
		/// When it comes to transitive includes this module is seen as modified from the outside.
		/// </summary>
		KeepAsIsForNow,

		/// <summary>
		/// Same as KeepAsIsForNow but will allow iwyu to update private headers and cpp files.
		/// </summary>
		KeepPublicAsIsForNow,

		/// <summary>
		/// Full iwyu support. When running with -Mode=IWYU this module will be modified if needed
		/// </summary>
		Full,
	}

	/// <summary>
	/// ModuleRules is a data structure that contains the rules for defining a module
	/// </summary>
	public partial class ModuleRules
	{
		/// <summary>
		/// Type of module
		/// </summary>
		public enum ModuleType
		{
			/// <summary>
			/// C++
			/// </summary>
			CPlusPlus,

			/// <summary>
			/// External (third-party)
			/// </summary>
			External,
		}

		/// <summary>
		/// Override the settings of the UHTModuleType to have a different set of
		/// PKG_ flags. Cannot set on a plugin because that value already set in
		/// the '.uplugin' file
		/// </summary>
		public enum PackageOverrideType
		{
			/// <summary>
			/// Do not override the package type on this module
			/// </summary>
			None,

			/// <summary>
			/// Set the PKG_EditorOnly flag on this module
			/// </summary>
			EditorOnly,

			/// <summary>
			/// Set the PKG_Developer on this module
			/// </summary>
			EngineDeveloper,

			/// <summary>
			/// Set the PKG_Developer on this module
			/// </summary>
			GameDeveloper,

			/// <summary>
			/// Set the PKG_UncookedOnly flag on this module
			/// </summary>
			EngineUncookedOnly,

			/// <summary>
			/// Set the PKG_UncookedOnly flag on this module as a game
			/// </summary>
			GameUncookedOnly
		}

		/// <summary>
		/// Code optimization settings
		/// </summary>
		public enum CodeOptimization
		{
			/// <summary>
			/// Code should never be optimized if possible.
			/// </summary>
			Never,

			/// <summary>
			/// Code should only be optimized in non-debug builds (not in Debug).
			/// </summary>
			InNonDebugBuilds,

			/// <summary>
			/// Code should only be optimized in shipping builds (not in Debug, DebugGame, Development)
			/// </summary>
			InShippingBuildsOnly,

			/// <summary>
			/// Code should always be optimized if possible.
			/// </summary>
			Always,

			/// <summary>
			/// Default: 'InNonDebugBuilds' for game modules, 'Always' otherwise.
			/// </summary>
			Default,
		}

		/// <summary>
		/// What type of PCH to use for this module.
		/// </summary>
		public enum PCHUsageMode
		{
			/// <summary>
			/// Default: Engine modules use shared PCHs, game modules do not
			/// </summary>
			Default,

			/// <summary>
			/// Never use any PCHs.
			/// </summary>
			NoPCHs,

			/// <summary>
			/// Never use shared PCHs.  Always generate a unique PCH for this module if appropriate
			/// </summary>
			NoSharedPCHs,

			/// <summary>
			/// Shared PCHs are OK!
			/// </summary>
			UseSharedPCHs,

			/// <summary>
			/// Shared PCHs may be used if an explicit private PCH is not set through PrivatePCHHeaderFile. In either case, none of the source files manually include a module PCH, and should include a matching header instead.
			/// </summary>
			UseExplicitOrSharedPCHs,
		}

		/// <summary>
		/// Which type of targets this module should be precompiled for
		/// </summary>
		public enum PrecompileTargetsType
		{
			/// <summary>
			/// Never precompile this module.
			/// </summary>
			None,

			/// <summary>
			/// Inferred from the module's directory. Engine modules under Engine/Source/Runtime will be compiled for games, those under Engine/Source/Editor will be compiled for the editor, etc...
			/// </summary>
			Default,

			/// <summary>
			/// Any game targets.
			/// </summary>
			Game,

			/// <summary>
			/// Any editor targets.
			/// </summary>
			Editor,

			/// <summary>
			/// Any targets.
			/// </summary>
			Any,
		}

		/// <summary>
		/// Control visibility of symbols in this module for special cases
		/// </summary>
		public enum SymbolVisibility
		{
			/// <summary>
			/// Standard visibility rules
			/// </summary>
			Default,

			/// <summary>
			/// Make sure symbols in this module are visible in Dll builds
			/// </summary>
			VisibileForDll,
		}

		/// <summary>
		/// File types that can use unity files for this module, if enabled.
		/// </summary>
		[Flags]
		public enum UnityFileType
		{
			/// <summary>
			/// No files
			/// </summary>
			None,

			/// <summary>
			/// C++
			/// </summary>
			Cpp,

			/// <summary>
			/// C
			/// </summary>
			C,

			/// <summary>
			/// C++ (alternate extension)
			/// </summary>
			CC,

			/// <summary>
			/// C++ (alternate extension)
			/// </summary>
			CXX,

			/// <summary>
			/// Objective-C
			/// </summary>
			M,

			/// <summary>
			/// Objective-C++
			/// </summary>
			MM,
		}

#pragma warning disable CA1034 // Nested types should not be visible - preserve API of APIs exposed to Rules files
		/// <summary>
		/// Information about a file which is required by the target at runtime, and must be moved around with it.
		/// </summary>
		[Serializable]
		public class RuntimeDependency
		{
			/// <summary>
			/// The file that should be staged. Should use $(EngineDir) and $(ProjectDir) variables as a root, so that the target can be relocated to different machines.
			/// </summary>
			public string Path { get; }

			/// <summary>
			/// The initial location for this file. It will be copied to Path at build time, ready for staging.
			/// </summary>
			public string? SourcePath { get; }

			/// <summary>
			/// How to stage this file.
			/// </summary>
			public StagedFileType Type { get; }

			/// <summary>
			/// Constructor
			/// </summary>
			/// <param name="path">Path to the runtime dependency</param>
			/// <param name="type">How to stage the given path</param>
			public RuntimeDependency(string path, StagedFileType type = StagedFileType.NonUFS)
			{
				Path = path;
				Type = type;
			}

			/// <summary>
			/// Constructor
			/// </summary>
			/// <param name="path">Path to the runtime dependency</param>
			/// <param name="sourcePath">Source path for the file in the working tree</param>
			/// <param name="type">How to stage the given path</param>
			[JsonConstructor]
			public RuntimeDependency(string path, string? sourcePath, StagedFileType type)
			{
				Path = path;
				SourcePath = sourcePath;
				Type = type;
			}
		}

		/// <summary>
		/// List of runtime dependencies, with convenience methods for adding new items
		/// </summary>
		[Serializable]
		public class RuntimeDependencyList
		{
			/// <summary>
			/// Inner list of runtime dependencies
			/// </summary>
			readonly List<RuntimeDependency> _inner = [];

			/// <summary>
			/// Readonly access of inner list of runtime dependencies
			/// </summary>
			internal IReadOnlyList<RuntimeDependency> Inner => _inner.AsReadOnly();

			/// <summary>
			/// Default constructor
			/// </summary>
			public RuntimeDependencyList()
			{
			}

			/// <summary>
			/// Add a runtime dependency to the list
			/// </summary>
			/// <param name="inPath">Path to the runtime dependency. May include wildcards.</param>
			public void Add(string inPath)
			{
				_inner.Add(new RuntimeDependency(inPath));
			}

			/// <summary>
			/// Add a runtime dependency to the list
			/// </summary>
			/// <param name="inPath">Path to the runtime dependency. May include wildcards.</param>
			/// <param name="inType">How to stage this file</param>
			public void Add(string inPath, StagedFileType inType)
			{
				_inner.Add(new RuntimeDependency(inPath, inType));
			}

			/// <summary>
			/// Add a runtime dependency to the list
			/// </summary>
			/// <param name="inPath">Path to the runtime dependency. May include wildcards.</param>
			/// <param name="inSourcePath">Source path for the file to be added as a dependency. May include wildcards.</param>
			/// <param name="inType">How to stage this file</param>
			public void Add(string inPath, string inSourcePath, StagedFileType inType = StagedFileType.NonUFS)
			{
				_inner.Add(new RuntimeDependency(inPath, inSourcePath, inType));
			}
		}

		/// <summary>
		/// List of runtime dependencies, with convenience methods for adding new items
		/// </summary>
		[Serializable]
		public class ReceiptPropertyList
		{
			/// <summary>
			/// Inner list of runtime dependencies
			/// </summary>
			readonly List<ReceiptProperty> _inner = [];

			/// <summary>
			/// Readonly access of inner list of runtime dependencies
			/// </summary>
			internal IReadOnlyList<ReceiptProperty> Inner => _inner.AsReadOnly();

			/// <summary>
			/// Default constructor
			/// </summary>
			public ReceiptPropertyList()
			{
			}

			/// <summary>
			/// Add a receipt property to the list
			/// </summary>
			/// <param name="name">Name of the property</param>
			/// <param name="value">Value for the property</param>
			public void Add(string name, string value)
			{
				_inner.Add(new ReceiptProperty(name, value));
			}

			/// <summary>
			/// Remove recepit properties from the list
			/// </summary>
			/// <param name="match">the maatcher predicate</param>
			/// <returns>the number of items removed</returns>
			public int RemoveAll(Predicate<ReceiptProperty> match)
			{
				return _inner.RemoveAll(match);
			}
		}

		/// <summary>
		/// Stores information about a framework on IOS or MacOS
		/// </summary>
#pragma warning disable CA1724 // Type names should not match namespaces
		public class Framework
#pragma warning restore CA1724
		{
			/// <summary>
			/// Name of the framework
			/// </summary>
			internal string Name { get; init; }

			/// <summary>
			/// Specifies the path to a zip file that contains it or where the framework is located on disk
			/// </summary>
			internal string Path { get; init; }

			/// <summary>
			/// 
			/// </summary>
			internal string? CopyBundledAssets { get; init; } = null;

			/// <summary>
			/// How to handle linking and copying the framework
			/// </summary>
			public enum FrameworkMode
			{
				/// <summary>
				/// Pass this framework to the linker
				/// </summary>
				Link,

				/// <summary>
				/// Copy this framework into the final .app
				/// </summary>
				Copy,

				/// <summary>
				/// Both link into executable and copy into .app
				/// </summary>
				LinkAndCopy,
			}

			/// <summary>
			/// How to treat the framework during linking and creating the .app
			/// </summary>
			internal FrameworkMode Mode { get; init; }

			/// <summary>
			/// Constructor
			/// </summary>
			/// <param name="name">Name of the framework</param>
			/// <param name="path">Path to a zip file containing the framework or a framework on disk</param>
			/// <param name="copyBundledAssets"></param>
			/// <param name="bCopyFramework">Copy the framework to the target's Framework directory</param>
			public Framework(string name, string path, string? copyBundledAssets = null, bool bCopyFramework = false)
				: this(name, path, bCopyFramework ? FrameworkMode.LinkAndCopy : FrameworkMode.Link, copyBundledAssets)
			{
			}

			/// <summary>
			/// Constructor
			/// </summary>
			/// <param name="name">Name of the framework</param>
			/// <param name="path">Path to a zip file containing the framework or a framework on disk</param>
			/// <param name="mode">How to treat the framework during linking and creating the .app</param>
			/// <param name="copyBundledAssets"></param>
			public Framework(string name, string path, FrameworkMode mode, string? copyBundledAssets = null)
			{
				Name = name;
				Path = path;
				Mode = mode;
				CopyBundledAssets = copyBundledAssets;
			}

			/// <summary>
			/// Specifies if the file is a zip file
			/// </summary>
			public bool IsZipFile() => Path.EndsWith(".zip", StringComparison.OrdinalIgnoreCase);
		}

		/// <summary>
		/// 
		/// </summary>
		public class BundleResource
		{
			/// <summary>
			/// 
			/// </summary>
			public string? ResourcePath { get; init; } = null;

			/// <summary>
			/// 
			/// </summary>
			public string? BundleContentsSubdir { get; init; } = null;

			/// <summary>
			/// 
			/// </summary>
			public bool bShouldLog { get; init; } = true;

			/// <summary>
			/// Constructor
			/// </summary>
			/// <param name="resourcePath"></param>
			/// <param name="bundleContentsSubdir"></param>
			/// <param name="bShouldLog"></param>
			public BundleResource(string resourcePath, string bundleContentsSubdir = "Resources", bool bShouldLog = true)
			{
				ResourcePath = resourcePath;
				BundleContentsSubdir = bundleContentsSubdir;
				this.bShouldLog = bShouldLog;
			}
		}

		/// <summary>
		/// Information about a Windows type library (TLB/OLB file) which requires a generated header.
		/// </summary>
		public class TypeLibrary
		{
			/// <summary>
			/// Name of the type library
			/// </summary>
			public string FileName { get; init; }

			/// <summary>
			/// Additional attributes for the #import directive
			/// </summary>
			public string Attributes { get; init; }

			/// <summary>
			/// Name of the output header
			/// </summary>
			public string Header { get; init; }

			/// <summary>
			/// Name of the output include
			/// </summary>
			public string? Include { get; init; }

			/// <summary>
			/// Constructor
			/// </summary>
			/// <param name="fileName">Name of the type library. Follows the same conventions as the filename parameter in the MSVC #import directive.</param>
			/// <param name="attributes">Additional attributes for the import directive</param>
			/// <param name="header">Name of the output header</param>
			/// <param name="include">Name of the output include</param>
			public TypeLibrary(string fileName, string attributes, string header, string? include = null)
			{
				FileName = fileName;
				Attributes = attributes;
				Header = header;
				Include = include;
			}
		}
#pragma warning restore CA1034

		/// <summary>
		/// Name of this module
		/// </summary>
		public string Name { get; internal set; }

		/// <summary>
		/// File containing this module
		/// </summary>
		internal FileReference File { get; set; }

		/// <summary>
		/// Directory containing this module
		/// </summary>
		internal DirectoryReference Directory { get; set; }

		/// <summary>
		/// The Directory that contains either the Build.cs file, or the platform extension's SubClass directory
		/// </summary>
		protected DirectoryReference PlatformDirectory => DirectoriesForModuleSubClasses[GetType()]; // this is expected to always exist if we are able to be queried

		/// <summary>
		/// Returns true if the rules are a platform extension subclass
		/// </summary>
		protected bool bIsPlatformExtension => (Directory != PlatformDirectory);

		/// <summary>
		/// Additional directories that contribute to this module (likely in UnrealBuildTool.EnginePlatformExtensionsDirectory). 
		/// The dictionary tracks module subclasses 
		/// </summary>
		internal Dictionary<Type, DirectoryReference> DirectoriesForModuleSubClasses;

		/// <summary>
		/// Additional directories that contribute to this module but are not based on a subclass (NotForLicensees, etc)
		/// </summary>
		private List<DirectoryReference> AdditionalModuleDirectories { get; } = [];

		/// <summary>
		/// The rules assembly to use when searching for modules
		/// </summary>
		internal RulesAssembly RulesAssembly;

		/// <summary>
		/// Plugin containing this module
		/// </summary>
		internal PluginInfo? Plugin;

		/// <summary>
		/// True if a Plugin contains this module
		/// </summary>
		public bool IsPlugin => Plugin != null;

		/// <summary>
		/// The rules context for this instance
		/// </summary>
		internal ModuleRulesContext Context { get; set; }

		/// <summary>
		/// Rules for the target that this module belongs to
		/// </summary>
		public ReadOnlyTargetRules Target { get; init; }

		/// <summary>
		/// Type of module
		/// </summary>
		public ModuleType Type { get; set; } = ModuleType.CPlusPlus;

		/// <summary>
		/// Accessor for the target logger
		/// </summary>
		public ILogger Logger => Target.Logger;

		/// <summary>
		/// Overridden type of module that will set different package flags.
		/// Cannot be used for modules that are a part of a plugin because that is 
		/// set in the `.uplugin` file already. 
		/// </summary>
		public PackageOverrideType OverridePackageType
		{
			get => overridePackageType ?? PackageOverrideType.None;
			set => overridePackageType = !IsPlugin
					? value
					: throw new CompilationResultException(CompilationResult.RulesError, KnownLogEvents.RulesAssembly, "Module '{ModuleName}' cannot override package type because it is part of a plugin!", Name);
		}

		private PackageOverrideType? overridePackageType;

		/// <summary>
		/// Returns true if there has been an override type specified on this module
		/// </summary>
		public bool HasPackageOverride => OverridePackageType != PackageOverrideType.None;

		/// <summary>
		/// Subfolder of Binaries/PLATFORM folder to put this module in when building DLLs. This should only be used by modules that are found via searching like the
		/// TargetPlatform or ShaderFormat modules. If FindModules is not used to track them down, the modules will not be found.
		/// </summary>
		public string BinariesSubFolder { get; set; } = String.Empty;

		private CodeOptimization? OptimizeCodeOverride;

		/// <summary>
		/// When this module's code should be optimized.
		/// </summary>
		public CodeOptimization OptimizeCode
		{
			get
			{
				if (bCodeCoverage)
				{
					return CodeOptimization.Never;
				}

				if (OptimizeCodeOverride.HasValue)
				{
					return OptimizeCodeOverride.Value;
				}

				bool? shouldOptimizeCode = null;
				if (Target.EnableOptimizeCodeForModules?.Contains(Name) ?? false)
				{
					shouldOptimizeCode = true;
				}

				if (Target.DisableOptimizeCodeForModules?.Contains(Name) ?? false)
				{
					shouldOptimizeCode = false;
				}

				if (!shouldOptimizeCode.HasValue)
				{
					return CodeOptimization.Default;
				}

				return shouldOptimizeCode.Value ? CodeOptimization.Always : CodeOptimization.Never;
			}
			set => OptimizeCodeOverride = value;
		}

		private OptimizationMode? OptimizationLevelOverride = null;

		/// <summary>
		/// Allows fine tuning optimization level for speed and\or code size.
		/// </summary>
		public OptimizationMode OptimizationLevel
		{
			get
			{
				if (Target.OptimizeForSizeModules?.Contains(Name) ?? false)
				{
					return OptimizationMode.Size;
				}
				if (Target.OptimizeForSizeAndSpeedModules?.Contains(Name) ?? false)
				{
					return OptimizationMode.SizeAndSpeed;
				}
				if (OptimizationLevelOverride.HasValue)
				{
					return OptimizationLevelOverride.Value;
				}
				else
				{
					return Target.OptimizationLevel;
				}
			}
			set => OptimizationLevelOverride = value;
		}

		/// <summary>
		/// Allows overriding the FP semantics for this module.
		/// </summary>
		public FPSemanticsMode FPSemantics
		{
			get => FPSemanticsPrivate ?? Target.FPSemantics;
			set => FPSemanticsPrivate = value;
		}

		private FPSemanticsMode? FPSemanticsPrivate = null;

		/// <summary>
		/// Header files that should be force included for every source file in this module.
		/// </summary>
		public List<string> ForceIncludeFiles { get; } = [];

		/// <summary>
		/// Explicit private PCH for this module. Implies that this module will not use a shared PCH.
		/// </summary>
		public string? PrivatePCHHeaderFile { get; set; }

		/// <summary>
		/// Header file name for a shared PCH provided by this module.  Must be a valid relative path to a public C++ header file.
		/// This should only be set for header files that are included by a significant number of other C++ modules.
		/// </summary>
		public string? SharedPCHHeaderFile { get; set; }

		/// <summary>
		/// Specifies an alternate name for intermediate directories and files for intermediates of this module. Useful when hitting path length limitations.
		/// </summary>
		public string? ShortName { get; set; } = null;

		/// <summary>
		/// Precompiled header usage for this module
		/// </summary>
		public PCHUsageMode PCHUsage
		{
			get
			{
				if (PCHUsagePrivate.HasValue)
				{
					// Use the override
					return PCHUsagePrivate.Value;
				}
				else if (Target.bEnableCppModules)
				{
					return PCHUsageMode.NoPCHs;
				}
				else if (DefaultBuildSettings >= BuildSettingsVersion.V2)
				{
					// Use shared or explicit PCHs
					return PCHUsageMode.UseExplicitOrSharedPCHs;
				}
				else if (Plugin != null)
				{
					// Older plugins use shared PCHs by default, since they aren't typically large enough to warrant their own PCH.
					return PCHUsageMode.UseSharedPCHs;
				}
				else
				{
					// Older game modules do not enable shared PCHs by default, because games usually have a large precompiled header of their own.
					return PCHUsageMode.NoSharedPCHs;
				}
			}
			set => PCHUsagePrivate = value;
		}
		private PCHUsageMode? PCHUsagePrivate;

		/// <summary>
		/// Whether this module should be treated as an engine module (eg. using engine definitions, PCHs, compiled with optimizations enabled in DebugGame configurations, etc...).
		/// Initialized to a default based on the rules assembly it was created from.
		/// </summary>
		public bool bTreatAsEngineModule { get; set; }

		/// <summary>
		/// List of files that will be generated for this module. Entries in list are (FileName, FileContent)
		/// </summary>
		public Dictionary<string, IEnumerable<string>> FilesToGenerate { get; } = [];

		/// <summary>
		/// If this contains a platform and the project has had it's SDK version overriden from default, this module will be compiled as a project module, instead of shared engine module
		/// </summary>
		public List<UnrealTargetPlatform> SDKVersionRelevantPlatforms { get; } = [];

		/// <summary>
		/// Emits compilation errors for incorrect UE_LOG format strings.
		/// </summary>
		public bool bValidateFormatStrings
		{
			get => bValidateFormatStringsPrivate ?? (bTreatAsEngineModule || Target.bValidateFormatStrings);
			set => bValidateFormatStringsPrivate = value;
		}
		private bool? bValidateFormatStringsPrivate;

		/// <summary>
		/// Emits deprecated warnings\errors for internal API usage for non-engine modules 
		/// </summary>
		public bool bValidateInternalApi
		{
			get => bValidateInternalApiPrivate ?? !bTreatAsEngineModule;
			set => bValidateInternalApiPrivate = value;
		}
		private bool? bValidateInternalApiPrivate;

		/// <summary>
		/// Emits deprecated warnings\errors for experimental API usage for non-engine modules 
		/// </summary>
		public bool bValidateExperimentalApi
		{
			get => bValidateExperimentalApiPrivate ?? !bTreatAsEngineModule;
			set => bValidateExperimentalApiPrivate = value;
		}
		private bool? bValidateExperimentalApiPrivate;

		/// <summary>
		/// Which engine version's build settings to use by default. 
		/// </summary>
		public BuildSettingsVersion DefaultBuildSettings
		{
			get => DefaultBuildSettingsPrivate ?? Target.DefaultBuildSettings;
			set => DefaultBuildSettingsPrivate = value;
		}
		private BuildSettingsVersion? DefaultBuildSettingsPrivate;

		/// <summary>
		/// What version of include order to use when compiling this module. Can be overridden via -ForceIncludeOrder on the command line or in a module's rules.
		/// </summary>
		public EngineIncludeOrderVersion IncludeOrderVersion
		{
			get
			{
				if (Target.ForcedIncludeOrder != null)
				{
					return Target.ForcedIncludeOrder.Value;
				}
				if (bTreatAsEngineModule)
				{
					return EngineIncludeOrderVersion.Latest;
				}
				return IncludeOrderVersionPrivate ?? Target.IncludeOrderVersion;
			}
			set => IncludeOrderVersionPrivate = value;
		}
		private EngineIncludeOrderVersion? IncludeOrderVersionPrivate;

		/// <summary>
		/// Use run time type information
		/// </summary>
		public bool bUseRTTI { get; set; }

		/// <summary>
		/// Whether to direct MSVC to remove unreferenced COMDAT functions and data.
		/// </summary>
		/// <seealso href="https://learn.microsoft.com/en-us/cpp/build/reference/zc-inline-remove-unreferenced-comdat">zc-inline-remove-unreferenced-comdat</seealso>
		public bool bVcRemoveUnreferencedComdat
		{
			get => bVcRemoveUnreferencedComdatPrivate ?? Target.WindowsPlatform.bStrictInlineConformance;
			set => bVcRemoveUnreferencedComdatPrivate = value;
		}
		private bool? bVcRemoveUnreferencedComdatPrivate;

		/// <summary>
		/// Enable code coverage compilation/linking support.
		/// </summary>
		public bool bCodeCoverage { get; set; }

		/// <summary>
		/// Direct the compiler to generate AVX instructions wherever SSE or AVX intrinsics are used, on the x64 platforms that support it.
		/// Note that by enabling this you are changing the minspec for the PC platform, and the resultant executable will crash on machines without AVX support.
		/// </summary>
		public MinimumCpuArchitectureX64? MinCpuArchX64 { get; set; } = null;

		/// <summary>
		/// Direct the compiler to compile with APX support for X64.
		/// Note that by enabling this you are changing the minspec for the PC platform, and the resultant executable will crash on machines without APX support.
		/// </summary>
		public bool? bMinCpuArchAPX { get; set; } = null;

		/// <summary>
		/// Enable buffer security checks.  This should usually be enabled as it prevents severe security risks.
		/// </summary>
		public bool bEnableBufferSecurityChecks { get; set; } = true;

		/// <summary>
		/// Enable exception handling
		/// </summary>
		public bool bEnableExceptions { get; set; }

		/// <summary>
		/// Enable objective C exception handling
		/// </summary>
		public bool bEnableObjCExceptions { get; set; }

		/// <summary>
		/// Enable objective C automatic reference counting (ARC)
		/// If you set this to true you should not use shared PCHs for this module. The engine won't be extensively using ARC in the short term  
		/// Not doing this will result in a compile errors because shared PCHs were compiled with different flags than consumer
		/// </summary>
		public bool bEnableObjCAutomaticReferenceCounting { get; set; }

		/// <summary>
		/// Settings object used by the module.
		/// </summary>
		public CppCompileWarnings CppCompileWarningSettings { get; init; }

		/// <summary>
		/// Whether to enable all warnings as errors. UE enables most warnings as errors already, but disables a few (such as deprecation warnings).
		/// </summary>`
		public bool bWarningsAsErrors { get; set; }

		/// <summary>
		/// Disable all static analysis - clang, msvc, pvs-studio.
		/// </summary>
		public bool bDisableStaticAnalysis { get; set; }

		/// <summary>
		/// Enable additional analyzer extension warnings using the EspXEngine plugin. This is only supported for MSVC.
		/// See https://learn.microsoft.com/en-us/cpp/code-quality/using-the-cpp-core-guidelines-checkers
		/// This will add a large number of warnings by default. It's recommended to use StaticAnalyzerRulesets if this is enabled.
		/// </summary>
		public bool bStaticAnalyzerExtensions { get; set; }

		/// <summary>
		/// The static analyzer rulesets that should be used to filter warnings. This is only supported for MSVC.
		/// See https://learn.microsoft.com/en-us/cpp/code-quality/using-rule-sets-to-specify-the-cpp-rules-to-run
		/// </summary>
		public HashSet<FileReference> StaticAnalyzerRulesets = [];

		/// <summary>
		/// The static analyzer checkers that should be enabled rather than the defaults. This is only supported for Clang.
		/// See https://clang.llvm.org/docs/analyzer/checkers.html for a full list. Or run:
		///    'clang -Xclang -analyzer-checker-help' 
		/// or: 
		///    'clang -Xclang -analyzer-checker-help-alpha' 
		/// for the list of experimental checkers.
		/// </summary>
		public HashSet<string> StaticAnalyzerCheckers { get; } = [];

		/// <summary>
		/// The static analyzer default checkers that should be disabled. Unused if StaticAnalyzerCheckers is populated. This is only supported for Clang.
		/// See https://clang.llvm.org/docs/analyzer/checkers.html for a full list. Or run:
		///    'clang -Xclang -analyzer-checker-help' 
		/// or: 
		///    'clang -Xclang -analyzer-checker-help-alpha' 
		/// for the list of experimental checkers.
		/// </summary>
		public HashSet<string> StaticAnalyzerDisabledCheckers { get; } = []; // This is set to DefaultStaticAnalyzerDisabledCheckers if static analyzer is set on target

		private static readonly HashSet<string> DefaultStaticAnalyzerDisabledCheckers = [
			// Work in progress, remove disabled checkers as all warnings are fixed
			"core.BitwiseShift",
			"core.CallAndMessage",
			"core.DivideZero",
			"core.NonNullParamChecker",
			"core.NullDereference",
			"core.UndefinedBinaryOperatorResult",
			"core.uninitialized.Assign",
			"core.uninitialized.Branch",
			"core.uninitialized.UndefReturn",
			"cplusplus.Move",
			"cplusplus.NewDelete",
			"cplusplus.NewDeleteLeaks",
			"unix.cstring.NullArg",

			// Needs evaluation
			"deadcode.DeadStores",
		];

		/// <summary>
		/// The static analyzer non-default checkers that should be enabled. Unused if StaticAnalyzerCheckers is populated. This is only supported for Clang.
		/// See https://clang.llvm.org/docs/analyzer/checkers.html for a full list. Or run:
		///    'clang -Xclang -analyzer-checker-help' 
		/// or: 
		///    'clang -Xclang -analyzer-checker-help-alpha' 
		/// for the list of experimental checkers.
		/// </summary>
		public HashSet<string> StaticAnalyzerAdditionalCheckers { get; } = [];

		/// <summary>
		/// The PVS Studio analysis errors that should be disabled.
		/// </summary>
		public HashSet<string> StaticAnalyzerPVSDisabledErrors { get; } = [];

		private bool? bUseUnityOverride;
		/// <summary>
		/// If unity builds are enabled this can be used to override if this specific module will build using Unity.
		/// This is set using the per module configurations in BuildConfiguration.
		/// </summary>
		public bool bUseUnity
		{
			set => bUseUnityOverride = value;
			get => bUseUnityOverride ?? Target.DisableUnityBuildForModules?.Contains(Name) != true;
		}

		/// <summary>
		/// Whether to merge module and generated unity files for faster compilation.
		/// </summary>
		public bool bMergeUnityFiles { get; set; } = true;

		/// <summary>
		/// Source file types that will be placed into unity files when enabled.
		/// </summary>
		public UnityFileType UnityFileTypes { get; set; } = UnityFileType.Cpp;

		/// <summary>
		/// The number of source files in this module before unity build will be activated for that module.  If set to
		/// anything besides -1, will override the default setting which is controlled by MinGameModuleSourceFilesForUnityBuild
		/// </summary>
		public int MinSourceFilesForUnityBuildOverride { get; set; } = 0;

		/// <summary>
		/// Overrides BuildConfiguration.MinFilesUsingPrecompiledHeader if non-zero.
		/// </summary>
		public int MinFilesUsingPrecompiledHeaderOverride { get; set; } = 0;

		/// <summary>
		/// Overrides Target.NumIncludedBytesPerUnityCPP if non-zero.
		/// </summary>
		public int NumIncludedBytesPerUnityCPPOverride { get; set; } = 0;

		/// <summary>
		/// Helper function to get the number of byes per unity cpp file
		/// </summary>
		public int GetNumIncludedBytesPerUnityCPP() => (NumIncludedBytesPerUnityCPPOverride != 0 && !Target.bDisableModuleNumIncludedBytesPerUnityCPPOverride) ? NumIncludedBytesPerUnityCPPOverride : Target.NumIncludedBytesPerUnityCPP;

		/// <summary>
		/// Module uses a #import so must be built locally when compiling with SN-DBS
		/// </summary>
		public bool bBuildLocallyWithSNDBS { get; set; }

		/// <summary>
		/// Redistribution override flag for this module.
		/// </summary>
		public bool? IsRedistributableOverride { get; set; } = null;

		/// <summary>
		/// Whether the output from this module can be publicly distributed, even if it has code/
		/// dependencies on modules that are not (i.e. CarefullyRedist, NotForLicensees, NoRedist).
		/// This should be used when you plan to release binaries but not source.
		/// </summary>
		public bool bLegalToDistributeObjectCode { get; set; }

		/// <summary>
		/// List of folders which are allowed to be referenced when compiling this binary, without propagating restricted folder names
		/// </summary>
		public List<string> AllowedRestrictedFolders = [];

		/// <summary>
		/// Set of aliased restricted folder references
		/// </summary>
		public Dictionary<string, string> AliasRestrictedFolders = [];

		/// <summary>
		/// Allows "include what you use" to modify the source code when run. bEnforceIWYU must be true for this variable to matter.
		/// 
		/// </summary>
		public IWYUSupport IWYUSupport { get; set; } = IWYUSupport.Full;

		/// <summary>
		/// Whether to add all the default include paths to the module (eg. the Source/Classes folder, subfolders under Source/Public).
		/// </summary>
		public bool bAddDefaultIncludePaths { get; set; } = true;

		/// <summary>
		/// Whether to ignore dangling (i.e. unresolved external) symbols in modules
		/// </summary>
		public bool bIgnoreUnresolvedSymbols { get; set; }

		/// <summary>
		/// Whether this module should be precompiled. Defaults to the bPrecompile flag from the target. Clear this flag to prevent a module being precompiled.
		/// </summary>
		public bool bPrecompile { get; set; }

		/// <summary>
		/// Whether this module should use precompiled data. Always true for modules created from installed assemblies.
		/// </summary>
		public bool bUsePrecompiled { get; set; }

		/// <summary>
		/// Whether this module can use PLATFORM_XXXX style defines, where XXXX is a confidential platform name. This is used to ensure engine or other 
		/// shared code does not reveal confidential information inside an #if PLATFORM_XXXX block. Licensee game code may want to allow for them, however.
		/// </summary>
		public bool bAllowConfidentialPlatformDefines { get; set; }

		/// <summary>
		/// Experimental: Whether this module requires platform specific SDK files. Not supported by all platforms yet
		/// </summary>
		public bool bRequiresPlatformSDK { get; set; }

		/// <summary>
		/// Disables AutoRTFM instrumentation in this module. Only matters when the AutoRTFM compiler is being used.
		/// The default value of this depends on whether any `TargetType` can be deduced for the module.
		/// It will default to `true` if:
		/// - The module has `SupportedTargetTypes` specified and `TargetType.Server` is not one of the target types.
		/// - The module is part of a plugin that has a "Type" that is one of "Editor", "EditorAndProgram",
		///   "EditorNoCommandlet", or "UncookedOnly".
		/// - The module is part of a plugin where the module has `"TargetAllowList"` specified and `"Server"` is not
		///   one of the allowed target types.
		/// - The module is part of a plugin where the module has `"TargetDenyList"` specified and `"Server"` is one
		///   of the denied target types.
		/// Otherwise it will default to `false`.
		/// </summary>
		public bool bDisableAutoRTFMInstrumentation
		{
			get
			{
				if (bDisableAutoRTFMInstrumentationPrivate != null)
				{
					return (bool)bDisableAutoRTFMInstrumentationPrivate;
                }

                IEnumerable<TargetType> supportedTargetTypes = GetType().GetCustomAttributes<SupportedTargetTypesAttribute>().SelectMany(x => x.TargetTypes).Distinct();
                if (supportedTargetTypes.Any() && !supportedTargetTypes.Any(x => x == TargetType.Server))
                {
                    return true;
                }

                ModuleDescriptor? descriptor = Plugin?.Descriptor.Modules?.FirstOrDefault(x => x.Name == Name);

				switch (descriptor?.Type)
				{
					case ModuleHostType.Editor:
					case ModuleHostType.EditorAndProgram:
                    case ModuleHostType.EditorNoCommandlet:
					case ModuleHostType.UncookedOnly:
						return true;
				}

                if (descriptor?.TargetAllowList != null && descriptor.TargetAllowList.Any() && !descriptor.TargetAllowList.Any(x => x == TargetType.Server))
                {
                    return true;
                }

                if (descriptor?.TargetDenyList != null && descriptor.TargetDenyList.Any() && descriptor.TargetDenyList.Any(x => x == TargetType.Server))
                {
                    return true;
                }

                return false;
            }
			set => bDisableAutoRTFMInstrumentationPrivate = value;
		}
		private bool? bDisableAutoRTFMInstrumentationPrivate;

		/// <summary>
		/// If `bDisableAutoRTFMInstrumentation` is true, will make the modules `*_API` define contain `AUTORTFM_DISABLE` to let us enable errors for code
		/// that uses transactionally-disabled code in places that are transactionally-enabled.
		/// </summary>
		public bool bInjectAutoRTFMAttributeDisables { get; set; }

		/// <summary>
		/// If true, autogenerated functions of USTRUCTS are always exported
		/// </summary>
		public bool bAlwaysExportStructs = true;

		/// <summary>
		/// If true, autogenerated functions of UENUMS are always exported
		/// </summary>
		public bool bAlwaysExportEnums = true;

		/// <summary>
		/// If true, this module can contain UE types (i.e. UCLASS) in namespaces
		/// </summary>
		public bool bAllowUETypesInNamespaces = false;

		/// <summary>
		/// If true, minimizes the amount of header files included in .generated.h files, which may require more headers from CoreUObject to be explicitly included.
		/// </summary>
		public bool bMinimizeGeneratedIncludes = false;

		/// <summary>
		/// List of modules names (no path needed) with header files that our module's public headers needs access to, but we don't need to "import" or link against.
		/// </summary>
		public List<string> PublicIncludePathModuleNames = [];

		/// <summary>
		/// List of public dependency module names (no path needed) (automatically does the private/public include). These are modules that are required by our public source files.
		/// </summary>
		public List<string> PublicDependencyModuleNames = [];

		/// <summary>
		/// List of modules name (no path needed) with header files that our module's private code files needs access to, but we don't need to "import" or link against.
		/// </summary>
		public List<string> PrivateIncludePathModuleNames = [];

		/// <summary>
		/// List of private dependency module names.  These are modules that our private code depends on but nothing in our public
		/// include files depend on.
		/// </summary>
		public List<string> PrivateDependencyModuleNames = [];

		/// <summary>
		/// Only for legacy reason, should not be used in new code. List of module dependencies that should be treated as circular references.  This modules must have already been added to
		/// either the public or private dependent module list.
		/// </summary>
		public HashSet<string> CircularlyReferencedDependentModules = [];

		/// <summary>
		/// List of system/library include paths - typically used for External (third party) modules.  These are public stable header file directories that are not checked when resolving header dependencies.
		/// </summary>
		public List<string> PublicSystemIncludePaths = [];

		/// <summary>
		/// (This setting is currently not need as we discover all files from the 'Public' folder) List of all paths to include files that are exposed to other modules
		/// </summary>
		public List<string> PublicIncludePaths = [];

		/// <summary>
		/// (This setting is currently not need as we discover all files from the 'Internal' folder) List of all paths to include files that are exposed to other internal modules
		/// </summary>
		public List<string> InternalIncludePaths = [];

		/// <summary>
		/// List of all paths to this module's internal include files, not exposed to other modules (at least one include to the 'Private' path, more if we want to avoid relative paths)
		/// </summary>
		public List<string> PrivateIncludePaths = [];

		/// <summary>
		/// List of system library paths (directory of .lib files) - for External (third party) modules please use the PublicAdditionalLibaries instead
		/// </summary>
		public List<string> PublicSystemLibraryPaths = [];

		/// <summary>
		/// List of search paths for libraries at runtime (eg. .so files)
		/// </summary>
		public List<string> PrivateRuntimeLibraryPaths = [];

		/// <summary>
		/// List of search paths for libraries at runtime (eg. .so files)
		/// </summary>
		public List<string> PublicRuntimeLibraryPaths = [];

		/// <summary>
		/// List of additional libraries (names of the .lib files including extension) - typically used for External (third party) modules
		/// </summary>
		public List<string> PublicAdditionalLibraries = [];

		/// <summary>
		/// List of additional debug visualizers (.natvis, .natstepfilter, and .natjmc) exposed to other modules - typically used for External (third party) modules
		/// </summary>
		public List<string> PublicDebugVisualizerPaths = [];

		/// <summary>
		/// Per-architecture lists of dependencies for linking to ignore (useful when building for multiple architectures, and a lib only is needed for one architecture), it's up to the Toolchain to use this
		/// </summary>
		public Dictionary<string, List<UnrealArch>> DependenciesToSkipPerArchitecture = [];

		/// <summary>
		/// Returns the directory of where the passed in module name lives.
		/// </summary>
		/// <param name="moduleName">Name of the module</param>
		/// <returns>Directory where the module lives</returns>
		public string GetModuleDirectory(string moduleName)
		{
			FileReference? moduleFileReference = RulesAssembly.GetModuleFileName(moduleName)
				?? throw new CompilationResultException(CompilationResult.RulesError, KnownLogEvents.RulesAssembly, "Could not find a module named '{ModuleName}'.", moduleName);
			return moduleFileReference.Directory.FullName;
		}

		/// <summary>
		/// List of additional pre-build libraries (names of the .lib files including extension) - typically used for additional targets which are still built, but using either TargetRules.PreBuildSteps or TargetRules.PreBuildTargets.
		/// </summary>
		public List<string> PublicPreBuildLibraries = [];

		/// <summary>
		/// List of system libraries to use - these are typically referenced via name and then found via the system paths. If you need to reference a .lib file use the PublicAdditionalLibraries instead
		/// </summary>
		public List<string> PublicSystemLibraries = [];

		/// <summary>
		/// List of XCode frameworks (iOS and MacOS)
		/// </summary>
		public List<string> PublicFrameworks = [];

		/// <summary>
		/// List of weak frameworks (for OS version transitions)
		/// </summary>
		public List<string> PublicWeakFrameworks = [];

		/// <summary>
		/// List of addition frameworks - typically used for External (third party) modules on Mac and iOS
		/// </summary>
		public List<Framework> PublicAdditionalFrameworks = [];

		/// <summary>
		/// List of addition resources that should be copied to the app bundle for Mac or iOS
		/// </summary>
		public List<BundleResource> AdditionalBundleResources = [];

		/// <summary>
		/// List of type libraries that we need to generate headers for (Windows only)
		/// </summary>
		public List<TypeLibrary> TypeLibraries = [];

		/// <summary>
		/// List of delay load DLLs - typically used for External (third party) modules
		/// </summary>
		public List<string> PublicDelayLoadDLLs = [];

		/// <summary>
		/// Private compiler definitions for this module
		/// </summary>
		public List<string> PrivateDefinitions = [];

		/// <summary>
		/// Public compiler definitions for this module
		/// </summary>
		public List<string> PublicDefinitions = [];

		/// <summary>
		/// AutoRTFM external open -> closed mapping files
		/// </summary>
		public List<string> AutoRTFMExternalMappingFiles = [];

		/// <summary>
		/// Append (or create)
		/// </summary>
		/// <param name="definition"></param>
		/// <param name="text"></param>
		public void AppendStringToPublicDefinition(string definition, string text)
		{
			string withEquals = definition + "=";
			for (int index = 0; index < PublicDefinitions.Count; index++)
			{
				if (PublicDefinitions[index].StartsWith(withEquals, StringComparison.Ordinal))
				{
					PublicDefinitions[index] = PublicDefinitions[index] + text;
					return;
				}
			}

			// if we get here, we need to make a new entry
			PublicDefinitions.Add(definition + "=" + text);
		}

		/// <summary>
		/// Addition modules this module may require at run-time 
		/// </summary>
		public List<string> DynamicallyLoadedModuleNames = [];

		/// <summary>
		/// List of files which this module depends on at runtime. These files will be staged along with the target.
		/// </summary>
		public RuntimeDependencyList RuntimeDependencies = new();

		/// <summary>
		/// List of additional properties to be added to the build receipt
		/// </summary>
		public ReceiptPropertyList AdditionalPropertiesForReceipt = new();

		/// <summary>
		/// Which targets this module should be precompiled for
		/// </summary>
		public PrecompileTargetsType PrecompileForTargets { get; set; } = PrecompileTargetsType.Default;

		/// <summary>
		/// External files which invalidate the makefile if modified. Relative paths are resolved relative to the .build.cs file.
		/// </summary>
		public List<string> ExternalDependencies = [];

		/// <summary>
		/// Subclass rules files which invalidate the makefile if modified.
		/// </summary>
		public List<string>? SubclassRules;

		/// <summary>
		/// Whether this module requires the IMPLEMENT_MODULE macro to be implemented. Most UE modules require this, since we use the IMPLEMENT_MODULE macro
		/// to do other global overloads (eg. operator new/delete forwarding to GMalloc).
		/// </summary>
		public bool? bRequiresImplementModule { get; set; }

		/// <summary>
		/// Module can provide ExtraRootPath. This will be part of the rootpaths table and is needed if module is using paths outside normal environment
		/// Id should be the same for all extra roots with the same path otherwise we can end up with cache misses that could be cache hits
		/// Make sure path is as long as possible. I.e, don't use "Program Files" as root path but rather "Program Files/MyProgram"
		/// </summary>
		public (string id, string path)? ExtraRootPath { get; set; }

		/// <summary>
		/// If this module has associated Verse code, this is the Verse root path of it
		/// </summary>
		public string? VersePath { get; private set; }

		/// <summary>
		/// Visibility of Verse code in this module's Source/Verse folder
		/// </summary>
		public VerseScope VerseScope { get; private set; } = VerseScope.PublicUser;

		/// <summary>
		/// Whether this module qualifies included headers from other modules relative to the root of their 'Public' folder. This reduces the number
		/// of search paths that have to be passed to the compiler, improving performance and reducing the length of the compiler command line.
		/// </summary>
		public bool bLegacyPublicIncludePaths
		{
			set => bLegacyPublicIncludePathsPrivate = value;
			get => bLegacyPublicIncludePathsPrivate ?? ((DefaultBuildSettings < BuildSettingsVersion.V2) && Target.bLegacyPublicIncludePaths);
		}
		private bool? bLegacyPublicIncludePathsPrivate;

		/// <summary>
		/// Whether this module qualifies included headers from other modules relative to the parent directory. This reduces the number
		/// of search paths that have to be passed to the compiler, improving performance and reducing the length of the compiler command line.
		/// </summary>
		public bool bLegacyParentIncludePaths
		{
			set => bLegacyParentIncludePathsPrivate = value;
			get => bLegacyParentIncludePathsPrivate ?? ((DefaultBuildSettings < BuildSettingsVersion.V3) && Target.bLegacyParentIncludePaths);
		}
		private bool? bLegacyParentIncludePathsPrivate;

		/// <summary>
		/// Whether circular dependencies will be validated against the allow list
		/// Circular module dependencies result in slower builds. Disabling this option is strongly discouraged.
		/// This option is ignored for Engine modules which will always be validated against the allow list.
		/// </summary>
		public bool bValidateCircularDependencies { get; set; } = true;

		/// <summary>
		/// Which stanard to use for compiling this module
		/// </summary>
		public CppStandardVersion? CppStandard { get; set; }

		/// <summary>
		/// Which standard to use for compiling this module
		/// </summary>
		public CStandardVersion? CStandard { get; set; }

		/// <summary>
		/// A list of subdirectory names and functions that are invoked to generate header files.
		/// The subdirectory name is appended to the generated code directory to form a new directory
		/// that headers are generated inside.
		/// </summary>
		public List<(string, Action<ILogger, DirectoryReference>)> GenerateHeaderFuncs = [];

		/// <summary>
		///  Control visibility of symbols
		/// </summary>
		public SymbolVisibility ModuleSymbolVisibility { get; set; } = SymbolVisibility.Default;

		/// <summary>
		/// The AutoSDK directory for the active host platform
		/// </summary>
		public static string? AutoSdkDirectory => UEBuildPlatformSDK.TryGetHostPlatformAutoSDKDir(out DirectoryReference? autoSdkDir) ? autoSdkDir.FullName : null;

		/// <summary>
		/// The current engine directory
		/// </summary>
		public static string EngineDirectory => Unreal.EngineDirectory.FullName;

		/// <summary>
		/// Property for the directory containing this plugin. Useful for adding paths to third party dependencies.
		/// </summary>
		public string PluginDirectory => Plugin == null
					? throw new CompilationResultException(CompilationResult.RulesError, KnownLogEvents.RulesAssembly, "Module '{ModuleName}' does not belong to a plugin; PluginDirectory property is invalid.", Name)
					: Plugin.Directory.FullName;

		/// <summary>
		/// Property for the directory containing this module. Useful for adding paths to third party dependencies.
		/// </summary>
		[SuppressMessage("Naming", "CA1721:Property names should not match get methods", Justification = "GetModuleDirectory() is to get the path to other modules")]
		public string ModuleDirectory => Directory.FullName;

		/// <summary>
		/// Property for the directory containing this module or the platform extension's subclass. Useful for adding paths to third party dependencies.
		/// </summary>
		protected string PlatformModuleDirectory => PlatformDirectory.FullName;

		/// <summary>
		/// Name of a platform under the PlatformModuleDirectory - for a PlatfomrExtension, we don't use platform subdirectories since it's already in a platform dir, so this returns '.'
		/// </summary>
		protected string PlatformSubdirectoryName => bIsPlatformExtension ? "." : Target.Platform.ToString();

		/// <summary>
		/// Returns module's low level tests directory "Tests".
		/// </summary>
		public string TestsDirectory => IsTestModule ? Directory.FullName : Path.Combine(Directory.FullName, "Tests");

		/// <summary>
		/// If set, this module will generate a bridging header between C++ and Swift, using this input header
		/// </summary>
		public string? SwiftInteropHeader = null;

		/// <summary>
		/// A List of files to be used as a LLVM FPass-Plugins for all clang compilation units of the current module.
		///    'clang -fpass-plugin=[FPassPlugin]' 
		/// </summary>
		public List<string> FPassPlugins = [];

		/// <summary>
		/// The metadata for external (third-party) modules.
		/// Should be non-<see langword="null"/> if and only if the module is of type <see cref="ModuleType.External"/>.
		/// </summary>
		public ExternalModuleMetadata? ExternalMetadata { get; set; }

#nullable disable
		/// <summary>
		/// Constructor. For backwards compatibility while the parameterless constructor is being phased out, initialization which would happen here is done by 
		/// RulesAssembly.CreateModulRules instead.
		/// </summary>
		/// <param name="target">Rules for building this target</param>
		public ModuleRules(ReadOnlyTargetRules target)
		{
			if (target.StaticAnalyzer != StaticAnalyzer.None)
			{
				StaticAnalyzerDisabledCheckers = new(DefaultStaticAnalyzerDisabledCheckers);
			}

			Target = target;
			CppCompileWarningSettings = new CppCompileWarnings(this, Logger);
		}
#nullable restore

		/// <summary>
		/// Add the given Engine ThirdParty modules as static private dependencies
		///	Statically linked to this module, meaning they utilize exports from the other module
		///	Private, meaning the include paths for the included modules will not be exposed when giving this modules include paths
		///	NOTE: There is no AddThirdPartyPublicStaticDependencies function.
		/// </summary>
		/// <param name="target">The target this module belongs to</param>
		/// <param name="moduleNames">The names of the modules to add</param>
		public void AddEngineThirdPartyPrivateStaticDependencies(ReadOnlyTargetRules target, params string[] moduleNames)
		{
			if (!bUsePrecompiled || target.LinkType == TargetLinkType.Monolithic)
			{
				PrivateDependencyModuleNames.AddRange(moduleNames);
			}
		}

		/// <summary>
		/// Add the given Engine ThirdParty modules as dynamic private dependencies
		///	Dynamically linked to this module, meaning they do not utilize exports from the other module
		///	Private, meaning the include paths for the included modules will not be exposed when giving this modules include paths
		///	NOTE: There is no AddThirdPartyPublicDynamicDependencies function.
		/// </summary>
		/// <param name="target">Rules for the target being built</param>
		/// <param name="moduleNames">The names of the modules to add</param>
		public void AddEngineThirdPartyPrivateDynamicDependencies(ReadOnlyTargetRules target, params string[] moduleNames)
		{
			if (!bUsePrecompiled || target.LinkType == TargetLinkType.Monolithic)
			{
				PrivateIncludePathModuleNames.AddRange(moduleNames);
				DynamicallyLoadedModuleNames.AddRange(moduleNames);
			}
		}

		/// <summary>
		/// Make sure the necessary linker dependencies are set for compiling code using VerseVM
		/// </summary>
		public void SetupVerse()
		{
			bAllowUETypesInNamespaces = true;
			if (Target.bUseVerseVM)
			{
				PublicDependencyModuleNames.AddRange(
					[
						"libpas",
						"CorePreciseFP",
					]
				);
			}
		}

		/// <summary>
		/// Setup this module for Verse
		/// </summary>
		public void SetupVerse(string versePath, VerseScope verseScope)
		{
			VersePath = versePath;
			VerseScope = verseScope;

			SetupVerse();
		}

		/// <summary>
		/// Setup this module for Mesh Editor support (based on the settings in UEBuildConfiguration)
		/// </summary>
		public void EnableMeshEditorSupport(ReadOnlyTargetRules target) => PublicDefinitions.Add($"ENABLE_MESH_EDITOR={(target.bEnableMeshEditor ? 1 : 0)}");

		/// <summary>
		/// Setup this module for GameplayDebugger support
		/// </summary>
		public void SetupGameplayDebuggerSupport(ReadOnlyTargetRules target, bool bAddAsPublicDependency = false)
		{
			if (target.bUseGameplayDebugger || target.bUseGameplayDebuggerCore)
			{
				PublicDefinitions.Add("WITH_GAMEPLAY_DEBUGGER_CORE=1");
				PublicDefinitions.Add("WITH_GAMEPLAY_DEBUGGER=" + (target.bUseGameplayDebugger ? 1 : 0));
				if (target.bUseGameplayDebugger || (target.bUseGameplayDebuggerCore && target.Configuration != UnrealTargetConfiguration.Shipping))
				{
					PublicDefinitions.Add("WITH_GAMEPLAY_DEBUGGER_MENU=1");
				}
				else
				{
					PublicDefinitions.Add("WITH_GAMEPLAY_DEBUGGER_MENU=0");
				}

				if (bAddAsPublicDependency)
				{
					PublicDependencyModuleNames.Add("GameplayDebugger");
					if (target.Type == TargetType.Editor)
					{
						PublicDependencyModuleNames.Add("GameplayDebuggerEditor");
					}
				}
				else
				{
					PrivateDependencyModuleNames.Add("GameplayDebugger");
					if (target.Type == TargetType.Editor)
					{
						PrivateDependencyModuleNames.Add("GameplayDebuggerEditor");
					}
				}
			}
			else
			{
				PublicDefinitions.Add("WITH_GAMEPLAY_DEBUGGER_CORE=0");
				PublicDefinitions.Add("WITH_GAMEPLAY_DEBUGGER=0");
				PublicDefinitions.Add("WITH_GAMEPLAY_DEBUGGER_MENU=0");
			}
		}

		/// <summary>
		/// Returns true if any trace-based debugger is supported by the provided target
		/// </summary>
		protected static bool AreTraceBasedDebuggersSupported(ReadOnlyTargetRules target)
		{
			return IsRewindDebuggerSupported(target) || IsChaosVisualDebuggerSupported(target);
		}

		/// <summary>
		/// Setup this module for trace-based debuggers
		/// </summary>
		public void SetupModuleTraceBasedDebuggersSupport(ReadOnlyTargetRules target)
		{
			SetupModuleRewindDebuggerSupport(target);
			SetupModuleChaosVisualDebuggerSupport(target);

			if (AreTraceBasedDebuggersSupported(target))
			{
				PublicDefinitions.Add("WITH_TRACE_BASED_DEBUGGERS=1");
			}
			else
			{
				PublicDefinitions.Add("WITH_TRACE_BASED_DEBUGGERS=0");
			}
		}

		/// <summary>
		/// Returns true if the RewindDebugger is supported by the provided target
		/// </summary>
		protected static bool IsRewindDebuggerSupported(ReadOnlyTargetRules target)
		{
			return target.bCompileRewindDebuggerSupport;
		}

		/// <summary>
		/// Setup this module for Rewind Debugger support (Required for recording debug data that will be visualized in the Rewind Debugger tool)
		/// </summary>
		public static void SetupModuleRewindDebuggerSupport(ReadOnlyTargetRules target)
		{
			//TODO:
		}

		/// <summary>
		/// Setup this module for Iris support (based on the settings in UEBuildConfiguration)
		/// </summary>
		public void SetupIrisSupport(ReadOnlyTargetRules target, bool bAddAsPublicDependency = false)
		{
			PrivateDefinitions.Add("UE_NET_HAS_IRIS_FASTARRAY_BINDING=1");

			// To be removed once UE_WITH_IRIS has been removed in code.
			PublicDefinitions.Add("UE_WITH_IRIS=1");

			if (bAddAsPublicDependency)
			{
				PublicDependencyModuleNames.Add("IrisCore");
			}
			else
			{
				PrivateDependencyModuleNames.Add("IrisCore");
			}
		}

		/// <summary>
		/// Returns true if the Chaos Visual Debugger is supported by the provided target
		/// </summary>
		protected static bool IsChaosVisualDebuggerSupported(ReadOnlyTargetRules target)
		{
			return target.bCompileChaosVisualDebuggerSupport;
		}

		/// <summary>
		/// Setup this module for Chaos Visual Debugger support (Required for recording debug data that will be visualized in the Chaos Visual Debugger tool)
		/// </summary>
		public void SetupModuleChaosVisualDebuggerSupport(ReadOnlyTargetRules target)
		{
			//TODO: Emit some form of warning if CVD support is explicitly enabled, but UE Trace is disabled
			if (IsChaosVisualDebuggerSupported(target))
			{
				PublicDependencyModuleNames.Add("ChaosVDRuntime");

				PublicDefinitions.Add("WITH_CHAOS_VISUAL_DEBUGGER=1");
			}
			else
			{
				PublicDefinitions.Add("WITH_CHAOS_VISUAL_DEBUGGER=0");
			}
		}

		/// <summary>
		/// Setup this module for physics support (based on the settings in UEBuildConfiguration)
		/// </summary>
		public void SetupModulePhysicsSupport(ReadOnlyTargetRules target)
		{
			PublicIncludePathModuleNames.AddRange(
				[
					"Chaos",
				]
			);

			PublicDependencyModuleNames.AddRange(
				[
					"PhysicsCore",
					"ChaosCore",
					"Chaos",
					"NNE",
				]
			);

			PublicDefinitions.Add("WITH_CLOTH_COLLISION_DETECTION=1");

			SetupModuleChaosVisualDebuggerSupport(target);
		}

		/// <summary>
		/// Determines if a module type is valid for a target, based on custom attributes
		/// </summary>
		/// <param name="moduleType">The type of the module to check</param>
		/// <param name="targetRules">The target to check against</param>
		/// <param name="invalidReason">Out, reason this module was invalid</param>
		/// <returns>True if the module is valid, false otherwise</returns>
		internal static bool IsValidForTarget(Type moduleType, ReadOnlyTargetRules targetRules, [NotNullWhen(false)] out string? invalidReason)
		{
			HashSet<TargetType>? targetTypes = GetSupportedTargetTypesByAttribute(moduleType);
			if (targetTypes != null && !targetTypes.Contains(targetRules.Type))
			{
				invalidReason = $"TargetType '{targetRules.Type}'";
				return false;
			}

			UnrealTargetConfiguration[]? supportedConfigurations = TargetRules.GetSupportedConfigurationFromAttribute(moduleType);
			if (supportedConfigurations != null && !supportedConfigurations.Contains(targetRules.Configuration))
			{
				invalidReason = $"Configuration '{targetRules.Configuration}'";
				return false;
			}

			// Skip platform extension modules. We only care about the base modules, not the platform overrides.
			// The platform overrides get applied at a later stage when we actually come to build the module.
			bool isPlatformExtension = false;
			int lastUnderscore = moduleType.Name.LastIndexOf('_');
			if (lastUnderscore != -1)
			{
				string potentialPlatform = moduleType.Name.Substring(lastUnderscore);
				isPlatformExtension = UEBuildPlatform.GetPlatformFolderNames().Any(s => s.Equals(potentialPlatform, StringComparison.OrdinalIgnoreCase));
			}
			if (!isPlatformExtension)
			{
				HashSet<UnrealTargetPlatform>? supportedPlatforms = GetSupportedPlatformsByAttribute(moduleType);
				if (supportedPlatforms != null && !supportedPlatforms.Contains(targetRules.Platform))
				{
					invalidReason = $"Platform '{targetRules.Platform}'";
					return false;
				}
			}

			IEnumerable<ObsoleteAttribute> obsoleteAttributes = moduleType.GetCustomAttributes<ObsoleteAttribute>();
			if (obsoleteAttributes.Any())
			{
				invalidReason = $"Obsolete '{(obsoleteAttributes.First().Message ?? "unspecified")}'";
				return false;
			}

			invalidReason = null;
			return true;
		}

		internal static bool IsValidForTarget(ModuleRules module, ReadOnlyTargetRules targetRules, [NotNullWhen(false)] out string? invalidReason)
		{
			// Additional checks for game/client/server based on module file path
			if (targetRules.Type != TargetType.Editor && targetRules.Type != TargetType.Program)
			{
				// Check if a module is set as non-redistributable
				if (targetRules.Configuration == UnrealTargetConfiguration.Shipping && targetRules.bCheckLicenseViolations && module.IsRedistributableOverride == false)
				{
					invalidReason = $"Target type '{targetRules.Type}' Configuration '{targetRules.Configuration}' and IsRedistributableOverride=false";
					return false;
				}

				foreach (DirectoryReference ExtensionDir in Unreal.GetExtensionDirs(Unreal.EngineDirectory))
				{
					// Check if an engine Developer module is referenced
					if (targetRules.Configuration == UnrealTargetConfiguration.Shipping && targetRules.bCheckLicenseViolations && module.IsRedistributableOverride == null)
					{
						DirectoryReference SourceDeveloperDir = DirectoryReference.Combine(ExtensionDir, "Source/Developer");
						if (module.File.IsUnderDirectory(SourceDeveloperDir))
						{
							invalidReason = $"Non-Redistributable (rules file located under '{SourceDeveloperDir.MakeRelativeTo(Unreal.RootDirectory)}')";
							return false;
						}
					}

					// Check if an engine Editor module is referenced
					DirectoryReference SourceEditorDir = DirectoryReference.Combine(ExtensionDir, "Source/Editor");
					if (module.File.IsUnderDirectory(SourceEditorDir))
					{
						invalidReason = $"Editor module for target type '{targetRules.Type}' (rules file located under '{SourceEditorDir.MakeRelativeTo(Unreal.RootDirectory)}')";
						return false;
					}
				}
			}

			return IsValidForTarget(module.GetType(), targetRules, out invalidReason);
		}

		/// <summary>
		/// Determines if this module can be precompiled for the current target.
		/// </summary>
		/// <param name="rulesFile">Path to the module rules file</param>
		/// <returns>True if the module can be precompiled, false otherwise</returns>
		internal bool IsValidForTarget(FileReference rulesFile)
		{
			if (Type == ModuleType.CPlusPlus)
			{
				switch (PrecompileForTargets)
				{
					case PrecompileTargetsType.None:
						return false;
					case PrecompileTargetsType.Default:
						return (Target.Type == TargetType.Editor || !Unreal.GetExtensionDirs(Unreal.EngineDirectory, "Source/Developer").Any(dir => rulesFile.IsUnderDirectory(dir)) || Plugin != null);
					case PrecompileTargetsType.Game:
						return (Target.Type == TargetType.Client || Target.Type == TargetType.Server || Target.Type == TargetType.Game);
					case PrecompileTargetsType.Editor:
						return (Target.Type == TargetType.Editor);
					case PrecompileTargetsType.Any:
						return true;
				}
			}
			return false;
		}

		/// <summary>
		/// Determines whether a given platform is available in the context of the current Target
		/// </summary>
		/// <param name="inPlatform">The platform to check for</param>
		/// <param name="bIgnoreSDKCheck">Ignore the sdks presence when checking for platform availablity (many platforms can be cooked as a target platform without requiring sdk)</param>
		/// <returns>True if it's available, false otherwise</returns>
		protected bool IsPlatformAvailable(UnrealTargetPlatform inPlatform, bool bIgnoreSDKCheck = false)
		{
			return UEBuildPlatform.IsPlatformAvailableForTarget(inPlatform, Target, bIgnoreSDKCheck);
		}

		/// <summary>
		/// Returns all the modules that are in the given module group
		/// </summary>
		/// <param name="moduleGroup">The name of the module group, as defined by a [ModuleGroup("Name")] attribute</param>
		/// <param name="bOnlyValid">Only include modules that are valid for the current Target</param>
		/// <returns></returns>
		public IEnumerable<string> GetModulesInGroup(string moduleGroup, bool bOnlyValid = true)
		{
			// figure out what platforms/groups aren't allowed with this opted in list
			List<string>? disallowedPlatformsAndGroups = (bOnlyValid && Target.OptedInModulePlatforms != null) ? Utils.MakeListOfUnsupportedPlatforms([.. Target.OptedInModulePlatforms], false, Logger) : null;

			foreach (Type moduleType in RulesAssembly.GetTypes()
				.Where(t => t.IsSubclassOf(typeof(ModuleRules)) && t.IsDefined(typeof(ModuleGroupsAttribute), true))
				.Where(t => !bOnlyValid || IsValidForTarget(t, Target, out string? _))
				)
			{
				// check if the module file is disallowed
				if (disallowedPlatformsAndGroups != null)
				{
					FileReference? moduleFileName = RulesAssembly.GetModuleFileName(moduleType.Name);
					if (moduleFileName != null)
					{
						if (moduleFileName.ContainsAnyNames(disallowedPlatformsAndGroups, Unreal.EngineDirectory) ||
							(Target.ProjectFile != null && moduleFileName.ContainsAnyNames(disallowedPlatformsAndGroups, Target.ProjectFile.Directory)))
						{
							continue;
						}
					}
				}

				if (moduleType.GetCustomAttributes<ModuleGroupsAttribute>().Any(x => x.ModuleGroups.Contains(moduleGroup)))
				{
					yield return moduleType.Name;
				}
			}
		}

		/// <summary>
		/// Prepares a module for building a low level tests executable.
		/// If we're building a module as part of a test module chain, then they require the LowLevelTestsRunner dependency.
		/// </summary>
		internal void PrepareModuleForTests()
		{
			TestTargetRules? testTargetRules = Target.InnerTestTargetRules;
			if (testTargetRules == null)
			{
				return;
			}

			lock (testTargetRules)
			{
				if (Name != "LowLevelTestsRunner")
				{
					if (!PrivateIncludePathModuleNames.Contains("LowLevelTestsRunner"))
					{
						PrivateIncludePathModuleNames.Add("LowLevelTestsRunner");
					}
				}
			}
		}

		internal bool IsTestModule => bIsTestModuleOverride ?? false;

		/// <summary>
		/// Whether this is a low level tests module.
		/// </summary>
		protected bool? bIsTestModuleOverride { get; set; }

		/// <summary>
		/// Returns the module directory for a given subclass of the module (platform extensions add subclasses of ModuleRules to add in platform-specific settings)
		/// </summary>
		/// <param name="type">typeof the subclass</param>
		/// <returns>Directory where the subclass's .Build.cs lives, or null if not found</returns>
		public DirectoryReference? GetModuleDirectoryForSubClass(Type type) => DirectoriesForModuleSubClasses.TryGetValue(type, out DirectoryReference? directory) ? directory : null;

		/// <summary>
		/// Returns the directories for all subclasses of this module, as well as any additional directories specified by the rules
		/// </summary>
		/// <returns>List of directories, or null if none were added</returns>
		public DirectoryReference[] GetAllModuleDirectories() => [.. DirectoriesForModuleSubClasses.Values, .. AdditionalModuleDirectories];

		/// <summary>
		/// Adds an additional module directory, if it exists (useful for NotForLicensees/NoRedist)
		/// </summary>
		/// <param name="directory"></param>
		/// <returns>true if the directory exists</returns>
		protected bool ConditionalAddModuleDirectory(DirectoryReference directory)
		{
			if (DirectoryReference.Exists(directory))
			{
				AdditionalModuleDirectories.Add(directory);
				return true;
			}

			return false;
		}

		/// <summary>
		/// Returns if VcPkg is supported for the build configuration.
		/// </summary>
		/// <returns>True if supported</returns>
		public bool IsVcPackageSupported => Target.Platform == UnrealTargetPlatform.Win64 ||
					Target.Platform == UnrealTargetPlatform.Linux ||
					Target.Platform == UnrealTargetPlatform.LinuxArm64 ||
					Target.Platform == UnrealTargetPlatform.Mac;

		/// <summary>
		/// Returns the VcPkg root directory for the build configuration
		/// </summary>
		/// <param name="packageName">The name of the third-party package</param>
		/// <returns></returns>
		[SuppressMessage("Globalization", "CA1308:Normalize strings to uppercase", Justification = "Architecture path is lowercase")]
		public string GetVcPackageRoot(string packageName)
		{
			// TODO: MacOS support, other platform support
			string targetPlatform = Target.Platform.ToString();
			string? platform = null;
			string? architecture = null;
			string linkage = String.Empty;
			string toolset = String.Empty;
			if (Target.Platform == UnrealTargetPlatform.Win64)
			{
				platform = "windows";
				architecture = Target.WindowsPlatform.Architecture.ToString().ToLowerInvariant();
				if (Target.bUseStaticCRT)
				{
					linkage = "-static";
				}
				else
				{
					linkage = "-static-md";
				}
				toolset = "-v142";
			}
			else if (Target.Platform == UnrealTargetPlatform.Linux)
			{
				architecture = "x86_64";
				platform = "unknown-linux-gnu";
			}
			else if (Target.Platform == UnrealTargetPlatform.LinuxArm64)
			{
				architecture = "aarch64";
				platform = "unknown-linux-gnueabi";
			}
			else if (Target.Platform == UnrealTargetPlatform.Mac)
			{
				architecture = "x86_64";
				platform = "osx";
			}

			if (String.IsNullOrEmpty(targetPlatform) || String.IsNullOrEmpty(platform) || String.IsNullOrEmpty(architecture))
			{
				throw new System.NotSupportedException($"Platform {Target.Platform} not currently supported by vcpkg");
			}

			string triplet = $"{architecture}-{platform}{linkage}{toolset}";

			return Path.Combine("ThirdParty", "vcpkg", targetPlatform, triplet, $"{packageName}_{triplet}");
		}

		/// <summary>
		/// Adds libraries compiled with vcpkg to the current module
		/// </summary>
		/// <param name="packageName">The name of the third-party package</param>
		/// <param name="addInclude">Should the include directory be added to PublicIncludePaths</param>
		/// <param name="libraries">The names of the libaries to add to PublicAdditionalLibraries/</param>
		public void AddVcPackage(string packageName, bool addInclude, params string[] libraries)
		{
			string vcPackageRoot = GetVcPackageRoot(packageName);

			if (!System.IO.Directory.Exists(vcPackageRoot))
			{
				throw new DirectoryNotFoundException(vcPackageRoot);
			}

			string libraryExtension = String.Empty;
			if (Target.Platform == UnrealTargetPlatform.Win64)
			{
				libraryExtension = ".lib";
			}
			else if (Target.Platform == UnrealTargetPlatform.Linux || Target.Platform == UnrealTargetPlatform.LinuxArm64 || Target.Platform == UnrealTargetPlatform.Mac)
			{
				libraryExtension = ".a";
			}

			foreach (string library in libraries)
			{
				string libraryPath = Path.Combine(vcPackageRoot, "lib", $"{library}{libraryExtension}");
				if ((Target.Platform == UnrealTargetPlatform.Linux || Target.Platform == UnrealTargetPlatform.LinuxArm64 || Target.Platform == UnrealTargetPlatform.Mac) && !library.StartsWith("lib", StringComparison.OrdinalIgnoreCase))
				{
					libraryPath = Path.Combine(vcPackageRoot, "lib", $"lib{library}{libraryExtension}");
				}
				if (!System.IO.File.Exists(libraryPath))
				{
					throw new FileNotFoundException(libraryPath);
				}
				PublicAdditionalLibraries.Add(libraryPath);
			}

			if (addInclude)
			{
				string includePath = Path.Combine(vcPackageRoot, "include");
				if (!System.IO.Directory.Exists(includePath))
				{
					throw new DirectoryNotFoundException(includePath);
				}

				PublicSystemIncludePaths.Add(Path.Combine(vcPackageRoot, "include"));
			}
		}

		/// <summary>
		/// Replace an expected value in a list of definitions with a new value
		/// </summary>
		/// <param name="definitions">List of definitions e.g. PublicDefinitions</param>
		/// <param name="name">Name of the define to change</param>
		/// <param name="previousValue">Expected value</param>
		/// <param name="newValue">New value</param>
		/// <exception cref="Exception"></exception>
		protected static void ChangeDefinition(List<string> definitions, string name, string previousValue, string newValue)
		{
			if (!definitions.Remove($"{name}={previousValue}"))
			{
				throw new Exception("Failed to removed expected definition");
			}
			definitions.Add($"{name}={newValue}");
		}

		/// <summary>
		/// Replace an expected value in a list of module names with a new value
		/// </summary>
		/// <param name="definitions">List of module names e.g. PublicDependencyModuleNames</param>
		/// <param name="previousModule">Expected value</param>
		/// <param name="newModule">New value</param>
		/// <exception cref="Exception"></exception>
		protected static void ReplaceModule(List<string> definitions, string previousModule, string newModule)
		{
			if (!definitions.Remove(previousModule))
			{
				throw new Exception("Failed to removed expected module name");
			}
			definitions.Add(newModule);
		}

		/// <summary>
		/// Cached function for getting supported target types from type attributes
		/// </summary>
		/// <param name="t"></param>
		/// <returns></returns>
		internal static HashSet<TargetType>? GetSupportedTargetTypesByAttribute(Type t) => s_supportedTargetTypesAttribute.GetOrAdd(t, static type => new Lazy<HashSet<TargetType>?>(() =>
		{
			object[] attributes = type.GetCustomAttributes(typeof(SupportedTargetTypesAttribute), false);
			if (attributes.Length == 0)
			{
				if (type.BaseType != null)
				{
					return GetSupportedTargetTypesByAttribute(type.BaseType);
				}
				else
				{
					return null;
				}
			}
			HashSet<TargetType> targetTypes = new(attributes.OfType<SupportedTargetTypesAttribute>().SelectMany(x => x.TargetTypes));
			if (type.BaseType != null)
			{
				HashSet<TargetType>? parentTargetTypes = GetSupportedTargetTypesByAttribute(type.BaseType);
				if (parentTargetTypes != null)
				{
					targetTypes.UnionWith(parentTargetTypes);
				}
			}

			return targetTypes;

		})).Value;
		static readonly ConcurrentDictionary<Type, Lazy<HashSet<TargetType>?>> s_supportedTargetTypesAttribute = new();

		/// <summary>
		/// Cached function for getting supported platforms from type attributes
		/// </summary>
		/// <param name="t"></param>
		/// <returns></returns>
		internal static HashSet<UnrealTargetPlatform>? GetSupportedPlatformsByAttribute(Type t) => s_supportedPlatformsAttribute.GetOrAdd(t, static type => new Lazy<HashSet<UnrealTargetPlatform>?>(() =>
		{
			object[] attributes = type.GetCustomAttributes(typeof(SupportedPlatformsAttribute), false);
			if (attributes.Length == 0)
			{
				if (type.BaseType != null)
				{
					return GetSupportedPlatformsByAttribute(type.BaseType);
				}
				else
				{
					return null;
				}
			}
			HashSet<UnrealTargetPlatform> platforms = new(attributes.OfType<SupportedPlatformsAttribute>().SelectMany(x => x.Platforms));
			if (type.BaseType != null)
			{
				HashSet<UnrealTargetPlatform>? parentPlatforms = GetSupportedPlatformsByAttribute(type.BaseType);
				if (parentPlatforms != null)
				{
					platforms.UnionWith(parentPlatforms);
				}
			}

			return platforms;

		})).Value;
		static readonly ConcurrentDictionary<Type, Lazy<HashSet<UnrealTargetPlatform>?>> s_supportedPlatformsAttribute = new();
	}

	/// <summary>
	/// AddRange added to HashSet
	/// </summary>
	public static class HashSetExtensions
	{
		/// <summary>
		/// Add a range of objects
		/// </summary>
		/// <typeparam name="T"></typeparam>
		/// <param name="set"></param>
		/// <param name="items"></param>
		/// <exception cref="ArgumentNullException"></exception>
		public static void AddRange<T>(this HashSet<T> set, IEnumerable<T> items)
		{
			set.UnionWith(items);
		}

		/// <summary>
		/// Add a range of objects
		/// </summary>
		/// <typeparam name="T"></typeparam>
		/// <param name="set"></param>
		/// <param name="items"></param>
		public static void AddRange<T>(this HashSet<T> set, params T[] items)
		{
			set.UnionWith(items);
		}
	}
}
