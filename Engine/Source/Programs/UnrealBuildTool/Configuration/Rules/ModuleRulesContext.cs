// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;

namespace UnrealBuildTool
{
	/// <summary>
	/// Stores information about where a module rules object came from, and how it can be used. 
	/// </summary>
	internal class ModuleRulesContext
	{
		/// <summary>
		/// The scope for this module. Used to validate references to other modules.
		/// </summary>
		public RulesScope Scope { get; set; }

		/// <summary>
		/// The default directory for output files
		/// </summary>
		public DirectoryReference DefaultOutputBaseDir { get; set; }

		/// <summary>
		/// The plugin that this module belongs to
		/// </summary>
		public PluginInfo? Plugin { get; set; }

		/// <summary>
		/// Whether this module should be included in the default hot reload set
		/// </summary>
		public bool bCanHotReload { get; set; }

		/// <summary>
		/// Whether this module should be compiled with optimization disabled in DebugGame configurations (ie. whether it's a game module).
		/// </summary>
		public bool bCanBuildDebugGame { get; set; }

		/// <summary>
		/// Whether this module can be used for generating shared PCHs
		/// </summary>
		public bool bCanUseForSharedPCH { get; set; }

		/// <summary>
		/// Whether to treat this module as a game module for UHT ordering
		/// </summary>
		public bool bClassifyAsGameModuleForUHT { get; set; }

		/// <summary>
		/// The default module type for UnrealHeaderTool. Do not use this for inferring other things about the module.
		/// </summary>
		public UHTModuleType? DefaultUHTModuleType { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		public ModuleRulesContext(RulesScope scope, DirectoryReference defaultOutputBaseDir)
		{
			Scope = scope;
			DefaultOutputBaseDir = defaultOutputBaseDir;
			bCanUseForSharedPCH = true;
		}

		/// <summary>
		/// Copy constructor
		/// </summary>
		/// <param name="other">The context to copy from</param>
		public ModuleRulesContext(ModuleRulesContext other)
		{
			Scope = other.Scope;
			DefaultOutputBaseDir = other.DefaultOutputBaseDir;
			Plugin = other.Plugin;
			bCanHotReload = other.bCanHotReload;
			bCanBuildDebugGame = other.bCanBuildDebugGame;
			bCanUseForSharedPCH = other.bCanUseForSharedPCH;
			bClassifyAsGameModuleForUHT = other.bClassifyAsGameModuleForUHT;
			DefaultUHTModuleType = other.DefaultUHTModuleType;
		}
	}
}
