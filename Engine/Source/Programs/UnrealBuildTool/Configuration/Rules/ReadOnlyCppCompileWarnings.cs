// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool
{
	/// <summary>
	/// Read-only wrapper around an existing CppCompileWarnings instance. This exposes CppCompileWarnings settings without letting them be modified.
	/// </summary>
	public class ReadOnlyCppCompileWarnings
	{
		/// <summary>
		/// The writeable CppCompileWarnings instance
		/// </summary>
		readonly CppCompileWarnings _inner;
		internal ReadOnlyCppCompileWarnings? ParentCppCompileWarnings { get; init; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="inner">The CppCompileWarnings instance to wrap around</param>
		public ReadOnlyCppCompileWarnings(CppCompileWarnings inner)
		{
			_inner = inner;
			ParentCppCompileWarnings = inner.ParentCppCompileWarnings;
		}

		/// <summary>
		/// Clones the <see cref="ReadOnlyCppCompileWarnings"/> as a writeable <see cref="CppCompileWarnings"/>.
		/// </summary>
		/// <returns>A writeable clone of the <see cref="ReadOnlyCppCompileWarnings"/>.</returns>
		internal CppCompileWarnings CloneAsWriteable()
		{
			return CppCompileWarnings.CreateShallowCopy(_inner);
		}

		/// <summary>
		/// Accessors for fields on the inner CppCompileWarnings instance
		/// </summary>
		#region Read-only accessor properties
#pragma warning disable CS1591
		#region -- Compiler Warning Settings --

		public WarningLevel ShadowVariableWarningLevel => _inner.ShadowVariableWarningLevel;
		public WarningLevel UndefinedIdentifierWarningLevel => _inner.UndefinedIdentifierWarningLevel;
		public WarningLevel UnreachableCodeWarningLevel => _inner.UnreachableCodeWarningLevel;
		public WarningLevel UnsafeTypeCastWarningLevel => _inner.UnsafeTypeCastWarningLevel;
		public WarningLevel SwitchUnhandledEnumeratorWarningLevel => _inner.SwitchUnhandledEnumeratorWarningLevel;
		public WarningLevel ShortenSizeTToIntWarningLevel => _inner.ShortenSizeTToIntWarningLevel;

		#region -- Clang Compiler Disables --

		public WarningLevel DeleteNonVirtualDtorWarningLevel => _inner.DeleteNonVirtualDtorWarningLevel;
		public WarningLevel EnumConversionWarningLevel => _inner.EnumConversionWarningLevel;
		public WarningLevel ClangBitfieldEnumConversion => _inner.ClangBitfieldEnumConversion;
		public WarningLevel EnumEnumConversionWarningLevel => _inner.EnumEnumConversionWarningLevel;
		public WarningLevel EnumFloatConversionWarningLevel => _inner.EnumFloatConversionWarningLevel;
		public WarningLevel AmbiguousReversedOperatorWarningLevel => _inner.AmbiguousReversedOperatorWarningLevel;
		public WarningLevel DeprecatedAnonEnumEnumConversionWarningLevel => _inner.DeprecatedAnonEnumEnumConversionWarningLevel;
		public WarningLevel DeprecatedVolatileWarningLevel => _inner.DeprecatedVolatileWarningLevel;
		public WarningLevel OrderedCompareFunctionPointers => _inner.OrderedCompareFunctionPointers;
		public WarningLevel UnusedWarningLevel => _inner.UnusedWarningLevel;
		public WarningLevel UnusedValueWarningLevel => _inner.UnusedValueWarningLevel;
		public WarningLevel UnknownPragmasWarningLevel => _inner.UnknownPragmasWarningLevel;
		public WarningLevel TautologicalCompareWarningLevel => _inner.TautologicalCompareWarningLevel;
		public WarningLevel SwitchWarningLevel => _inner.SwitchWarningLevel;
		public WarningLevel InvalidOffsetWarningLevel => _inner.InvalidOffsetWarningLevel;
		public WarningLevel InconsistentMissingOverrideWarningLevel => _inner.InconsistentMissingOverrideWarningLevel;
		public WarningLevel GNUStringLiteralOperatorTemplateWarningLevel => _inner.GNUStringLiteralOperatorTemplateWarningLevel;
		public WarningLevel UndefinedVarTemplateWarningLevel => _inner.UndefinedVarTemplateWarningLevel;
		public WarningLevel DeprecatedCopyWithUserProvidedCopyWarningLevel => _inner.DeprecatedCopyWithUserProvidedCopyWarningLevel;
		public WarningLevel BitwiseInsteadOfLogicalWarningLevel => _inner.BitwiseInsteadOfLogicalWarningLevel;
		public WarningLevel DeprecatedCopyWarningLevel => _inner.DeprecatedCopyWarningLevel;
		public WarningLevel InvalidUnevaluatedStringWarningLevel => _inner.InvalidUnevaluatedStringWarningLevel;
		public WarningLevel NaNInfinityDisabledWarningLevel => _inner.NaNInfinityDisabledWarningLevel;
		public WarningLevel CastFunctionTypeMismatchWarningLevel => _inner.CastFunctionTypeMismatchWarningLevel;
		public WarningLevel MissingTemplateArgListAfterTemplateWarningLevel => _inner.MissingTemplateArgListAfterTemplateWarningLevel;
		public WarningLevel LevelExtraQualificationWarningLevel => _inner.LevelExtraQualificationWarningLevel;
		public WarningLevel ProfileInstructWarningLevel => _inner.ProfileInstructWarningLevel;
		public WarningLevel BackendPluginWarningLevel => _inner.BackendPluginWarningLevel;
		public WarningLevel Shorten64To32WarningLevel => _inner.Shorten64To32WarningLevel;
		public WarningLevel DllExportExplicitInstantiationDeclWarningLevel => _inner.DllExportExplicitInstantiationDeclWarningLevel;
		public WarningLevel NonTrivialMemAccessWarningLevel => _inner.NonTrivialMemAccessWarningLevel;
		public WarningLevel DeprecationWarningLevel => _inner.DeprecationWarningLevel;
		public WarningLevel DeterministicWarningLevel => _inner.DeterministicWarningLevel;

		#endregion -- Clang Compiler Disables --

		#region -- Clang VC Compiler Disables --
		public WarningLevel MicrosoftGroupWarningLevel => _inner.MicrosoftGroupWarningLevel;
		public WarningLevel MSVCIncludeWarningLevel => _inner.MSVCIncludeWarningLevel;
		public WarningLevel PragmaPackWarningLevel => _inner.PragmaPackWarningLevel;
		public WarningLevel InlineNewDeleteWarningLevel => _inner.InlineNewDeleteWarningLevel;
		public WarningLevel ImplicitExceptionSpecMismatchWarningLevel => _inner.ImplicitExceptionSpecMismatchWarningLevel;
		public WarningLevel UndefinedBoolConversionWarningLevel => _inner.UndefinedBoolConversionWarningLevel;
		public WarningLevel DeprecatedWriteableStringsWarningLevel => _inner.DeprecatedWriteableStringsWarningLevel;
		public WarningLevel DeprecatedRegisterWarningLevel => _inner.DeprecatedRegisterWarningLevel;
		public WarningLevel LogicalOpParenthesesWarningLevel => _inner.LogicalOpParenthesesWarningLevel;
		public WarningLevel NullArithmeticWarningLevel => _inner.NullArithmeticWarningLevel;
		public WarningLevel ReturnTypeCLinkageWarningLevel => _inner.ReturnTypeCLinkageWarningLevel;
		public WarningLevel IgnoredAttributesWarningLevel => _inner.IgnoredAttributesWarningLevel;
		public WarningLevel UninitializedWarningLevel => _inner.UninitializedWarningLevel;
		public WarningLevel ReturnTypeWarningLevel => _inner.ReturnTypeWarningLevel;
		public WarningLevel UnusedParameterWarningLevel => _inner.UnusedParameterWarningLevel;
		public WarningLevel IgnoredQualifiersWarningLevel => _inner.IgnoredQualifiersWarningLevel;
		public WarningLevel ExpansionToDefined => _inner.ExpansionToDefined;
		public WarningLevel SignCompareWarningLevel => _inner.SignCompareWarningLevel;
		public WarningLevel MissingFieldInitializersWarningLevel => _inner.MissingFieldInitializersWarningLevel;
		public WarningLevel NonPortableIncludePathWarningLevel => _inner.NonPortableIncludePathWarningLevel;
		public WarningLevel InvalidTokenPasteWarningLevel => _inner.InvalidTokenPasteWarningLevel;
		public WarningLevel NullPointerArithmeticWarningLevel => _inner.NullPointerArithmeticWarningLevel;
		public WarningLevel ConstantLogicalOperandWarningLevel => _inner.ConstantLogicalOperandWarningLevel;
		public WarningLevel BitfieldEnumConversion => _inner.BitfieldEnumConversion;
		public WarningLevel NullPointerSubtractionWarningLevel => _inner.NullPointerSubtractionWarningLevel;
		public WarningLevel DanglingWarningLevel => _inner.DanglingWarningLevel;

		#region -- Duplicated Clang VC Compiler Disables --
		
		public WarningLevel MSVCSwitchEnumWarningLevel => _inner.MSVCSwitchEnumWarningLevel;
		public WarningLevel MSVCUnusedValueWarningLevel => _inner.MSVCUnusedValueWarningLevel;
		public WarningLevel MSVCDeprecationWarningLevel => _inner.MSVCDeprecationWarningLevel;

		#endregion -- Duplicated Clang VC Compiler Disables --

		#endregion -- Clang VC Compiler Disables --

		#region -- Intel Compiler Disables --

		public WarningLevel FormatWarningLevel => _inner.FormatWarningLevel;
		public WarningLevel ImplicitIntConversionWarningLevel => _inner.ImplicitIntConversionWarningLevel;
		public WarningLevel SingleBitfieldConstantConversionWarningLevel => _inner.SingleBitfieldConstantConversionWarningLevel;
		public WarningLevel UnusedCommandLineArgumentWarningLevel => _inner.UnusedCommandLineArgumentWarningLevel;
		public WarningLevel CommentWarningLevel => _inner.CommentWarningLevel;
		public WarningLevel RangeLoopConstructWarningLevel => _inner.RangeLoopConstructWarningLevel;
		public WarningLevel PragmaOnceOutsideHeaderWarningLevel => _inner.PragmaOnceOutsideHeaderWarningLevel;
		public WarningLevel LogicalNotParenthesesWarningLevel => _inner.LogicalNotParenthesesWarningLevel;
		public WarningLevel Cpp20ExtensionsWarningLevel => _inner.Cpp20ExtensionsWarningLevel;

		#endregion

		#endregion -- Compiler Warning Settings --

		#region -- Performance Warning Settings --

		public WarningLevel PCHPerformanceIssueWarningLevel => _inner.PCHPerformanceIssueWarningLevel;

		#endregion -- Performance Warning Settings --

		#region -- Rules Validation Warning Settings --

		public WarningLevel ModuleUnsupportedWarningLevel => _inner.ModuleUnsupportedWarningLevel;
		public WarningLevel PluginModuleUnsupportedWarningLevel => _inner.PluginModuleUnsupportedWarningLevel;
		public WarningLevel ModuleIncludePathWarningLevel => _inner.ModuleIncludePathWarningLevel;
		public WarningLevel ModuleIncludePrivateWarningLevel => _inner.ModuleIncludePrivateWarningLevel;
		public WarningLevel ModuleIncludeSubdirectoryWarningLevel => _inner.ModuleIncludeSubdirectoryWarningLevel;
		public WarningLevel DisablePluginsConflictWarningLevel => _inner.DisablePluginsConflictWarningLevel;
		public WarningLevel NonInlinedGenCppWarningLevel => _inner.NonInlinedGenCppWarningLevel;

		#endregion -- Rules Validation Warning Settings --
#pragma warning restore C1591
		#endregion
	}
}
