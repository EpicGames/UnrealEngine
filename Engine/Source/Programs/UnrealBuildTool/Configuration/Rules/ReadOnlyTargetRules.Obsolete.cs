// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace UnrealBuildTool
{
	public partial class ReadOnlyTargetRules
	{
#pragma warning disable CS1591
		[Obsolete("Deprecated in UE5.5 - No longer used in engine.")]
		public bool bUsesSteam => Inner.bUsesSteam;

		[Obsolete("Deprecated in UE5.1 - No longer used as Chaos is always enabled.")]
		public bool bCompileChaos => Inner.bCompileChaos;

		[Obsolete("Deprecated in UE5.1 - No longer used as Chaos is always enabled.")]
		public bool bUseChaos => Inner.bUseChaos;

		[Obsolete("Deprecated in UE5.1 - No longer used in engine.")]
		public bool bCustomSceneQueryStructure => Inner.bCustomSceneQueryStructure;

		[Obsolete("Deprecated in UE5.1 - No longer used as Chaos is always enabled.")]
		public bool bCompilePhysX => Inner.bCompilePhysX;

		[Obsolete("Deprecated in UE5.1 - No longer used as Chaos is always enabled.")]
		public bool bCompileAPEX => Inner.bCompileAPEX;

		[Obsolete("Deprecated in UE5.1 - No longer used as Chaos is always enabled.")]
		public bool bCompileNvCloth => Inner.bCompileNvCloth;

		[Obsolete("Deprecated in UE5.4 - No longer used.")]
		public bool bCompileIntelMetricsDiscovery => Inner.bCompileIntelMetricsDiscovery;

		[Obsolete("Deprecated in UE 5.6 - DirectXMath path was incompatible with LWC and unused since 5.0")]
		public bool bWithDirectXMath => Inner.bWithDirectXMath;

		[Obsolete("Deprecated in UE5.1 - Please use OptimizationLevel instead.")]
		public bool bCompileForSize => Inner.bCompileForSize;

		[Obsolete("Deprecated in UE5.6")]
		public bool bDetailedUnityFiles => Inner.bDetailedUnityFiles;

		[Obsolete("Deprecated in UE5.4 - Replace with ReadOnlyTargetRules.DebugInfo")]
		public bool bDisableDebugInfo => Inner.bDisableDebugInfo;

		[Obsolete("Deprecated in UE5.3 - No longer used as MallocProfiler is removed in favor of UnrealInsights.")]
		public bool bUseMallocProfiler => Inner.bUseMallocProfiler;

		[Obsolete("Deprecated in UE5.6 - Replace with ReadOnlyTargetRules.CppCompileWarningSettings.DeprecationWarningLevel")]
		public WarningLevel DeprecationWarningLevel => Inner.CppCompileWarningSettings.DeprecationWarningLevel;

		[Obsolete("Deprecated in UE5.6 - Replace with ReadOnlyTargetRules.CppCompileWarningSettings.ShadowVariableWarningLevel")]
		public WarningLevel ShadowVariableWarningLevel => Inner.CppCompileWarningSettings.ShadowVariableWarningLevel;

		[Obsolete("Deprecated in UE5.6 - Replace with ReadOnlyTargetRules.CppCompileWarningSettings.UnsafeTypeCastWarningLevel")]
		public WarningLevel UnsafeTypeCastWarningLevel => Inner.CppCompileWarningSettings.UnsafeTypeCastWarningLevel;

		[Obsolete("Deprecated in UE5.6 - Replace with ReadOnlyTargetRules.CppCompileWarningSettings.UndefinedIdentifierWarningLevel")]
		public WarningLevel UndefinedIdentifierWarningLevel => Inner.CppCompileWarningSettings.UndefinedIdentifierWarningLevel;

		[Obsolete("Deprecated in UE5.5 - Replace with ModuleRules.UndefinedIdentifierWarningLevel")]
		public bool bUndefinedIdentifierErrors => Inner.bUndefinedIdentifierErrors;

		[Obsolete("Deprecated in UE5.6 - Replace with ReadOnlyTargetRules.CppCompileWarningSettings.PCHPerformanceIssueWarningLevel")]
		public WarningLevel PCHPerformanceIssueWarningLevel => Inner.CppCompileWarningSettings.PCHPerformanceIssueWarningLevel;

		[Obsolete("Deprecated in UE5.6 - Replace with ReadOnlyTargetRules.CppCompileWarningSettings.ModuleIncludePathWarningLevel")]
		public WarningLevel ModuleIncludePathWarningLevel => Inner.CppCompileWarningSettings.ModuleIncludePathWarningLevel;

		[Obsolete("Deprecated in UE5.6 - Replace with ReadOnlyTargetRules.CppCompileWarningSettings.ModuleIncludePrivateWarningLevel")]
		public WarningLevel ModuleIncludePrivateWarningLevel => Inner.CppCompileWarningSettings.ModuleIncludePrivateWarningLevel;

		[Obsolete("Deprecated in UE5.6 - Replace with ReadOnlyTargetRules.CppCompileWarningSettings.ModuleIncludeSubdirectoryWarningLevel")]
		public WarningLevel ModuleIncludeSubdirectoryWarningLevel => Inner.CppCompileWarningSettings.ModuleIncludeSubdirectoryWarningLevel;
#pragma warning restore CS1591
	}
}