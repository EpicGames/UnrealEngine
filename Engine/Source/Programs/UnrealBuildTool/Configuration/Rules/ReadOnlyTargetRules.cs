// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// Read-only wrapper around an existing TargetRules instance. This exposes target settings to modules without letting them to modify the global environment.
	/// </summary>
	public partial class ReadOnlyTargetRules
	{
		/// <summary>
		/// The writeable TargetRules instance
		/// </summary>
		readonly TargetRules Inner;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="inner">The TargetRules instance to wrap around</param>
		public ReadOnlyTargetRules(TargetRules inner)
		{
			Inner = inner;
			AndroidPlatform = new ReadOnlyAndroidTargetRules(inner.AndroidPlatform);
			IOSPlatform = new ReadOnlyIOSTargetRules(inner.IOSPlatform);
			LinuxPlatform = new ReadOnlyLinuxTargetRules(inner.LinuxPlatform);
			MacPlatform = new ReadOnlyMacTargetRules(inner.MacPlatform);
			WindowsPlatform = new ReadOnlyWindowsTargetRules(inner.WindowsPlatform);
			CppCompileWarningSettings = new ReadOnlyCppCompileWarnings(inner.CppCompileWarningSettings);
		}

		/// <summary>
		/// Accessors for fields on the inner TargetRules instance
		/// </summary>
		#region Read-only accessor properties
#pragma warning disable CS1591

		public string Name => Inner.Name;

		public string OriginalName => Inner.OriginalName;

		internal FileReference File => Inner.File!;

		internal FileReference TargetSourceFile => Inner.TargetSourceFile!;

		internal IReadOnlySet<FileReference> TargetFiles => Inner.TargetFiles!;

		public object? AdditionalData => Inner.AdditionalData;

		public UnrealTargetPlatform Platform => Inner.Platform;

		public UnrealTargetConfiguration Configuration => Inner.Configuration;

		public UnrealArchitectures Architectures => Inner.Architectures;

		public UnrealIntermediateEnvironment IntermediateEnvironment => Inner.IntermediateEnvironment;

		public UnrealArch Architecture => Inner.Architecture;

		public FileReference? ProjectFile => Inner.ProjectFile;

		public ReadOnlyBuildVersion Version => Inner.Version;

		public TargetType Type => Inner.Type;

		/// <inheritdoc cref="TargetRules.Logger"/>
		public ILogger Logger => Inner.Logger;

		public BuildSettingsVersion DefaultBuildSettings => Inner.DefaultBuildSettings;

		public EngineIncludeOrderVersion? ForcedIncludeOrder => Inner.ForcedIncludeOrder;

		public EngineIncludeOrderVersion IncludeOrderVersion => Inner.IncludeOrderVersion;

		internal ConfigValueTracker ConfigValueTracker => Inner.ConfigValueTracker;

		public string? OutputFile => Inner.OutputFile;

		public bool bUsesCEF3 => Inner.bUsesCEF3;

		public bool bUsesSlate => Inner.bUsesSlate;

		public bool bUseStaticCRT => Inner.bUseStaticCRT;

		public bool bDebugBuildsActuallyUseDebugCRT => Inner.bDebugBuildsActuallyUseDebugCRT;

		public bool bLegalToDistributeBinary => Inner.bLegalToDistributeBinary;

		public UnrealTargetConfiguration UndecoratedConfiguration => Inner.UndecoratedConfiguration;

		public string DecoratedSeparator => Inner.DecoratedSeparator;

		public bool bAllowHotReload => Inner.bAllowHotReload;

		public bool bBuildAllModules => Inner.bBuildAllModules;

		public IReadOnlyList<string> AdditionalPlugins => Inner.AdditionalPlugins;

		public IReadOnlyList<string> EnablePlugins => Inner.EnablePlugins;

		public IReadOnlyList<string> DisablePlugins => Inner.DisablePlugins;

		public IReadOnlyList<string> OptionalPlugins => Inner.OptionalPlugins;

		public bool bAllowEnginePluginsEnabledByDefault => Inner.bAllowEnginePluginsEnabledByDefault;

		public WarningLevel DisablePluginsConflictWarningLevel => Inner.CppCompileWarningSettings.DisablePluginsConflictWarningLevel;

		public IReadOnlyList<string> BuildPlugins => Inner.BuildPlugins;

		public IReadOnlyList<string> InternalPluginDependencies => Inner.InternalPluginDependencies;
		public bool bRuntimeDependenciesComeFromBuildPlugins => Inner.bRuntimeDependenciesComeFromBuildPlugins;

		public string PakSigningKeysFile => Inner.PakSigningKeysFile;

		public string SolutionDirectory => Inner.SolutionDirectory;

		public string CustomConfig => Inner.CustomConfig;

		public bool? bBuildInSolutionByDefault => Inner.bBuildInSolutionByDefault;

		public string ExeBinariesSubFolder => Inner.ExeBinariesSubFolder;

		public EGeneratedCodeVersion GeneratedCodeVersion => Inner.GeneratedCodeVersion;
		public bool bEnableMeshEditor => Inner.bEnableMeshEditor;

		public bool bUseVerseBPVM => Inner.bUseVerseBPVM;

		public bool bUseVerseVM => !bUseVerseBPVM || Inner.GlobalDefinitions.Contains("WITH_VERSE_VM=1") || Inner.GlobalDefinitions.Contains("WITH_VERSE_VM=WITH_COREUOBJECT");

		public bool bUseAutoRTFMCompiler => Inner.bUseAutoRTFMCompiler;
		public bool bUseAutoRTFMVerifier => Inner.bUseAutoRTFMVerifier;
		public bool bAutoRTFMVerify => Inner.bAutoRTFMVerify;
		public bool bAutoRTFMClosedStaticLinkage => Inner.bAutoRTFMClosedStaticLinkage;
		public bool bAutoRTFMRedirectionInstrumentation => Inner.bAutoRTFMRedirectionInstrumentation;

		public bool bUseChaosMemoryTracking => Inner.bUseChaosMemoryTracking;

		public bool bCompileRewindDebuggerSupport => Inner.bCompileRewindDebuggerSupport;

		public bool bCompileChaosVisualDebuggerSupport => Inner.bCompileChaosVisualDebuggerSupport;
	
		public bool bCompileAsioSSLSupport => Inner.bCompileAsioSSLSupport;

		public bool bUseChaosChecked => Inner.bUseChaosChecked;

		public bool bCompileICU => Inner.bCompileICU;

		public bool bCompileCEF3 => Inner.bCompileCEF3;

		public bool bCompileISPC => Inner.bCompileISPC;

		public bool bMultiViewPerSlateWindowSupport => Inner.bMultiViewPerSlateWindowSupport;

		public bool bUseGameplayDebugger => Inner.bUseGameplayDebugger;

		public bool bUseGameplayDebuggerCore => Inner.bUseGameplayDebuggerCore;

		public bool bCompileIoStoreOnDemand => Inner.bCompileIoStoreOnDemand;

		public bool bTrackRHIResourceInfoForTest => Inner.bTrackRHIResourceInfoForTest;

		public bool bCompilePython => Inner.bCompilePython;

		public bool bEnableConfigSystem => Inner.bEnableConfigSystem;

		public bool bBuildEditor => Inner.bBuildEditor;

		public bool bBuildRequiresCookedData => Inner.bBuildRequiresCookedData;

		public bool bBuildWithEditorOnlyData => Inner.bBuildWithEditorOnlyData;

		public bool bBuildDeveloperTools => Inner.bBuildDeveloperTools;

		public bool bBuildTargetDeveloperTools => Inner.bBuildTargetDeveloperTools;

		public bool bForceBuildTargetPlatforms => Inner.bForceBuildTargetPlatforms;

		public bool bForceBuildShaderFormats => Inner.bForceBuildShaderFormats;

		public bool bNeedsExtraShaderFormats => Inner.bNeedsExtraShaderFormats;

		public bool bCompileCustomSQLitePlatform => Inner.bCompileCustomSQLitePlatform;

		public bool bUseCacheFreedOSAllocs => Inner.bUseCacheFreedOSAllocs;

		public bool bCompileAgainstEngine => Inner.bCompileAgainstEngine;

		public bool bCompileAgainstCoreUObject => Inner.bCompileAgainstCoreUObject;

		public bool bEnableConstInitUObject => Inner.bEnableConstInitUObject;

		public bool bCompileAgainstApplicationCore => Inner.bCompileAgainstApplicationCore;

		public bool bEnableTrace => Inner.bEnableTrace;

		public bool bForceEnableTrace => Inner.bForceEnableTrace;

		public bool bEnableTraceSecureConnection => Inner.bEnableTraceSecureConnection;

		public bool bCompileAgainstEditor => Inner.bCompileAgainstEditor;

		public bool bCompileRecast => Inner.bCompileRecast;

		public bool bCompileNavmeshSegmentLinks => Inner.bCompileNavmeshSegmentLinks;

		public bool bCompileNavmeshClusterLinks => Inner.bCompileNavmeshClusterLinks;

		public bool bCompileSpeedTree => Inner.bCompileSpeedTree;

		public bool bForceEnableExceptions => Inner.bForceEnableExceptions;

		public bool bForceEnableObjCExceptions => Inner.bForceEnableObjCExceptions;

		public bool bForceEnableRTTI => Inner.bForceEnableRTTI;

		public bool bEnablePrivateBuildInformation => Inner.bEnablePrivateBuildInformation;

		public bool bEnablePIE => Inner.bEnablePIE;

		public bool bEnableStackProtection => Inner.bEnableStackProtection;

		public bool bUseInlining => Inner.bUseInlining;

		public bool bWithClientCode => Inner.bWithClientCode;

		public bool bWithServerCode => Inner.bWithServerCode;

		public bool bFNameOutlineNumber => Inner.bFNameOutlineNumber;

		public bool bWithPushModel => Inner.bWithPushModel;

		public bool bExcludeParallelIrisSupport => Inner.bExcludeParallelIrisSupport;

		public bool bCompileWithStatsWithoutEngine => Inner.bCompileWithStatsWithoutEngine;

		public bool bCompileWithPluginSupport => Inner.bCompileWithPluginSupport;

		public bool bIncludePluginsForTargetPlatforms => Inner.bIncludePluginsForTargetPlatforms;

		public bool bCompileWithAccessibilitySupport => Inner.bCompileWithAccessibilitySupport;

		public bool bWithPerfCounters => Inner.bWithPerfCounters;

		public bool bWithLiveCoding => Inner.bWithLiveCoding;

		public bool bUseDebugLiveCodingConsole => Inner.bUseDebugLiveCodingConsole;

		public bool bWithFixedTimeStepSupport => Inner.bWithFixedTimeStepSupport;

		public bool bUseLoggingInShipping => Inner.bUseLoggingInShipping;

		public bool bUseConsoleInShipping => Inner.bUseConsoleInShipping;

		public bool bLoggingToMemoryEnabled => Inner.bLoggingToMemoryEnabled;

		public bool bUseLauncherChecks => Inner.bUseLauncherChecks;

		public bool bUseChecksInShipping => Inner.bUseChecksInShipping;

		public bool bAllowProfileGPUInTest => Inner.bAllowProfileGPUInTest;

		public bool bAllowProfileGPUInShipping => Inner.bAllowProfileGPUInShipping;

		public bool bTCHARIsUTF8 => Inner.bTCHARIsUTF8;

		public bool bUseEstimatedUtcNow => Inner.bUseEstimatedUtcNow;

		public bool bCompileFreeType => Inner.bCompileFreeType;

		public bool bUseExecCommandsInShipping => Inner.bUseExecCommandsInShipping;

		public OptimizationMode OptimizationLevel => Inner.OptimizationLevel;

		public bool bDefineForceInlineHintToInline => Inner.bDefineForceInlineHintToInline;

		public bool bUseSparseSetStrictMode => Inner.bUseSparseSetStrictMode;

		public bool bUseCompactSetAsDefault => Inner.bUseCompactSetAsDefault;

		public bool bUseAndroidJni => Inner.bUseAndroidJni;

		public FPSemanticsMode FPSemantics => Inner.FPSemantics;

		public bool bRetainFramePointers => Inner.bRetainFramePointers;

		public bool bForceCompileDevelopmentAutomationTests => Inner.bForceCompileDevelopmentAutomationTests;

		public bool bForceCompilePerformanceAutomationTests => Inner.bForceCompilePerformanceAutomationTests;

		public bool bForceDisableAutomationTests => Inner.bForceDisableAutomationTests;

		public bool bUseXGEController => Inner.bUseXGEController;

		public bool bEventDrivenLoader => Inner.bEventDrivenLoader;

		public PointerMemberBehavior? NativePointerMemberBehaviorOverride => Inner.NativePointerMemberBehaviorOverride;

		public bool bIWYU => Inner.bIWYU;

		public bool bIncludeHeaders => Inner.bIncludeHeaders;

		public bool bHeadersOnly => Inner.bHeadersOnly;

		public bool bEnforceIWYU => Inner.bEnforceIWYU;

		public bool bWarnAboutMonolithicHeadersIncluded => Inner.bWarnAboutMonolithicHeadersIncluded;

		public bool bHasExports => Inner.bHasExports;

		public bool bPrecompile => Inner.bPrecompile;

		public bool bEnableOSX109Support => Inner.bEnableOSX109Support;

		public bool bIsBuildingConsoleApplication => Inner.bIsBuildingConsoleApplication;

		public bool bBuildAdditionalConsoleApp => Inner.bBuildAdditionalConsoleApp;

		public bool bBuildConsoleAppOnly => Inner.bBuildConsoleAppOnly;

		public bool bDisableSymbolCache => Inner.bDisableSymbolCache;

		public bool bUseUnityBuild => Inner.bUseUnityBuild;

		public bool bForceUnityBuild => Inner.bForceUnityBuild;

		public bool bAlwaysUseUnityForGeneratedFiles => Inner.bAlwaysUseUnityForGeneratedFiles;

		public IReadOnlyList<string>? DisableMergingModuleAndGeneratedFilesInUnityFiles => Inner.DisableMergingModuleAndGeneratedFilesInUnityFiles;

		public bool bAdaptiveUnityDisablesOptimizations => Inner.bAdaptiveUnityDisablesOptimizations;

		public IReadOnlyList<UnrealTargetConfiguration> AdaptiveUnityDisablesOptimizationsConfigurations => Inner.AdaptiveUnityDisablesOptimizationsConfigurations;

		public bool bAdaptiveUnityDisablesPCH => Inner.bAdaptiveUnityDisablesPCH;

		public bool bAdaptiveUnityDisablesPCHForProject => Inner.bAdaptiveUnityDisablesPCHForProject;

		public bool bAdaptiveUnityCreatesDedicatedPCH => Inner.bAdaptiveUnityCreatesDedicatedPCH;

		public bool bAdaptiveUnityEnablesEditAndContinue => Inner.bAdaptiveUnityEnablesEditAndContinue;

		public bool bAdaptiveUnityCompilesHeaderFiles => Inner.bAdaptiveUnityCompilesHeaderFiles;

		public int MinGameModuleSourceFilesForUnityBuild => Inner.MinGameModuleSourceFilesForUnityBuild;

		public bool bValidateFormatStrings => Inner.bValidateFormatStrings;

		public WarningLevel DefaultWarningLevel => Inner.DefaultWarningLevel;

		public ReadOnlyCppCompileWarnings CppCompileWarningSettings { get; init; }

		public bool bShowIncludes => Inner.bShowIncludes;

		public bool bWarningsAsErrors => Inner.bWarningsAsErrors;

		public bool bUseFastMonoCalls => Inner.bUseFastMonoCalls;

		public int NumIncludedBytesPerUnityCPP => Inner.NumIncludedBytesPerUnityCPP;

		public bool bDisableModuleNumIncludedBytesPerUnityCPPOverride => Inner.bDisableModuleNumIncludedBytesPerUnityCPPOverride;

		public bool bStressTestUnity => Inner.bStressTestUnity;

		public DebugInfoMode DebugInfo => Inner.DebugInfo;

		public IReadOnlySet<string> DisableDebugInfoModules => Inner.DisableDebugInfoModules;

		public IReadOnlySet<string> DisableDebugInfoPlugins => Inner.DisableDebugInfoPlugins;

		public DebugInfoMode DebugInfoLineTablesOnly => Inner.DebugInfoLineTablesOnly;

		public IReadOnlySet<string> DebugInfoLineTablesOnlyModules => Inner.DebugInfoLineTablesOnlyModules;

		public IReadOnlySet<string> DebugInfoLineTablesOnlyPlugins => Inner.DebugInfoLineTablesOnlyPlugins;

		public DebugInfoMode DebugInfoNoInlineLineTables => Inner.DebugInfoNoInlineLineTables;
		public IReadOnlySet<string> DebugInfoInlineLineTablesModules => Inner.DebugInfoInlineLineTablesModules;
		public IReadOnlySet<string> DebugInfoInlineLineTablesPlugins => Inner.DebugInfoInlineLineTablesPlugins;
		
		public DebugInfoMode DebugInfoSimpleTemplateNames => Inner.DebugInfoSimpleTemplateNames;
		public IReadOnlySet<string> DebugInfoSimpleTemplateNamesModules => Inner.DebugInfoSimpleTemplateNamesModules;
		public IReadOnlySet<string> DebugInfoSimpleTemplateNamesPlugins => Inner.DebugInfoSimpleTemplateNamesPlugins;

		public bool bDisableDebugInfoForGeneratedCode => Inner.bDisableDebugInfoForGeneratedCode;

		public bool bOmitPCDebugInfoInDevelopment => Inner.bOmitPCDebugInfoInDevelopment;

		public bool bUsePDBFiles => Inner.bUsePDBFiles;

		public bool bUsePCHFiles => Inner.bUsePCHFiles;

		public bool bDeterministic => Inner.bDeterministic;

		public bool bUseVFS => Inner.bUseVFS;

		public string? CacheSalt => Inner.CacheSalt;

		public bool bChainPCHs => Inner.bChainPCHs;

		public bool bForceIncludePCHHeadersForGenCppFilesWhenPCHIsDisabled => Inner.bForceIncludePCHHeadersForGenCppFilesWhenPCHIsDisabled;

		public bool bPreprocessOnly => Inner.bPreprocessOnly;

		public bool bWithAssembly => Inner.bWithAssembly;

		public bool bAllowUbaCompression => Inner.bAllowUbaCompression;

		public StaticAnalyzer StaticAnalyzer => Inner.StaticAnalyzer;

		public StaticAnalyzerOutputType StaticAnalyzerOutputType => Inner.StaticAnalyzerOutputType;

		public StaticAnalyzerMode StaticAnalyzerMode => Inner.StaticAnalyzerMode;

		public int StaticAnalyzerPVSPrintLevel => Inner.StaticAnalyzerPVSPrintLevel;

		public IReadOnlyList<string> StaticAnalyzerPVSConfigs => Inner.StaticAnalyzerPVSConfigs;

		public bool bStaticAnalyzerProjectOnly => Inner.bStaticAnalyzerProjectOnly;

		public bool bStaticAnalyzerIncludeGenerated => Inner.bStaticAnalyzerIncludeGenerated;

		public int MinFilesUsingPrecompiledHeader => Inner.MinFilesUsingPrecompiledHeader;

		public bool bForcePrecompiledHeaderForGameModules => Inner.bForcePrecompiledHeaderForGameModules;

		public bool bUseIncrementalLinking => Inner.bUseIncrementalLinking;

		public bool bAllowLTCG => Inner.bAllowLTCG;

		public bool bPreferThinLTO => Inner.bPreferThinLTO;

		public bool bThinLTODistributed => Inner.bThinLTODistributed;

		public bool bCompressDebugFile => Inner.bCompressDebugFile;

		public bool bPGOProfile => Inner.bPGOProfile;

		public bool bPGOOptimize => Inner.bPGOOptimize;

		public bool bSupportEditAndContinue => Inner.bSupportEditAndContinue;

		public bool bCodeCoverage => Inner.bCodeCoverage;

		public bool bOmitFramePointers => Inner.bOmitFramePointers;

		public bool bEnableCppModules => Inner.bEnableCppModules;

		public bool bEnableProcessPriorityControl => Inner.bEnableProcessPriorityControl;

		public bool bShaderCompilerWorkerTrace => Inner.bShaderCompilerWorkerTrace;

		public bool bUseSharedPCHs => Inner.bUseSharedPCHs;

		public bool bUseShippingPhysXLibraries => Inner.bUseShippingPhysXLibraries;

		public bool bUseCheckedPhysXLibraries => Inner.bUseCheckedPhysXLibraries;

		public bool bCheckLicenseViolations => Inner.bCheckLicenseViolations;

		public bool bBreakBuildOnLicenseViolation => Inner.bBreakBuildOnLicenseViolation;

		[Obsolete("Deprecated in UE5.7 - No longer recommended")]
		public bool? bUseFastPDBLinking => Inner.bUseFastPDBLinking;

		public bool bCreateMapFile => Inner.bCreateMapFile;

		public bool bAllowRuntimeSymbolFiles => Inner.bAllowRuntimeSymbolFiles;

		public string? PackagePath => Inner.PackagePath;

		public string? CrashDiagnosticDirectory => Inner.CrashDiagnosticDirectory;

		public string? ThinLTOCacheDirectory => Inner.ThinLTOCacheDirectory;

		public string? ThinLTOCachePruningArguments => Inner.ThinLTOCachePruningArguments;

		public string? BundleVersion => Inner.BundleVersion;

		public bool bDeployAfterCompile => Inner.bDeployAfterCompile;

		public bool bAllowRemotelyCompiledPCHs => Inner.bAllowRemotelyCompiledPCHs;

		public bool bUseHeaderUnitsForPch => Inner.bUseHeaderUnitsForPch;

		public bool bCheckSystemHeadersForModification => Inner.bCheckSystemHeadersForModification;

		public bool bDisableLinking => Inner.bDisableLinking;

		public bool bIgnoreBuildOutputs => Inner.bIgnoreBuildOutputs;

		public bool bFormalBuild => Inner.bFormalBuild;

		public bool bUseAdaptiveUnityBuild => Inner.bUseAdaptiveUnityBuild;

		public bool bFlushBuildDirOnRemoteMac => Inner.bFlushBuildDirOnRemoteMac;

		public bool bPrintToolChainTimingInfo => Inner.bPrintToolChainTimingInfo;

		public bool bParseTimingInfoForTracing => Inner.bParseTimingInfoForTracing;

		public bool bPublicSymbolsByDefault => Inner.bPublicSymbolsByDefault;

		public bool bIdenticalCodeFolding => Inner.bIdenticalCodeFolding;

		public string? ToolChainName => Inner.ToolChainName;

		public float MSVCCompileActionWeight => Inner.MSVCCompileActionWeight;

		public float ClangCompileActionWeight => Inner.ClangCompileActionWeight;

		public bool bLegacyPublicIncludePaths => Inner.bLegacyPublicIncludePaths;

		public bool bLegacyParentIncludePaths => Inner.bLegacyParentIncludePaths;

		public bool bDisableEngineDeprecations => Inner.bDisableEngineDeprecations;

		public CppStandardVersion CppStandardEngine => Inner.CppStandardEngine;

		public CppStandardVersion CppStandard => Inner.CppStandard;

		public CStandardVersion CStandard => Inner.CStandard;

		public MinimumCpuArchitectureX64 MinCpuArchX64 => Inner.MinCpuArchX64;

		public bool bMinCpuArchAPX => Inner.bMinCpuArchAPX;

		public Arm64TargetCpuType MinArm64CpuTarget => Inner.MinArm64CpuTarget;

		internal bool bNoManifestChanges => Inner.bNoManifestChanges;

		public string? BuildVersion => Inner.BuildVersion;

		public TargetLinkType LinkType => Inner.LinkType;

		public bool bStripExports => Inner.bStripExports;

		public bool bMergeModules => Inner.bMergeModules;

		public bool bCheckDelegateLifetime => Inner.bCheckDelegateLifetime;

		public IReadOnlySet<string> MergeAdditionalCommonModules => Inner.MergeAdditionalCommonModules;

		public IReadOnlyList<string> MergePlugins => Inner.MergePlugins;

		public IReadOnlyList<string> MergePluginsIgnored => Inner.MergePluginsIgnored;

		public IReadOnlyList<string> MergePluginsExcluded => Inner.MergePluginsExcluded;

		public IReadOnlyDictionary<string, IEnumerable<string>> MergePluginsShared => Inner.MergePluginsShared.ToDictionary(k => k.Key, v => v.Value.AsEnumerable());

		public IReadOnlyList<string> MergePluginsLaunch => Inner.MergePluginsLaunch;

		public bool MergeOnlyGameFeatures => Inner.bMergeOnlyGameFeatures;

		public bool bReportModuleMergingData => Inner.bReportModuleMergingData;

		public IReadOnlyList<string> GlobalDefinitions => Inner.GlobalDefinitions.AsReadOnly();

		public IReadOnlyList<string> ProjectDefinitions => Inner.ProjectDefinitions.AsReadOnly();

		public IReadOnlyDictionary<string, string> BuildSettingsVariables => Inner.BuildSettingsVariables.AsReadOnly();

		public ReadOnlyAdditionalProperties AdditionalProperties => Inner.AdditionalProperties;

		public string? LaunchModuleName => Inner.LaunchModuleName;

		public string? ExportPublicHeader => Inner.ExportPublicHeader;

		public IReadOnlyList<string> ExtraModuleNames => Inner.ExtraModuleNames.AsReadOnly();

		public IReadOnlyList<FileReference> ManifestFileNames => Inner.ManifestFileNames.AsReadOnly();

		public IReadOnlyList<FileReference> DependencyListFileNames => Inner.DependencyListFileNames.AsReadOnly();

		public TargetBuildEnvironment BuildEnvironment => Inner.BuildEnvironment;

		public bool bOverrideBuildEnvironment => Inner.bOverrideBuildEnvironment;

		public IReadOnlyList<TargetInfo> PreBuildTargets => Inner.PreBuildTargets;

		public IReadOnlyList<string> PreBuildSteps => Inner.PreBuildSteps;

		public IReadOnlyList<string> PostBuildSteps => Inner.PostBuildSteps;

		public IReadOnlyList<string> AdditionalBuildProducts => Inner.AdditionalBuildProducts;

		public string? AdditionalCompilerArguments => Inner.AdditionalCompilerArguments;

		public string? AdditionalLinkerArguments => Inner.AdditionalLinkerArguments;

		public double MemoryPerActionGB => Inner.MemoryPerActionGB;

		public string? GeneratedProjectName => Inner.GeneratedProjectName;

		public ReadOnlyAndroidTargetRules AndroidPlatform { get; init; }

		public ReadOnlyLinuxTargetRules LinuxPlatform { get; init; }

		public ReadOnlyIOSTargetRules IOSPlatform { get; init; }

		public ReadOnlyMacTargetRules MacPlatform { get; init; }

		public ReadOnlyWindowsTargetRules WindowsPlatform { get; init; }

		public bool bShouldCompileAsDLL => Inner.bShouldCompileAsDLL;

		public bool bGenerateProjectFiles => Inner.bGenerateProjectFiles;

		public bool bIsEngineInstalled => Inner.bIsEngineInstalled;

		public IReadOnlyList<string>? DisableUnityBuildForModules => Inner.DisableUnityBuildForModules;

		public IReadOnlyList<string>? EnableOptimizeCodeForModules => Inner.EnableOptimizeCodeForModules;

		public IReadOnlyList<string>? DisableOptimizeCodeForModules => Inner.DisableOptimizeCodeForModules;

		public IReadOnlyList<string>? OptimizeForSizeModules => Inner.OptimizeForSizeModules;

		public IReadOnlyList<string>? OptimizeForSizeAndSpeedModules => Inner.OptimizeForSizeAndSpeedModules;

		public IReadOnlyList<UnrealTargetPlatform>? OptedInModulePlatforms => Inner.OptedInModulePlatforms;

		public TestTargetRules? InnerTestTargetRules => Inner as TestTargetRules;

#pragma warning restore C1591
		#endregion

		/// <summary>
		/// Provide access to the RelativeEnginePath property for code referencing ModuleRules.BuildConfiguration.
		/// </summary>
#pragma warning disable CA1822 // Mark members as static - used in module build files, leave as-is
		public string RelativeEnginePath => Unreal.EngineDirectory.MakeRelativeTo(DirectoryReference.GetCurrentDirectory()) + Path.DirectorySeparatorChar;
#pragma warning restore CA1822

		/// <summary>
		/// Provide access to the UEThirdPartySourceDirectory property for code referencing ModuleRules.UEBuildConfiguration.
		/// </summary>
#pragma warning disable CA1822 // Mark members as static - used in module build files, leave as-is
		public string UEThirdPartySourceDirectory => "ThirdParty/";
#pragma warning restore CA1822

		/// <summary>
		/// Provide access to the UEThirdPartyBinariesDirectory property for code referencing ModuleRules.UEBuildConfiguration.
		/// </summary>
#pragma warning disable CA1822 // Mark members as static - used in module build files, leave as-is
		public string UEThirdPartyBinariesDirectory => "../Binaries/ThirdParty/";
#pragma warning restore CA1822

		/// <summary>
		/// Whether this is a low level tests target.
		/// </summary>
		public bool IsTestTarget => Inner.IsTestTarget;

		/// <summary>
		/// The selection of test modules to be exclusively included in compilation (no other modules tests will be included). 
		/// </summary>
		public List<string> SelectedTestModuleNames => Inner.SelectedTestModuleNames;

		/// <summary>
		/// Whether this is a test target explicitly defined.
		/// Explicitley defined test targets always inherit from TestTargetRules and define their own tests.
		/// Implicit test targets are created from existing targets when building with -Mode=Test and they include tests from all dependencies.
		/// </summary>
		public bool ExplicitTestsTarget => Inner.ExplicitTestsTarget;

		/// <summary>
		/// Controls the value of WITH_LOW_LEVEL_TESTS that dictates whether module-specific low level tests are compiled in or not.
		/// </summary>
		public bool WithLowLevelTests => Inner.WithLowLevelTests;

		/// <summary>
		/// Get the platforms this target supports
		/// </summary>
		public IEnumerable<UnrealTargetPlatform> SupportedPlatforms => Inner.GetSupportedPlatforms();

		/// <summary>
		/// Get the configurations this target supports
		/// </summary>
		public IEnumerable<UnrealTargetConfiguration> SupportedConfigurations => Inner.GetSupportedConfigurations();

		/// <summary>
		/// Checks if current platform is part of a given platform group
		/// </summary>
		/// <param name="group">The platform group to check</param>
		/// <returns>True if current platform is part of a platform group</returns>
		public bool IsInPlatformGroup(UnrealPlatformGroup group) => UEBuildPlatform.IsPlatformInGroup(Platform, group);

		/// <summary>
		/// Gets diagnostic messages about default settings which have changed in newer versions of the engine
		/// </summary>
		/// <param name="diagnostics">List of messages to be appended to</param>
		internal void GetBuildSettingsInfo(List<string> diagnostics) => Inner.GetBuildSettingsInfo(diagnostics);

		public bool IsPlatformOptedIn(UnrealTargetPlatform platform) => Inner.OptedInModulePlatforms == null || Inner.OptedInModulePlatforms.Contains(platform);

		/// <summary>
		/// Checks if a plugin should be programmatically allowed in the build
		/// </summary>
		/// <returns>true if the plugin is allowed</returns>
		public bool ShouldIgnorePluginDependency(PluginInfo parentInfo, PluginReferenceDescriptor childDescriptor) => Inner.ShouldIgnorePluginDependency(parentInfo, childDescriptor);

		/// <summary>
		/// Report merged modules structure to the target so that the game may use this information.
		/// </summary>
		public virtual void ReportMergedModules(Dictionary<string, IEnumerable<ModuleRules>> ModuleGroups, Dictionary<string, Dictionary<string, List<string>>> RootPluginUsage)
			=> Inner.ReportMergedModules(ModuleGroups, RootPluginUsage);

		/// <summary>
		/// Determines if the automation tests should be compiled based on the current configuration and optional forced settings.
		/// If either the development tests or performance tests are compiled (unless explicitly disabled), this property returns true.
		/// </summary>
		private bool? _bWithAutomationTestsPrivate = null;
		public bool WithAutomationTests
		{
			get
			{
				if (!_bWithAutomationTestsPrivate.HasValue)
				{
					bool bCompileDevTests = (Configuration != UnrealTargetConfiguration.Test && Configuration != UnrealTargetConfiguration.Shipping);
					bool bCompilePerfTests = bCompileDevTests;

					if (bForceCompileDevelopmentAutomationTests)
					{
						bCompileDevTests = true;
					}
					if (bForceCompilePerformanceAutomationTests)
					{
						bCompilePerfTests = true;
					}
					if (bForceDisableAutomationTests)
					{
						bCompileDevTests = bCompilePerfTests = false;
					}

					_bWithAutomationTestsPrivate = bCompileDevTests || bCompilePerfTests;
				}

				return _bWithAutomationTestsPrivate.Value;
			}
		}

		public StaticAllocatorType StaticAllocator => Inner.StaticAllocator;
	}
}
