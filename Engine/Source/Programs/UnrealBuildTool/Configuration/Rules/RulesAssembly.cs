// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using System.Linq;
using System.Reflection;
using System.Runtime.CompilerServices;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Logging.Abstractions;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// Controls how the target and SDK are validated in CreateTargetRulesInstance
	/// </summary>
	public enum TargetRulesValidationOptions
	{
		/// <summary>
		///  This will perform full target validation (allows the platform to update the target to make sure it's valid), and
		///  SDK validation (makes sure SDK overrides are not conflicting).
		///  This is the standard, default mode.
		/// </summary>
		ValidateTargetAndSDK,

		/// <summary>
		/// This will validate the target only (see ValidateTargetAndSDK)
		/// </summary>
		ValidateTargetOnly,

		/// <summary>
		/// This will validate the SDK only (see ValidateTargetAndSDK)
		/// </summary>
		ValidateSDKOnly,

		/// <summary>
		/// This will perform neither of the validations
		/// </summary>
		ValidateNothing
	}

	/// <summary>
	/// Stores information about a compiled rules assembly and the types it contains
	/// </summary>
	public class RulesAssembly
	{
		/// <summary>
		/// Outers scope for items created by this assembly. Used for chaining assemblies together.
		/// </summary>
		internal readonly RulesScope Scope;

		/// <summary>
		/// The compiled assembly
		/// </summary>
		private readonly Assembly? CompiledAssembly;

		/// <summary>
		/// Returns the simple name of the assembly e.g. "UE5ProgramRules"
		/// </summary>
		/// <returns></returns>
		public string? GetSimpleAssemblyName()
		{
			if (CompiledAssembly != null)
			{
				return CompiledAssembly.GetName().Name;
			}
			else
			{
				return null;
			}
		}

		/// <summary>
		/// Returns all the types contained in the compiled rules assembly
		/// </summary>
		/// <returns></returns>
		public IEnumerable<Type> GetTypes()
		{
			if (CompiledAssembly != null)
			{
				return CompiledAssembly.GetTypes();
			}
			return Enumerable.Empty<Type>();
		}

		/// <summary>
		/// The base directories for this assembly
		/// </summary>
		private readonly IReadOnlyList<DirectoryReference> BaseDirs;

		/// <summary>
		/// All the plugins included in this assembly
		/// </summary>
		private readonly IReadOnlyList<PluginInfo> Plugins;

		/// <summary>
		/// Maps module names to their actual xxx.Module.cs file on disk
		/// </summary>
		private readonly IReadOnlyDictionary<string, FileReference> ModuleNameToModuleFile;

		/// <summary>
		/// Maps target names to their actual xxx.Target.cs file on disk
		/// </summary>
		private readonly IReadOnlyDictionary<string, FileReference> TargetNameToTargetFile;

		/// <summary>
		/// Mapping from module file to its context.
		/// </summary>
		private readonly IReadOnlyDictionary<FileReference, ModuleRulesContext> ModuleFileToContext;

		/// <summary>
		/// Whether this assembly contains engine modules. Used to set default values for bTreatAsEngineModule.
		/// </summary>
		private readonly bool bContainsEngineModules;

		/// <summary>
		/// Whether to use backwards compatible default settings for module and target rules. This is enabled by default for game projects to support a simpler migration path, but
		/// is disabled for engine modules.
		/// </summary>
		private readonly BuildSettingsVersion? DefaultBuildSettings;

		/// <summary>
		/// Whether the modules and targets in this assembly are read-only
		/// </summary>
		private readonly bool bReadOnly;

		/// <summary>
		/// The parent rules assembly that this assembly inherits. Game assemblies inherit the engine assembly, and the engine assembly inherits nothing.
		/// </summary>
		public RulesAssembly? Parent { get; }

		/// <summary>
		/// The set of files that were compiled to create this assembly
		/// </summary>
		public IEnumerable<FileReference>? AssemblySourceFiles { get; }

		/// <summary>
		/// Any preprocessor defines that were set when this assembly was created
		/// </summary>
		public IEnumerable<string>? PreprocessorDefines { get; }

		/// <summary>
		/// Constructor. Compiles a rules assembly from the given source files.
		/// </summary>
		/// <param name="Scope">The scope of items created by this assembly</param>
		/// <param name="BaseDirs">The base directories for this assembly</param>
		/// <param name="Plugins">All the plugins included in this assembly</param>
		/// <param name="ModuleFileToContext">List of module files to compile</param>
		/// <param name="TargetFiles">List of target files to compile</param>
		/// <param name="AssemblyFileName">The output path for the compiled assembly</param>
		/// <param name="bContainsEngineModules">Whether this assembly contains engine modules. Used to initialize the default value for ModuleRules.bTreatAsEngineModule.</param>
		/// <param name="DefaultBuildSettings">Optional override for the default build settings version for modules created from this assembly.</param>
		/// <param name="bReadOnly">Whether the modules and targets in this assembly are installed, and should be created with the bUsePrecompiled flag set</param> 
		/// <param name="bSkipCompile">Whether to skip compiling this assembly</param>
		/// <param name="bForceCompile">Whether to always compile this assembly</param>
		/// <param name="Parent">The parent rules assembly</param>
		/// <param name="Logger"></param>
		internal RulesAssembly(RulesScope Scope, IReadOnlyList<DirectoryReference> BaseDirs, IReadOnlyList<PluginInfo> Plugins, IReadOnlyDictionary<FileReference, ModuleRulesContext> ModuleFileToContext, IReadOnlyList<FileReference> TargetFiles, FileReference AssemblyFileName, bool bContainsEngineModules, BuildSettingsVersion? DefaultBuildSettings, bool bReadOnly, bool bSkipCompile, bool bForceCompile, RulesAssembly? Parent, ILogger Logger)
		{
			this.Scope = Scope;
			this.BaseDirs = BaseDirs;
			this.Plugins = Plugins;
			this.ModuleFileToContext = ModuleFileToContext;
			this.bContainsEngineModules = bContainsEngineModules;
			this.DefaultBuildSettings = DefaultBuildSettings;
			this.bReadOnly = bReadOnly;
			this.Parent = Parent;

			// Find all the source files
			HashSet<FileReference> AssemblySourceFilesHashSet = new HashSet<FileReference>();
			AssemblySourceFilesHashSet.UnionWith(ModuleFileToContext.Keys);
			AssemblySourceFilesHashSet.UnionWith(TargetFiles);
			AssemblySourceFiles = AssemblySourceFilesHashSet;

			// Compile the assembly
			if (AssemblySourceFiles.Any())
			{
				// Add the parent assemblies as references so they can be used
				List<string>? ReferencedAssembies = null;
				RulesAssembly? CurrentParent = Parent;
				while (CurrentParent != null && CurrentParent.CompiledAssembly != null)
				{
					ReferencedAssembies ??= new List<string>();

					ReferencedAssembies.Add(CurrentParent.CompiledAssembly.Location);
					CurrentParent = CurrentParent.Parent;
				}

				PreprocessorDefines = GetPreprocessorDefinitions();
				CompiledAssembly = DynamicCompilation.CompileAndLoadAssembly(AssemblyFileName, AssemblySourceFiles, Logger, ReferencedAssembies: ReferencedAssembies, PreprocessorDefines: PreprocessorDefines, DoNotCompile: bSkipCompile, ForceCompile: bForceCompile);
			}

			// Setup the module map
			Dictionary<string, FileReference> ModuleNameToModuleFileDict = new Dictionary<string, FileReference>(StringComparer.InvariantCultureIgnoreCase);
			ModuleNameToModuleFile = ModuleNameToModuleFileDict;
			foreach (FileReference ModuleFile in ModuleFileToContext.Keys)
			{
				string ModuleName = ModuleFile.GetFileNameWithoutAnyExtensions();
				if (!ModuleNameToModuleFile.ContainsKey(ModuleName))
				{
					ModuleNameToModuleFileDict.Add(ModuleName, ModuleFile);
				}
			}

			// Setup the target map
			Dictionary<string, FileReference> TargetNameToTargetFileDict = new Dictionary<string, FileReference>(StringComparer.InvariantCultureIgnoreCase);
			TargetNameToTargetFile = TargetNameToTargetFileDict;
			foreach (FileReference TargetFile in TargetFiles)
			{
				string TargetName = TargetFile.GetFileNameWithoutAnyExtensions();
				if (!TargetNameToTargetFile.ContainsKey(TargetName))
				{
					TargetNameToTargetFileDict.Add(TargetName, TargetFile);
				}
			}

			// Write any deprecation warnings for methods overridden from a base with the [ObsoleteOverride] attribute. Unlike the [Obsolete] attribute, this ensures the message
			// is given because the method is implemented, not because it's called.
			if (CompiledAssembly != null)
			{
				Task.Run(() =>
				{
					//using ITimelineEvent timelineScope = Timeline.ScopeEvent("PrejitRules");
					ParallelQueue queue = new();
					foreach (Type CompiledType in CompiledAssembly.GetTypes())
					{
						if (CompiledType.BaseType == typeof(ModuleRules))
						{
							foreach (ConstructorInfo Constructor in CompiledType.GetConstructors())
							{
								ParameterInfo[] parameters = Constructor.GetParameters();
								if (parameters.Length > 0)
								{
									if (parameters[0].ParameterType == typeof(ReadOnlyTargetRules))
									{
										queue.Enqueue(() => RuntimeHelpers.PrepareMethod(Constructor.MethodHandle));
									}
								}
							}
						}
					}
					queue.Drain();
					//timelineScope.Finish();

					// Write out warnings as well
					foreach (Type CompiledType in CompiledAssembly.GetTypes())
					{
						foreach (MethodInfo Method in CompiledType.GetMethods(BindingFlags.Instance | BindingFlags.Public | BindingFlags.DeclaredOnly))
						{
							ObsoleteOverrideAttribute? Attribute = Method.GetCustomAttribute<ObsoleteOverrideAttribute>(true);
							if (Attribute != null)
							{
								FileReference? Location;
								if (!TryGetFileNameFromType(CompiledType, out Location))
								{
									Location = new FileReference(CompiledAssembly.Location);
								}
								Logger.LogWarning("{Location}: warning: {AttributeMessage}", Location, Attribute.Message);
							}
						}
						if (CompiledType.BaseType == typeof(ModuleRules))
						{
							ConstructorInfo? Constructor = CompiledType.GetConstructor([typeof(TargetInfo)]);
							if (Constructor != null)
							{
								FileReference? Location;
								if (!TryGetFileNameFromType(CompiledType, out Location))
								{
									Location = new FileReference(CompiledAssembly.Location);
								}
								Logger.LogWarning("{Location}: warning: Module constructors should take a ReadOnlyTargetRules argument (rather than a TargetInfo argument) and pass it to the base class constructor from 4.15 onwards. Please update the method signature.", Location);
							}
						}
					}
				});
			}
		}

		/// <summary>
		/// Determines if the given path is read-only
		/// </summary>
		/// <param name="Location">The location to check</param>
		/// <returns>True if the path is read-only, false otherwise</returns>
		public bool IsReadOnly(FileSystemReference Location)
		{
			if (BaseDirs.Any(x => Location.IsUnderDirectory(x)))
			{
				return bReadOnly;
			}
			else if (Parent != null)
			{
				return Parent.IsReadOnly(Location);
			}
			else
			{
				return false;
			}
		}

		/// <summary>
		/// Finds all the preprocessor definitions that need to be set for the current engine.
		/// </summary>
		/// <returns>List of preprocessor definitions that should be set</returns>
		public static List<string> GetPreprocessorDefinitions()
		{
			List<string> PreprocessorDefines = new List<string>();
			PreprocessorDefines.Add("WITH_FORWARDED_MODULE_RULES_CTOR");
			PreprocessorDefines.Add("WITH_FORWARDED_TARGET_RULES_CTOR");

			// Define macros for the Unreal engine version, starting with 4.17
			// Assumes the current MajorVersion is 5
			BuildVersion? Version;
			if (BuildVersion.TryRead(BuildVersion.GetDefaultFileName(), out Version))
			{
				for (int MinorVersion = 17; MinorVersion <= 30; MinorVersion++)
				{
					PreprocessorDefines.Add(String.Format("UE_4_{0}_OR_LATER", MinorVersion));
				}
				for (int MinorVersion = 0; MinorVersion <= Version.MinorVersion; MinorVersion++)
				{
					PreprocessorDefines.Add(String.Format("UE_5_{0}_OR_LATER", MinorVersion));
				}
			}
			return PreprocessorDefines;
		}

		/// <summary>
		/// Fills a list with all the module names in this assembly (or its parent)
		/// </summary>
		/// <param name="ModuleNames">List to receive the module names</param>
		public void GetAllModuleNames(List<string> ModuleNames)
		{
			Parent?.GetAllModuleNames(ModuleNames);
			if (CompiledAssembly != null)
			{
				ModuleNames.AddRange(CompiledAssembly.GetTypes().Where(x => x.IsClass && x.IsSubclassOf(typeof(ModuleRules)) && ModuleNameToModuleFile.ContainsKey(x.Name)).Select(x => x.Name));
			}
		}

		/// <summary>
		/// Fills a list with all the target names in this assembly
		/// </summary>
		/// <param name="TargetNames">List to receive the target names</param>
		/// <param name="bIncludeParentAssembly">Whether to include targets in the parent assembly</param>
		public void GetAllTargetNames(List<string> TargetNames, bool bIncludeParentAssembly)
		{
			if (Parent != null && bIncludeParentAssembly)
			{
				Parent.GetAllTargetNames(TargetNames, true);
			}
			TargetNames.AddRange(TargetNameToTargetFile.Keys);
		}

		/// <summary>
		/// Tries to get the filename that declared the given type
		/// </summary>
		/// <param name="ExistingType"></param>
		/// <param name="File"></param>
		/// <returns>True if the type was found, false otherwise</returns>
		public bool TryGetFileNameFromType(Type ExistingType, [NotNullWhen(true)] out FileReference? File)
		{
			if (ExistingType.Assembly == CompiledAssembly)
			{
				string Name = ExistingType.Name;
				if (ModuleNameToModuleFile.TryGetValue(Name, out File))
				{
					return true;
				}

				string NameWithoutTarget = Name;
				if (NameWithoutTarget.EndsWith("Target", StringComparison.Ordinal))
				{
					NameWithoutTarget = NameWithoutTarget.Substring(0, NameWithoutTarget.Length - 6);
				}
				if (TargetNameToTargetFile.TryGetValue(NameWithoutTarget, out File))
				{
					return true;
				}
			}
			else
			{
				if (Parent != null && Parent.TryGetFileNameFromType(ExistingType, out File))
				{
					return true;
				}
			}

			File = null;
			return false;
		}

		/// <summary>
		/// Gets the source file containing rules for the given module
		/// </summary>
		/// <param name="ModuleName">The name of the module</param>
		/// <returns>The filename containing rules for this module, or an empty string if not found</returns>
		public FileReference? GetModuleFileName(string ModuleName)
		{
			FileReference? ModuleFile;
			if (ModuleNameToModuleFile.TryGetValue(ModuleName, out ModuleFile))
			{
				return ModuleFile;
			}
			else
			{
				return Parent?.GetModuleFileName(ModuleName);
			}
		}

		/// <summary>
		/// Gets the type defining rules for the given module
		/// </summary>
		/// <param name="ModuleName">The name of the module</param>
		/// <returns>The rules type for this module, or null if not found</returns>
		public Type? GetModuleRulesType(string ModuleName)
		{
			if (ModuleNameToModuleFile.ContainsKey(ModuleName))
			{
				return GetModuleRulesTypeInternal(ModuleName);
			}
			else
			{
				return Parent?.GetModuleRulesType(ModuleName);
			}
		}

		private readonly ConcurrentDictionary<string, Type?> _moduleNameToType = [];

		/// <summary>
		/// Gets the type defining rules for the given module within this assembly
		/// </summary>
		/// <param name="ModuleName">The name of the module</param>
		/// <returns>The rules type for this module, or null if not found</returns>
		private Type? GetModuleRulesTypeInternal(string ModuleName)
		{
			return _moduleNameToType.GetOrAdd(ModuleName, ModuleName =>
			{
				// The build module must define a type named 'Rules' that derives from our 'ModuleRules' type.  
				Type? RulesObjectType = CompiledAssembly?.GetType(ModuleName);
				if (RulesObjectType == null)
				{
					// Temporary hack to avoid System namespace collisions
					// @todo projectfiles: Make rules assemblies require namespaces.
					RulesObjectType = CompiledAssembly?.GetType("UnrealBuildTool.Rules." + ModuleName);
				}
				return RulesObjectType;
			});
		}

		/// <summary>
		/// Gets the source file containing rules for the given target
		/// </summary>
		/// <param name="TargetName">The name of the target</param>
		/// <returns>The filename containing rules for this target, or an empty string if not found</returns>
		public FileReference? GetTargetFileName(string TargetName)
		{
			FileReference? TargetFile;
			if (TargetNameToTargetFile.TryGetValue(TargetName, out TargetFile))
			{
				return TargetFile;
			}
			else
			{
				return Parent?.GetTargetFileName(TargetName);
			}
		}

		/// <summary>
		/// Creates an instance of a module rules descriptor object for the specified module name
		/// </summary>
		/// <param name="ModuleName">Name of the module</param>
		/// <param name="Target">Information about the target associated with this module</param>
		/// <param name="ReferenceChain">Chain of references leading to this module</param>
		/// <param name="Logger">Logger for output</param>
		/// <returns>Compiled module rule info</returns>
		public ModuleRules CreateModuleRules(string ModuleName, ReadOnlyTargetRules Target, string ReferenceChain, ILogger Logger)
		{
			if (Target.IsTestTarget && !Target.ExplicitTestsTarget)
			{
				ModuleName = TargetDescriptor.GetTestedName(ModuleName);
			}

			// Currently, we expect the user's rules object type name to be the same as the module name
			string ModuleTypeName = ModuleName;

			// Make sure the base module file is known to us
			FileReference? ModuleFileName;
			if (!ModuleNameToModuleFile.TryGetValue(ModuleTypeName, out ModuleFileName))
			{
				if (Parent == null)
				{
					throw new CompilationResultException(CompilationResult.RulesError, KnownLogEvents.RulesAssembly, "Could not find definition for module '{ModuleTypeName}', (referenced via {ReferenceChain})", ModuleTypeName, ReferenceChain);
				}
				else
				{
					return Parent.CreateModuleRules(ModuleName, Target, ReferenceChain, Logger);
				}
			}

			// get the standard Rules object class from the assembly
			Type BaseRulesObjectType = GetModuleRulesTypeInternal(ModuleTypeName)!;

			// look around for platform/group modules that we will use instead of the basic module
			Type? PlatformRulesObjectType = GetModuleRulesTypeInternal(ModuleTypeName + "_" + Target.Platform.ToString());
			if (PlatformRulesObjectType == null)
			{
				foreach (UnrealPlatformGroup Group in UEBuildPlatform.GetPlatformGroups(Target.Platform))
				{
					// look to see if the group has an override
					Type? GroupRulesObjectType = GetModuleRulesTypeInternal(ModuleName + "_" + Group.ToString());

					// we expect only one platform group to be found in the extensions
					if (GroupRulesObjectType != null && PlatformRulesObjectType != null)
					{
						throw new CompilationResultException(CompilationResult.RulesError, KnownLogEvents.RulesAssembly, "Found multiple platform group overrides ({GroupRulesName} and {PlatformRulesName}) for module {ModuleName} without a platform specific override. Create a platform override with the class hierarchy as needed.",
							GroupRulesObjectType.Name, PlatformRulesObjectType.Name, ModuleName);
					}
					// remember the platform group if we found it, but keep searching to verify there isn't more than one
					if (GroupRulesObjectType != null)
					{
						PlatformRulesObjectType = GroupRulesObjectType;
					}
				}
			}

			// verify that we aren't creating a platform module when we definitely don't want it
			if (Target.OptedInModulePlatforms != null)
			{
				// figure out what platforms/groups aren't allowed with this opted in list
				List<string> DisallowedPlatformsAndGroups = Utils.MakeListOfUnsupportedPlatforms(Target.OptedInModulePlatforms.ToList(), false, Logger);

				// check if the module file is disallowed
				if (ModuleFileName.ContainsAnyNames(DisallowedPlatformsAndGroups, Unreal.EngineDirectory) ||
					(Target.ProjectFile != null && ModuleFileName.ContainsAnyNames(DisallowedPlatformsAndGroups, Target.ProjectFile.Directory)))
				{
					throw new CompilationResultException(CompilationResult.RulesError, KnownLogEvents.RulesAssembly, "Platform module file {ModuleFileName} is not allowed (only platforms '{Platforms}', and their groups, are allowed. This indicates a module reference not being checked with something like IsPlatformAvailableForTarget()).",
						ModuleFileName, String.Join(",", Target.OptedInModulePlatforms));
				}
			}

			// Figure out the best rules object to use
			Type? RulesObjectType = PlatformRulesObjectType ?? BaseRulesObjectType;
			if (RulesObjectType == null)
			{
				throw new CompilationResultException(CompilationResult.RulesError, KnownLogEvents.RulesAssembly, "Expecting to find a type to be declared in a module rules named '{ModuleTypeName}' in '{AssemblyName}'.  This type must derive from the 'ModuleRules' type defined by UnrealBuildTool.",
					ModuleTypeName, CompiledAssembly?.FullName ?? "Unknown Assembly");
			}

			// Create an instance of the module's rules object
			try
			{
				// Create an uninitialized ModuleRules object and set some defaults.
				ModuleRules RulesObject = (ModuleRules)RuntimeHelpers.GetUninitializedObject(RulesObjectType);

				// even if we created a platform-extension version of the module rules, we are pretending to be
				// the base type, so that no one else needs to manage this
				RulesObject.Name = ModuleName;
				RulesObject.File = ModuleFileName;
				RulesObject.Directory = ModuleFileName.Directory;
				RulesObject.Context = ModuleFileToContext[RulesObject.File];
				RulesObject.Plugin = RulesObject.Context.Plugin;
				RulesObject.bTreatAsEngineModule = bContainsEngineModules;
				if (DefaultBuildSettings.HasValue)
				{
					RulesObject.DefaultBuildSettings = DefaultBuildSettings.Value;
				}
				RulesObject.bPrecompile = (RulesObject.bTreatAsEngineModule || ModuleName.Equals("UnrealGame", StringComparison.OrdinalIgnoreCase)) && Target.bPrecompile;
				RulesObject.bUsePrecompiled = bReadOnly;
				RulesObject.RulesAssembly = this;
				RulesObject.DirectoriesForModuleSubClasses = new();
				RulesObject.DirectoriesForModuleSubClasses.Add(BaseRulesObjectType, RulesObject.Directory);

				// go up the type hierarchy (if there is a hierarchy), looking for any extra directories for the module
				if (RulesObjectType != BaseRulesObjectType && RulesObjectType != typeof(ModuleRules))
				{
					Type SubType = RulesObjectType;

					RulesObject.SubclassRules = new List<string>();
					while (SubType != BaseRulesObjectType)
					{
						FileReference? SubTypeFileName;
						if (TryGetFileNameFromType(SubType, out SubTypeFileName))
						{
							RulesObject.DirectoriesForModuleSubClasses.Add(SubType, SubTypeFileName.Directory);
							RulesObject.SubclassRules.Add(SubTypeFileName.FullName);
						}
						if (SubType.BaseType == null)
						{
							throw new CompilationResultException(CompilationResult.RulesError, KnownLogEvents.RulesAssembly, "{TypeName} is not derived from {BaseTypeName}", RulesObjectType.Name, BaseRulesObjectType.Name);
						}
						SubType = SubType.BaseType;
					}
				}

				// Call the constructor
				ConstructorInfo? Constructor = RulesObjectType.GetConstructor(new Type[] { typeof(ReadOnlyTargetRules) });
				if (Constructor == null)
				{
					throw new CompilationResultException(CompilationResult.RulesError, KnownLogEvents.RulesAssembly, "No valid constructor found for {ModuleName}.", ModuleName);
				}

				// Add the parent assemblies to the assembly cache so the types in them can be used when the constructor is called
				RulesAssembly? CurrentParent = Parent;
				while (CurrentParent != null && CurrentParent.CompiledAssembly != null)
				{
					EpicGames.Core.AssemblyUtils.AddFileToAssemblyCache(CurrentParent.CompiledAssembly.Location);
					CurrentParent = CurrentParent.Parent;
				}

				Constructor.Invoke(RulesObject, new object[] { Target });

				if (Target.IsTestTarget && !RulesObject.IsTestModule)
				{
					if (!Target.ExplicitTestsTarget)
					{
						if (Target.LaunchModuleName != null && ModuleName == TargetDescriptor.GetTestedName(Target.LaunchModuleName))
						{
							RulesObject = new TestModuleRules(RulesObject);
						}
					}
					RulesObject.PrepareModuleForTests();
				}

				return RulesObject;
			}
			catch (Exception Ex)
			{
				Exception MessageEx = (Ex is TargetInvocationException && Ex.InnerException != null) ? Ex.InnerException : Ex;
				throw new CompilationResultException(CompilationResult.RulesError, Ex, KnownLogEvents.RulesAssembly, "Unable to instantiate module '{ModuleName}': {ExceptionMessage}\n(referenced via {ReferenceChain})", ModuleName, MessageEx.ToString(), ReferenceChain);
			}
		}

		/// <summary>
		/// Construct an instance of the given target rules
		/// Will return null if the requested type name does not exist in the assembly 
		/// </summary>
		/// <param name="TypeName">Type name of the target rules</param>
		/// <param name="TargetInfo">Target configuration information to pass to the constructor</param>
		/// <param name="Logger">Logger for output</param>
		/// <param name="IsTestTarget">If building a low level tests target</param>
		/// <param name="ValidationOptions">Controls validation of target and SDK</param>
		/// <returns>Instance of the corresponding TargetRules or null if requested type name does not exist</returns>
		protected TargetRules? CreateTargetRulesInstance(string TypeName, TargetInfo TargetInfo, ILogger Logger, bool IsTestTarget = false, TargetRulesValidationOptions ValidationOptions = TargetRulesValidationOptions.ValidateTargetAndSDK)
		{
			// The build module must define a type named '<TargetName>Target' that derives from our 'TargetRules' type.  
			Type? BaseRulesType = CompiledAssembly?.GetType(TypeName);
			if (BaseRulesType == null)
			{
				return null;
			}

			// Look for platform/group rules that we will use instead of the base rules
			string PlatformRulesName = TargetInfo.Name + "_" + TargetInfo.Platform.ToString();
			Type? PlatformRulesType = CompiledAssembly?.GetType(TypeName + "_" + TargetInfo.Platform.ToString());
			if (PlatformRulesType == null)
			{
				foreach (UnrealPlatformGroup Group in UEBuildPlatform.GetPlatformGroups(TargetInfo.Platform))
				{
					// look to see if the group has an override
					string GroupRulesName = TargetInfo.Name + "_" + Group.ToString();
					Type? GroupRulesObjectType = CompiledAssembly?.GetType(TypeName + "_" + Group.ToString());

					// we expect only one platform group to be found in the extensions
					if (GroupRulesObjectType != null && PlatformRulesType != null)
					{
						throw new CompilationResultException(CompilationResult.RulesError, KnownLogEvents.RulesAssembly, "Found multiple platform group overrides ({GroupRulesName} and {PlatformRulesName}) for rules {TypeName} without a platform specific override. Create a platform override with the class hierarchy as needed.",
							GroupRulesObjectType.Name, PlatformRulesType.Name, TypeName);
					}
					// remember the platform group if we found it, but keep searching to verify there isn't more than one
					if (GroupRulesObjectType != null)
					{
						PlatformRulesName = GroupRulesName;
						PlatformRulesType = GroupRulesObjectType;
					}
				}
			}
			if (PlatformRulesType != null && !PlatformRulesType.IsSubclassOf(BaseRulesType))
			{
				throw new CompilationResultException(CompilationResult.RulesError, KnownLogEvents.RulesAssembly, "Expecting {PlatformRulesType} to be a specialization of {BaseRulesType}", PlatformRulesType, BaseRulesType);
			}

			// Create an instance of the module's rules object, and set some defaults before calling the constructor.
			Type RulesType = PlatformRulesType ?? BaseRulesType;
			FileReference BaseFile = TargetNameToTargetFile[TargetInfo.Name];
			FileReference PlatformFile = TargetNameToTargetFile.TryGetValue(PlatformRulesName, out FileReference? PlatformTargetFile) ? PlatformTargetFile : BaseFile;
			TargetRules Rules = TargetRules.Create(RulesType, TargetInfo, BaseFile, PlatformFile, TargetNameToTargetFile.Values, DefaultBuildSettings, Logger);
			bool bValidateTarget = ValidationOptions == TargetRulesValidationOptions.ValidateTargetAndSDK || ValidationOptions == TargetRulesValidationOptions.ValidateTargetOnly;
			bool bValidateSDK = ValidationOptions == TargetRulesValidationOptions.ValidateTargetAndSDK || ValidationOptions == TargetRulesValidationOptions.ValidateSDKOnly;

			// Set the default overriddes for the configured target type
			Rules.SetOverridesForTargetType();

			// Set the final value for the link type in the target rules
			if (Rules.LinkType == TargetLinkType.Default)
			{
				throw new CompilationResultException(CompilationResult.RulesError, KnownLogEvents.RulesAssembly, "TargetRules.LinkType should be inferred from TargetType");
			}

			if (bValidateTarget)
			{
				// Delayed-fixup of TargetBuildEnvironment.UniqueIfNeeded
				Rules.UpdateBuildEnvironmentIfNeeded(this, arguments: null, Logger);

				// Set the default value for whether to use the shared build environment
				if (Rules.BuildEnvironment == TargetBuildEnvironment.Unique && Unreal.IsEngineInstalled())
				{
					throw new CompilationResultException(CompilationResult.RulesError, KnownLogEvents.RulesAssembly, "Targets with a unique build environment cannot be built with an installed engine.");
				}
			}

			// Automatically include CoreUObject
			if (Rules.bCompileAgainstEngine)
			{
				Rules.bCompileAgainstCoreUObject = true;
			}

			if (Rules.Type == TargetType.Editor)
			{
				Rules.bBuildWithEditorOnlyData = true; // Must have editor only data if building the editor.
				Rules.bCompileAgainstEditor = true;
			}

			// Apply the override to force debug info to be enabled
			if (Rules.bForceDebugInfo)
			{
				Rules.DebugInfo = DebugInfoMode.Full;
				Rules.bOmitPCDebugInfoInDevelopment = false;
			}

			// If merging modules, force modular and strip exports
			if (Rules.bMergeModules)
			{
				Rules.BuildEnvironment = TargetBuildEnvironment.Unique;
				Rules.LinkType = TargetLinkType.Modular;
				Rules.bStripExports = true;
			}

			// Setup utrace for Shader Compiler Worker
			if (Rules.bShaderCompilerWorkerTrace)
			{
				Rules.GlobalDefinitions.Add("USE_SHADER_COMPILER_WORKER_TRACE=1");
			}
			else
			{
				Rules.GlobalDefinitions.Add("USE_SHADER_COMPILER_WORKER_TRACE=0");
			}

			// Set a macro if we allow using generated inis
			if (!Rules.bAllowGeneratedIniWhenCooked)
			{
				Rules.GlobalDefinitions.Add("DISABLE_GENERATED_INI_WHEN_COOKED=1");
			}

			if (!Rules.bAllowNonUFSIniWhenCooked)
			{
				Rules.GlobalDefinitions.Add("DISABLE_NONUFS_INI_WHEN_COOKED=1");
			}

			if (Rules.bDisableUnverifiedCertificates)
			{
				Rules.GlobalDefinitions.Add("DISABLE_UNVERIFIED_CERTIFICATE_LOADING=1");
			}

			if (Rules.bRequireObjectPtrForAddReferencedObjects)
			{
				Rules.GlobalDefinitions.Add("UE_REFERENCE_COLLECTOR_REQUIRE_OBJECTPTR=1");
			}

			// Until VNI fully supports the new VM, we need the ability to have both the old and new
			// available in some rare cases.  If we are using the old VM and the target hasn't overridden
			// the new VM define, then set the define based on the old VM flag.
			if (!Rules.GlobalDefinitions.Any(x => x.StartsWith("WITH_VERSE_VM=", StringComparison.Ordinal)))
			{
				Rules.GlobalDefinitions.Add($"WITH_VERSE_VM={(Rules.bUseVerseBPVM ? 0 : 1)}");
			}

			// if the Target has opted in only some platforms, disable any plugins of other platforms (there may be editor, etc, modules that
			// will just add themselves, with no other reference to be able to remove them, other than disabling them here)
			if (Rules.OptedInModulePlatforms != null)
			{
				// figure out what platforms/groups aren't allowed with this opted in list
				List<string> DisallowedPlatformsAndGroups = Utils.MakeListOfUnsupportedPlatforms(Rules.OptedInModulePlatforms.ToList(), false, Logger);

				// This is really slow (disallow list can contain 20 items and there can be 1000 plugins with long paths.. so search space is huuuuge)
				// Use a bloom filter to minimize the expensive testing
				BloomFilter bloomFilter = new();
				bloomFilter.AddRange(DisallowedPlatformsAndGroups);

				DirectoryReference? ProjectDir = Rules.ProjectFile?.Directory;

				// look in all plugins' paths to see if any disallowed
				List<PluginInfo> DisallowedPluginsList = [];

				foreach (PluginSet set in EnumeratePlugins())
				{
					if (set.ChoiceVersion == null)
					{
						continue;
					}

					PluginInfo info = set.ChoiceVersion;

					bool mightContain = false;
					if (info.File.IsUnderDirectory(Unreal.EngineDirectory))
					{
						ReadOnlySpan<char> span = info.File.FullName.AsSpan(Unreal.EngineDirectory.FullName.Length + 1);
						if (bloomFilter.MightContainSubString(span, Path.DirectorySeparatorChar))
						{
							mightContain = true;
						}
					}

					if (!mightContain && ProjectDir != null && info.File.IsUnderDirectory(ProjectDir))
					{
						ReadOnlySpan<char> span = info.File.FullName.AsSpan(ProjectDir.FullName.Length + 1);
						if (bloomFilter.MightContainSubString(span, Path.DirectorySeparatorChar))
						{
							mightContain = true;
						}
					}

					if (mightContain)
					{
						if (info.File.ContainsAnyNames(DisallowedPlatformsAndGroups, Unreal.EngineDirectory) ||
						(ProjectDir != null && info.File.ContainsAnyNames(DisallowedPlatformsAndGroups, ProjectDir)))
						{
							DisallowedPluginsList.Add(info);
						}
					}
				}

				DisallowedPluginsList.ForEach(x => Logger.LogDebug("Disallowing non-opted-in platform plugin {PluginFile}", x.File));
				// and, disable these plugins
				Rules.DisablePlugins.AddRange(DisallowedPluginsList.Select(x => x.Name));
			}

			// Allow the platform to finalize the settings
			if (bValidateTarget)
			{
				UEBuildPlatform Platform = UEBuildPlatform.GetBuildPlatform(Rules.Platform);
				Platform.ValidateTarget(Rules);
			}

			// make sure any SDK overrides are valid
			if (bValidateSDK)
			{
				ValidateSDKs(Rules);
			}

			// Some platforms may *require* monolithic compilation...
			if (Rules.LinkType != TargetLinkType.Monolithic && UEBuildPlatform.PlatformRequiresMonolithicBuilds(Rules.Platform))
			{
				throw new CompilationResultException(CompilationResult.RulesError, KnownLogEvents.RulesAssembly, "{RulesName}: {RulesPlatform} does not support modular builds", Rules.Name, Rules.Platform);
			}

			if (IsTestTarget)
			{
				Rules = TestTargetRules.Create(Rules, TargetInfo);
			}

			return Rules;
		}

		private static void ValidateSDKs(TargetRules Rules)
		{
			// ask if each plaform SDK matters to this target

			foreach (UnrealTargetPlatform Platform in UnrealTargetPlatform.GetValidPlatforms())
			{
				if (!Rules.IsSDKVersionRelevant(Platform))
				{
					continue;
				}

				UEBuildPlatformSDK? SDK = UEBuildPlatformSDK.GetSDKForPlatform(Platform.ToString());
				if (SDK == null)
				{
					continue;
				}

				if (SDK.bHasSDKOverride)
				{
					// if the target doesn't allow for an override at all, error
					if (!Rules.AllowsPerProjectSDKVersion())
					{
						throw new CompilationResultException(CompilationResult.RulesError, KnownLogEvents.RulesAssembly, "Target {RulesName} is being built with a overridden {Platform} SDK version to '{SdkVersion}', but this target is not allowed - likely due to a modular build using a Shared BuildEnvironment. If this target doesn't care about SDK versions, set 'bAreTargetSDKVersionsRelevantOverride = false' in your Target.cs file. Be aware this could cause confusion with multiple targets fighting over shared modules.",
							Rules.Name, Platform, SDK.GetMainVersion());
					}

					// check to see if the project _doesn't_ override the SDK version (if it did, this was already handled during early processing)
					if (Rules.ProjectFile == null || !SDK.ProjectsThatOverrodeSDK.Contains(Rules.ProjectFile))
					{
						string OverrideProject = SDK.ProjectsThatOverrodeSDK[0].GetFileNameWithoutAnyExtensions();
						throw new CompilationResultException(CompilationResult.RulesError, KnownLogEvents.RulesAssembly, "Target {RulesName} is using default {Platform} SDK version, but another target (probably {OverrideProject}) has overridden the SDK version to '{SdkVersion}'. If this target doesn't care about SDK versions, set 'bAreTargetSDKVersionsRelevantOverride = false' in your Target.cs file",
							Rules.Name, Platform, OverrideProject, SDK.GetMainVersion());
					}
				}
			}
		}

		/// <summary>
		/// Creates a target rules object for the specified target name.
		/// </summary>
		/// <param name="TargetName">Name of the target</param>
		/// <param name="Platform">Platform being compiled</param>
		/// <param name="Configuration">Configuration being compiled</param>
		/// <param name="Architectures">Architectures being built</param>
		/// <param name="ProjectFile">Path to the project file for this target</param>
		/// <param name="Arguments">Command line arguments for this target</param>
		/// <param name="InLogger"></param>
		/// <param name="IsTestTarget">If building a low level test target</param>
		/// <param name="ValidationOptions">Controls validation of target and SDK</param>
		/// <param name="IntermediateEnvironment">Intermediate environment to use</param>
		/// <returns>The build target rules for the specified target</returns>
		public TargetRules CreateTargetRules(string TargetName, UnrealTargetPlatform Platform, UnrealTargetConfiguration Configuration, UnrealArchitectures? Architectures, FileReference? ProjectFile, CommandLineArguments? Arguments, ILogger InLogger, bool IsTestTarget = false, TargetRulesValidationOptions ValidationOptions = TargetRulesValidationOptions.ValidateTargetAndSDK, UnrealIntermediateEnvironment IntermediateEnvironment = UnrealIntermediateEnvironment.Default)
		{
			if (IsTestTarget)
			{
				TargetName = TargetDescriptor.GetTestedName(TargetName);
			}

			// Suppress TargetRules logging during project gen
			ILogger Logger = ProjectFileGenerator.bGenerateProjectFiles ? NullLogger.Instance : InLogger;

			Architectures ??= UnrealArchitectureConfig.ForPlatform(Platform).ActiveArchitectures(ProjectFile, TargetName);

			bool bFoundTargetName = TargetNameToTargetFile.ContainsKey(TargetName);
			if (bFoundTargetName == false)
			{
				if (Parent == null)
				{
					string ExceptionMessage = "Couldn't find target rules file for target '{TargetName}' in rules assembly '{RulesAssembly}'" + Environment.NewLine;
					ExceptionMessage += "Location: {AssemblyLocation}" + Environment.NewLine;
					if (TargetNameToTargetFile.Any())
					{
						ExceptionMessage += "Target rules found:" + Environment.NewLine;
						foreach (KeyValuePair<string, FileReference> entry in TargetNameToTargetFile)
						{
							ExceptionMessage += "\t" + entry.Key + " - " + entry.Value + Environment.NewLine;
						}
					}

					throw new CompilationResultException(CompilationResult.RulesError, KnownLogEvents.RulesAssembly, ExceptionMessage,
						TargetName, CompiledAssembly?.FullName ?? "Unknown Assembly", CompiledAssembly?.Location ?? "Unknown Location");
				}
				else
				{
					return Parent.CreateTargetRules(TargetName, Platform, Configuration, Architectures, ProjectFile, Arguments, Logger, IsTestTarget, ValidationOptions, IntermediateEnvironment);
				}
			}

			// Currently, we expect the user's rules object type name to be the same as the module name + 'Target'
			string TargetTypeName = TargetName + "Target";

			// The build module must define a type named '<TargetName>Target' that derives from our 'TargetRules' type.  
			TargetRules? TargetRules = CreateTargetRulesInstance(TargetTypeName, new TargetInfo(TargetName, Platform, Configuration, Architectures, ProjectFile, Arguments, IntermediateEnvironment), Logger, IsTestTarget, ValidationOptions);

			if (TargetRules == null)
			{
				throw new CompilationResultException(CompilationResult.RulesError, KnownLogEvents.RulesAssembly, "Expecting to find a type to be declared in a target rules named '{TargetTypeName}'.  This type must derive from the 'TargetRules' type defined by UnrealBuildTool.", TargetTypeName);
			}

			return TargetRules;
		}

		/// <summary>
		/// Determines a target name based on the type of target we're trying to build
		/// </summary>
		/// <param name="Type">The type of target to look for</param>
		/// <param name="Platform">The platform being built</param>
		/// <param name="Configuration">The configuration being built</param>
		/// <param name="ProjectFile">Project file for the target being built</param>
		/// <param name="Logger">Logger for output</param>
		/// <returns>Name of the target for the given type</returns>
		public string GetTargetNameByType(TargetType Type, UnrealTargetPlatform Platform, UnrealTargetConfiguration Configuration, FileReference? ProjectFile, ILogger Logger)
		{
			// Create all the targets in this assembly 
			List<string> Matches = new List<string>();
			foreach (KeyValuePair<string, FileReference> TargetPair in TargetNameToTargetFile)
			{
				TargetRules? Rules = CreateTargetRulesInstance(TargetPair.Key + "Target", new TargetInfo(TargetPair.Key, Platform, Configuration, null, ProjectFile, null), Logger);
				if (Rules != null && Rules.Type == Type && !Rules.bExplicitTargetForType)
				{
					Matches.Add(TargetPair.Key);
				}
			}

			// If we got a result, return it. If there were multiple results, fail.
			if (Matches.Count == 0)
			{
				if (Parent == null)
				{
					throw new CompilationResultException(CompilationResult.RulesError, KnownLogEvents.RulesAssembly, "Unable to find target of type '{Type}' for project '{1}'", Type, ProjectFile?.FullName ?? "NoProject");
				}
				else
				{
					return Parent.GetTargetNameByType(Type, Platform, Configuration, ProjectFile, Logger);
				}
			}
			else
			{
				if (Matches.Count == 1)
				{
					return Matches[0];
				}

				// attempt to get a default target (like DefaultEditorTarget) from the Engine.ini
				string KeyName = $"Default{Type}Target";
				string? DefaultTargetName;
				// read in the engine config hierarchy and get the value
				ConfigHierarchy EngineConfig = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, ProjectFile?.Directory, Platform);
				if (EngineConfig.GetString("/Script/BuildSettings.BuildSettings", KeyName, out DefaultTargetName))
				{
					// if a value was found, make sure that this is one of the found targets
					if (Matches.Contains(DefaultTargetName))
					{
						return DefaultTargetName;
					}
				}

				throw new CompilationResultException(CompilationResult.RulesError, KnownLogEvents.RulesAssembly, "Found multiple targets with TargetType={Type}: {Matches}.\nSpecify a default with a {KeyName} entry in [/Script/BuildSettings.BuildSettings] section of your DefaultEngine.ini", Type, String.Join(", ", Matches), KeyName);
			}
		}

		/// <summary>
		/// Enumerates all the plugins that are available
		/// </summary>
		/// <returns></returns>
		public IEnumerable<PluginSet> EnumeratePlugins()
		{
			return global::UnrealBuildTool.Plugins.FilterPlugins(EnumeratePluginsInternal());
		}

		/// <summary>
		/// Enumerates all the plugins that are available
		/// </summary>
		/// <returns></returns>
		protected IEnumerable<PluginInfo> EnumeratePluginsInternal()
		{
			if (Parent == null)
			{
				return Plugins;
			}
			else
			{
				return Plugins.Concat(Parent.EnumeratePluginsInternal());
			}
		}

		/// <summary>
		/// Tries to find the ModuleRulesContext associated with a given module file
		/// </summary>
		internal ModuleRulesContext? TryGetContextForModule(FileReference ModuleFile)
		{
			for (RulesAssembly? Assembly = this; Assembly != null; Assembly = Assembly.Parent)
			{
				if (Assembly.ModuleFileToContext.TryGetValue(ModuleFile, out ModuleRulesContext? Context))
				{
					return Context;
				}
			}

			return null;
		}
	}
}
