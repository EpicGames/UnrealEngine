// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Threading;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using OpenTracing.Util;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// Class which compiles (and caches) rules assemblies for different folders.
	/// </summary>
	public static class RulesCompiler
	{
		/// <summary>
		/// 
		/// </summary>
		const string FrameworkAssemblyExtension = ".dll";

		/// <summary>
		/// 
		/// </summary>
		const BuildSettingsVersion DefaultEngineBuildSettingsVersion = BuildSettingsVersion.Latest;

		/// <summary>
		/// Find all the module rules files under a given directory
		/// </summary>
		/// <param name="BaseDirectory">The directory to search under</param>
		/// <param name="ModuleContext">The module context for each found rules instance</param>
		/// <param name="ModuleFileToContext">Map of module files to their context</param>
		private static void AddModuleRulesWithContext(DirectoryReference BaseDirectory, ModuleRulesContext ModuleContext, Dictionary<FileReference, ModuleRulesContext> ModuleFileToContext)
		{
			IReadOnlyList<FileReference> RulesFiles = Rules.Get([BaseDirectory], Rules.RulesFileType.Module);
			foreach (FileReference RulesFile in RulesFiles)
			{
				ModuleFileToContext[RulesFile] = ModuleContext;
			}
		}

		/// <summary>
		/// Lock for creating engine rule assembly
		/// </summary>
		private static readonly Lock EngineRulesAssemblyLock = new();

		/// <summary>
		/// The cached rules assembly for engine modules and targets.
		/// </summary>
		private static RulesAssembly? EngineRulesAssembly;

		/// <summary>
		/// Map of assembly names we've already compiled and loaded to their Assembly and list of game folders.  This is used to prevent
		/// trying to recompile the same assembly when ping-ponging between different types of targets
		/// </summary>
		private static readonly ConcurrentDictionary<FileReference, Lazy<RulesAssembly>> LoadedAssemblyMap = new();

		/// <summary>
		/// Creates the engine rules assembly
		/// </summary>
		/// <param name="bUsePrecompiled">Whether to use a precompiled engine</param>
		/// <param name="bSkipCompile">Whether to skip compilation for this assembly</param>
		/// <param name="bForceCompile">Whether to always compile this assembly</param>
		/// <param name="Logger">Logger for output</param>
		/// <returns>New rules assembly</returns>
		public static RulesAssembly CreateEngineRulesAssembly(bool bUsePrecompiled, bool bSkipCompile, bool bForceCompile, ILogger Logger)
		{
			lock (EngineRulesAssemblyLock)
			{
				if (EngineRulesAssembly == null)
				{
					// Prevent multiple conflicting processes building rules assembly at the same time
					using IDisposable _ = SingleInstanceMutex.Acquire(GlobalSingleInstanceMutex.GetUniqueMutexForPath("UnrealBuildTool_CreateTargetRulesAssembly", Unreal.EngineDirectory));
					{
						List<PluginInfo> EnginePlugins = [];
						List<PluginInfo> MarketplacePlugins = [];
						{
							using ITimelineEvent ite = Timeline.ScopeEvent("EngineRulesPlugins");
							DirectoryReference MarketplaceDirectory = DirectoryReference.Combine(Unreal.EngineDirectory, "Plugins", "Marketplace");
							foreach (PluginInfo PluginInfo in Plugins.ReadEnginePlugins(Unreal.EngineDirectory))
							{
								if (PluginInfo.File.IsUnderDirectory(MarketplaceDirectory))
								{
									MarketplacePlugins.Add(PluginInfo);
								}
								else
								{
									EnginePlugins.Add(PluginInfo);
								}
							}
						}
						EngineRulesAssembly = CreateEngineRulesAssemblyInternal(Unreal.GetExtensionDirs(Unreal.EngineDirectory), ProjectFileGenerator.EngineRulesAssemblyName, EnginePlugins, Unreal.IsEngineInstalled() || bUsePrecompiled, bSkipCompile, bForceCompile, null, Logger);

						if (MarketplacePlugins.Count > 0)
						{
							EngineRulesAssembly = CreateMarketplaceRulesAssembly(MarketplacePlugins, Unreal.IsEngineInstalled() || bUsePrecompiled, bSkipCompile, bForceCompile, EngineRulesAssembly, Logger);
						}
					}
				}
				return EngineRulesAssembly;
			}
		}

		/// <summary>
		/// Creates a rules assembly
		/// </summary>
		/// <param name="RootDirectories">The root directories to create rules for</param>
		/// <param name="AssemblyPrefix">A prefix for the assembly file name</param>
		/// <param name="Plugins">List of plugins to include in this assembly</param>
		/// <param name="bReadOnly">Whether the assembly should be marked as installed</param>
		/// <param name="bSkipCompile">Whether to skip compilation for this assembly</param>
		/// <param name="bForceCompile">Whether to always compile this assembly</param>
		/// <param name="Parent">The parent rules assembly</param>
		/// <param name="Logger">Logger for output</param>
		/// <returns>New rules assembly</returns>
		private static RulesAssembly CreateEngineRulesAssemblyInternal(List<DirectoryReference> RootDirectories, string AssemblyPrefix, IReadOnlyList<PluginInfo> Plugins, bool bReadOnly, bool bSkipCompile, bool bForceCompile, RulesAssembly? Parent, ILogger Logger)
		{
			// Scope hierarchy
			RulesScope Scope = new RulesScope("Engine", null);
			RulesScope PluginsScope = new RulesScope("Engine Plugins", Scope);
			RulesScope ProgramsScope = new RulesScope("Engine Programs", PluginsScope);

			// Find the shared modules, excluding the programs directory. These are used to create an assembly with the bContainsEngineModules flag set to true.
			Dictionary<FileReference, ModuleRulesContext> ModuleFileToContext = new Dictionary<FileReference, ModuleRulesContext>();
			ModuleRulesContext DefaultModuleContext = new ModuleRulesContext(Scope, RootDirectories[0]);
			List<FileReference> ProgramTargetFiles = new List<FileReference>();

			// We want to prefetch the directories in one call to make it go as wide as possible against filesystem
			{
				using ITimelineEvent fdt = Timeline.ScopeEvent("EngineRulesFetchDirs");

				List<DirectoryReference> populateDirs = [];
				foreach (PluginInfo info in Plugins)
				{
					populateDirs.Add(DirectoryReference.Combine(info.Directory, "Source"));
					populateDirs.Add(DirectoryReference.Combine(info.Directory, "Tests"));
					foreach (FileReference ChildFile in info.ChildFiles)
					{
						populateDirs.Add(DirectoryReference.Combine(ChildFile.Directory, "Source"));
						populateDirs.Add(DirectoryReference.Combine(ChildFile.Directory, "Tests"));
					}
				}
				foreach (DirectoryReference RootDirectory in RootDirectories)
				{
					populateDirs.Add(DirectoryReference.Combine(RootDirectory, "Source"));
					populateDirs.Add(DirectoryReference.Combine(RootDirectory, "Shaders"));
				}
				Rules.Populate(populateDirs);
			}

			foreach (DirectoryReference RootDirectory in RootDirectories)
			{
				using (GlobalTracer.Instance.BuildSpan("Finding engine modules").StartActive())
				{
					void AddEngineModuleRulesWithContext(DirectoryReference BaseDirectory, string SubDirectoryName, ModuleRulesContext BaseModuleContext, UHTModuleType? DefaultUHTModuleType, Dictionary<FileReference, ModuleRulesContext> ModuleFileToContext)
					{
						DirectoryReference Directory = DirectoryReference.Combine(BaseDirectory, SubDirectoryName);
						if (DirectoryLookupCache.DirectoryExists(Directory))
						{
							ModuleRulesContext ModuleContext = new ModuleRulesContext(BaseModuleContext) { DefaultUHTModuleType = DefaultUHTModuleType, bCanHotReload = true };

							IReadOnlyList<FileReference> RulesFiles = Rules.Get([Directory], Rules.RulesFileType.Module);
							foreach (FileReference RulesFile in RulesFiles)
							{
								ModuleFileToContext[RulesFile] = ModuleContext;
							}
						}
					}

					DirectoryReference SourceDirectory = DirectoryReference.Combine(RootDirectory, "Source");

					AddEngineModuleRulesWithContext(SourceDirectory, "Runtime", DefaultModuleContext, UHTModuleType.EngineRuntime, ModuleFileToContext);
					AddEngineModuleRulesWithContext(SourceDirectory, "Developer", DefaultModuleContext, UHTModuleType.EngineDeveloper, ModuleFileToContext);
					AddEngineModuleRulesWithContext(SourceDirectory, "Editor", DefaultModuleContext, UHTModuleType.EngineEditor, ModuleFileToContext);
					AddEngineModuleRulesWithContext(SourceDirectory, "ThirdParty", DefaultModuleContext, UHTModuleType.EngineThirdParty, ModuleFileToContext);
					AddEngineModuleRulesWithContext(RootDirectory, "Shaders", DefaultModuleContext, UHTModuleType.EngineThirdParty, ModuleFileToContext);
				}
			}

			// Add all the plugin modules too (don't need to loop over RootDirectories since the plugins come in already found
			using (GlobalTracer.Instance.BuildSpan("Finding plugin modules").StartActive())
			{
				ModuleRulesContext PluginsModuleContext = new ModuleRulesContext(PluginsScope, RootDirectories[0]);
				PluginsModuleContext.bCanHotReload = true;
				FindModuleRulesForPlugins(Plugins, PluginsModuleContext, ModuleFileToContext);
				// Plugin test target rules only
				FindTestRulesForPlugins(Plugins, PluginsModuleContext, ModuleFileToContext, ProgramTargetFiles);
			}

			// Create the assembly
			using ITimelineEvent erat = Timeline.ScopeEvent("EngineRulesAssembly");
			DirectoryReference AssemblyDir = RootDirectories[0];
			FileReference EngineAssemblyFileName = FileReference.Combine(AssemblyDir, "Intermediate", "Build", "BuildRules", AssemblyPrefix + "Rules" + FrameworkAssemblyExtension);
			RulesAssembly EngineAssembly = new RulesAssembly(Scope, RootDirectories, Plugins, ModuleFileToContext, new List<FileReference>(), EngineAssemblyFileName, bContainsEngineModules: true, DefaultBuildSettings: DefaultEngineBuildSettingsVersion, bReadOnly: bReadOnly, bSkipCompile: bSkipCompile, bForceCompile: bForceCompile, Parent: Parent, Logger: Logger);
			erat.Finish();

			using ITimelineEvent prat = Timeline.ScopeEvent("ProgramRulesAssembly");
			Dictionary<FileReference, ModuleRulesContext> ProgramModuleFiles = new Dictionary<FileReference, ModuleRulesContext>();
			foreach (DirectoryReference RootDirectory in RootDirectories)
			{
				DirectoryReference SourceDirectory = DirectoryReference.Combine(RootDirectory, "Source");
				DirectoryReference ProgramsDirectory = DirectoryReference.Combine(SourceDirectory, "Programs");

				// Also create a scope for them, and update the UHT module type
				ModuleRulesContext ProgramsModuleContext = new ModuleRulesContext(ProgramsScope, RootDirectory);
				ProgramsModuleContext.DefaultUHTModuleType = UHTModuleType.Program;

				using (GlobalTracer.Instance.BuildSpan("Finding program modules").StartActive())
				{
					// Find all the rules files
					AddModuleRulesWithContext(ProgramsDirectory, ProgramsModuleContext, ProgramModuleFiles);
				}

				using (GlobalTracer.Instance.BuildSpan("Finding program targets").StartActive())
				{
					ProgramTargetFiles.AddRange(Rules.Get([SourceDirectory], Rules.RulesFileType.Target));
				}
			}

			// Create a path to the assembly that we'll either load or compile
			FileReference ProgramAssemblyFileName = FileReference.Combine(AssemblyDir, "Intermediate", "Build", "BuildRules", AssemblyPrefix + "ProgramRules" + FrameworkAssemblyExtension);
			RulesAssembly ProgramAssembly = new RulesAssembly(ProgramsScope, RootDirectories, new List<PluginInfo>().AsReadOnly(), ProgramModuleFiles, ProgramTargetFiles, ProgramAssemblyFileName, bContainsEngineModules: false, DefaultBuildSettings: DefaultEngineBuildSettingsVersion, bReadOnly: bReadOnly, bSkipCompile: bSkipCompile, bForceCompile: bForceCompile, Parent: EngineAssembly, Logger: Logger);

			// Return the combined assembly
			return ProgramAssembly;
		}

		/// <summary>
		/// Creates a rules assembly
		/// </summary>
		/// <param name="Plugins">List of plugins to include in this assembly</param>
		/// <param name="bReadOnly">Whether the assembly should be marked as installed</param>
		/// <param name="bSkipCompile">Whether to skip compilation for this assembly</param>
		/// <param name="bForceCompile">Whether to always compile this assembly</param>
		/// <param name="Parent">The parent rules assembly</param>
		/// <param name="Logger"></param>
		/// <returns>New rules assembly</returns>
		private static RulesAssembly CreateMarketplaceRulesAssembly(IReadOnlyList<PluginInfo> Plugins, bool bReadOnly, bool bSkipCompile, bool bForceCompile, RulesAssembly Parent, ILogger Logger)
		{
			RulesScope MarketplaceScope = new RulesScope("Marketplace", Parent.Scope);

			// We want to prefetch the directories in one call to make it go as wide as possible against filesystem
			{
				List<DirectoryReference> populateDirs = [];
				foreach (PluginInfo info in Plugins)
				{
					populateDirs.Add(DirectoryReference.Combine(info.Directory, "Source"));
					populateDirs.Add(DirectoryReference.Combine(info.Directory, "Tests"));
					foreach (FileReference ChildFile in info.ChildFiles)
					{
						populateDirs.Add(DirectoryReference.Combine(ChildFile.Directory, "Source"));
						populateDirs.Add(DirectoryReference.Combine(ChildFile.Directory, "Tests"));
					}
				}
				Rules.Populate(populateDirs);
			}

			// Add all the plugin modules too (don't need to loop over RootDirectories since the plugins come in already found
			Dictionary<FileReference, ModuleRulesContext> ModuleFileToContext = new Dictionary<FileReference, ModuleRulesContext>();
			using (GlobalTracer.Instance.BuildSpan("Finding marketplace plugin modules").StartActive())
			{
				ModuleRulesContext PluginsModuleContext = new ModuleRulesContext(MarketplaceScope, Unreal.EngineDirectory);
				FindModuleRulesForPlugins(Plugins, PluginsModuleContext, ModuleFileToContext);
			}

			// Create the assembly
			RulesAssembly Result = Parent;
			if (ModuleFileToContext.Count > 0)
			{
				FileReference AssemblyFileName = FileReference.Combine(Unreal.WritableEngineDirectory, "Intermediate", "Build", "BuildRules", "MarketplaceRules.dll");
				Result = new RulesAssembly(MarketplaceScope, new List<DirectoryReference> { DirectoryReference.Combine(Unreal.EngineDirectory, "Plugins", "Marketplace") }, Plugins, ModuleFileToContext, new List<FileReference>(), AssemblyFileName, bContainsEngineModules: true, DefaultBuildSettings: null, bReadOnly: bReadOnly, bSkipCompile: bSkipCompile, bForceCompile: bForceCompile, Parent: Parent, Logger: Logger);
			}
			return Result;
		}

		/// <summary>
		/// Creates a rules assembly with the given parameters.
		/// </summary>
		/// <param name="ProjectFileName">The project file to create rules for. Null for the engine.</param>
		/// <param name="bUsePrecompiled">Whether to use a precompiled engine</param>
		/// <param name="bSkipCompile">Whether to skip compilation for this assembly</param>
		/// <param name="bForceCompile">Whether to always compile this assembly</param>
		/// <param name="Logger">Logger for output</param>
		/// <returns>New rules assembly</returns>
		public static RulesAssembly CreateProjectRulesAssembly(FileReference ProjectFileName, bool bUsePrecompiled, bool bSkipCompile, bool bForceCompile, ILogger Logger)
		{
			// Check if there's an existing assembly for this project
			return LoadedAssemblyMap.GetOrAdd(ProjectFileName, new Lazy<RulesAssembly>(() =>
			{
				// Prevent multiple conflicting processes building rules assembly at the same time
				using IDisposable _ = SingleInstanceMutex.Acquire(GlobalSingleInstanceMutex.GetUniqueMutexForPath("UnrealBuildTool_CreateTargetRulesAssembly", ProjectFileName));

				Logger.LogTrace("Creating project rules assembly for {Project}...", ProjectFileName);

				ProjectDescriptor Project = ProjectDescriptor.FromFile(ProjectFileName);

				// Create the parent assembly
				RulesAssembly Parent = CreateEngineRulesAssembly(bUsePrecompiled, bSkipCompile, bForceCompile, Logger);

				using ITimelineEvent ite = Timeline.ScopeEvent("ProjectRulesAssembly");

				DirectoryReference MainProjectDirectory = ProjectFileName.Directory;
				//DirectoryReference MainProjectSourceDirectory = DirectoryReference.Combine(MainProjectDirectory, "Source");

				// Create a scope for things in this assembly
				RulesScope Scope = new RulesScope("Project", Parent.Scope);

				// Create a new context for modules created by this assembly
				ModuleRulesContext DefaultModuleContext = new ModuleRulesContext(Scope, MainProjectDirectory);
				DefaultModuleContext.bCanBuildDebugGame = true;
				DefaultModuleContext.bCanHotReload = true;
				DefaultModuleContext.bClassifyAsGameModuleForUHT = true;
				DefaultModuleContext.bCanUseForSharedPCH = false;

				// gather modules from project and platforms
				Dictionary<FileReference, ModuleRulesContext> ModuleFiles = new Dictionary<FileReference, ModuleRulesContext>();
				List<FileReference> TargetFiles = new List<FileReference>();

				// Find all the project directories
				List<DirectoryReference> ProjectDirectories = Unreal.GetExtensionDirs(ProjectFileName.Directory);
				if (Project.AdditionalRootDirectories != null)
				{
					ProjectDirectories.AddRange(Project.AdditionalRootDirectories);
				}

				// Find all the project plugins
				List<PluginInfo> ProjectPlugins = new List<PluginInfo>();
				ProjectPlugins.AddRange(Plugins.ReadProjectPlugins(MainProjectDirectory));

				// Add the project's additional plugin directories plugins too
				if (Project.AdditionalPluginDirectories != null)
				{
					foreach (DirectoryReference AdditionalPluginDirectory in Project.AdditionalPluginDirectories)
					{
						ProjectPlugins.AddRange(Plugins.ReadAdditionalPlugins(AdditionalPluginDirectory, Logger));
					}
				}

				// We want to prefetch the directories in one call to make it go as wide as possible against filesystem
				{
					List<DirectoryReference> populateDirs = [];
					foreach (PluginInfo info in ProjectPlugins)
					{
						populateDirs.Add(DirectoryReference.Combine(info.Directory, "Source"));
						populateDirs.Add(DirectoryReference.Combine(info.Directory, "Tests"));
						foreach (FileReference ChildFile in info.ChildFiles)
						{
							populateDirs.Add(DirectoryReference.Combine(ChildFile.Directory, "Source"));
							populateDirs.Add(DirectoryReference.Combine(ChildFile.Directory, "Tests"));
						}
					}
					foreach (DirectoryReference RootDirectory in ProjectDirectories)
					{
						populateDirs.Add(DirectoryReference.Combine(RootDirectory, "Source"));
						populateDirs.Add(DirectoryReference.Combine(RootDirectory, "Intermediate", "Source"));
					}
					Rules.Populate(populateDirs);
				}

				// Find all the rules/plugins under the project source directories
				foreach (DirectoryReference ProjectDirectory in ProjectDirectories)
				{
					DirectoryReference ProjectSourceDirectory = DirectoryReference.Combine(ProjectDirectory, "Source");
					AddModuleRulesWithContext(ProjectSourceDirectory, DefaultModuleContext, ModuleFiles);
					TargetFiles.AddRange(Rules.Get([ProjectSourceDirectory], Rules.RulesFileType.Target));
				}

				Logger.LogTrace(" Found {Count} Plugins:", ProjectPlugins.Count);
				ProjectPlugins.ForEach(x => { Logger.LogTrace("  {Plugin}", x.File); });

				// Find all the plugin module rules as well as plugin test target and module rules
				FindModuleRulesForPlugins(ProjectPlugins, DefaultModuleContext, ModuleFiles);
				FindTestRulesForPlugins(ProjectPlugins, DefaultModuleContext, ModuleFiles, TargetFiles);

				Logger.LogTrace(" Found {Count} Modules:", ModuleFiles.Count);
				foreach (KeyValuePair<FileReference, ModuleRulesContext> Item in ModuleFiles)
				{
					Logger.LogTrace("  {Module}", Item.Key);
				}

				// Add the games project's intermediate source folder
				DirectoryReference ProjectIntermediateSourceDirectory = DirectoryReference.Combine(MainProjectDirectory, "Intermediate", "Source");
				if (DirectoryReference.Exists(ProjectIntermediateSourceDirectory))
				{
					AddModuleRulesWithContext(ProjectIntermediateSourceDirectory, DefaultModuleContext, ModuleFiles);
					TargetFiles.AddRange(Rules.Get([ProjectIntermediateSourceDirectory], Rules.RulesFileType.Target));
				}

				RulesAssembly ProjectRulesAssembly;
				// Compile the assembly. If there are no module or target files, just use the parent assembly.
				FileReference AssemblyFileName = FileReference.Combine(MainProjectDirectory, "Intermediate", "Build", "BuildRules", ProjectFileName.GetFileNameWithoutExtension() + "ModuleRules" + FrameworkAssemblyExtension);
				if (ModuleFiles.Count == 0 && TargetFiles.Count == 0 && ProjectPlugins.Count == 0)
				{
					ProjectRulesAssembly = Parent;
				}
				else
				{
					ProjectRulesAssembly = new RulesAssembly(Scope, new List<DirectoryReference> { MainProjectDirectory }, ProjectPlugins, ModuleFiles, TargetFiles, AssemblyFileName, bContainsEngineModules: false, DefaultBuildSettings: null, bReadOnly: Unreal.IsProjectInstalled(), bSkipCompile: bSkipCompile, bForceCompile: bForceCompile, Parent: Parent, Logger: Logger);
				}
				return ProjectRulesAssembly;
			})).Value;
		}

		/// <summary>
		/// Creates a rules assembly with the given parameters.
		/// </summary>
		/// <param name="PluginFileName">The plugin file to create rules for</param>
		/// <param name="bSkipCompile">Whether to skip compilation for this assembly</param>
		/// <param name="bForceCompile">Whether to always compile this assembly</param>
		/// <param name="Parent">The parent rules assembly</param>
		/// <param name="bBuildPluginAsLocal">Whether the plugin should be built as though it is a local plugin.</param>
		/// <param name="bContainsEngineModules">Whether the plugin contains engine modules. Used to initialize the default value for ModuleRules.bTreatAsEngineModule.</param>
		/// <param name="Logger">Logger for ouptut</param>
		/// <returns>The new rules assembly</returns>
		public static RulesAssembly CreatePluginRulesAssembly(FileReference PluginFileName, bool bSkipCompile, bool bForceCompile, RulesAssembly Parent, bool bBuildPluginAsLocal, bool bContainsEngineModules, ILogger Logger)
		{
			// Check if there's an existing assembly for this project
			return LoadedAssemblyMap.GetOrAdd(PluginFileName, new Lazy<RulesAssembly>(() =>
			{
				// Prevent multiple conflicting processes building rules assembly at the same time
				using IDisposable _ = SingleInstanceMutex.Acquire(GlobalSingleInstanceMutex.GetUniqueMutexForPath("UnrealBuildTool_CreateTargetRulesAssembly", PluginFileName));

				Logger.LogTrace("Creating plugin rules assembly for {Plugin}...", PluginFileName);

				// Find all the rules source files
				Dictionary<FileReference, ModuleRulesContext> ModuleFiles = new Dictionary<FileReference, ModuleRulesContext>();
				List<FileReference> TargetFiles = new List<FileReference>();

				// Create a list of plugins for this assembly. We need to override the parent plugin, if it exists, due to overriding the
				// setting for bClassifyAsGameModuleForUHT below.
				List<PluginInfo> ForeignPlugins = new List<PluginInfo>();
				if (!Parent.EnumeratePlugins().Any(x => x.ChoiceVersion != null && x.ChoiceVersion.File == PluginFileName && x.ChoiceVersion.Type == PluginType.Engine))
				{
					PluginInfo ForeignPluginInfo = new PluginInfo(PluginFileName, PluginType.External)
					{
						bExplicitPluginTarget = true
					};
					ForeignPlugins.Add(ForeignPluginInfo);
				}

				Logger.LogTrace(" Found {Count} ForeignPlugins:", ForeignPlugins.Count);
				ForeignPlugins.ForEach(x => { Logger.LogTrace("  {ForeignPlugin}", x.File); });

				// Create a new scope for the plugin. It should not reference anything else.
				RulesScope Scope = new RulesScope("Plugin", Parent.Scope);

				// Find all the modules
				ModuleRulesContext PluginModuleContext = new ModuleRulesContext(Scope, PluginFileName.Directory);
				PluginModuleContext.bClassifyAsGameModuleForUHT = !bContainsEngineModules;
				if (bBuildPluginAsLocal)
				{
					PluginModuleContext.bCanBuildDebugGame = true;
					PluginModuleContext.bCanHotReload = true;
					PluginModuleContext.bClassifyAsGameModuleForUHT = true;
					PluginModuleContext.bCanUseForSharedPCH = false;
				}
				FindModuleRulesForPlugins(ForeignPlugins, PluginModuleContext, ModuleFiles);

				Logger.LogTrace(" Found {Count} Modules:", ModuleFiles.Count);
				foreach (KeyValuePair<FileReference, ModuleRulesContext> Item in ModuleFiles)
				{
					Logger.LogTrace("  {Module}", Item.Key);
				}

				// Compile the assembly
				FileReference AssemblyFileName = FileReference.Combine(PluginFileName.Directory, "Intermediate", "Build", "BuildRules", Path.GetFileNameWithoutExtension(PluginFileName.FullName) + "ModuleRules" + FrameworkAssemblyExtension);
				return new RulesAssembly(Scope, new List<DirectoryReference> { PluginFileName.Directory }, ForeignPlugins, ModuleFiles, TargetFiles, AssemblyFileName, bContainsEngineModules, DefaultBuildSettings: null, bReadOnly: false, bSkipCompile: bSkipCompile, bForceCompile: bForceCompile, Parent: Parent, Logger: Logger);
			})).Value;
		}

		/// <summary>
		/// Compile a rules assembly for the current target
		/// </summary>
		/// <param name="ProjectFile">The project file being compiled</param>
		/// <param name="TargetName">The target being built</param>
		/// <param name="bSkipRulesCompile">Whether to skip compiling any rules assemblies</param>
		/// <param name="bForceRulesCompile">Whether to always compile all rules assemblies</param>
		/// <param name="bUsePrecompiled">Whether to use a precompiled engine build</param>
		/// <param name="ForeignPlugin">Foreign plugin to be compiled</param>
		/// <param name="bBuildPluginAsLocal">Whether the plugin should be built as though it is a local plugin</param>
		/// <param name="Logger">Logger for output</param>
		/// <returns>The compiled rules assembly</returns>
		public static RulesAssembly CreateTargetRulesAssembly(FileReference? ProjectFile, string TargetName, bool bSkipRulesCompile, bool bForceRulesCompile, bool bUsePrecompiled, FileReference? ForeignPlugin, bool bBuildPluginAsLocal, ILogger Logger)
		{
			RulesAssembly RulesAssembly;
			if (ProjectFile != null)
			{
				RulesAssembly = CreateProjectRulesAssembly(ProjectFile, bUsePrecompiled, bSkipRulesCompile, bForceRulesCompile, Logger);
			}
			else
			{
				RulesAssembly = CreateEngineRulesAssembly(bUsePrecompiled, bSkipRulesCompile, bForceRulesCompile, Logger);
			}
			if (ForeignPlugin != null)
			{
				RulesAssembly = CreatePluginRulesAssembly(ForeignPlugin, bSkipRulesCompile, bForceRulesCompile, RulesAssembly, bBuildPluginAsLocal, true, Logger);
			}
			return RulesAssembly;
		}

		/// <summary>
		/// Finds all the module rules for plugins under the given directory.
		/// </summary>
		/// <param name="Plugins">The list of plugins to search modules for</param>
		/// <param name="DefaultContext">The default context for any files that are enumerated</param>
		/// <param name="ModuleFileToContext">Dictionary which is filled with mappings from the module file to its corresponding context</param>
		private static void FindModuleRulesForPlugins(IReadOnlyList<PluginInfo> Plugins, ModuleRulesContext DefaultContext, Dictionary<FileReference, ModuleRulesContext> ModuleFileToContext)
		{
			foreach (PluginInfo Plugin in Plugins)
			{
				FindModuleRulesForPluginInFolder(Plugin, "Source", DefaultContext, ModuleFileToContext);
			}
		}

		/// <summary>
		/// Finds all the module and target rules for plugins' tests.
		/// </summary>
		/// <param name="Plugins">The list of plugins to search test rules for</param>
		/// <param name="DefaultContext">The default context for any files that are enumerated</param>
		/// <param name="ModuleFileToContext">Dictionary which is filled with mappings from the module file to its corresponding context</param>
		/// <param name="TargetFiles">List of target files to add test target rules to</param>
		private static void FindTestRulesForPlugins(IReadOnlyList<PluginInfo> Plugins, ModuleRulesContext DefaultContext, Dictionary<FileReference, ModuleRulesContext> ModuleFileToContext, List<FileReference> TargetFiles)
		{
			foreach (PluginInfo Plugin in Plugins)
			{
				FindModuleRulesForPluginInFolder(Plugin, "Tests", DefaultContext, ModuleFileToContext);
				TargetFiles.AddRange(Rules.Get([DirectoryReference.Combine(Plugin.Directory, "Tests")], Rules.RulesFileType.Target));
			}
		}

		/// <summary>
		/// Finds all the module rules for a given plugin in a specified folder.
		/// </summary>
		/// <param name="Plugin">The plugin to search module rules for</param>
		/// <param name="Folder">The folder relative to the plugin root to look into, usually "Source" or "Tests"</param>
		/// <param name="DefaultContext">The default context for any files that are enumerated</param>
		/// <param name="ModuleFileToContext">Dictionary which is filled with mappings from the module file to its corresponding context</param>
		private static void FindModuleRulesForPluginInFolder(PluginInfo Plugin, string Folder, ModuleRulesContext DefaultContext, Dictionary<FileReference, ModuleRulesContext> ModuleFileToContext)
		{
			List<FileReference> PluginModuleFiles = [];
			PluginModuleFiles.AddRange(Rules.Get([DirectoryReference.Combine(Plugin.Directory, Folder)], Rules.RulesFileType.Module));
			foreach (FileReference ChildFile in Plugin.ChildFiles)
			{
				PluginModuleFiles.AddRange(Rules.Get([DirectoryReference.Combine(ChildFile.Directory, Folder)], Rules.RulesFileType.Module));
			}
			foreach (FileReference ModuleFile in PluginModuleFiles)
			{
				ModuleRulesContext PluginContext = new ModuleRulesContext(DefaultContext);
				PluginContext.DefaultOutputBaseDir = Plugin.Directory;
				PluginContext.Plugin = Plugin;
				ModuleFileToContext[ModuleFile] = PluginContext;
			}
		}

		/// <summary>
		/// Gets the filename that declares the given type.
		/// </summary>
		/// <param name="ExistingType">The type to search for.</param>
		/// <returns>The filename that declared the given type, or null</returns>
		public static string? GetFileNameFromType(Type ExistingType)
		{
			FileReference? FileName;
			if (EngineRulesAssembly != null && EngineRulesAssembly.TryGetFileNameFromType(ExistingType, out FileName))
			{
				return FileName.FullName;
			}

			foreach (Lazy<RulesAssembly> RulesAssembly in LoadedAssemblyMap.Values)
			{
				if (RulesAssembly.Value.TryGetFileNameFromType(ExistingType, out FileName))
				{
					return FileName.FullName;
				}
			}
			return null;
		}
	}
}
