// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using System.Linq;
using System.Reflection;
using System.Text.RegularExpressions;
using System.Xml;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace UnrealBuildTool
{
	static class RulesDocumentation
	{
		class SettingInfo
		{
			public string Name;
			public Type Type;
			public List<string> Description;

			public SettingInfo(string Name, Type Type, List<string> Description)
			{
				this.Name = Name;
				this.Type = Type;
				this.Description = Description;
			}
		}

		public static void WriteDocumentation(Type RulesType, FileReference OutputFile, ILogger Logger)
		{
			// Get the path to the XML documentation
			FileReference InputDocumentationFile = new FileReference(Assembly.GetExecutingAssembly().Location).ChangeExtension(".xml");
			if (!FileReference.Exists(InputDocumentationFile))
			{
				throw new BuildException("Generated assembly documentation not found at {0}.", InputDocumentationFile);
			}

			// Read the documentation
			XmlDocument InputDocumentation = new XmlDocument();
			InputDocumentation.Load(InputDocumentationFile.FullName);

			// Filter the settings into read-only and read/write lists
			List<SettingInfo> ReadOnlySettings = new List<SettingInfo>();
			List<SettingInfo> ReadWriteSettings = new List<SettingInfo>();

			// First read the fields
			foreach (FieldInfo Field in RulesType.GetFields(BindingFlags.Instance | BindingFlags.SetProperty | BindingFlags.Public))
			{
				if (!Field.FieldType.IsClass || !Field.FieldType.Name.EndsWith("TargetRules", StringComparison.Ordinal))
				{
					List<string>? Lines;
					if (TryGetXmlComment(InputDocumentation, Field, Logger, out Lines))
					{
						SettingInfo Setting = new SettingInfo(Field.Name, Field.FieldType, Lines);
						if (Field.IsInitOnly)
						{
							ReadOnlySettings.Add(Setting);
						}
						else
						{
							ReadWriteSettings.Add(Setting);
						}
					}
				}
			}

			// ...then read all the properties
			foreach (PropertyInfo Property in RulesType.GetProperties(BindingFlags.Instance | BindingFlags.SetProperty | BindingFlags.Public))
			{
				if (!Property.PropertyType.IsClass || !Property.PropertyType.Name.EndsWith("TargetRules", StringComparison.Ordinal))
				{
					List<string>? Lines;
					if (TryGetXmlComment(InputDocumentation, Property, Logger, out Lines))
					{
						SettingInfo Setting = new SettingInfo(Property.Name, Property.PropertyType, Lines);
						if (Property.SetMethod == null)
						{
							ReadOnlySettings.Add(Setting);
						}
						else
						{
							ReadWriteSettings.Add(Setting);
						}
					}
				}
			}

			// Make sure the output file is writable
			if (FileReference.Exists(OutputFile))
			{
				FileReference.MakeWriteable(OutputFile);
			}
			else
			{
				DirectoryReference.CreateDirectory(OutputFile.Directory);
			}

			// Generate the documentation file
			if (OutputFile.HasExtension(".udn"))
			{
				WriteDocumentationUDN(OutputFile, ReadOnlySettings, ReadWriteSettings);
			}
			else if (OutputFile.HasExtension(".html"))
			{
				WriteDocumentationHTML(OutputFile, ReadOnlySettings, ReadWriteSettings);
			}
			else
			{
				throw new BuildException("Unable to detect format from extension of output file ({0})", OutputFile);
			}

			// Success!
			Logger.LogInformation("Written documentation to {OutputFile}.", OutputFile);
		}

		static void WriteDocumentationUDN(FileReference OutputFile, List<SettingInfo> ReadOnlySettings, List<SettingInfo> ReadWriteSettings)
		{
			// Generate the UDN documentation file
			using (StreamWriter Writer = new StreamWriter(OutputFile.FullName))
			{
				Writer.WriteLine("Availability: NoPublish");
				Writer.WriteLine("Title: Build Configuration Properties Page");
				Writer.WriteLine("Crumbs:");
				Writer.WriteLine("Description: This is a procedurally generated markdown page.");
				Writer.WriteLine("Version: {0}.{1}", ReadOnlyBuildVersion.Current.MajorVersion, ReadOnlyBuildVersion.Current.MinorVersion);
				Writer.WriteLine("");
				if (ReadOnlySettings.Count > 0)
				{
					Writer.WriteLine("### Read-Only Properties");
					Writer.WriteLine();
					foreach (SettingInfo Field in ReadOnlySettings)
					{
						WriteFieldUDN(Field, Writer);
					}
					Writer.WriteLine();
				}
				if (ReadWriteSettings.Count > 0)
				{
					Writer.WriteLine("### Read/Write Properties");
					foreach (SettingInfo Field in ReadWriteSettings)
					{
						WriteFieldUDN(Field, Writer);
					}
					Writer.WriteLine("");
				}
			}
		}

		/// <summary>
		/// Gets the XML comment for a particular field
		/// </summary>
		/// <param name="Documentation">The XML documentation</param>
		/// <param name="Field">The member to search for</param>
		/// <param name="Logger">Logger for output</param>
		/// <param name="Lines">Receives the description for the requested field</param>
		/// <returns>True if a comment was found for the field</returns>
		public static bool TryGetXmlComment(XmlDocument Documentation, FieldInfo Field, ILogger Logger, [NotNullWhen(true)] out List<string>? Lines)
		{
			return TryGetXmlComment(Documentation, $"F:{Field.DeclaringType}.{Field.Name}", Logger, out Lines);
		}

		/// <summary>
		/// Gets the XML comment for a particular field
		/// </summary>
		/// <param name="Documentation">The XML documentation</param>
		/// <param name="Property">The member to search for</param>
		/// <param name="Logger">Logger for output</param>
		/// <param name="Lines">Receives the description for the requested field</param>
		/// <returns>True if a comment was found for the field</returns>
		public static bool TryGetXmlComment(XmlDocument Documentation, PropertyInfo Property, ILogger Logger, [NotNullWhen(true)] out List<string>? Lines)
		{
			return TryGetXmlComment(Documentation, $"P:{Property.DeclaringType}.{Property.Name}", Logger, out Lines);
		}

		/// <summary>
		/// Gets the XML comment for a particular field
		/// </summary>
		/// <param name="Documentation">The XML documentation</param>
		/// <param name="Member">The member to search for (field or property)</param>
		/// <param name="Logger">Logger for output</param>
		/// <param name="Lines">Receives the description for the requested member</param>
		/// <returns>True if a comment was found for the member</returns>
		public static bool TryGetXmlComment(XmlDocument Documentation, MemberInfo Member, ILogger Logger, [NotNullWhen(true)] out List<string>? Lines)
		{
			if (Member is FieldInfo)
			{
				return TryGetXmlComment(Documentation, (Member as FieldInfo)!, Logger, out Lines);
			}
			else if (Member is PropertyInfo)
			{
				return TryGetXmlComment(Documentation, (Member as PropertyInfo)!, Logger, out Lines);
			}

			Lines = null;

			return false;
		}

		/// <summary>
		/// Gets the XML comment for a particular field
		/// </summary>
		/// <param name="Documentation">The XML documentation</param>
		/// <param name="MemberName">The member to search for</param>
		/// <param name="Logger">Logger for output</param>
		/// <param name="Lines">Receives the description for the requested field</param>
		/// <returns>True if a comment was found for the field</returns>
		public static bool TryGetXmlComment(XmlDocument Documentation, string MemberName, ILogger Logger, [NotNullWhen(true)] out List<string>? Lines)
		{
			HashSet<string> VisitedProperties = new HashSet<string>(StringComparer.Ordinal);

			bool bExclude = false;

			XmlNode? Node = null;
			for (; ; )
			{
				// Nested types are delineated with '+' in Type.FullName, but generated Documentation.xml uses '.' in all cases
				string XmlMemberName = MemberName.Replace('+', '.');

				Node = Documentation.SelectSingleNode($"//member[@name='{XmlMemberName}']/exclude");
				if (Node != null)
				{
					bExclude = true;
					break;
				}

				Node = Documentation.SelectSingleNode($"//member[@name='{XmlMemberName}']/summary");
				if (Node != null)
				{
					break;
				}

				XmlNode? InheritNode = Documentation.SelectSingleNode($"//member[@name='{XmlMemberName}']/inheritdoc/@cref");
				if (InheritNode == null || !VisitedProperties.Add(InheritNode.InnerText))
				{
					break;
				}

				MemberName = InheritNode.InnerText;
			}

			if (Node != null && !bExclude)
			{
				// Reflow the comments into paragraphs, assuming that each paragraph will be separated by a blank line
				List<string> NewLines = new List<string>(Node.InnerText.Trim().Split('\n').Select(x => x.Trim()));
				for (int Idx = NewLines.Count - 1; Idx > 0; Idx--)
				{
					if (NewLines[Idx - 1].Length > 0 && !NewLines[Idx].StartsWith("*", StringComparison.Ordinal) && !NewLines[Idx].StartsWith("-", StringComparison.Ordinal))
					{
						NewLines[Idx - 1] += " " + NewLines[Idx];
						NewLines.RemoveAt(Idx);
					}
				}

				if (NewLines.Count > 0)
				{
					Lines = NewLines;
					return true;
				}
			}

			if (!bExclude)
			{
				Logger.LogWarning("Missing XML comment for {Member}", Regex.Replace(MemberName, @"^[A-Z]:", ""));
			}
			Lines = null;
			return false;
		}

		static void WriteFieldUDN(SettingInfo Field, TextWriter Writer)
		{
			// Write the values of the enum
			/*				if(Field.FieldType.IsEnum)
							{
								Lines.Add("Valid values are:");
								foreach(string Value in Enum.GetNames(Field.FieldType))
								{
									Lines.Add(String.Format("* {0}.{1}", Field.FieldType.Name, Value));
								}
							}
			*/

			List<string> Lines = Field.Description;

			// Write the result to the .udn file
			Writer.WriteLine("$ {0} ({1}): {2}", Field.Name, GetPrettyTypeName(Field.Type), Lines[0]);
			for (int Idx = 1; Idx < Lines.Count; Idx++)
			{
				if (Lines[Idx].StartsWith("*", StringComparison.Ordinal) || Lines[Idx].StartsWith("-", StringComparison.Ordinal))
				{
					Writer.WriteLine("        * {0}", Lines[Idx].Substring(1).TrimStart());
				}
				else
				{
					Writer.WriteLine("    * {0}", Lines[Idx]);
				}
			}
			Writer.WriteLine();
		}

		static void WriteDocumentationHTML(FileReference OutputFile, List<SettingInfo> ReadOnlySettings, List<SettingInfo> ReadWriteSettings)
		{
			using (StreamWriter Writer = new StreamWriter(OutputFile.FullName))
			{
				Writer.WriteLine("<html>");
				Writer.WriteLine("  <body>");
				if (ReadOnlySettings.Count > 0)
				{
					Writer.WriteLine("    <h2>Read-Only Properties</h2>");
					Writer.WriteLine("    <dl>");
					foreach (SettingInfo Setting in ReadOnlySettings)
					{
						WriteFieldHTML(Setting, Writer);
					}
					Writer.WriteLine("    </dl>");
				}
				if (ReadWriteSettings.Count > 0)
				{
					Writer.WriteLine("    <h2>Read/Write Properties</h2>");
					Writer.WriteLine("    <dl>");
					foreach (SettingInfo Setting in ReadWriteSettings)
					{
						WriteFieldHTML(Setting, Writer);
					}
					Writer.WriteLine("    </dl>");
				}
				Writer.WriteLine("  </body>");
				Writer.WriteLine("</html>");
			}
		}

		static void WriteFieldHTML(SettingInfo Setting, TextWriter Writer)
		{

			// Write the values of the enum
			/*				if(Field.FieldType.IsEnum)
							{
								Lines.Add("Valid values are:");
								foreach(string Value in Enum.GetNames(Field.FieldType))
								{
									Lines.Add(String.Format("* {0}.{1}", Field.FieldType.Name, Value));
								}
							}
			*/
			// Write the result to the HTML file
			List<string> Lines = Setting.Description;
			if (Lines.Count > 0)
			{
				Writer.WriteLine("      <dt>{0} ({1})</dt>", Setting.Name, GetPrettyTypeName(Setting.Type));

				if (Lines.Count == 1)
				{
					Writer.WriteLine("      <dd>{0}</dd>", Lines[0]);
				}
				else
				{
					Writer.WriteLine("      <dd>");
					for (int Idx = 0; Idx < Lines.Count; Idx++)
					{
						if (Lines[Idx].StartsWith("*", StringComparison.Ordinal) || Lines[Idx].StartsWith("-", StringComparison.Ordinal))
						{
							Writer.WriteLine("        <ul>");
							for (; Idx < Lines.Count && (Lines[Idx].StartsWith("*", StringComparison.Ordinal) || Lines[Idx].StartsWith("-", StringComparison.Ordinal)); Idx++)
							{
								Writer.WriteLine("          <li>{0}</li>", Lines[Idx].Substring(1).TrimStart());
							}
							Writer.WriteLine("        </ul>");
						}
						else
						{
							Writer.WriteLine("        {0}", Lines[Idx]);
						}
					}
					Writer.WriteLine("      </dd>");
				}
			}
		}

		static string GetPrettyTypeName(Type FieldType)
		{
			if (FieldType.IsGenericType)
			{
				return String.Format("{0}&lt;{1}&gt;", FieldType.Name.Substring(0, FieldType.Name.IndexOf('`', StringComparison.Ordinal)), String.Join(", ", FieldType.GenericTypeArguments.Select(x => GetPrettyTypeName(x))));
			}
			else
			{
				return FieldType.Name;
			}
		}
	}
}
