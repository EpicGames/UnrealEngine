// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace UnrealBuildTool
{
	/// <summary>
	/// Represents a layer within the rules hierarchy. Any module is created within a certain scope, and may only reference modules in an equal or parent scope (eg. engine modules cannot reference project modules).
	/// </summary>
	class RulesScope
	{
		/// <summary>
		/// Name of this scope
		/// </summary>
		public string Name;

		/// <summary>
		/// The parent scope
		/// </summary>
		public RulesScope? Parent;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="Name">Name of this scope</param>
		/// <param name="Parent">The parent scope</param>
		public RulesScope(string Name, RulesScope? Parent)
		{
			this.Name = Name;
			this.Parent = Parent;
		}

		/// <summary>
		/// Checks whether this scope contains another scope
		/// </summary>
		/// <param name="Other">The other scope to check</param>
		/// <returns>True if this scope contains the other scope</returns>
		public bool Contains(RulesScope Other)
		{
			for (RulesScope? Scope = this; Scope != null; Scope = Scope.Parent)
			{
				if (Scope == Other)
				{
					return true;
				}
			}
			return false;
		}

		/// <summary>
		/// Formats the hierarchy of scopes
		/// </summary>
		/// <returns>String representing the hierarchy of scopes</returns>
		public string FormatHierarchy()
		{
			if (Parent == null)
			{
				return Name;
			}
			else
			{
				return String.Format("{0} -> {1}", Name, Parent.FormatHierarchy());
			}
		}
	}
}
