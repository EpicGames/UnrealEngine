// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace UnrealBuildTool
{
	public abstract partial class TargetRules
	{
		/// <summary>
		/// Whether the target uses Steam.
		/// </summary>
		[Obsolete("Deprecated in UE5.5 - No longer used in engine.")]
		public bool bUsesSteam { get; set; }

		/// <summary>
		/// Whether to compile the Chaos physics plugin.
		/// </summary>
		[Obsolete("Deprecated in UE5.1 - No longer used as Chaos is always enabled.")]
		public bool bCompileChaos { get; set; }

		/// <summary>
		/// Whether to use the Chaos physics interface. This overrides the physx flags to disable APEX and NvCloth
		/// </summary>
		[Obsolete("Deprecated in UE5.1 - No longer used as Chaos is always enabled.")]
		public bool bUseChaos { get; set; }

		/// <summary>
		/// Whether scene query acceleration is done by UE. The physx scene query structure is still created, but we do not use it.
		/// </summary>
		[Obsolete("Deprecated in UE5.1 - No longer used in engine.")]
		public bool bCustomSceneQueryStructure { get; set; }

		/// <summary>
		/// Whether to include PhysX support.
		/// </summary>
		[Obsolete("Deprecated in UE5.1 - No longer used as Chaos is always enabled.")]
		public bool bCompilePhysX { get; set; }

		/// <summary>
		/// Whether to include PhysX APEX support.
		/// </summary>
		[Obsolete("Deprecated in UE5.1 - No longer used as Chaos is always enabled.")]
		public bool bCompileAPEX { get; set; }

		/// <summary>
		/// Whether to include NvCloth.
		/// </summary>
		[Obsolete("Deprecated in UE5.1 - No longer used as Chaos is always enabled.")]
		public bool bCompileNvCloth { get; set; }

		/// <summary>
		/// Whether to compile IntelMetricsDiscovery.
		/// </summary>
		[Obsolete("Deprecated in UE5.4 - No longer used.")]
		public bool bCompileIntelMetricsDiscovery { get; set; } = false;

		/// <summary>
		/// Obsolete: whether to enable support for DirectX Math
		/// Not compatible with LWC so has been disabled since UE 5.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[Obsolete("Deprecated in UE 5.6 - DirectXMath path was incompatible with LWC and unused since 5.0")]
		public bool bWithDirectXMath { get; set; }

		/// <summary>
		/// True if we want to favor optimizing size over speed.
		/// </summary>
		[Obsolete("Deprecated in UE5.1 - Please use OptimizationLevel instead.")]
		public bool bCompileForSize { get; set; }

		/// <summary>
		/// Whether to merge module and generated unity files for faster compilation.
		/// </summary>
		[Obsolete("Deprecated in UE5.3 - use DisableMergingModuleAndGeneratedFilesInUnityFiles instead")]
		public bool bMergeModuleAndGeneratedUnityFiles { get; set; }

		/// <summary>
		/// Whether to add additional information to the unity files, such as '_of_X' in the file name. Not recommended.
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		[Obsolete("Deprecated in UE5.6 - Caused unnecessary rebuilds when generated unity files change")]
		public bool bDetailedUnityFiles { get; set; }

		/// <summary>
		/// Whether to use the :FASTLINK option when building with /DEBUG to create local PDBs on Windows. No longer recommended.
		/// </summary>
		[Obsolete("Deprecated in UE5.7 - No longer recommended")]
		public bool? bUseFastPDBLinking { get; set; }

		/// <summary>
		/// Whether to globally disable debug info generation; Obsolete, please use TargetRules.DebugInfoMode instead
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration", Deprecated = true)]
		[Obsolete("Deprecated in UE5.4 - Replace with TargetRules.DebugInfo")]
		public bool bDisableDebugInfo
		{
			get => DebugInfo == DebugInfoMode.None;
			set => DebugInfo = value ? DebugInfo = DebugInfoMode.None : DebugInfo = DebugInfoMode.Full;
		}

		/// <summary>
		/// Generate dependency files by preprocessing. Obsolete.
		/// </summary>
		[Obsolete("Deprecated in UE5.6 - bPreprocessDepends is no longer supported")]
		public bool bPreprocessDepends { get; set; }

		/// <summary>
		/// Whether to enable support for C++20 coroutines
		/// This option is provided to facilitate evaluation of the feature.
		/// Expect the name of the option to change. Use of coroutines in with UE is untested and unsupported.
		/// </summary>
		[Obsolete("CppCoroutines are not supported")]
		public bool bEnableCppCoroutinesForEvaluation { get; set; }

		/// <summary>
		/// If true, then enable memory profiling in the build (defines USE_MALLOC_PROFILER=1 and forces bOmitFramePointers=false).
		/// </summary>
		[Obsolete("Deprecated in UE5.3 - No longer used as MallocProfiler is removed in favor of UnrealInsights.")]
		public bool bUseMallocProfiler { get; set; }

		/// <summary>
		/// Disable supports for inlining gen.cpps
		/// </summary>
		[Obsolete("Deprecated in UE5.3 - the engine code relies on inlining generated cpp files to compile")]
		public bool bDisableInliningGenCpps { get; set; }

		/// <summary>
		/// How to treat conflicts when a disabled plugin is being enabled by another plugin referencing it
		/// </summary>
		[Obsolete("Deprecated in UE5.6 - Replace with TargetRules.CppCompileWarningSettings.DisablePluginsConflictWarningLevel")]
		public WarningLevel DisablePluginsConflictWarningLevel
		{
			get => CppCompileWarningSettings.DisablePluginsConflictWarningLevel;
			set => CppCompileWarningSettings.DisablePluginsConflictWarningLevel = value;
		}

		/// <summary>
		/// Level to report deprecation warnings as errors
		/// </summary>
		[Obsolete("Deprecated in UE5.6 - Replace with TargetRules.CppCompileWarningSettings.DeprecationWarningLevel")]
		public WarningLevel DeprecationWarningLevel
		{
			get => CppCompileWarningSettings.DeprecationWarningLevel;
			set => CppCompileWarningSettings.DeprecationWarningLevel = value;
		}

		/// <summary>
		/// Forces shadow variable warnings to be treated as errors on platforms that support it.
		/// </summary>
		[Obsolete("Deprecated in UE5.6 - Replace with TargetRules.CppCompileWarningSettings.ShadowVariableWarningLevel")]
		public WarningLevel ShadowVariableWarningLevel
		{
			get => CppCompileWarningSettings.ShadowVariableWarningLevel;
			set => CppCompileWarningSettings.ShadowVariableWarningLevel = value;
		}

		/// <summary>
		/// Indicates what warning/error level to treat unsafe type casts as on platforms that support it (e.g., double->float or int64->int32)
		/// </summary>
		[Obsolete("Deprecated in UE5.6 - Replace with TargetRules.CppCompileWarningSettings.UnsafeTypeCastWarningLevel")]
		public WarningLevel UnsafeTypeCastWarningLevel
		{
			get => CppCompileWarningSettings.UnsafeTypeCastWarningLevel;
			set => CppCompileWarningSettings.UnsafeTypeCastWarningLevel = value;
		}

		/// <summary>
		/// Forces the use of undefined identifiers in conditional expressions to be treated as errors.
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration", Deprecated = true)]
		[Obsolete("Deprecated in UE5.5 - Replace with TargetRules.UndefinedIdentifierWarningLevel")]
		public bool bUndefinedIdentifierErrors { get; set; } = true;

		/// <summary>
		/// Indicates what warning/error level to treat undefined identifiers in conditional expressions.
		/// </summary>
		[Obsolete("Deprecated in UE5.6 - Replace with TargetRules.CppCompileWarningSettings.UndefinedIdentifierWarningLevel")]
		public WarningLevel UndefinedIdentifierWarningLevel
		{
			get => CppCompileWarningSettings.UndefinedIdentifierWarningLevel;
			set => CppCompileWarningSettings.UndefinedIdentifierWarningLevel = value;
		}

		/// <summary>
		/// Indicates what warning/error level to treat potential PCH performance issues.
		/// </summary>
		[Obsolete("Deprecated in UE5.6 - Replace with TargetRules.CppCompileWarningSettings.PCHPerformanceIssueWarningLevel")]
		public WarningLevel PCHPerformanceIssueWarningLevel
		{
			get => CppCompileWarningSettings.PCHPerformanceIssueWarningLevel;
			set => CppCompileWarningSettings.PCHPerformanceIssueWarningLevel = value;
		}

		/// <summary>
		/// How to treat general module include path validation messages
		/// </summary>
		[Obsolete("Deprecated in UE5.6 - Replace with TargetRules.CppCompileWarningSettings.ModuleIncludePathWarningLevel")]
		public WarningLevel ModuleIncludePathWarningLevel
		{
			get => CppCompileWarningSettings.ModuleIncludePathWarningLevel;
			set => CppCompileWarningSettings.ModuleIncludePathWarningLevel = value;
		}

		/// <summary>
		/// How to treat private module include path validation messages, where a module is adding an include path that exposes private headers
		/// </summary>
		[Obsolete("Deprecated in UE5.6 - Replace with TargetRules.CppCompileWarningSettings.ModuleIncludePrivateWarningLevel")]
		public WarningLevel ModuleIncludePrivateWarningLevel
		{
			get => CppCompileWarningSettings.ModuleIncludePrivateWarningLevel;
			set => CppCompileWarningSettings.ModuleIncludePrivateWarningLevel = value;
		}

		/// <summary>
		/// How to treat unnecessary module sub-directory include path validation messages
		/// </summary>
		[Obsolete("Deprecated in UE5.6 - Replace with TargetRules.CppCompileWarningSettings.ModuleIncludeSubdirectoryWarningLevel")]
		public WarningLevel ModuleIncludeSubdirectoryWarningLevel
		{
			get => CppCompileWarningSettings.ModuleIncludeSubdirectoryWarningLevel;
			set => CppCompileWarningSettings.ModuleIncludeSubdirectoryWarningLevel = value;
		}
	}
}