// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using System.Linq;
using System.Reflection;
using System.Runtime.CompilerServices;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// The type of target
	/// </summary>
	[Serializable]
	public enum TargetType
	{
		/// <summary>
		/// Cooked monolithic game executable (GameName.exe).  Also used for a game-agnostic engine executable (UnrealGame.exe or RocketGame.exe)
		/// </summary>
		Game,

		/// <summary>
		/// Uncooked modular editor executable and DLLs (UnrealEditor.exe, UnrealEditor*.dll, GameName*.dll)
		/// </summary>
		Editor,

		/// <summary>
		/// Cooked monolithic game client executable (GameNameClient.exe, but no server code)
		/// </summary>
		Client,

		/// <summary>
		/// Cooked monolithic game server executable (GameNameServer.exe, but no client code)
		/// </summary>
		Server,

		/// <summary>
		/// Program (standalone program, e.g. ShaderCompileWorker.exe, can be modular or monolithic depending on the program)
		/// </summary>
		Program
	}

	/// <summary>
	/// Specifies how to link all the modules in this target
	/// </summary>
	[Serializable]
	public enum TargetLinkType
	{
		/// <summary>
		/// Use the default link type based on the current target type
		/// </summary>
		Default,

		/// <summary>
		/// Link all modules into a single binary
		/// </summary>
		Monolithic,

		/// <summary>
		/// Link modules into individual dynamic libraries
		/// </summary>
		Modular,
	}

	/// <summary>
	/// Specifies whether to share engine binaries and intermediates with other projects, or to create project-specific versions. By default,
	/// editor builds always use the shared build environment (and engine binaries are written to Engine/Binaries/Platform), but monolithic builds
	/// and programs do not (except in installed builds). Using the shared build environment prevents target-specific modifications to the build
	/// environment.
	/// </summary>
	[Serializable]
	public enum TargetBuildEnvironment
	{
		/// <summary>
		/// Engine binaries and intermediates are output to the engine folder. Target-specific modifications to the engine build environment will be ignored.
		/// </summary>
		Shared,

		/// <summary>
		/// Engine binaries and intermediates are specific to this target
		/// </summary>
		Unique,

		/// <summary>
		/// Will switch to Unique if needed - per-project sdk is enabled, or a property that requires unique is set away from default
		/// </summary>
		UniqueIfNeeded,
	}

	/// <summary>
	/// Specifies how UnrealHeaderTool should enforce member pointer declarations in UCLASSes and USTRUCTs.  This should match (by name, not value) the
	/// EPointerMemberBehavior enum in BaseParser.h so that it can be interpreted correctly by UnrealHeaderTool.
	/// </summary>
	[Serializable]
	public enum PointerMemberBehavior
	{
		/// <summary>
		/// Pointer members of the associated type will be disallowed and result in an error emitted from UnrealHeaderTool.
		/// </summary>
		Disallow,

		/// <summary>
		/// Pointer members of the associated type will be allowed and not emit any messages to log or screen.
		/// </summary>
		AllowSilently,

		/// <summary>
		/// Pointer members of the associated type will be allowed but will emit messages to log.
		/// </summary>
		AllowAndLog,
	}

	/// <summary>
	/// Determines which version of the engine to take default build settings from. This allows for backwards compatibility as new options are enabled by default.
	/// </summary>
#pragma warning disable CA1027 // Mark enums with FlagsAttribute
	public enum BuildSettingsVersion
#pragma warning restore CA1027 // Mark enums with FlagsAttribute
	{
		/// <summary>
		/// Legacy default build settings for 4.23 and earlier.
		/// </summary>
		V1,

		/// <summary>
		/// New defaults for 4.24:
		/// * ModuleRules.PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs
		/// * ModuleRules.bLegacyPublicIncludePaths = false
		/// </summary>
		V2,

		/// <summary>
		/// New defaults for 5.2:
		/// * ModuleRules.bLegacyParentIncludePaths = false
		/// </summary>
		V3,

		/// <summary>
		/// New defaults for 5.3:
		/// * TargetRules.CppStandard = CppStandardVersion.Default has changed from Cpp17 to Cpp20
		/// * TargetRules.WindowsPlatform.bStrictConformanceMode = true
		/// </summary>
		V4,

		/// <summary>
		/// New defaults for 5.4:
		/// * TargetRules.bValidateFormatStrings = true
		/// </summary>
		V5,

		/// <summary>
		/// New defaults for 5.7:
		/// * TargetRules.WindowsPlatform.bStrictInlineConformance = true
		/// * ModuleRules.CppCompileWarningSettings.UndefinedIdentifierWarningLevel = WarningLevel.Error
		/// </summary>
		V6,

		/// <summary>
		/// New defaults for 5.8:
		/// * ModuleRules.CppCompileWarningSettings.ReturnTypeWarningLevel = WarningLevel.Error
		/// * ModuleRules.CppCompileWarningSettings.DanglingWarningLevel = WarningLevel.Error
		/// * ModuleRules.CppCompileWarningSettings.UnreachableCodeWarningLevel = WarningLevel.Error
		/// </summary>
		V7,

		// *** When adding new entries here, be sure to update GameProjectUtils::GetDefaultBuildSettingsVersion() to ensure that new projects are created correctly. ***

		/// <summary>
		/// Always use the defaults for the current engine version. Note that this may cause compatibility issues when upgrading.
		/// </summary>
		Latest = V7,
	}

	/// <summary>
	/// What version of include order to use when compiling. This controls which UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_X defines are enabled when compiling the target.
	/// The UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_X defines are used when removing implicit includes from Unreal Public headers.
	/// Specific versions can be used by licensees to avoid compile errors when integrating new versions of Unreal.
	/// Using Latest comes with a high risk of introducing compile errors in your code on newer Unreal versions.
	/// </summary>
#pragma warning disable CA1707 // Identifiers should not contain underscores - public facing API
#pragma warning disable CA1027 // Mark enums with FlagsAttribute
	public enum EngineIncludeOrderVersion
#pragma warning restore CA1027 // Mark enums with FlagsAttribute
	{
		/// <summary>
		/// Include order used in Unreal 5.1
		/// </summary>
		[Obsolete("The Unreal 5.1 include order is unsupported.")]
		Unreal5_1,

		/// <summary>
		/// Include order used in Unreal 5.2
		/// </summary>
		[Obsolete("The Unreal 5.2 include order is unsupported.")]
		Unreal5_2,

		/// <summary>
		/// Include order used in Unreal 5.3
		/// </summary>
		[Obsolete("The Unreal 5.3 include order is unsupported.")]
		Unreal5_3,

		/// <summary>
		/// Include order used in Unreal 5.4
		/// </summary>
		[Obsolete("The Unreal 5.4 include order is unsupported.")]
		Unreal5_4,

		/// <summary>
		/// Include order used in Unreal 5.5
		/// </summary>
		[Obsolete("The Unreal 5.5 include order is unsupported.")]
		Unreal5_5,

		/// <summary>
		/// Include order used in Unreal 5.6
		/// </summary>
		[Obsolete("The Unreal 5.6 include order is deprecated and will be unsupported in 5.9.")]
		Unreal5_6,

		/// <summary>
		/// Include order used in Unreal 5.7
		/// </summary>
		Unreal5_7,

		/// <summary>
		/// Include order used in Unreal 5.8
		/// </summary>
		Unreal5_8,

		// *** When adding new entries here, be sure to update EngineIncludeOrderHelper.GetDeprecationDefine to ensure that the correct guard is used. ***

		/// <summary>
		/// Always use the latest version of include order. This value is updated every Unreal release, use with caution if you intend to integrate newer Unreal releases.
		/// </summary>
		Latest = Unreal5_8,

		/// <summary>
		/// Contains the oldest version of include order that the engine supports.
		/// Do not delete old enum entries to prevent breaking project generation
		/// </summary>
#pragma warning disable CS0618 // Type or member is obsolete
		Oldest = Unreal5_6,
#pragma warning restore CS0618 // Type or member is obsolete
	}
#pragma warning restore CA1707

	/// <summary>
	/// Which static analyzer to use
	/// </summary>
	public enum StaticAnalyzer
	{
		/// <summary>
		/// Do not perform static analysis
		/// </summary>
		None,

		/// <summary>
		/// Use the default static analyzer for the selected compiler, if it has one. For
		/// Visual Studio and Clang, this means using their built-in static analysis tools.
		/// Any compiler that doesn't support static analysis will ignore this option.
		/// </summary>
		Default,

		/// <summary>
		/// Use the built-in Visual C++ static analyzer
		/// </summary>
		VisualCpp,

		/// <summary>
		/// Use PVS-Studio for static analysis
		/// </summary>
		PVSStudio,

		/// <summary>
		/// Use clang for static analysis. This forces the compiler to clang.
		/// </summary>
		Clang,
	}

	/// <summary>
	/// Output type for the static analyzer. This currently only works for the Clang static analyzer.
	/// The Clang static analyzer can do either Text, which prints the analysis to stdout, or
	/// html, where it writes out a navigable HTML page for each issue that it finds, per file.
	/// The HTML is output in the same directory as the object file that would otherwise have
	/// been generated. 
	/// All other analyzers default automatically to Text. 
	/// </summary>
	public enum StaticAnalyzerOutputType
	{
		/// <summary>
		/// Output the analysis to stdout.
		/// </summary>
		Text,

		/// <summary>
		/// Output the analysis to an HTML file in the object folder.
		/// </summary>
		Html,
	}

	/// <summary>
	/// Output type for the static analyzer. This currently only works for the Clang static analyzer.
	/// The Clang static analyzer can do a shallow quick analysis. However the default deep is recommended.
	/// </summary>
	public enum StaticAnalyzerMode
	{
		/// <summary>
		/// Default deep analysis.
		/// </summary>
		Deep,

		/// <summary>
		/// Quick analysis. Not recommended.
		/// </summary>
		Shallow,
	}

	/// <summary>
	/// Optimization mode for compiler settings
	/// </summary>
	public enum OptimizationMode
	{
		/// <summary>
		/// Favor speed
		/// </summary>
		Speed,

		/// <summary>
		/// Favor minimal code size
		/// </summary>
		Size,

		/// <summary>
		/// Somewhere between Speed and Size
		/// </summary>
		SizeAndSpeed
	}

	/// <summary>
	/// Debug info mode for compiler settings to determine how much debug info is available
	/// Note that in some cases this flag is used for features which strip certain debug info, in which case the meaning is inverted.
	/// i.e. 'Full' removes that type of debug info from all modules
	/// </summary>
	[Flags]
	public enum DebugInfoMode
	{
		/// <summary>
		/// Disable all debugging info.
		/// MSVC: object files will be compiled without /Z7 or /Zi but pdbs will still be created
		///       callstacks should be available in this mode but there have been reports with them being incorrect
		/// </summary>
		None = 0,

		/// <summary>
		/// Enable debug info for engine modules
		/// </summary>
		Engine = 1 << 0,

		/// <summary>
		/// Enable debug info for engine plugins
		/// </summary>
		EnginePlugins = 1 << 1,

		/// <summary>
		/// Enable debug info for project modules
		/// </summary>
		Project = 1 << 2,

		/// <summary>
		/// Enable debug info for project plugins
		/// </summary>
		ProjectPlugins = 1 << 3,

		/// <summary>
		/// Only include debug info for engine modules and plugins
		/// </summary>
		EngineOnly = Engine | EnginePlugins,

		/// <summary>
		/// Only include debug info for project modules and project plugins
		/// </summary>
		ProjectOnly = Project | ProjectPlugins,

		/// <summary>
		/// Include full debugging information for all modules
		/// </summary>
		Full = Engine | EnginePlugins | Project | ProjectPlugins,
	}

	/// <summary>
	/// Floating point math semantics
	/// </summary>
	public enum FPSemanticsMode
	{
		/// <summary>
		/// Use the default semantics for the platform.
		/// </summary>
		Default,

		/// <summary>
		/// FP math is IEEE-754 compliant, assuming that FP exceptions are disabled and the rounding
		/// mode is round-to-nearest-even.
		/// </summary>
		Precise,

		/// <summary>
		/// FP math isn't IEEE-754 compliant: the compiler is allowed to transform math expressions
		/// in a ways that might result in differently rounded results from what IEEE-754 requires.
		/// </summary>
		Imprecise,
	}

	/// <summary>
	/// Determines how the Gameplay Debugger plugin will be activated.
	/// </summary>
	public enum GameplayDebuggerOverrideState
	{
		/// <summary>
		/// Default => Not overriden, default usage behavior.
		/// </summary>
		Default,

		/// <summary>
		/// Core => WITH_GAMEPLAY_DEBUGGER = 0 and WITH_GAMEPLAY_DEBUGGER_CORE = 1
		/// </summary>
		Core,

		/// <summary>
		/// Full => WITH_GAMEPLAY_DEBUGGER = 1 and WITH_GAMEPLAY_DEBUGGER_CORE = 1
		/// </summary>
		Full
	}

	/// <summary>
	/// List of supported allocators to select in compile time for target.
	/// Can be none for Desktop\Editor targets where allocator can be changed with command line.
	/// </summary>
	public enum StaticAllocatorType
	{
		/// <summary>
		/// Default C allocator
		/// </summary>
		Ansi,

		/// <summary>
		/// Binned malloc
		/// </summary>
		Binned2,

		/// <summary>
		/// Newer VM-based binned malloc, 64 bit only
		/// </summary>
		Binned3,

		/// <summary>
		/// Default for Desktop\Editor targets that allows allocator selection to be overridden with a command line
		/// </summary>
		None,
	}

	/// <summary>
	/// Utility class for EngineIncludeOrderVersion defines
	/// </summary>
	public static class EngineIncludeOrderHelper
	{
		private static string GetDeprecationDefine(EngineIncludeOrderVersion inVersion) => inVersion switch
		{
			EngineIncludeOrderVersion.Unreal5_8 => "UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8",
			EngineIncludeOrderVersion.Unreal5_7 => "UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7",
#pragma warning disable CS0618 // Type or member is obsolete
			EngineIncludeOrderVersion.Unreal5_6 => "UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6",
#pragma warning restore CS0618 // Type or member is obsolete
			_ => throw new BuildLogEventException("Unsupported EngineIncludeOrderVersion {Value}", inVersion)
		};

		private static string GetDeprecationDefine(EngineIncludeOrderVersion inTestVersion, EngineIncludeOrderVersion inVersion) => $"{GetDeprecationDefine(inTestVersion)}={(inVersion < inTestVersion ? "1" : "0")}";

		/// <summary>
		/// Returns a list of every deprecation define available.
		/// </summary>
		/// <returns></returns>
		public static IEnumerable<string> GetAllDeprecationDefines() => Enum.GetValues<EngineIncludeOrderVersion>()
			.Where(x => x >= EngineIncludeOrderVersion.Oldest)
			.Select(GetDeprecationDefine)
			.Distinct()
			.Order();

		/// <summary>
		/// Get a list of every deprecation define and their value for the specified engine include order.
		/// </summary>
		/// <param name="inVersion"></param>
		/// <returns></returns>
		public static IEnumerable<string> GetDeprecationDefines(EngineIncludeOrderVersion inVersion) => Enum.GetValues<EngineIncludeOrderVersion>()
			.Where(x => x >= EngineIncludeOrderVersion.Oldest)
			.Select(x => GetDeprecationDefine(x, inVersion))
			.Distinct()
			.Order();

		/// <summary>
		/// Returns the latest deprecation define.
		/// </summary>
		/// <returns></returns>
		public static string GetLatestDeprecationDefine() => GetDeprecationDefine(EngineIncludeOrderVersion.Latest);
	}

	/// <summary>
	/// Attribute used to mark fields which must match between targets in the shared build environment
	/// </summary>
	[AttributeUsage(AttributeTargets.Field | AttributeTargets.Property, AllowMultiple = false)]
	sealed class RequiresUniqueBuildEnvironmentAttribute : Attribute
	{
	}

	/// <summary>
	/// Attribute used to mark configurable sub-objects
	/// </summary>
	[AttributeUsage(AttributeTargets.Field | AttributeTargets.Property, AllowMultiple = false)]
	sealed class ConfigSubObjectAttribute : Attribute
	{
	}

	/// <summary>
	/// Read-only additional properties. Will fallback to real environment variables if not found in internal lookup
	/// </summary>
	public class ReadOnlyAdditionalProperties
	{
		/// <summary>
		/// Get property
		/// </summary>
		public string? GetProperty(string name)
		{
			if (_properties.TryGetValue(name, out string? value))
			{
				return value;
			}
			return Environment.GetEnvironmentVariable(name);
		}

		/// <summary>
		/// Internal name->value property lookup
		/// </summary>
		protected Dictionary<string, string?> _properties = [];
	}

	/// <summary>
	/// Writable additional properties.
	/// </summary>
	public sealed class AdditionalProperties : ReadOnlyAdditionalProperties
	{
		/// <summary>
		/// Set property
		/// </summary>
		public void SetProperty(string name, string value) { _properties[name] = value; }

		/// <summary>
		/// Remove property
		/// </summary>
		public void RemoveProperty(string name) { _properties[name] = null; }
	}

	/// <summary>
	/// TargetRules is a data structure that contains the rules for defining a target (application/executable)
	/// </summary>
	public abstract partial class TargetRules
	{
#pragma warning disable CA1034 // Nested types should not be visible - preserve API of APIs exposed to Rules files
		/// <summary>
		/// Static class wrapping constants aliasing the global TargetType enum.
		/// </summary>
		public static class TargetType
		{
			/// <summary>
			/// Alias for TargetType.Game
			/// </summary>
			public const global::UnrealBuildTool.TargetType Game = global::UnrealBuildTool.TargetType.Game;

			/// <summary>
			/// Alias for TargetType.Editor
			/// </summary>
			public const global::UnrealBuildTool.TargetType Editor = global::UnrealBuildTool.TargetType.Editor;

			/// <summary>
			/// Alias for TargetType.Client
			/// </summary>
			public const global::UnrealBuildTool.TargetType Client = global::UnrealBuildTool.TargetType.Client;

			/// <summary>
			/// Alias for TargetType.Server
			/// </summary>
			public const global::UnrealBuildTool.TargetType Server = global::UnrealBuildTool.TargetType.Server;

			/// <summary>
			/// Alias for TargetType.Program
			/// </summary>
			public const global::UnrealBuildTool.TargetType Program = global::UnrealBuildTool.TargetType.Program;
		}
#pragma warning restore CA1034

		/// <summary>
		/// The name of this target
		/// </summary>
		public string Name
		{
			get => !String.IsNullOrEmpty(NameOverride) ? NameOverride : DefaultName;
			set => NameOverride = value;
		}

		/// <summary>
		/// If the Name of this target has been overriden
		/// </summary>
		public bool IsNameOverriden() => !String.IsNullOrEmpty(NameOverride);

		/// <summary>
		/// Return the original target name without overrides or adornments
		/// </summary>
		public string OriginalName => DefaultName;

		/// <summary>
		/// Override the name used for this target
		/// </summary>
		[CommandLine("-TargetNameOverride=")]
		private string? NameOverride;

		private readonly string DefaultName;

		/// <summary>
		/// Whether this is a low level tests target.
		/// </summary>
		public bool IsTestTarget => bIsTestTargetOverride;
		/// <summary>
		/// Override this boolean flag in inheriting classes for low level test targets.
		/// </summary>
		protected bool bIsTestTargetOverride { get; set; }

#pragma warning disable CA2227
		/// <summary>
		/// The selection of test modules to be exclusively included in compilation (no other modules tests will be included). 
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		public List<string> SelectedTestModuleNames { get; protected set; } = [];
#pragma warning restore CA2227

		/// <summary>
		/// Whether this is a test target explicitly defined.
		/// Explicitley defined test targets always inherit from TestTargetRules and define their own tests.
		/// Implicit test targets are created from existing targets when building with -Mode=Test and they include tests from all dependencies.
		/// </summary>
		public bool ExplicitTestsTarget => GetType().IsSubclassOf(typeof(TestTargetRules));

		/// <summary>
		/// Controls the value of WITH_LOW_LEVEL_TESTS that dictates whether module-specific low level tests are compiled in or not.
		/// </summary>
		public bool WithLowLevelTests => (IsTestTarget && !ExplicitTestsTarget) || bWithLowLevelTestsOverride;
		/// <summary>
		/// Override the value of WithLowLevelTests by setting this to true in inherited classes.
		/// </summary>
		protected bool bWithLowLevelTestsOverride { get; set; }

		/// <summary>
		/// File containing the general type for this target (not including platform/group)
		/// </summary>
		internal FileReference? File { get; set; }

		/// <summary>
		/// File containing the platform/group-specific type for this target
		/// </summary>
		internal FileReference? TargetSourceFile { get; set; }

		/// <summary>
		/// All target files that could be referenced by this target and will require updating the Makefile if changed
		/// </summary>
		internal HashSet<FileReference>? TargetFiles { get; set; }

		/// <summary>
		/// Logger for output relating to this target. Set before the constructor is run from <see cref="RulesCompiler"/>
		/// </summary>
		public ILogger Logger { get; internal set; }

		/// <summary>
		/// Generic nullable object so a user can set additional data in a project's TargetRule and access in a project's ModuleRule without needing to add new properties post-release.
		/// For example:
		/// * in .Target.cs:	AdditionalData = "data";
		/// * in .Build.cs:		if ((Target.AdditionalData as string) == "data") { ... }
		/// </summary>
		public object? AdditionalData { get; set; }

		/// <summary>
		/// Platform that this target is being built for.
		/// </summary>
		public UnrealTargetPlatform Platform { get; init; }

		/// <summary>
		/// The configuration being built.
		/// </summary>
		public UnrealTargetConfiguration Configuration { get; init; }

		/// <summary>
		/// Architecture that the target is being built for (or an empty string for the default).
		/// </summary>
		public UnrealArchitectures Architectures { get; init; }

		/// <summary>
		/// Gets the Architecture in the normal case where there is a single Architecture in Architectures
		/// (this will throw an exception if there is more than one architecture specified)
		/// </summary>
		public UnrealArch Architecture => Architectures.SingleArchitecture;

		/// <summary>
		/// Intermediate environment. Determines if the intermediates end up in a different folder than normal.
		/// </summary>
		public UnrealIntermediateEnvironment IntermediateEnvironment { get; init; }

		/// <summary>
		/// Path to the project file for the project containing this target.
		/// </summary>
		public FileReference? ProjectFile { get; init; }

		/// <summary>
		/// The current build version
		/// </summary>
		public ReadOnlyBuildVersion Version { get; init; }

		/// <summary>
		/// The type of target.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		public global::UnrealBuildTool.TargetType Type { get; set; } = global::UnrealBuildTool.TargetType.Game;

		/// <summary>
		/// Specifies the engine version to maintain backwards-compatible default build settings with (eg. DefaultSettingsVersion.Release_4_23, DefaultSettingsVersion.Release_4_24). Specify DefaultSettingsVersion.Latest to always
		/// use defaults for the current engine version, at the risk of introducing build errors while upgrading.
		/// </summary>
		public BuildSettingsVersion DefaultBuildSettings
		{
			get => DefaultBuildSettingsPrivate ?? BuildSettingsVersion.V1;
			set => DefaultBuildSettingsPrivate = value;
		}
		private BuildSettingsVersion? DefaultBuildSettingsPrivate; // Cannot be initialized inline; potentially overridden before the constructor is called.

		/// <summary>
		/// Force the include order to a specific version. Overrides any Target and Module rules.
		/// </summary>
		[CommandLine("-ForceIncludeOrder=")]
		public EngineIncludeOrderVersion? ForcedIncludeOrder { get; set; }

		/// <summary>
		/// What version of include order to use when compiling this target. Can be overridden via -ForceIncludeOrder on the command line. ModuleRules.IncludeOrderVersion takes precedence.
		/// </summary>
		public EngineIncludeOrderVersion IncludeOrderVersion
		{
			get => ForcedIncludeOrder ?? IncludeOrderVersionPrivate ?? EngineIncludeOrderVersion.Oldest;
			set => IncludeOrderVersionPrivate = value;
		}
		private EngineIncludeOrderVersion? IncludeOrderVersionPrivate;

		/// <summary>
		/// Path to the output file for the main executable, relative to the Engine or project directory.
		/// This setting is only typically useful for non-UE programs, since the engine uses paths relative to the executable to find other known folders (eg. Content).
		/// </summary>
		public string? OutputFile { get; set; }

		/// <summary>
		/// Tracks a list of config values read while constructing this target
		/// </summary>
		internal readonly ConfigValueTracker ConfigValueTracker;

		/// <summary>
		/// Whether the target uses CEF3.
		/// </summary>
		public bool bUsesCEF3 { get; set; }

		/// <summary>
		/// Whether the project uses visual Slate UI (as opposed to the low level windowing/messaging, which is always available).
		/// </summary>
		public bool bUsesSlate { get; set; } = true;

		/// <summary>
		/// Forces linking against the static CRT. This is not fully supported across the engine due to the need for allocator implementations to be shared (for example), and TPS
		/// libraries to be consistent with each other, but can be used for utility programs.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		public bool bUseStaticCRT { get; set; }

		/// <summary>
		/// Enables the debug C++ runtime (CRT) for debug builds. By default we always use the release runtime, since the debug
		/// version isn't particularly useful when debugging Unreal Engine projects, and linking against the debug CRT libraries forces
		/// our third party library dependencies to also be compiled using the debug CRT (and often perform more slowly). Often
		/// it can be inconvenient to require a separate copy of the debug versions of third party static libraries simply
		/// so that you can debug your program's code.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bDebugBuildsActuallyUseDebugCRT { get; set; }

		/// <summary>
		/// Whether the output from this target can be publicly distributed, even if it has dependencies on modules that are in folders
		/// with special restrictions (eg. CarefullyRedist, NotForLicensees, NoRedist).
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bLegalToDistributeBinary { get; set; }

		/// <summary>
		/// Specifies the configuration whose binaries do not require a "-Platform-Configuration" suffix.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		public UnrealTargetConfiguration UndecoratedConfiguration { get; set; } = UnrealTargetConfiguration.Development;

		/// <summary>
		/// Specifies the separator charcter for binary filenames.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		public string DecoratedSeparator { get; set; } = "-";

		/// <summary>
		/// Whether this target supports hot reload
		/// </summary>
		public bool bAllowHotReload
		{
			get => bAllowHotReloadOverride ?? (Type == TargetType.Editor && LinkType == TargetLinkType.Modular);
			set => bAllowHotReloadOverride = value;
		}
		private bool? bAllowHotReloadOverride;

		/// <summary>
		/// Build all the modules that are valid for this target type. Used for CIS and making installed engine builds.
		/// </summary>
		[CommandLine("-AllModules")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bBuildAllModules { get; set; }

		/// <summary>
		/// Set this to reference a VSTest run settings file from generated projects.
		/// </summary>
		public FileReference? VSTestRunSettingsFile { get; set; }

		/// <summary>
		/// Additional plugins that are built for this target type but not enabled.
		/// </summary>
		[CommandLine("-BuildPlugin=", ListSeparator = '+')]
		public List<string> BuildPlugins = [];

		/// <summary>
		/// If this is true, then the BuildPlugins list will be used to populate RuntimeDependencies, rather than EnablePlugins
		/// </summary>
		public bool bRuntimeDependenciesComeFromBuildPlugins { get; set; }

		/// <summary>
		/// A list of additional plugins which need to be included in this target. This allows referencing non-optional plugin modules
		/// which cannot be disabled, and allows building against specific modules in program targets which do not fit the categories
		/// in ModuleHostType.
		/// </summary>
		[CommandLine("-AdditionalPlugins=", ListSeparator = '+')]
		public List<string> AdditionalPlugins = [];

		/// <summary>
		/// Additional plugins that should be included for this target.
		/// </summary>
		[CommandLine("-EnablePlugin=", ListSeparator = '+')]
		public List<string> EnablePlugins = [];

		/// <summary>
		/// List of plugins to be disabled for this target. Note that the project file may still reference them, so they should be marked
		/// as optional to avoid failing to find them at runtime.
		/// </summary>
		[CommandLine("-DisablePlugin=", ListSeparator = '+')]
		public List<string> DisablePlugins = [];

		/// <summary>
		/// Additional plugins that should be included for this target if they are found.
		/// </summary>
		[CommandLine("-OptionalPlugins=", ListSeparator = '+')]
		public List<string> OptionalPlugins = [];

		/// <summary>
		/// If false, suppress loading "enabled by default" engine plugins not explicitly enabled by the project or target files.
		/// </summary>
		public bool bAllowEnginePluginsEnabledByDefault { get; set; } = true;

		/// <summary>
		/// A list of Plugin names that are allowed to exist as dependencies without being defined in the uplugin descriptor
		/// </summary>
		public List<string> InternalPluginDependencies = [];

		/// <summary>
		/// Path to the set of pak signing keys to embed in the executable.
		/// </summary>
		public string PakSigningKeysFile { get; set; } = String.Empty;

		/// <summary>
		/// Allows a Program Target to specify it's own solution folder path.
		/// </summary>
		public string SolutionDirectory { get; set; } = String.Empty;

		/// <summary>
		/// Force a Target to be treated as a Program for the purposes of project file generation.
		/// </summary>
		public bool bGenerateProgramProject { get; set; }

		/// <summary>
		/// If true, GetTargetNameByType will not consider this target, thereby disambiguating -TargetType=X.
		/// </summary>
		public bool bExplicitTargetForType { get; set; }

		/// <summary>
		/// Whether the target should be included in the default solution build configuration
		/// Setting this to false will skip building when running in the IDE
		/// </summary>
		public bool? bBuildInSolutionByDefault { get; set; }

		/// <summary>
		/// Whether this target should be compiled as a DLL.  Requires LinkType to be set to TargetLinkType.Monolithic.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[CommandLine("-CompileAsDll")]
		public bool bShouldCompileAsDLL { get; set; }

		/// <summary>
		/// Extra subdirectory to load config files out of, for making multiple types of builds with the same platform
		/// This will get baked into the game executable as CUSTOM_CONFIG and used when staging to filter files and settings
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[CommandLine("-CustomConfig")]
		public string CustomConfig { get; set; } = String.Empty;

		/// <summary>
		/// Subfolder to place executables in, relative to the default location.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[CommandLine("-BinariesSubfolder")]
		public string ExeBinariesSubFolder { get; set; } = String.Empty;

		/// <summary>
		/// Allow target module to override UHT code generation version.
		/// </summary>
		public EGeneratedCodeVersion GeneratedCodeVersion { get; set; } = EGeneratedCodeVersion.None;

		/// <summary>
		/// Whether to enable the mesh editor.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		public bool bEnableMeshEditor { get; set; }

		/// <summary>
		/// Whether to use the BPVM to run Verse.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[CommandLine("-NoUseVerseBPVM", Value = "false")]
		[CommandLine("-UseVerseBPVM", Value = "true")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bUseVerseBPVM { get; set; } = true;

		/// <summary>
		/// Whether to use the AutoRTFM Clang compiler.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		public bool bUseAutoRTFMCompiler
		{
			get => bForceNoAutoRTFMCompiler ? false : _bUseAutoRTFMCompilerPrivate;
			set => _bUseAutoRTFMCompilerPrivate = value;
		}
		private bool _bUseAutoRTFMCompilerPrivate = false;

		/// <summary>
		/// Whether to use force AutoRTFM Clang compiler off.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[CommandLine("-NoUseAutoRTFM", Value = "true")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bForceNoAutoRTFMCompiler { get; set; } = false;

		/// <summary>
		/// Whether to enable emitting AutoRTFM verification metadata
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[CommandLine("-UseAutoRTFMVerifier", Value = "true")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bUseAutoRTFMVerifier { get; set; } = false;

		/// <summary>
		/// Whether to run LLVM verification after the AutoRTFM compiler pass.
		/// This is used by our compiler folks to ensure the pass works with
		/// the various code-paths UBT can take us down.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[CommandLine("-AutoRTFMVerify", Value = "true")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bAutoRTFMVerify { get; set; } = false;

		/// <summary>
		/// Whether to link closed function declarations statically.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[CommandLine("-AutoRTFMClosedStaticLinkage", Value = "true")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bAutoRTFMClosedStaticLinkage { get; set; } = false;

		/// <summary>
		/// Whether to instrument all memory accesses for redirection.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[CommandLine("-AutoRTFMRedirectionInstrumentation", Value = "true")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bAutoRTFMRedirectionInstrumentation { get; set; } = false;

		/// <summary>
		/// Whether to compile in checked chaos features for debugging
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		public bool bUseChaosChecked { get; set; }

		/// <summary>
		/// Whether to compile in chaos memory tracking features
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		public bool bUseChaosMemoryTracking { get; set; }

		/// <summary>
		/// Whether to compile in Chaos Visual Debugger (CVD) support features to record the state of the physics simulation
		/// </summary>
		[XmlConfigFile(Name = "bCompileChaosVisualDebuggerSupport")]
		[CommandLine("-CompileChaosVisualDebuggerSupport")]
		[RequiresUniqueBuildEnvironment]
		public bool bCompileChaosVisualDebuggerSupport 
		{
			get 
			{
				if (bEnableChaosVisualDebuggerSupport.HasValue)
				{
					return bEnableChaosVisualDebuggerSupport.Value;
				}

				// By default, if we didn't explicitly ask for CVD support, we should only enable it in non-shipping builds and only if trace is enabled
				return Configuration != UnrealTargetConfiguration.Shipping && bEnableTrace;
			}  
			set => bEnableChaosVisualDebuggerSupport = value;
		}
		private bool? bEnableChaosVisualDebuggerSupport;

		/// <summary>
		/// Whether to compile in RewindDebugger (RWD) support features to record traces for debug primitives and logs
		/// </summary>
		[XmlConfigFile(Name = "bCompileRewindDebuggerSupport")]
		[CommandLine("-bCompileRewindDebuggerSupport")]
		[RequiresUniqueBuildEnvironment]
		public bool bCompileRewindDebuggerSupport
		{
			get
			{
				if (bEnableRewindDebuggerSupport.HasValue)
				{
					return bEnableRewindDebuggerSupport.Value;
				}

				// By default, if we didn't explicitly ask for RWD support, we should only enable it in non-shipping builds and only if trace is enabled
				return Configuration != UnrealTargetConfiguration.Shipping && bEnableTrace;
			}
			set => bEnableRewindDebuggerSupport = value;
		}
		private bool? bEnableRewindDebuggerSupport;
		
		/// <summary>
		/// Whether to compile in ssl support in the third party asio module
		/// </summary>
		[XmlConfigFile(Name = "bCompileAsioSSLSupport")]
		[CommandLine("-bCompileAsioSSLSupport")]
		[RequiresUniqueBuildEnvironment]
		public bool bCompileAsioSSLSupport { get; set; } = false;

		/// <summary>
		/// Whether to include ICU unicode/i18n support in Core.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/BuildSettings.BuildSettings", "bCompileICU")]
		public bool bCompileICU { get; set; } = true;

		/// <summary>
		/// Whether to compile CEF3 support.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/BuildSettings.BuildSettings", "bCompileCEF3")]
		public bool bCompileCEF3 { get; set; } = true;

		/// <summary>
		/// Whether to compile using ISPC.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		public bool bCompileISPC { get; set; }

		/// <summary>
		/// Whether to compile in python support
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		public bool bCompilePython { get; set; } = true;

		/// <summary>
		/// Whether to enable the config system (GConfig)
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		public bool bEnableConfigSystem { get; set; } = true;

		/// <summary>
		/// Whether to compile with support for multiple platform-native views per slate window (right now only implemented on macOS)
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/BuildSettings.BuildSettings", "bMultiViewPerSlateWindowSupport")]
		public bool bMultiViewPerSlateWindowSupport { get; set; } = false;

		/// <summary>
		/// Whether to compile with WITH_GAMEPLAY_DEBUGGER enabled with all Engine's default gameplay debugger categories.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		public bool bUseGameplayDebugger
		{
			get
			{
				if (UseGameplayDebuggerOverride == GameplayDebuggerOverrideState.Default)
				{
					return (bBuildDeveloperTools || (Configuration != UnrealTargetConfiguration.Test && Configuration != UnrealTargetConfiguration.Shipping));
				}
				else
				{
					return UseGameplayDebuggerOverride == GameplayDebuggerOverrideState.Full;
				}
			}
			set
			{
				if (value)
				{
					UseGameplayDebuggerOverride = GameplayDebuggerOverrideState.Full;
				}
				else if (UseGameplayDebuggerOverride != GameplayDebuggerOverrideState.Core)
				{
					UseGameplayDebuggerOverride = GameplayDebuggerOverrideState.Default;
				}
			}
		}

		/// <summary>
		/// Set to true when bUseGameplayDebugger is false but GameplayDebugger's core parts are required.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		public bool bUseGameplayDebuggerCore
		{
			get => (UseGameplayDebuggerOverride == GameplayDebuggerOverrideState.Core) || bUseGameplayDebugger;
			set
			{
				if (UseGameplayDebuggerOverride != GameplayDebuggerOverrideState.Full)
				{
					UseGameplayDebuggerOverride = value ? GameplayDebuggerOverrideState.Core : GameplayDebuggerOverrideState.Default;
				}
			}
		}
		GameplayDebuggerOverrideState UseGameplayDebuggerOverride;

		/// <summary>
		/// Whether to use I/O store on-demand
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[CommandLine("-CompileIoStoreOnDemand", Value = "true")]
		public bool bCompileIoStoreOnDemand { get; set; } = false;

		/// <summary>
		/// Whether to track owner (asset name) of RHI resource for Test configuration.
		/// Useful for ListShaderMaps and ListShaderLibraries commands.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		public bool bTrackRHIResourceInfoForTest { get; set; } = false;

		/// <summary>
		/// Whether we are compiling editor code or not. Prefer the more explicit bCompileAgainstEditor instead.
		/// </summary>
		public bool bBuildEditor
		{
			get => (Type == TargetType.Editor || bCompileAgainstEditor);
			[Obsolete("Deprecated, replace with TargetRules.Type")]
			set => Logger.LogWarning("Setting {Type}.bBuildEditor is deprecated. Set {Type}.Type instead.", GetType().Name, GetType().Name);
		}

		/// <summary>
		/// Whether to compile code related to building assets. Consoles generally cannot build assets. Desktop platforms generally can.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		public bool bBuildRequiresCookedData
		{
			get => bBuildRequiresCookedDataOverride ?? (Type == TargetType.Game || Type == TargetType.Client || Type == TargetType.Server);
			set => bBuildRequiresCookedDataOverride = value;
		}
		bool? bBuildRequiresCookedDataOverride;

		/// <summary>
		/// Whether to compile WITH_EDITORONLY_DATA disabled. Only Windows will use this, other platforms force this to false.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		public bool bBuildWithEditorOnlyData
		{
			get => bBuildWithEditorOnlyDataOverride ?? (Type == TargetType.Editor || Type == TargetType.Program);
			set => bBuildWithEditorOnlyDataOverride = value;
		}
		private bool? bBuildWithEditorOnlyDataOverride;

		/// <summary>
		/// Whether to compile the developer tools.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		public bool bBuildDeveloperTools
		{
			set => bBuildDeveloperToolsOverride = value;
			get => bBuildDeveloperToolsOverride ?? (bCompileAgainstEngine && (Type == TargetType.Editor || Type == TargetType.Program || (Configuration != UnrealTargetConfiguration.Test && Configuration != UnrealTargetConfiguration.Shipping)));
		}
		private bool? bBuildDeveloperToolsOverride;

		/// <summary>
		/// Whether to compile the developer tools that are for target platforms or connected devices (defaults to bBuildDeveloperTools)
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		public bool bBuildTargetDeveloperTools
		{
			set => bBuildTargetDeveloperToolsOverride = value;
			get => bBuildTargetDeveloperToolsOverride ?? bBuildDeveloperTools;
		}
		private bool? bBuildTargetDeveloperToolsOverride;

		/// <summary>
		/// Whether to force compiling the target platform modules, even if they wouldn't normally be built.
		/// </summary>
		public bool bForceBuildTargetPlatforms { get; set; }

		/// <summary>
		/// Whether to force compiling shader format modules, even if they wouldn't normally be built.
		/// </summary>
		public bool bForceBuildShaderFormats { get; set; }

		/// <summary>
		/// Override for including extra shader formats
		/// </summary>
		public bool? bNeedsExtraShaderFormatsOverride { get; set; }

		/// <summary>
		/// Whether we should include any extra shader formats. By default this is only enabled for Program and Editor targets.
		/// </summary>
		public bool bNeedsExtraShaderFormats
		{
			set => bNeedsExtraShaderFormatsOverride = value;
			get => bNeedsExtraShaderFormatsOverride ?? (bForceBuildShaderFormats || bBuildDeveloperTools) && (Type == TargetType.Editor || Type == TargetType.Program);
		}

		/// <summary>
		/// Checks the above flags to see if target SDKs are likely to be compiled in
		/// </summary>
		private bool bUsesTargetSDKs => bNeedsExtraShaderFormats || bBuildDeveloperTools || bBuildTargetDeveloperTools || bForceBuildShaderFormats || bForceBuildTargetPlatforms;

		/// <summary>
		/// Whether we should compile SQLite using the custom "Unreal" platform (true), or using the native platform (false).
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/BuildSettings.BuildSettings", "bCompileCustomSQLitePlatform")]
		public bool bCompileCustomSQLitePlatform { get; set; } = true;

		/// <summary>
		/// Whether to utilize cache freed OS allocs with MallocBinned
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/BuildSettings.BuildSettings", "bUseCacheFreedOSAllocs")]
		public bool bUseCacheFreedOSAllocs { get; set; } = true;

		/// <summary>
		/// Enabled for all builds that include the engine project.  Disabled only when building standalone apps that only link with Core.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		public virtual bool bCompileAgainstEngine
		{
			get => bCompileAgainstEnginePrivate;
			set => bCompileAgainstEnginePrivate = value;
		}

		/// <summary>
		/// variable backing bCompileAgainstEngine
		/// </summary>
		protected bool bCompileAgainstEnginePrivate = true;

		/// <summary>
		/// Enabled for all builds that include the CoreUObject project.  Disabled only when building standalone apps that only link with Core.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		public virtual bool bCompileAgainstCoreUObject
		{
			get => bCompileAgainstCoreUObjectPrivate;
			set => bCompileAgainstCoreUObjectPrivate = value;
		}
		private bool bCompileAgainstCoreUObjectPrivate = true;

		/// <summary>
		/// Enable compilation of UClass and related reflection objects as statically initialized global variables.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[CommandLine("-ConstInitUObject")]
		[CommandLine("-NoConstInitUObject", Value="false")]
		public virtual bool bEnableConstInitUObject 
		{
			get => bEnableConstInitUObjectPrivate;
			set => bEnableConstInitUObjectPrivate = value;
		}
		private bool bEnableConstInitUObjectPrivate;

		/// <summary>
		/// Enabled for builds that need to initialize the ApplicationCore module. Command line utilities do not normally need this.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		public virtual bool bCompileAgainstApplicationCore
		{
			get => bCompileAgainstApplicationCorePrivate;
			set => bCompileAgainstApplicationCorePrivate = value;
		}
		private bool bCompileAgainstApplicationCorePrivate = true;

		/// <summary>
		/// Enabled for builds that want to use against the Trace module for profiling and diagnostics.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		public virtual bool bEnableTrace
		{
			get
			{
				if (bEnableTracePrivate.HasValue)
				{
					return bEnableTracePrivate.Value;
				}
				if (Configuration == UnrealTargetConfiguration.Shipping || Type == TargetType.Program)
				{
					return false;
				}
				if (UEBuildPlatform.IsPlatformInGroup(Platform, UnrealPlatformGroup.Apple)
				|| UEBuildPlatform.IsPlatformInGroup(Platform, UnrealPlatformGroup.Unix)
				|| UEBuildPlatform.IsPlatformInGroup(Platform, UnrealPlatformGroup.Android)
				|| UEBuildPlatform.IsPlatformInGroup(Platform, UnrealPlatformGroup.Windows))
				{
					return true;
				}
				return false;
			}
			set => bEnableTracePrivate = value;
		}
		private bool? bEnableTracePrivate;

		/// <summary>
		/// Force enable tracing - used to allow testing programs to verify tracing works as intended.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		public virtual bool bForceEnableTrace
		{
			get
			{
				if (bForceEnableTracePrivate.HasValue)
				{
					return bForceEnableTracePrivate.Value;
				}

				return false;
			}
			set => bForceEnableTracePrivate = value;
		}
		private bool? bForceEnableTracePrivate;

		/// <summary>
		/// Enables secure socket connection support for Trace.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		public virtual bool bEnableTraceSecureConnection
		{
			get 
			{
				if (bEnableTraceSecureConnectionPrivate.HasValue)
				{
					return bEnableTraceSecureConnectionPrivate.Value;
				}

				return false;
			}
			set => bEnableTraceSecureConnectionPrivate = value;
		}
		private bool? bEnableTraceSecureConnectionPrivate;

		/// <summary>
		/// Enabled for editor builds (TargetType.Editor). Can be overridden for programs (TargetType.Program) that would need to compile against editor code. Not available for other target types.
		/// Mainly drives the value of WITH_EDITOR.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		public virtual bool bCompileAgainstEditor
		{
			set => bCompileAgainstEditorOverride = value;
			get => bCompileAgainstEditorOverride ?? (Type == TargetType.Editor);
		}
		private bool? bCompileAgainstEditorOverride;

		/// <summary>
		/// Whether to compile Recast navmesh generation.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/BuildSettings.BuildSettings", "bCompileRecast")]
		public bool bCompileRecast { get; set; } = true;

		/// <summary>
		/// Whether to compile with navmesh segment links.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		public bool bCompileNavmeshSegmentLinks { get; set; } = true;

		/// <summary>
		/// Whether to compile with navmesh cluster links.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		public bool bCompileNavmeshClusterLinks { get; set; } = true;

		/// <summary>
		/// Whether to compile SpeedTree support.
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/BuildSettings.BuildSettings", "bCompileSpeedTree")]
		bool? bOverrideCompileSpeedTree { get; set; }

		/// <summary>
		/// Whether we should compile in support for SpeedTree or not.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		public bool bCompileSpeedTree
		{
			set => bOverrideCompileSpeedTree = value;
			get => bOverrideCompileSpeedTree ?? Type == TargetType.Editor;
		}

		/// <summary>
		/// Enable exceptions for all modules.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		public bool bForceEnableExceptions { get; set; }

		/// <summary>
		/// Enable inlining for all modules.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bUseInlining { get; set; } = true;

		/// <summary>
		/// Enable exceptions for all modules.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		public bool bForceEnableObjCExceptions { get; set; }

		/// <summary>
		/// Enable RTTI for all modules.
		/// </summary>
		[CommandLine("-rtti")]
		[RequiresUniqueBuildEnvironment]
		public bool bForceEnableRTTI { get; set; }

		/// <summary>
		/// Enable BuildSettings to contain private information about build. Such as machine name, user name and user domain name (exposed in BuildSettings.h)
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		public bool bEnablePrivateBuildInformation { get; set; }

		/// <summary>
		/// Enable Position Independent Executable (PIE). Has an overhead cost
		/// </summary>
		[CommandLine("-pie")]
		public bool bEnablePIE { get; set; }

		/// <summary>
		/// Enable Stack Protection. Has an overhead cost
		/// </summary>
		[CommandLine("-stack-protect")]
		public bool bEnableStackProtection { get; set; }

		/// <summary>
		/// Compile client-only code.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		public bool bWithClientCode
		{
			get => bWithClientCodeOverride ?? (Type != TargetType.Server);
			set => bWithClientCodeOverride = value;
		}
		private bool? bWithClientCodeOverride;

		/// <summary>
		/// Compile server-only code.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		public bool bWithServerCode
		{
			get => bWithServerCodeOverride ?? (Type != TargetType.Client);
			set => bWithServerCodeOverride = value;
		}
		private bool? bWithServerCodeOverride;

		/// <summary>
		/// Compile with FName storing the number part in the name table. 
		/// Saves memory when most names are not numbered and those that are are referenced multiple times.
		/// The game and engine must ensure they reuse numbered names similarly to name strings to avoid leaking memory.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		public bool bFNameOutlineNumber
		{
			get => bFNameOutlineNumberOverride ?? false;
			set => bFNameOutlineNumberOverride = value;
		}
		private bool? bFNameOutlineNumberOverride;

		/// <summary>
		/// When enabled, Push Model Networking support will be compiled in.
		/// This can help reduce CPU overhead of networking, at the cost of more memory.
		/// Always enabled in editor builds.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		public bool bWithPushModel
		{
			get => bWithPushModelOverride ?? (Type == TargetType.Editor);
			set => bWithPushModelOverride = value;
		}
		private bool? bWithPushModelOverride;

		/// <summary>
		/// When enabled, certain thread safety mechanisms will be removed from Iris code
		/// this may result in a small cpu perf gain if running a single-threaded server
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[CommandLine("-ExcludeParallelIrisSupport", Value = "true")]
		public bool bExcludeParallelIrisSupport { get; set; } = false;

		/// <summary>
		/// Whether to include stats support even without the engine.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		public bool bCompileWithStatsWithoutEngine { get; set; }

		/// <summary>
		/// Whether to include plugin support.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/BuildSettings.BuildSettings", "bCompileWithPluginSupport")]
		public bool bCompileWithPluginSupport { get; set; }

		/// <summary>
		/// Whether to allow plugins which support all target platforms.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		public bool bIncludePluginsForTargetPlatforms
		{
			get => bIncludePluginsForTargetPlatformsOverride ?? (Type == TargetType.Editor);
			set => bIncludePluginsForTargetPlatformsOverride = value;
		}
		private bool? bIncludePluginsForTargetPlatformsOverride;

		/// <summary>
		/// Whether to allow accessibility code in both Slate and the OS layer.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		public bool bCompileWithAccessibilitySupport { get; set; } = true;

		/// <summary>
		/// Whether to include PerfCounters support.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		public bool bWithPerfCounters
		{
			get => bWithPerfCountersOverride ?? (Type == TargetType.Editor || Type == TargetType.Server);
			set => bWithPerfCountersOverride = value;
		}

		[ConfigFile(ConfigHierarchyType.Engine, "/Script/BuildSettings.BuildSettings", "bWithPerfCounters")]
		bool? bWithPerfCountersOverride;

		/// <summary>
		/// Whether to enable support for live coding
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[XmlConfigFile(Category = "BuildConfiguration")]
		[CommandLine("-LiveCoding")]
		[CommandLine("-NoLiveCoding", Value = "false")]
		public bool bWithLiveCoding
		{
			get => bWithLiveCodingPrivate ?? (Platform == UnrealTargetPlatform.Win64 && Architecture.bIsX64 && Configuration != UnrealTargetConfiguration.Shipping && Configuration != UnrealTargetConfiguration.Test && Type != TargetType.Program);
			set => bWithLiveCodingPrivate = value;
		}
		bool? bWithLiveCodingPrivate;

		/// <summary>
		/// Whether to enable support for live coding
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bUseDebugLiveCodingConsole { get; set; }

		/// <summary>
		/// Whether to enable the support for FixedTimeStep in the engine
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		public bool bWithFixedTimeStepSupport { get; set; } = true;

		/// <summary>
		/// Whether to turn on logging for test/shipping builds.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		public bool bUseLoggingInShipping { get; set; }

		/// <summary>
		/// Whether to turn on console for shipping builds.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		public bool bUseConsoleInShipping { get; set; }

		/// <summary>
		/// Whether to turn on logging to memory for test/shipping builds.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		public bool bLoggingToMemoryEnabled { get; set; }

		/// <summary>
		/// Whether to check that the process was launched through an external launcher.
		/// </summary>
		public bool bUseLauncherChecks { get; set; }

		/// <summary>
		/// Whether to turn on checks (asserts) for test/shipping builds.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		public bool bUseChecksInShipping { get; set; }

		/// <summary>
		/// Whether to turn on GPU markers for Test builds.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		public bool bAllowProfileGPUInTest { get; set; }

		/// <summary>
		/// Whether to turn on GPU markers for Shipping builds.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		public bool bAllowProfileGPUInShipping { get; set; }

		/// <summary>
		/// Whether to turn on UTF-8 mode, mapping TCHAR to UTF8CHAR.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		public bool bTCHARIsUTF8 { get; set; }

		/// <summary>
		/// Whether to use the EstimatedUtcNow or PlatformUtcNow.  EstimatedUtcNow is appropriate in
		/// cases where PlatformUtcNow can be slow.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		public bool bUseEstimatedUtcNow { get; set; }

		/// <summary>
		/// True if we need FreeType support.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/BuildSettings.BuildSettings", "bCompileFreeType")]
		public bool bCompileFreeType { get; set; } = true;

		/// <summary>
		/// Whether to turn allow exec commands for shipping builds.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		public bool bUseExecCommandsInShipping { get; set; } = true;

		/// <summary>
		/// Allows to fine tune optimizations level for speed and\or code size
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[CommandLine("-OptimizationLevel=")]
		public OptimizationMode OptimizationLevel { get; set; } = OptimizationMode.Speed;

		/// <summary>
		/// By default UE_FORCEINLINE_HINT is defined to FORCEINLINE. If this flag is true it will instead be defined to 'inline'.
		/// This flag should only be true for targets where code size is a concern and/or is being compiled with PGO or LTO.
		/// Note that if this flag is true when PGOOptimize is enabled, it should also be true when PGOProfile is enabled.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		public bool bDefineForceInlineHintToInline
		{
			get => bDefineForceInlineHintToInlinePrivate ?? (bPGOOptimize || bPGOProfile || bAllowLTCG);
			set => bDefineForceInlineHintToInlinePrivate = value;
		}
		private bool? bDefineForceInlineHintToInlinePrivate;

		/// <summary>
		/// If enabled sparse set will have additional runtime checks against usage that conflicts with compact sets. This can be used to catch 
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[CommandLine("-UseSparseSetStrictMode")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bUseSparseSetStrictMode { get; set; } = false;

		/// <summary>
		/// By default TSparseSet is used as the underlying TSet container. Use this flag to change that to use TCompactSet instead.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[CommandLine("-UseCompactSetAsDefault")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bUseCompactSetAsDefault { get; set; } = false;

		/// <summary>
		/// Whether to use Android Java Native Interface.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		public bool bUseAndroidJni { get; set; } = true;

		/// <summary>
		/// Allows setting the FP semantics.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[CommandLine("-FPSemantics=")]
		public FPSemanticsMode FPSemantics { get; set; } = FPSemanticsMode.Default;

		/// <summary>
		/// Whether to compile development automation tests.
		/// </summary>
		public bool bForceCompileDevelopmentAutomationTests { get; set; }

		/// <summary>
		/// Whether to compile performance automation tests.
		/// </summary>
		public bool bForceCompilePerformanceAutomationTests { get; set; }

		/// <summary>
		/// Whether to override the defaults for automation tests (Debug/Development configs)
		/// </summary>
		public bool bForceDisableAutomationTests { get; set; }

		/// <summary>
		/// If true, event driven loader will be used in cooked builds. @todoio This needs to be replaced by a runtime solution after async loading refactor.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		public bool bEventDrivenLoader { get; set; }

		/// <summary>
		/// Used to override the behavior controlling whether UCLASSes and USTRUCTs are allowed to have native pointer members, if disallowed they will be a UHT error and should be substituted with TObjectPtr members instead.
		/// </summary>
		public PointerMemberBehavior? NativePointerMemberBehaviorOverride { get; set; }

		/// <summary>
		/// Whether the XGE controller worker and modules should be included in the engine build.
		/// These are required for distributed shader compilation using the XGE interception interface.
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bUseXGEController { get; set; } = true;

		/// <summary>
		/// Enables "include what you use" by default for modules in this target. Changes the default PCH mode for any module in this project to PCHUsageMode.UseExplicitOrSharedPCHs.
		/// </summary>
		public bool bIWYU => IntermediateEnvironment == UnrealIntermediateEnvironment.IWYU;

		/// <summary>
		/// Adds header files in included modules to the build.
		/// </summary>
		[CommandLine("-IncludeHeaders")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bIncludeHeaders
		{
			get => bIncludeHeadersPrivate || bIWYU;
			set => bIncludeHeadersPrivate = value;
		}
		private bool bIncludeHeadersPrivate = false;

		/// <summary>
		/// When used with -IncludeHeaders, only header files will be compiled.
		/// </summary>
		[CommandLine("-HeadersOnly")]
		public bool bHeadersOnly { get; set; }

		/// <summary>
		/// Enforce "include what you use" rules; warns if monolithic headers (Engine.h, UnrealEd.h, etc...) are used, and checks that source files include their matching header first.
		/// </summary>
		public bool bEnforceIWYU { get; set; } = true;

		/// <summary>
		/// Emit a warning when an old-style monolithic header is included while compiling this target.
		/// </summary>
		public bool bWarnAboutMonolithicHeadersIncluded { get; set; }

		/// <summary>
		/// Whether the final executable should export symbols.
		/// </summary>
		public bool bHasExports
		{
			get => bHasExportsOverride ?? (LinkType == TargetLinkType.Modular);
			set => bHasExportsOverride = value;
		}
		private bool? bHasExportsOverride;

		/// <summary>
		/// Make static libraries for all engine modules as intermediates for this target.
		/// </summary>
		[CommandLine("-Precompile")]
		public bool bPrecompile { get; set; }

		/// <summary>
		/// Whether we should compile with support for OS X 10.9 Mavericks. Used for some tools that we need to be compatible with this version of OS X.
		/// </summary>
		public bool bEnableOSX109Support { get; set; }

		/// <summary>
		/// True if this is a console application that's being built.
		/// </summary>
		public bool bIsBuildingConsoleApplication { get; set; }

		/// <summary>
		/// If true, creates an additional console application. Hack for Windows, where it's not possible to conditionally inherit a parent's console Window depending on how
		/// the application is invoked; you have to link the same executable with a different subsystem setting.
		/// </summary>
		public bool bBuildAdditionalConsoleApp
		{
			get => bBuildAdditionalConsoleAppOverride ?? (Type == TargetType.Editor);
			set => bBuildAdditionalConsoleAppOverride = value;
		}
		private bool? bBuildAdditionalConsoleAppOverride;

		/// <summary>
		/// If true, only creates an additional console application. Overrides bBuildAdditionalConsoleApp.		
		/// </summary>
		public bool bBuildConsoleAppOnly { get; set; } = false;
		/// <summary>
		/// True if debug symbols that are cached for some platforms should not be created.
		/// </summary>
		public bool bDisableSymbolCache { get; set; } = true;

		/// <summary>
		/// Whether to unify C++ code into larger files for faster compilation.
		/// </summary>
		public bool bUseUnityBuild
		{
			get => bUseUnityBuildOverride ?? IntermediateEnvironment.IsUnity() && !bEnableCppModules;
			set => bUseUnityBuildOverride = value;
		}

		/// <summary>
		/// Whether to unify C++ code into larger files for faster compilation.
		/// </summary>
		bool? bUseUnityBuildOverride { get; set; }

		/// <summary>
		/// Whether to force C++ source files to be combined into larger files for faster compilation.
		/// </summary>
		[CommandLine("-ForceUnity")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bForceUnityBuild { get; set; }

		/// <summary>
		/// Whether to force generated files to be placed in a unity file even when unity files are otherwise disabled.
		/// It is not recommended to disable this unless debugging issues with UHT file generation.
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bAlwaysUseUnityForGeneratedFiles { get; set; } = true;

		/// <summary>
		/// List of modules that disables merging module and generated cpp files in the same unity files.
		/// </summary>
		[XmlConfigFile(Category = "ModuleConfiguration", Name = "DisableMergingModuleAndGeneratedFilesInUnityFiles")]
		public string[]? DisableMergingModuleAndGeneratedFilesInUnityFiles = null;

		/// <summary>
		/// Use a heuristic to determine which files are currently being iterated on and exclude them from unity blobs. This results in faster
		/// incremental compile times. For Perforce repositories, the current implementation uses the read-only flag to distinguish the working
		/// set, assuming that files will be made writable by the source control system if they are being modified. For Git repositories, the
		/// implementation uses the git status command. Source code archives downloaded from Git as .zip files are not supported.
		/// </summary>
		[CommandLine("-DisableAdaptiveUnity", Value = "false")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bUseAdaptiveUnityBuild { get; set; } = true;

		/// <summary>
		/// Disable optimization for files that are in the adaptive non-unity working set.
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bAdaptiveUnityDisablesOptimizations { get; set; }

		/// <summary>
		/// Disable optimizations for files that are in the adaptive non-unity working set for the given configurations.
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		public UnrealTargetConfiguration[] AdaptiveUnityDisablesOptimizationsConfigurations { get; set; } = [];

		/// <summary>
		/// Disables force-included PCHs for files that are in the adaptive non-unity working set.
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bAdaptiveUnityDisablesPCH { get; set; }

		/// <summary>
		/// Backing storage for bAdaptiveUnityDisablesProjectPCH.
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool? bAdaptiveUnityDisablesProjectPCHForProjectPrivate { get; set; }

		/// <summary>
		/// Whether to disable force-included PCHs for project source files in the adaptive non-unity working set. Defaults to bAdaptiveUnityDisablesPCH;
		/// </summary>
		public bool bAdaptiveUnityDisablesPCHForProject
		{
			get => bAdaptiveUnityDisablesProjectPCHForProjectPrivate ?? bAdaptiveUnityDisablesPCH;
			set => bAdaptiveUnityDisablesProjectPCHForProjectPrivate = value;
		}

		/// <summary>
		/// Creates a dedicated PCH for each source file in the working set, allowing faster iteration on cpp-only changes.
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bAdaptiveUnityCreatesDedicatedPCH { get; set; }

		/// <summary>
		/// Creates a dedicated PCH for each source file in the working set, allowing faster iteration on cpp-only changes.
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bAdaptiveUnityEnablesEditAndContinue { get; set; }

		/// <summary>
		/// Creates a dedicated source file for each header file in the working set to detect missing includes in headers.
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bAdaptiveUnityCompilesHeaderFiles { get; set; }

		/// <summary>
		/// The number of source files in a game module before unity build will be activated for that module.  This
		/// allows small game modules to have faster iterative compile times for single files, at the expense of slower full
		/// rebuild times.  This setting can be overridden by the bFasterWithoutUnity option in a module's Build.cs file.
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		public int MinGameModuleSourceFilesForUnityBuild { get; set; } = 32;

		/// <summary>
		/// Default treatment of uncategorized warnings
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		public WarningLevel DefaultWarningLevel
		{
			get => (DefaultWarningLevelPrivate == WarningLevel.Default) ? (bWarningsAsErrors ? WarningLevel.Error : WarningLevel.Warning) : DefaultWarningLevelPrivate;
			set => DefaultWarningLevelPrivate = value;
		}

		/// <inheritdoc cref="DefaultWarningLevel"/>
		[XmlConfigFile(Category = "BuildConfiguration", Name = nameof(DefaultWarningLevel))]
		private WarningLevel DefaultWarningLevelPrivate;

		/// <summary>
		/// Require TObjectPtr for FReferenceCollector API's. (Needed for compatibility with incremental GC.)
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bRequireObjectPtrForAddReferencedObjects
		{
			get => bRequireObjectPtrForAddReferencedObjectsPrivate ?? true;
			set => bRequireObjectPtrForAddReferencedObjectsPrivate = value;
		}
		private bool? bRequireObjectPtrForAddReferencedObjectsPrivate;

		/// <summary>
		/// Emits compilation errors for incorrect UE_LOG format strings.
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bValidateFormatStrings
		{
			get => bValidateFormatStringsPrivate ?? (DefaultBuildSettings >= BuildSettingsVersion.V5);
			set => bValidateFormatStringsPrivate = value;
		}
		private bool? bValidateFormatStringsPrivate;

		/// <summary>
		/// C++ Warnings settings object used by the target
		/// </summary>
		[ConfigSubObject]
		public CppCompileWarnings CppCompileWarningSettings { get; init; }

		/// <summary>
		/// Whether to enable all warnings as errors. UE enables most warnings as errors already, but disables a few (such as deprecation warnings).
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		[CommandLine("-WarningsAsErrors")]
		[RequiresUniqueBuildEnvironment]
		public bool bWarningsAsErrors { get; set; }

		/// <summary>
		/// Print out files that are included by each source file
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		[CommandLine("-ShowIncludes")]
		public bool bShowIncludes = false;

		/// <summary>
		/// Forces frame pointers to be retained this is usually required when you want reliable callstacks e.g. mallocframeprofiler
		/// </summary>
		public bool bRetainFramePointers
		{
			get =>
				// Default to disabled on Linux to maintain legacy behavior
				bRetainFramePointersOverride ?? Platform.IsInGroup(UnrealPlatformGroup.Linux) == false;
			set => bRetainFramePointersOverride = value;
		}

		/// <summary>
		/// Forces frame pointers to be retained this is usually required when you want reliable callstacks e.g. mallocframeprofiler
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration", Name = "bRetainFramePointers")]
		[CommandLine("-RetainFramePointers", Value = "true")]
		[CommandLine("-NoRetainFramePointers", Value = "false")]
		public bool? bRetainFramePointersOverride { get; set; }

		/// <summary>
		/// New Monolithic Graphics drivers have optional "fast calls" replacing various D3d functions
		/// </summary>
		[CommandLine("-FastMonoCalls", Value = "true")]
		[CommandLine("-NoFastMonoCalls", Value = "false")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bUseFastMonoCalls { get; set; } = true;

		/// <summary>
		/// An approximate number of bytes of C++ code to target for inclusion in a single unified C++ file.
		/// </summary>
		[CommandLine("-BytesPerUnityCPP")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public int NumIncludedBytesPerUnityCPP { get; set; } = 384 * 1024;

		/// <summary>
		/// Disables overrides that are set by the module
		/// </summary>
		[CommandLine("-DisableModuleNumIncludedBytesPerUnityCPPOverride", Value = "true")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bDisableModuleNumIncludedBytesPerUnityCPPOverride { get; set; }

		/// <summary>
		/// Whether to stress test the C++ unity build robustness by including all C++ files files in a project from a single unified file.
		/// </summary>
		[CommandLine("-StressTestUnity")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bStressTestUnity { get; set; }

		/// <summary>
		/// Whether to force debug info to be generated.
		/// </summary>
		[CommandLine("-ForceDebugInfo")]
		public bool bForceDebugInfo { get; set; }

		/// <summary>
		/// How much debug info should be generated. See DebugInfoMode enum for more details
		/// </summary>
		[CommandLine("-NoDebugInfo", Value = "None")]
		[CommandLine("-DebugInfo=")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public DebugInfoMode DebugInfo { get; set; } = DebugInfoMode.Full;

		/// <summary>
		/// Modules that should have debug info disabled
		/// </summary>
		[CommandLine("-DisableDebugInfoModules=", ListSeparator = '+')]
		public HashSet<string> DisableDebugInfoModules { get; } = [];

		/// <summary>
		/// Plugins that should have debug info disabled
		/// </summary>
		[CommandLine("-DisableDebugInfoPlugins=", ListSeparator = '+')]
		public HashSet<string> DisableDebugInfoPlugins { get; } = [];

		/// <summary>
		/// True if only debug line number tables should be emitted in debug information for compilers that support doing so. Overrides TargetRules.DebugInfo
		/// See https://clang.llvm.org/docs/UsersManual.html#cmdoption-gline-tables-only for more information
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		[CommandLine("-DebugInfoLineTablesOnly=")]
		public DebugInfoMode DebugInfoLineTablesOnly { get; set; } = DebugInfoMode.None;

		/// <summary>
		/// Modules that should emit line number tables instead of full debug information for compilers that support doing so. Overrides DisableDebugInfoModules
		/// </summary>
		[CommandLine("-DebugInfoLineTablesOnlyModules=", ListSeparator = '+')]
		public HashSet<string> DebugInfoLineTablesOnlyModules { get; } = [];

		/// <summary>
		/// Plugins that should emit line number tables instead of full debug information for compilers that support doing so. Overrides DisableDebugInfoPlugins
		/// </summary>
		[CommandLine("-DebugInfoLineTablesOnlyPlugins=", ListSeparator = '+')]
		public HashSet<string> DebugInfoLineTablesOnlyPlugins { get; } = [];

		/// <summary>
		/// True if debug line number tables should be omitted in debug information for inline functions, compilers that support doing so.
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		[CommandLine("-DebugInfoNoInlineLineTables=")]
		public DebugInfoMode DebugInfoNoInlineLineTables { get; set; } = DebugInfoMode.None;

		/// <summary>
		/// Modules that should include line number tables for inline functions for compilers that support doing so, even if DebugInfoNoInlineLineTables would otherwise remove them.
		/// </summary>
		[CommandLine("-DebugInfoInlineLineTablesModules=", ListSeparator = '+')]
		public HashSet<string> DebugInfoInlineLineTablesModules { get; } = [];

		/// <summary>
		/// Modules that should include line number tables for inline functions for compilers that support doing so, even if DebugInfoNoInlineLineTables would otherwise remove them.
		/// </summary>
		[CommandLine("-DebugInfoInlineLineTablesPlugins=", ListSeparator = '+')]
		public HashSet<string> DebugInfoInlineLineTablesPlugins { get; } = [];

		/// <summary>
		/// True if template type names should be simplified to reduce the size of debug info for compilers that support doing so.
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		[CommandLine("-DebugInfoSimpleTemplateNames=")]
		public DebugInfoMode DebugInfoSimpleTemplateNames { get; set; } = DebugInfoMode.None;

		/// <summary>
		/// Modules for which template type names should be simplified to reduce the size of debug info for compilers that support doing so.
		/// </summary>
		[CommandLine("-DebugInfoSimpleTemplateNamesModules=", ListSeparator = '+')]
		public HashSet<string> DebugInfoSimpleTemplateNamesModules { get; } = [];

		/// <summary>
		/// Plugins for which template type names should be simplified to reduce the size of debug info for compilers that support doing so.
		/// </summary>
		[CommandLine("-DebugInfoSimpleTemplateNamesPlugins=", ListSeparator = '+')]
		public HashSet<string> DebugInfoSimpleTemplateNamesPlugins { get; } = [];

		/// <summary>
		/// Whether to disable debug info generation for generated files. This improves link times and reduces pdb size for modules that have a lot of generated glue code.
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bDisableDebugInfoForGeneratedCode { get; set; }

		/// <summary>
		/// Whether to disable debug info on PC/Mac in development builds (for faster developer iteration, as link times are extremely fast with debug info disabled).
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bOmitPCDebugInfoInDevelopment { get; set; }

		/// <summary>
		/// Whether PDB files should be used for Visual C++ builds.
		/// </summary>
		[CommandLine("-NoPDB", Value = "false")]
		[CommandLine("-UsePDB", Value = "true")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bUsePDBFiles { get; set; }

		/// <summary>
		/// Whether PCH files should be used.
		/// </summary>
		[CommandLine("-NoPCH", Value = "false")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bUsePCHFiles { get; set; } = true;

		/// <summary>
		/// Set flags require for deterministic compiling and linking.
		/// Enabling deterministic mode for msvc disables codegen multithreading so compiling will be slower
		/// </summary>
		[CommandLine("-Deterministic", Value = "true")]
		[CommandLine("-NonDeterministic", Value = "false")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		[RequiresUniqueBuildEnvironment]
		public bool bDeterministic
		{
			get => bDeterministicPrivate ?? (Configuration == UnrealTargetConfiguration.Shipping);
			set => bDeterministicPrivate = value;
		}
		private bool? bDeterministicPrivate;

		/// <summary>
		/// Experimental work in progress: Set flags to use toolchain virtual file system support, to generate consistent pathing in outputs
		/// </summary>
		[CommandLine("-VFS", Value = "true")]
		[CommandLine("-NoVFS", Value = "false")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/BuildSettings.BuildSettings", "bUseVFS")]
		[RequiresUniqueBuildEnvironment]
		public bool bUseVFS { get; set; }

		/// <summary>
		/// Salt value to modify cache bucket calculation, can be used to invalidate cached actions outputs
		/// </summary>
		[CommandLine("-CacheSalt")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public string? CacheSalt { get; set; }

		/// <summary>
		/// Whether PCHs should be chained when compiling with clang.
		/// Note: The earliest PCH a header is included in the chain will be included, so if there is a mismatch
		/// in preprocessor state this can cause an unexpected version of a header to be included.
		/// Currently disabled as this is the case.
		/// </summary>
		[CommandLine("-NoPCHChain", Value = "false")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bChainPCHs { get; set; } = false;

		/// <summary>
		/// Whether PCH headers should be force included for gen.cpp files when PCH is disabled.
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bForceIncludePCHHeadersForGenCppFilesWhenPCHIsDisabled { get; set; } = true;

		/// <summary>
		/// Whether to just preprocess source files for this target, and skip compilation
		/// </summary>
		[CommandLine("-Preprocess")]
		public bool bPreprocessOnly { get; set; }

		/// <summary>
		/// Whether to generate assembly data while compiling this target. Works exclusively on MSVC for now.
		/// </summary>
		[CommandLine("-WithAssembly")]
		public bool bWithAssembly { get; set; }

		/// <summary>
		/// Experimental: Store object (.obj) compressed on disk. Requires UBA to link. Toggling this flag will invalidate actions.
		/// </summary>
		[XmlConfigFile(Category = "UnrealBuildAccelerator", Name = "bStoreObjFilesCompressed")]
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/BuildSettings.BuildSettings", "bStoreObjFilesCompressed")]
		[CommandLine("-UBAStoreObjFilesCompressed", Value = "true", MarkUsed = false)]
		[RequiresUniqueBuildEnvironment]
		public bool bAllowUbaCompression { get; set; }

		/// <summary>
		/// Whether static code analysis should be enabled.
		/// </summary>
		[CommandLine("-StaticAnalyzer")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public StaticAnalyzer StaticAnalyzer { get; set; } = StaticAnalyzer.None;

		/// <summary>
		/// The output type to use for the static analyzer. This is only supported for Clang.
		/// </summary>
		[CommandLine("-StaticAnalyzerOutputType")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public StaticAnalyzerOutputType StaticAnalyzerOutputType { get; set; } = StaticAnalyzerOutputType.Text;

		/// <summary>
		/// The mode to use for the static analyzer. This is only supported for Clang.
		/// Shallow mode completes quicker but is generally not recommended.
		/// </summary>
		[CommandLine("-StaticAnalyzerMode")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public StaticAnalyzerMode StaticAnalyzerMode { get; set; } = StaticAnalyzerMode.Deep;

		/// <summary>
		/// The level of warnings to print when analyzing using PVS-Studio
		/// </summary>
		[CommandLine("-StaticAnalyzerPVSPrintLevel")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public int StaticAnalyzerPVSPrintLevel { get; set; } = 1;

		/// <summary>
		/// Paths to PVS-Studio .pvsconfig configuration files
		/// </summary>
		[CommandLine("-StaticAnalyzerPVSConfigs=", ListSeparator = ';')]
		public List<string> StaticAnalyzerPVSConfigs = [];

		/// <summary>
		/// Only run static analysis against project modules, skipping engine modules
		/// </summary>
		/// <remarks>PVS-Studio: warnings that are disabled via Engine/Source/Runtime/Core/Public/Microsoft/MicrosoftPlatformCodeAnalysis.h will be reenabled</remarks>
		[CommandLine("-StaticAnalyzerProjectOnly")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bStaticAnalyzerProjectOnly { get; set; } = false;

		/// <summary>
		/// When enabled, generated source files will be analyzed
		/// </summary>
		[CommandLine("-StaticAnalyzerIncludeGenerated")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bStaticAnalyzerIncludeGenerated { get; set; } = false;

		/// <summary>
		/// The minimum number of files that must use a pre-compiled header before it will be created and used.
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		public int MinFilesUsingPrecompiledHeader { get; set; } = 6;

		/// <summary>
		/// When enabled, a precompiled header is always generated for game modules, even if there are only a few source files
		/// in the module.  This greatly improves compile times for iterative changes on a few files in the project, at the expense of slower
		/// full rebuild times for small game projects.  This can be overridden by setting MinFilesUsingPrecompiledHeaderOverride in
		/// a module's Build.cs file.
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bForcePrecompiledHeaderForGameModules { get; set; } = true;

		/// <summary>
		/// Whether to use incremental linking or not. Incremental linking can yield faster iteration times when making small changes.
		/// Currently disabled by default because it tends to behave a bit buggy on some computers (PDB-related compile errors).
		/// </summary>
		[CommandLine("-IncrementalLinking")]
		[CommandLine("-NoIncrementalLinking", Value = "false")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bUseIncrementalLinking { get; set; }

		/// <summary>
		/// Whether to allow the use of link time code generation (LTCG).
		/// </summary>
		[CommandLine("-LTCG")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bAllowLTCG { get; set; }
		
		/// <summary>
		/// When Link Time Code Generation (LTCG) is enabled, whether to 
		/// prefer using the lighter weight version on supported platforms.
		/// </summary>
		[CommandLine("-ThinLTO")]
		[CommandLine("-NoThinLTO", Value = "false")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bPreferThinLTO { get; set; } = true;

		/// <summary>
		/// Directory where to put the ThinLTO cache on supported platforms.
		/// </summary>
		[CommandLine("-ThinLTOCacheDirectory")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public string? ThinLTOCacheDirectory { get; set; }

		/// <summary>
		/// Arguments that will be applied to prune the ThinLTO cache on supported platforms.
		/// Arguments will only be applied if ThinLTOCacheDirectory is set.
		/// </summary>
		[CommandLine("-ThinLTOCachePruningArguments")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public string? ThinLTOCachePruningArguments { get; set; }

		/// <summary>
		/// When ThinLTO is enabled we use distributed linking if supported
		/// This can save a lot of time for large binaries
		/// </summary>
		[CommandLine("-ThinLTODistributed")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bThinLTODistributed { get; set; }

		/// <summary>
		/// Whether to enable Profile Guided Optimization (PGO) instrumentation in this build.
		/// </summary>
		[CommandLine("-PGOProfile", Value = "true")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bPGOProfile { get; set; }

		/// <summary>
		/// Whether to optimize this build with Profile Guided Optimization (PGO).
		/// </summary>
		[CommandLine("-PGOOptimize", Value = "true")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bPGOOptimize { get; set; }

		/// <summary>
		/// Whether the target requires code coverage compilation and linking.
		/// </summary>
		[CommandLine("-CodeCoverage", Value = "true")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bCodeCoverage { get; set; }

		/// <summary>
		/// Whether to support edit and continue.
		/// </summary>
		[CommandLine("-SupportEditAndContinue")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bSupportEditAndContinue { get; set; }

		/// <summary>
		/// Whether to omit frame pointers or not. Disabling is useful for e.g. memory profiling on the PC.
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bOmitFramePointers { get; set; } = true;

		/// <summary>
		/// If we want to compress the debug files
		/// </summary>
		public bool bCompressDebugFile { get; set; } = true;

		/// <summary>
		/// Whether to enable support for C++20 modules
		/// </summary>
		public bool bEnableCppModules { get; set; }

		/// <summary>
		/// Whether to enable engine's ability to set process priority on runtime.
		/// This option requires some environment setup on Linux, that's why it's disabled by default.
		/// Project has to opt-in this feature as it has to guarantee correct setup.
		/// </summary>
		public bool bEnableProcessPriorityControl { get; set; }

		/// <summary>
		/// If true, then enable Unreal Insights (utrace) profiling in the build for the Shader Compiler Worker (defines USE_SHADER_COMPILER_WORKER_TRACE=1).
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bShaderCompilerWorkerTrace { get; set; }

		/// <summary>
		/// Enables "Shared PCHs", a feature which significantly speeds up compile times by attempting to
		/// share certain PCH files between modules that UBT detects is including those PCH's header files.
		/// </summary>
		[CommandLine("-NoSharedPCH", Value = "false")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bUseSharedPCHs { get; set; } = true;

		/// <summary>
		/// True if Development and Release builds should use the release configuration of PhysX/APEX.
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bUseShippingPhysXLibraries { get; set; }

		/// <summary>
		/// True if Development and Release builds should use the checked configuration of PhysX/APEX. if bUseShippingPhysXLibraries is true this is ignored.
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bUseCheckedPhysXLibraries { get; set; }

		/// <summary>
		/// Tells the UBT to check if module currently being built is violating EULA.
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bCheckLicenseViolations { get; set; } = true;

		/// <summary>
		/// Tells the UBT to break build if module currently being built is violating EULA.
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bBreakBuildOnLicenseViolation { get; set; } = true;

		/// <summary>
		/// Outputs a map file as part of the build.
		/// </summary>
		[CommandLine("-MapFile")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bCreateMapFile { get; set; }

		/// <summary>
		/// True if runtime symbols files should be generated as a post build step for some platforms.
		/// These files are used by the engine to resolve symbol names of callstack backtraces in logs.
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bAllowRuntimeSymbolFiles { get; set; } = true;

		/// <summary>
		/// Package full path (directory + filename) where to store input files used at link time 
		/// Normally used to debug a linker crash for platforms that support it
		/// </summary>
		[CommandLine("-PackagePath")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public string? PackagePath { get; set; }

		/// <summary>
		/// Directory where to put crash report files for platforms that support it
		/// </summary>
		[CommandLine("-CrashDiagnosticDirectory")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public string? CrashDiagnosticDirectory { get; set; }

		/// <summary>
		/// Bundle version for Mac apps.
		/// </summary>
		[CommandLine("-BundleVersion")]
		public string? BundleVersion { get; set; }

		/// <summary>
		/// Whether to deploy the executable after compilation on platforms that require deployment.
		/// </summary>
		[CommandLine("-Deploy")]
		public bool bDeployAfterCompile { get; set; }

		/// <summary>
		/// Whether to force skipping deployment for platforms that require deployment by default.
		/// </summary>
		[CommandLine("-SkipDeploy")]
		private bool bForceSkipDeploy { get; set; }

		/// <summary>
		/// When enabled, allows XGE to compile pre-compiled header files on remote machines.  Otherwise, PCHs are always generated locally.
		/// </summary>
		public bool bAllowRemotelyCompiledPCHs { get; set; }

		/// <summary>
		/// Will replace pch with ifc and use header units instead. This is an experimental and msvc-only feature
		/// </summary>
		[CommandLine("-HeaderUnits")]
		public bool bUseHeaderUnitsForPch { get; set; }

		/// <summary>
		/// Whether headers in system paths should be checked for modification when determining outdated actions.
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bCheckSystemHeadersForModification { get; set; }

		/// <summary>
		/// Whether to disable linking for this target.
		/// </summary>
		[CommandLine("-NoLink")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bDisableLinking { get; set; }

		/// <summary>
		/// Whether to ignore tracking build outputs for this target.
		/// </summary>
		public bool bIgnoreBuildOutputs { get; set; }

		/// <summary>
		/// Whether to disable UE_DEPRECATED_FORENGINE deprecation warnings in non-engine modules.
		/// This should only be used in the short-term as these deprecations will get removed.
		/// </summary>
		public bool bDisableEngineDeprecations { get; set; }

		/// <summary>
		/// Indicates that this is a formal build, intended for distribution. This flag is automatically set to true when Build.version has a changelist set and is a promoted build.
		/// The only behavior currently bound to this flag is to compile the default resource file separately for each binary so that the OriginalFilename field is set correctly.
		/// By default, we only compile the resource once to reduce build times.
		/// </summary>
		[CommandLine("-Formal")]
		public bool bFormalBuild { get; set; }

		/// <summary>
		/// Whether to clean Builds directory on a remote Mac before building.
		/// </summary>
		[CommandLine("-FlushMac")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bFlushBuildDirOnRemoteMac { get; set; }

		/// <summary>
		/// Whether to write detailed timing info from the compiler and linker.
		/// </summary>
		[CommandLine("-Timing")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bPrintToolChainTimingInfo { get; set; }

		/// <summary>
		/// Whether to parse timing data into a tracing file compatible with chrome://tracing.
		/// </summary>
		[CommandLine("-Tracing")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bParseTimingInfoForTracing { get; set; }

		/// <summary>
		/// Whether to expose all symbols as public by default on POSIX platforms
		/// </summary>
		[CommandLine("-PublicSymbolsByDefault")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bPublicSymbolsByDefault { get; set; }

		/// <summary>
		/// Enables Identical Code Folding (Clang toolchains)
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		public bool bIdenticalCodeFolding
		{
			get => bIdenticalCodeFoldingPrivate ?? Configuration == UnrealTargetConfiguration.Test || Configuration == UnrealTargetConfiguration.Shipping;
			set => bIdenticalCodeFoldingPrivate = value;
		}
		private bool? bIdenticalCodeFoldingPrivate;

		/// <summary>
		/// Allows overriding the toolchain to be created for this target. This must match the name of a class declared in the UnrealBuildTool assembly.
		/// </summary>
		[CommandLine("-ToolChain")]
		public string? ToolChainName { get; set; }

		/// <summary>
		/// The weight(cpu/memory utilization) of a MSVC compile action
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		public float MSVCCompileActionWeight { get; set; } = 1.0f;

		/// <summary>
		/// The weight(cpu/memory utilization) of a clang compile action
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		public float ClangCompileActionWeight { get; set; } = 1.0f;

		/// <summary>
		/// Whether to allow engine configuration to determine if we can load unverified certificates.
		/// </summary>
		public bool bDisableUnverifiedCertificates { get; set; }

		/// <summary>
		/// Whether to load generated ini files in cooked build, (GameUserSettings.ini loaded either way)
		/// </summary>
		public bool bAllowGeneratedIniWhenCooked { get; set; } = true;

		/// <summary>
		/// Whether to load non-ufs ini files in cooked build, (GameUserSettings.ini loaded either way)
		/// </summary>
		public bool bAllowNonUFSIniWhenCooked { get; set; } = true;

		/// <summary>
		/// Add all the public folders as include paths for the compile environment.
		/// </summary>
		public bool bLegacyPublicIncludePaths
		{
			get => bLegacyPublicIncludePathsPrivate ?? (DefaultBuildSettings < BuildSettingsVersion.V2);
			set => bLegacyPublicIncludePathsPrivate = value;
		}
		private bool? bLegacyPublicIncludePathsPrivate;

		/// <summary>
		/// Add all the parent folders as include paths for the compile environment.
		/// </summary>
		public bool bLegacyParentIncludePaths
		{
			get => bLegacyParentIncludePathsPrivate ?? (DefaultBuildSettings < BuildSettingsVersion.V3);
			set => bLegacyParentIncludePathsPrivate = value;
		}
		private bool? bLegacyParentIncludePathsPrivate;

		/// <summary>
		/// Which C++ standard to use for compiling this target (for engine modules)
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[CommandLine("-CppStdEngine")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public CppStandardVersion CppStandardEngine
		{
			get => CppStandardEnginePrivate ?? CppStandardVersion.EngineDefault;
			set => CppStandardEnginePrivate = value;
		}
		private CppStandardVersion? CppStandardEnginePrivate;

		/// <summary>
		/// Which C++ standard to use for compiling this target (for non-engine modules)
		/// </summary>
		[CommandLine("-CppStd")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public CppStandardVersion CppStandard
		{
#pragma warning disable CS0618 // Type or member is obsolete
			get => CppStandardPrivate ?? (DefaultBuildSettings < BuildSettingsVersion.V4 ? CppStandardVersion.Cpp17 : CppStandardVersion.Default);
#pragma warning restore CS0618 // Type or member is obsolete
			set => CppStandardPrivate = value;
		}
		private CppStandardVersion? CppStandardPrivate;

		/// <summary>
		/// Which C standard to use for compiling this target
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[CommandLine("-CStd")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public CStandardVersion CStandard { get; set; } = CStandardVersion.Default;

		/// <summary>
		/// Direct the compiler to generate AVX instructions wherever SSE or AVX intrinsics are used, on the x64 platforms that support it. Ignored for arm64.
		/// Note that by enabling this you are changing the minspec for the target platform, and the resultant executable will crash on machines without AVX support.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[CommandLine("-MinCpuArchX64")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public MinimumCpuArchitectureX64 MinCpuArchX64 { get; set; } = MinimumCpuArchitectureX64.Default;

		/// <summary>
		/// Direct the compiler to compile with APX support for X64.
		/// Note that by enabling this you are changing the minspec for the PC platform, and the resultant executable will crash on machines without APX support.
		/// </summary>
		[CommandLine("-APX")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bMinCpuArchAPX { get; set; } = false;

		/// <summary>
		/// Select a minimum ARM64 target CPU. This will change what ARMv8(9)-A ISA + extension code will be compiled for.
		/// Changing it from default will guarantee not to work on all ARM64 CPUs.
		/// Currently supported by clang compiler only
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[CommandLine("-MinArm64CpuTarget")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public Arm64TargetCpuType MinArm64CpuTarget { get; set; } = Arm64TargetCpuType.Default;

		/// <summary>
		/// Do not allow manifest changes when building this target. Used to cause earlier errors when building multiple targets with a shared build environment.
		/// </summary>
		[CommandLine("-NoManifestChanges")]
		internal bool bNoManifestChanges { get; set; }

		/// <summary>
		/// The build version string
		/// </summary>
		[CommandLine("-BuildVersion")]
		public string? BuildVersion { get; set; }

		/// <summary>
		/// Specifies how to link modules in this target (monolithic or modular). This is currently protected for backwards compatibility. Call the GetLinkType() accessor
		/// until support for the deprecated ShouldCompileMonolithic() override has been removed.
		/// </summary>
		public TargetLinkType LinkType
		{
			get => (LinkTypePrivate != TargetLinkType.Default) ? LinkTypePrivate : ((Type == global::UnrealBuildTool.TargetType.Editor) ? TargetLinkType.Modular : TargetLinkType.Monolithic);
			set => LinkTypePrivate = value;
		}

		/// <summary>
		/// Experimental: Strip unused exports from libraries. Only applies when LinkType is Modular
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[CommandLine("-StripExports")]
		public bool bStripExports { get; set; }

		/// <summary>
		/// Experimental: Merge modular modules into combined libraries. Sets LinkType to Modular and enables bStripExports
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[CommandLine("-MergeModules")]
		public bool bMergeModules { get; set; }

		/// <summary>
		/// Experimental: Check lifetimes for delegate instances. Recommended for merged modules testing
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[CommandLine("-CheckDelegateLifetime")]
		public bool bCheckDelegateLifetime { get; set; }

		/// <summary>
		/// Experimental: List of modules to exclude from Engine to be placed in Common
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[CommandLine("-MergeAdditionalCommonModules=", ListSeparator = '+')]
		public HashSet<string> MergeAdditionalCommonModules { get; } = [];

		/// <summary>
		/// Experimental: List of plugins (and their dependencies) to each merge into separate libraries. Requires bMergeModules to be enabled
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[CommandLine("-MergePlugins=", ListSeparator = '+')]
		public List<string> MergePlugins { get; } = [];

		/// <summary>
		/// Experimental: List of plugins (and their dependencies) to ignore when processing merged libraries. Requires bMergeModules to be enabled
		/// Ignored plugins will still be compiled, but if they're kept in the base executable, their dependencies will not be forced out of merged library with them.
		/// This should be used carefully, for organisational plugins known to have dependencies they don't actually need at runtime.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[CommandLine("-MergePluginsIgnored=", ListSeparator = '+')]
		public List<string> MergePluginsIgnored { get; } = [];

		/// <summary>
		/// Experimental: List of plugins (and their dependencies) to explicitly exclude when processing merged libraries. Requires bMergeModules to be enabled
		/// Ignored plugins will still be compiled but forced into the executable.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[CommandLine("-MergePluginsExcluded=", ListSeparator = '+')]
		public List<string> MergePluginsExcluded { get; } = [];

		/// <summary>
		/// Experimental: List of merged plugins to move common dependencies to a shared library, requires MergePlugins. Can be chained
		/// </summary>
		public Dictionary<string, HashSet<string>> MergePluginsShared { get; } = [];

		/// <summary>
		/// Experimental: List of merged plugins to move to the main executable, requires MergePlugins. "Engine" for all shared engine modules and "Common" for shared project modules.
		/// </summary>
		public List<string> MergePluginsLaunch { get; } = [];

		/// <summary>
		/// Experimental: Whether to merge only game feature plugins, ignoring regular plugins.
		/// </summary>
		public bool bMergeOnlyGameFeatures { get; set; } = true;

		/// <summary>
		/// Experimental: Whether to report module merging data to the game target.
		/// </summary>
		[CommandLine("-MergeModulesReport")]
		public bool bReportModuleMergingData { get; set; } = false;

		/// <summary>
		/// Backing storage for the LinkType property.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[CommandLine("-Monolithic", Value = "Monolithic")]
		[CommandLine("-Modular", Value = "Modular")]
		TargetLinkType LinkTypePrivate = TargetLinkType.Default;

		/// <summary>
		/// Macros to define globally across the whole target.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[CommandLine("-Define:")]
		public List<string> GlobalDefinitions { get; } = [];

		/// <summary>
		/// Macros to define across all macros in the project.
		/// </summary>
		[CommandLine("-ProjectDefine:")]
		public List<string> ProjectDefinitions { get; } = [];

		/// <summary>
		/// Variables that will be stored in BuildSettings module. Can be fetched with BuildSettings::GetVariable
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		public Dictionary<string, string> BuildSettingsVariables { get; } = [];

		/// <summary>
		/// Environment variables. Should be used instead of Environment.Set/GetEnvironmentVariable in .build.cs
		/// </summary>
		public AdditionalProperties AdditionalProperties { get; } = new();

		/// <summary>
		/// Specifies the name of the launch module. For modular builds, this is the module that is compiled into the target's executable.
		/// </summary>
		public string? LaunchModuleName
		{
			get => (LaunchModuleNamePrivate == null && Type != global::UnrealBuildTool.TargetType.Program) ? "Launch" : LaunchModuleNamePrivate;
			set => LaunchModuleNamePrivate = value;
		}

		/// <summary>
		/// Backing storage for the LaunchModuleName property.
		/// </summary>
		private string? LaunchModuleNamePrivate { get; set; }

		/// <summary>
		/// Specifies the path to write a header containing public definitions for this target. Useful when building a DLL to be consumed by external build processes.
		/// </summary>
		public string? ExportPublicHeader { get; set; }

		/// <summary>
		/// List of additional modules to be compiled into the target.
		/// </summary>
		public List<string> ExtraModuleNames { get; } = [];

		/// <summary>
		/// Path to a manifest to output for this target
		/// </summary>
		[CommandLine("-Manifest")]
		public List<FileReference> ManifestFileNames { get; } = [];

		/// <summary>
		/// Path to a list of dependencies for this target, when precompiling
		/// </summary>
		[CommandLine("-DependencyList")]
		public List<FileReference> DependencyListFileNames { get; } = [];

		/// <summary>
		/// Backing storage for the BuildEnvironment property
		/// </summary>
		[CommandLine("-SharedBuildEnvironment", Value = "Shared")]
		[CommandLine("-UniqueBuildEnvironment", Value = "Unique")]
		private TargetBuildEnvironment? BuildEnvironmentOverride;

		/// <summary>
		/// Specifies the build environment for this target. See TargetBuildEnvironment for more information on the available options.
		/// </summary>
		public TargetBuildEnvironment BuildEnvironment
		{
			get
			{
				if (BuildEnvironmentOverride.HasValue)
				{
					if (BuildEnvironmentOverride.Value == TargetBuildEnvironment.UniqueIfNeeded)
					{
						throw new BuildException($"Target {Name} had BuildEnv set to UniqueIfNeeded when querying, which means UpdateBuildEnvironmentIfNeeded wasn't called in time");
					}
					return BuildEnvironmentOverride.Value;
				}
				if (Type == TargetType.Program && ProjectFile != null && File!.IsUnderDirectory(ProjectFile.Directory))
				{
					return TargetBuildEnvironment.Unique;
				}
				else if (Unreal.IsEngineInstalled() || LinkType != TargetLinkType.Monolithic)
				{
					return TargetBuildEnvironment.Shared;
				}
				else
				{
					return TargetBuildEnvironment.Unique;
				}
			}
			set => BuildEnvironmentOverride = value;
		}

		/// <summary>
		/// If this true, a Shared build environment target will allow for any modules that are SDK version sensitive to have
		/// a project side module when an SDK is overridden. For instance, if IOSTargetPlatform, which is marked as IOS SDK 
		/// version relevant, is compiled for a Target with this set to true, and that target overrides the IOS SDK, then
		/// this would compile to e.g. MyProject/Binaries/Win64/IOS/MyProject-IOSTargetPlatform.dll
		/// </summary>
		public bool bAllowSDKOverrideModulesWithSharedEnvironment { get; protected set; } = false;

		/// <summary>
		/// Whether to ignore violations to the shared build environment (eg. editor targets modifying definitions)
		/// </summary>
		[CommandLine("-OverrideBuildEnvironment")]
		public bool bOverrideBuildEnvironment { get; set; }

		/// <summary>
		/// Specifies a list of targets which should be built before this target is built.
		/// </summary>
		public List<TargetInfo> PreBuildTargets { get; } = [];

		/// <summary>
		/// Specifies a list of steps which should be executed before this target is built, in the context of the host platform's shell.
		/// The following variables will be expanded before execution:
		/// $(EngineDir), $(ProjectDir), $(TargetName), $(TargetPlatform), $(TargetConfiguration), $(TargetType), $(ProjectFile).
		/// </summary>
		public List<string> PreBuildSteps { get; } = [];

		/// <summary>
		/// Specifies a list of steps which should be executed after this target is built, in the context of the host platform's shell.
		/// The following variables will be expanded before execution:
		/// $(EngineDir), $(ProjectDir), $(TargetName), $(TargetPlatform), $(TargetConfiguration), $(TargetType), $(ProjectFile).
		/// </summary>
		public List<string> PostBuildSteps { get; } = [];

		/// <summary>
		/// Specifies additional build products produced as part of this target.
		/// </summary>
		public List<string> AdditionalBuildProducts { get; } = [];

		/// <summary>
		/// Additional arguments to pass to the compiler
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[CommandLine("-CompilerArguments=")]
		public string? AdditionalCompilerArguments { get; set; }

		/// <summary>
		/// Additional arguments to pass to the linker
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[CommandLine("-LinkerArguments=")]
		public string? AdditionalLinkerArguments { get; set; }

		/// <summary>
		/// Max amount of memory that each compile action may require. Used by ParallelExecutor to decide the maximum 
		/// number of parallel actions to start at one time.
		/// </summary>
		public double MemoryPerActionGB { get; set; } = 0.0;

		/// <summary>
		/// List of modules to disable unity builds for
		/// </summary>
		[XmlConfigFile(Category = "ModuleConfiguration", Name = "DisableUnityBuild")]
		public string[]? DisableUnityBuildForModules = null;

		/// <summary>
		///  List of modules to enable optimizations for
		/// </summary>
		[XmlConfigFile(Category = "ModuleConfiguration", Name = "EnableOptimizeCode")]
		public string[]? EnableOptimizeCodeForModules = null;

		/// <summary>
		/// List of modules to disable optimizations for
		/// </summary>
		[XmlConfigFile(Category = "ModuleConfiguration", Name = "DisableOptimizeCode")]
		public string[]? DisableOptimizeCodeForModules = null;

		/// <summary>
		///  List of modules to optimize for size. This allows the target to override module optimization level
		///  Note that this may disable PCH usage if a private PCH is not provided
		/// </summary>
		[XmlConfigFile(Category = "ModuleConfiguration", Name = "OptimizeForSize")]
		public string[]? OptimizeForSizeModules = null;

		/// <summary>
		///  List of modules to optimize for size and speed. This allows the target to override module optimization level
		///  Note that this may disable PCH usage if a private PCH is not provided
		/// </summary>
		[XmlConfigFile(Category = "ModuleConfiguration", Name = "OptimizeForSizeAndSpeed")]
		public string[]? OptimizeForSizeAndSpeedModules = null;

		/// <summary>
		/// When generating project files, specifies the name of the project file to use when there are multiple targets of the same type.
		/// </summary>
		public string? GeneratedProjectName { get; set; }

		/// <summary>
		/// If this is non-null, then any platforms NOT listed will not be allowed to have modules in their directories be created
		/// </summary>
		public UnrealTargetPlatform[]? OptedInModulePlatforms { get; set; } = null;

		/// <summary>
		/// Android-specific target settings.
		/// </summary>
		[ConfigSubObject]
		public AndroidTargetRules AndroidPlatform { get; init; } = new();

		/// <summary>
		/// IOS-specific target settings.
		/// </summary>
		[ConfigSubObject]
		public IOSTargetRules IOSPlatform { get; init; } = new();

		/// <summary>
		/// Linux-specific target settings.
		/// </summary>
		[ConfigSubObject]
		public LinuxTargetRules LinuxPlatform { get; init; } = new();

		/// <summary>
		/// Mac-specific target settings.
		/// </summary>
		[ConfigSubObject]
		public MacTargetRules MacPlatform { get; init; } = new();

		/// <summary>
		/// Windows-specific target settings.
		/// </summary>
		[ConfigSubObject]
		public WindowsTargetRules WindowsPlatform { get; init; } // Requires 'this' parameter; initialized in constructor

		/// <summary>
		/// Statically selected allocator to use
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		public StaticAllocatorType StaticAllocator { get; set; } = StaticAllocatorType.None;

		/// <summary>
		/// Checks if the target needs to care about sdk version - if not, we can share the target between projects that are on differnet sdk versions.
		/// For instance, ShaderCompileWorker is very tied to, say, console sdk versions, but UnrealPak doesn't have sdk things compiled in
		/// </summary>
		/// <returns></returns>
		public bool IsSDKVersionRelevant(UnrealTargetPlatform sdkPlatform)
		{
			// we always care bout the sdk of the platform we are compiling for (Platform is the platform this target is being built for)
			if (Platform == sdkPlatform)
			{
				return true;
			}

			// at this point, we are checking against an sdk that we are not compiling for, ie a target platform sdk, so check if the Target has overriden it
			if (bAreTargetSDKVersionsRelevantOverride != null)
			{
				return bAreTargetSDKVersionsRelevantOverride.Value;
			}

			// developer tools are what pull in sdk versions for other platforms, so Programs that don't use dev tools don't care about versions
			if (Type == global::UnrealBuildTool.TargetType.Program || Type == global::UnrealBuildTool.TargetType.Editor)
			{
				return bUsesTargetSDKs;
			}

			// games/client/servers are not expected to contain other platform sdk bits in them, so we don't need to care
			return false;
		}

		/// <summary>
		/// Allow a target to override whether or not AreSDKVersionsRelevant uses default logic
		/// </summary>
		protected bool? bAreTargetSDKVersionsRelevantOverride { get; set; }

		/// <summary>
		/// Checks if the target can use non-standard sdk versions. For instance, the editor is usually compiled as a modular build shared between projects,
		/// so if one projet uses sdk version 1, and another uses 2, then the generated engine modules will use the most recently built sdk, which could break
		/// </summary>
		/// <returns></returns>
		public bool AllowsPerProjectSDKVersion()
		{
			// modular target with TargetBuildEnvironment.Shared build type cannot allow per-project SDKs
			return LinkType == TargetLinkType.Monolithic || BuildEnvironment == TargetBuildEnvironment.Unique || bAllowSDKOverrideModulesWithSharedEnvironment;
		}

		/// <summary>
		/// Checks if a plugin should be programmatically allowed in the build
		/// </summary>
		/// <returns>true if the plugin is allowed</returns>
		public virtual bool ShouldIgnorePluginDependency(PluginInfo parentInfo, PluginReferenceDescriptor descriptor) => false;

		/// <summary>
		/// Validate that the merged modules setup is appropriate
		/// </summary>
		public virtual void ReportMergedModules(Dictionary<string, IEnumerable<ModuleRules>> ModuleGroups, Dictionary<string, Dictionary<string, List<string>>> RootPluginUsage) { }

		/// <summary>
		/// Checks if a property has been set with the RequiresUniqueBuildEnvironmentAttribute, and is different from it's base
		/// </summary>
		/// <param name="rulesAssembly">Assembly containing the target</param>
		/// <param name="arguments">Commandline options that may affect the target creation</param>
		/// <param name="propNamesThatRequireUnique">If the target requires a unique build environment, this will contain the names of the field/property that require unique, mapped to a pair of base and current value</param>
		/// <param name="baseTargetName">If the target requires a unique build environment, this will contain the name of the target this was based on (UnrealGame, UnrealEditor, etc)/property</param>
		/// <returns>true if a property was set such that it requires a unique build environment</returns>
		/// <exception cref="BuildException"></exception>
		public bool RequiresUniqueEnvironment(RulesAssembly rulesAssembly, CommandLineArguments? arguments, Dictionary<string, (string?, string?)> propNamesThatRequireUnique, [NotNullWhen(true)] out string? baseTargetName)
		{
			baseTargetName = null;

			TargetRules thisRules = this;
			// Allow disabling these checks
			if (thisRules.bOverrideBuildEnvironment)
			{
				return false;
			}

			// Get the name of the target with default settings
			switch (thisRules.Type)
			{
				case TargetType.Game:
					baseTargetName = "UnrealGame";
					break;
				case TargetType.Editor:
					baseTargetName = "UnrealEditor";
					break;
				case TargetType.Client:
					baseTargetName = "UnrealClient";
					break;
				case TargetType.Server:
					baseTargetName = "UnrealServer";
					break;
				default:
					return false;
			}

			// Create the target rules for it
			// we need to validate the target, but we don't care about SDK versions at all
			TargetRules baseRules = rulesAssembly.CreateTargetRules(baseTargetName, thisRules.Platform, thisRules.Configuration, thisRules.Architectures, null, arguments, Logger, IntermediateEnvironment: thisRules.IntermediateEnvironment, ValidationOptions: TargetRulesValidationOptions.ValidateTargetOnly);

			// Get all the configurable objects
			object[] baseObjects = baseRules.GetConfigurableObjects().ToArray();
			object[] thisObjects = GetConfigurableObjects().ToArray();
			if (baseObjects.Length != thisObjects.Length)
			{
				throw new BuildException("Expected same number of configurable objects from base rules object.");
			}

			// Iterate through all fields with the [SharedBuildEnvironment] attribute
			for (int idx = 0; idx < baseObjects.Length; idx++)
			{
				Type objectType = baseObjects[idx].GetType();
				foreach (FieldInfo field in objectType.GetFields().Where(field => field.GetCustomAttribute<RequiresUniqueBuildEnvironmentAttribute>() != null))
				{
					object? thisValue = field.GetValue(thisObjects[idx]);
					object? baseValue = field.GetValue(baseObjects[idx]);
					if (!CheckValuesMatch(field.FieldType, thisValue, baseValue))
					{
						propNamesThatRequireUnique.Add(field.Name, (ValueAsString(field.FieldType, thisValue), ValueAsString(field.FieldType, baseValue)));
					}
				}

				foreach (PropertyInfo property in objectType.GetProperties().Where(property => property.GetCustomAttribute<RequiresUniqueBuildEnvironmentAttribute>() != null))
				{
					object? thisValue = property.GetValue(thisObjects[idx]);
					object? baseValue = property.GetValue(baseObjects[idx]);
					if (!CheckValuesMatch(property.PropertyType, thisValue, baseValue))
					{
						propNamesThatRequireUnique.Add(property.Name, (ValueAsString(property.PropertyType, thisValue), ValueAsString(property.PropertyType, baseValue)));
					}
				}
			}

			// Iterate through all fields with the [ApplyWarnings] attribute, which are consumed by the CppCompileWarnings system and inserted into the compile arguments
			for (int idx = 0; idx < baseObjects.Length; idx++)
			{
				Type objectType = baseObjects[idx].GetType();
				foreach (FieldInfo field in objectType.GetFields().Where(field => field.GetCustomAttributes<ApplyWarningsAttribute>().FirstOrDefault() != null))
				{
					object? thisValue = field.GetValue(thisObjects[idx]);
					object? baseValue = field.GetValue(baseObjects[idx]);
					if (!CheckValuesMatch(field.FieldType, thisValue, baseValue))
					{
						propNamesThatRequireUnique.Add(field.Name, (ValueAsString(field.FieldType, thisValue), ValueAsString(field.FieldType, baseValue)));
					}
				}

				foreach (PropertyInfo property in objectType.GetProperties().Where(property => property.GetCustomAttributes<ApplyWarningsAttribute>().FirstOrDefault() != null))
				{
					object? thisValue = property.GetValue(thisObjects[idx]);
					object? baseValue = property.GetValue(baseObjects[idx]);
					if (!CheckValuesMatch(property.PropertyType, thisValue, baseValue))
					{
						propNamesThatRequireUnique.Add(property.Name, (ValueAsString(property.PropertyType, thisValue), ValueAsString(property.PropertyType, baseValue)));
					}
				}
			}

			// if any properties require a unique build environment, return true
			return propNamesThatRequireUnique.Count > 0;
		}

		/// <summary>
		/// For any target that has set BuildEnvironment = TargetBuildEnvironment.Unique, this will change the Environment to either Shared or Unique accordingly
		/// </summary>
		/// <param name="rulesAssembly">Assembly containing the target</param>
		/// <param name="arguments">Commandline options that may affect the target creation</param>
		/// <param name="logger">Logger</param>
		public void UpdateBuildEnvironmentIfNeeded(RulesAssembly rulesAssembly, CommandLineArguments? arguments, ILogger logger)
		{
			// only do anything here when using UniqueIfNeeded
			if (!BuildEnvironmentOverride.HasValue || BuildEnvironmentOverride.Value != TargetBuildEnvironment.UniqueIfNeeded)
			{
				return;
			}

			// RequiresUniqueEnvironment is expensive so early out when generating project files
			if (IntermediateEnvironment == UnrealIntermediateEnvironment.GenerateProjectFiles)
			{
				BuildEnvironment = TargetBuildEnvironment.Unique;
				return;
			}

			if (UEBuildPlatformSDK.bHasAnySDKOverride)
			{
				// check if any platform needs a unique environment for the sdk check
				foreach (UnrealTargetPlatform platform in UnrealTargetPlatform.GetValidPlatforms())
				{
					if (IsSDKVersionRelevant(platform))
					{
						UEBuildPlatformSDK? sdk = UEBuildPlatformSDK.GetSDKForPlatform(platform.ToString());
						if (sdk != null && sdk.bHasSDKOverride)
						{
							logger.LogInformation("Setting {Target}'s BuildEnvironment to Unique, because a project overrode the {Platform} sdk version", Name, platform);
							BuildEnvironment = TargetBuildEnvironment.Unique;
							break;
						}
					}
				}
			}

			// if we didn't set it above, check the properties
			if (BuildEnvironmentOverride.Value == TargetBuildEnvironment.UniqueIfNeeded)
			{
				Dictionary<string, (string?, string?)> propNames = [];
				string? baseTargetName;
				if (RequiresUniqueEnvironment(rulesAssembly, arguments, propNames, out baseTargetName))
				{
					logger.LogInformation("Setting {Target}'s BuildEnvironment to Unique, because it had changed the values of the propertues [ {Props} ] away from the values specified in {BaseTarget}",
						Name, String.Join(", ", propNames.Select(x => $"{x.Key}: {x.Value.Item1} != {x.Value.Item2}")), baseTargetName);
					BuildEnvironment = TargetBuildEnvironment.Unique;
				}
				else
				{
					BuildEnvironment = TargetBuildEnvironment.Shared;
				}
			}
		}

		static bool CheckValuesMatch(Type valueType, object? thisValue, object? baseValue)
		{
			// Check if the fields match, treating lists of strings (eg. definitions) differently to value types.
			bool bFieldsMatch;
			if (thisValue == null || baseValue == null)
			{
				bFieldsMatch = (thisValue == baseValue);
			}
			else if (typeof(IEnumerable<string>).IsAssignableFrom(valueType))
			{
				bFieldsMatch = Enumerable.SequenceEqual((IEnumerable<string>)thisValue, (IEnumerable<string>)baseValue);
			}
			else if (typeof(IDictionary<string, string>).IsAssignableFrom(valueType))
			{
				bFieldsMatch = Enumerable.SequenceEqual((IDictionary<string, string>)thisValue, (IDictionary<string, string>)baseValue);
			}
			else
			{
				bFieldsMatch = thisValue.Equals(baseValue);
			}

			return bFieldsMatch;
		}

		static string ValueAsString(Type valueType, object? value)
		{
			if (value == null)
			{
				return "null";
			}
			else if (typeof(IEnumerable<string>).IsAssignableFrom(valueType))
			{
				IEnumerable<string> enumerable = (IEnumerable<string>)value;
				return String.Join(',', (IEnumerable<string>)value);
			}
			else if (typeof(IDictionary<string, string>).IsAssignableFrom(valueType))
			{
				IDictionary<string, string> dictionary = (IDictionary<string, string>)value;
				return String.Join(',', dictionary.Select(x => $"{x.Key}:{x.Value}"));
			}
			return value.ToString() ?? String.Empty;
		}

		/// <summary>
		/// Create a TargetRules instance
		/// </summary>
		/// <param name="rulesType">Type to create</param>
		/// <param name="targetInfo">Target info</param>
		/// <param name="baseFile">Path to the file for the rules assembly</param>
		/// <param name="platformFile">Path to the platform specific rules file</param>
		/// <param name="targetFiles">Path to all possible rules file</param>
		/// <param name="defaultBuildSettings"></param>
		/// <param name="logger">Logger for the new target rules</param>
		/// <returns>Target instance</returns>
		public static TargetRules Create(Type rulesType, TargetInfo targetInfo, FileReference? baseFile, FileReference? platformFile, IEnumerable<FileReference>? targetFiles, BuildSettingsVersion? defaultBuildSettings, ILogger logger)
		{

			TargetRules rules = (TargetRules)RuntimeHelpers.GetUninitializedObject(rulesType);
			if (defaultBuildSettings.HasValue)
			{
				rules.DefaultBuildSettings = defaultBuildSettings.Value;
			}

			// The base target file name: this affects where the resulting build product is created so the platform/group is not desired in this case.
			rules.File = baseFile;

			// The platform/group-specific target file name
			rules.TargetSourceFile = platformFile;

			// All target files for this target that could cause invalidation
			rules.TargetFiles = targetFiles?.ToHashSet() ?? [];

			// Initialize the logger
			rules.Logger = logger;

			// Find the constructor
			ConstructorInfo? constructor = rulesType.GetConstructor(new Type[] { typeof(TargetInfo) })
				?? throw new CompilationResultException(CompilationResult.RulesError, KnownLogEvents.RulesAssembly, "No constructor found on {TargetName} which takes an argument of type TargetInfo.", rulesType.Name);

			// Invoke the regular constructor
			try
			{
				constructor.Invoke(rules, new object[] { targetInfo });
				CppCompileWarnings.ApplyTargetDefaults(rules.CppCompileWarningSettings);
			}
			catch (Exception ex)
			{
				throw new CompilationResultException(CompilationResult.RulesError, ex, KnownLogEvents.RulesAssembly, "Unable to instantiate instance of '{TargetName}' object type from compiled assembly '{AssemblyPath}'.  Unreal Build Tool creates an instance of your module's 'Rules' object in order to find out about your module's requirements.  The CLR exception details may provide more information: {ExceptionMessage}",
					rulesType.Name, Path.GetFileNameWithoutExtension(rulesType.Assembly?.Location) ?? "Unknown Assembly", ex.ToString());
			}

			return rules;
		}

		/// <summary>
		/// Validate the target rules for its current platform when using modular build
		/// </summary>
		/// <param name="target">Rules to validate</param>
		/// <param name="platform">Explicit platform to use</param>
		public static void ValidateMergedTarget(TargetRules target, UnrealTargetPlatform platform)
		{
			UEBuildPlatform BuildPlatform = UEBuildPlatform.GetBuildPlatform(platform);
			BuildPlatform.ValidateMergedTarget(target);
		}
		
		/// <summary>
		/// Constructor.
		/// </summary>
		/// <param name="target">Information about the target being built</param>
		protected TargetRules(TargetInfo target)
		{
			DefaultName = target.Name;
			Platform = target.Platform;
			Configuration = target.Configuration;
			Architectures = target.Architectures;
			IntermediateEnvironment = target.IntermediateEnvironment;
			ProjectFile = target.ProjectFile;
			Version = target.Version;
			WindowsPlatform = new WindowsTargetRules(this);
			CppCompileWarningSettings = new CppCompileWarnings(this, Logger);

			// Make sure the logger was initialized by the caller
			if (Logger == null)
			{
				throw new NotSupportedException("Logger property must be initialized by the caller.");
			}

			// TODO: Need to get the compression setting from UBA so we can invalidate actions if the compression changes.
			// Figure out a better way to do this.
			{
				UnrealBuildAcceleratorConfig ubaConfig = new();
				XmlConfig.ApplyTo(ubaConfig);
				bAllowUbaCompression = ubaConfig.bStoreObjFilesCompressed;
			}

			// Read settings from config files
			Dictionary<ConfigDependencyKey, IReadOnlyList<string>?> configValues = [];
			foreach (object configurableObject in GetConfigurableObjects())
			{
				ConfigCache.ReadSettings(DirectoryReference.FromFile(ProjectFile), Platform, configurableObject, configValues, target.Arguments);
				XmlConfig.ApplyTo(configurableObject);
				target.Arguments?.ApplyTo(configurableObject);
			}
			ConfigValueTracker = new ConfigValueTracker(configValues);

			// If we've got a changelist set, set that we're making a formal build
			bFormalBuild = bFormalBuild || (Version.Changelist != 0 && Version.IsPromotedBuild);

			// Allow the build platform to set defaults for this target
			UEBuildPlatform.GetBuildPlatform(Platform).ResetTarget(this);
			bDeployAfterCompile = !bForceSkipDeploy && bDeployAfterCompile;

			// Determine intermediate environment overrides based on command line flags
			if (StaticAnalyzer != StaticAnalyzer.None)
			{
				if (!bUseUnityBuild)
				{
					bUseUnityBuildOverride = false;
				}
				// Always override environment for analyzing, regardless of other settings
				IntermediateEnvironment = UnrealIntermediateEnvironment.Analyze;
			}
			else if (IntermediateEnvironment == UnrealIntermediateEnvironment.Default)
			{
				if (bIWYU)
				{
					IntermediateEnvironment = UnrealIntermediateEnvironment.IWYU;
				}
				else if (!bUseUnityBuild)
				{
					IntermediateEnvironment = UnrealIntermediateEnvironment.NonUnity;
				}
			}

			if (bIWYU)
			{
				WindowsPlatform.Compiler = WindowsCompiler.Clang;
				bDisableLinking = true; // No linking in iwyu builds
				CppCompileWarningSettings.UnusedWarningLevel = WarningLevel.Off; // Disable warnings to avoid extra noise
				bUsePCHFiles = false; // No pch for iwyu
				bUseVFS = false; // Do not use vfs because IWYUMode can't handle virtual paths (yet)
				bUseUnityBuild = false; // No unity for iwyu
				CppStandard = CppStandardVersion.Cpp23; // Must be set to 23 to handle function default values same as msvc
				CppStandardEngine = CppStandardVersion.Cpp23; // Must be set to 23 to handle function default values same as msvc
				// bAlwaysUseUnityForGeneratedFiles = false; // This should be used but adds too many extra files atm.
			}

			// Header units only work for vs 2026 (tested with unreleased 14.50.35615)
			// (For future I suspect we want bUseHeaderUnitsForPch to default to true with 2026)
			if (WindowsPlatform.Compiler != WindowsCompiler.VisualStudio2026)
			{
				bUseHeaderUnitsForPch = false;
			}

			// Set the default build version
			if (String.IsNullOrEmpty(BuildVersion))
			{
				if (String.IsNullOrEmpty(target.Version.BuildVersionString))
				{
					BuildVersion = $"{target.Version.BranchName}-CL-{target.Version.Changelist}";
				}
				else
				{
					BuildVersion = target.Version.BuildVersionString;
				}
			}

			// Get the directory to use for crypto settings. We can build engine targets (eg. UHT) with 
			// a project file, but we can't use that to determine crypto settings without triggering
			// constant rebuilds of UHT.
			DirectoryReference? cryptoSettingsDir = DirectoryReference.FromFile(ProjectFile);
			if (cryptoSettingsDir != null && File != null && !File.IsUnderDirectory(cryptoSettingsDir))
			{
				cryptoSettingsDir = null;
			}

			// Setup macros for signing and encryption keys
			EncryptionAndSigning.CryptoSettings cryptoSettings = EncryptionAndSigning.ParseCryptoSettings(cryptoSettingsDir, Platform, Logger);
			if (cryptoSettings.IsAnyEncryptionEnabled())
			{
				ProjectDefinitions.Add($"IMPLEMENT_ENCRYPTION_KEY_REGISTRATION()=UE_REGISTER_ENCRYPTION_KEY({FormatHexBytes(cryptoSettings.EncryptionKey!.Key!)})");
			}
			else
			{
				ProjectDefinitions.Add("IMPLEMENT_ENCRYPTION_KEY_REGISTRATION()=");
			}

			if (cryptoSettings.IsPakSigningEnabled())
			{
				ProjectDefinitions.Add($"IMPLEMENT_SIGNING_KEY_REGISTRATION()=UE_REGISTER_SIGNING_KEY(UE_LIST_ARGUMENT({FormatHexBytes(cryptoSettings.SigningKey!.PublicKey.Exponent!)}), UE_LIST_ARGUMENT({FormatHexBytes(cryptoSettings.SigningKey.PublicKey.Modulus!)}))");
			}
			else
			{
				ProjectDefinitions.Add("IMPLEMENT_SIGNING_KEY_REGISTRATION()=");
			}
		}

		/// <summary>
		/// Formats an array of bytes as a sequence of values
		/// </summary>
		/// <param name="data">The data to convert into a string</param>
		/// <returns>List of hexadecimal bytes</returns>
		private static string FormatHexBytes(byte[] data)
		{
			return String.Join(",", data.Select(x => $"0x{x:X2}"));
		}

		/// <summary>
		/// Override any settings required for the selected target type
		/// </summary>
		internal void SetOverridesForTargetType()
		{
			if (Type == global::UnrealBuildTool.TargetType.Game)
			{
				GlobalDefinitions.Add("UE_GAME=1");
			}
			else if (Type == global::UnrealBuildTool.TargetType.Client)
			{
				GlobalDefinitions.Add("UE_GAME=1");
				GlobalDefinitions.Add("UE_CLIENT=1");
			}
			else if (Type == global::UnrealBuildTool.TargetType.Editor)
			{
				GlobalDefinitions.Add("UE_EDITOR=1");
			}
			else if (Type == global::UnrealBuildTool.TargetType.Server)
			{
				GlobalDefinitions.Add("UE_SERVER=1");
				GlobalDefinitions.Add("USE_NULL_RHI=1");
			}
		}

		/// <summary>
		/// Override settings that all cooked editor targets will want
		/// </summary>
		protected void SetDefaultsForCookedEditor(bool bIsCookedCooker, bool bIsForExternalUse)
		{
			LinkType = TargetLinkType.Monolithic;
			BuildEnvironment = TargetBuildEnvironment.Unique;

			if (!bIsCookedCooker)
			{
				bBuildAdditionalConsoleApp = false;
			}

			GlobalDefinitions.Add("ASSETREGISTRY_FORCE_PREMADE_REGISTRY_IN_EDITOR=1");
			bUseLoggingInShipping = true;

			GlobalDefinitions.Add("UE_IS_COOKED_EDITOR=1");

			// remove some insecure things external users may not want
			if (bIsForExternalUse)
			{
				bWithServerCode = false;
				bBuildTargetDeveloperTools = false;
				GlobalDefinitions.Add("AUTOSDKS_ENABLED=0");
			}

			// this will allow shader compiling to work based on whether or not the shaders directory is present
			// to determine if we should allow shader compilation
			GlobalDefinitions.Add("UE_ALLOW_SHADER_COMPILING_BASED_ON_SHADER_DIRECTORY_EXISTENCE=1");
			// this setting can be used to compile out the shader compiler if that is important
			//GlobalDefinitions.Add("UE_ALLOW_SHADER_COMPILING=0");

			ConfigHierarchy projectGameIni = ConfigCache.ReadHierarchy(ConfigHierarchyType.Game, ProjectFile?.Directory, Platform, CustomConfig);
			List<string>? disabledPlugins;
			List<string> allDisabledPlugins = [];

			if (projectGameIni.GetArray("CookedEditorSettings", "DisabledPlugins", out disabledPlugins))
			{
				allDisabledPlugins.AddRange(disabledPlugins);
			}
			if (projectGameIni.GetArray("CookedEditorSettings" + (bIsCookedCooker ? "_CookedCooker" : "_CookedEditor"), "DisabledPlugins", out disabledPlugins))
			{
				allDisabledPlugins.AddRange(disabledPlugins);
			}
			if (Configuration == UnrealTargetConfiguration.Shipping)
			{
				if (projectGameIni.GetArray("CookedEditorSettings", "DisabledPluginsInShipping", out disabledPlugins))
				{
					allDisabledPlugins.AddRange(disabledPlugins);
				}
				if (projectGameIni.GetArray("CookedEditorSettings" + (bIsCookedCooker ? "_CookedCooker" : "_CookedEditor"), "DisabledPluginsInShipping", out disabledPlugins))
				{
					allDisabledPlugins.AddRange(disabledPlugins);
				}
			}

			// disable them, and remove them from Enabled in case they were there
			foreach (string pluginName in allDisabledPlugins)
			{
				DisablePlugins.Add(pluginName);
				EnablePlugins.Remove(pluginName);
				OptionalPlugins.Remove(pluginName);
			}
		}

		/// <summary>
		/// Gets a list of platforms that this target supports
		/// </summary>
		/// <returns>Array of platforms that the target supports</returns>
		internal UnrealTargetPlatform[] GetSupportedPlatforms()
		{
			// Take the SupportedPlatformsAttribute from the first type in the inheritance chain that supports it
			for (Type? currentType = GetType(); currentType != null; currentType = currentType.BaseType)
			{
				object[] attributes = currentType.GetCustomAttributes(typeof(SupportedPlatformsAttribute), false);
				if (attributes.Length > 0)
				{
					return attributes.OfType<SupportedPlatformsAttribute>().SelectMany(x => x.Platforms).Distinct().ToArray();
				}
			}

			// Otherwise, get the default for the target type
			if (Type == TargetType.Program)
			{
				return Utils.GetPlatformsInClass(UnrealPlatformClass.Desktop);
			}
			else if (Type == TargetType.Editor)
			{
				return Utils.GetPlatformsInClass(UnrealPlatformClass.Editor);
			}
			else if (Type == TargetType.Server)
			{
				return Utils.GetPlatformsInClass(UnrealPlatformClass.Server);
			}
			else
			{
				return Utils.GetPlatformsInClass(UnrealPlatformClass.All);
			}
		}

		/// <summary>
		/// Gets a list of configurations that this target supports
		/// </summary>
		/// <returns>Array of configurations that the target supports</returns>
		internal UnrealTargetConfiguration[] GetSupportedConfigurations()
		{

			UnrealTargetConfiguration[]? configs = GetSupportedConfigurationFromAttribute(GetType());
			if (configs != null)
			{
				return configs;
			}

			// Otherwise, get the default for the target type
			if (Type == TargetType.Editor)
			{
				return new[] { UnrealTargetConfiguration.Debug, UnrealTargetConfiguration.DebugGame, UnrealTargetConfiguration.Development };
			}
			else
			{
				return [.. Enum.GetValues<UnrealTargetConfiguration>().Where(x => x != UnrealTargetConfiguration.Unknown)];
			}
		}

		/// <summary>
		/// Finds all the subobjects which can be configured by command line options and config files
		/// </summary>
		/// <returns>Sequence of objects</returns>
		internal IEnumerable<object> GetConfigurableObjects()
		{
			yield return this;

			foreach (FieldInfo field in GetType().GetFields(BindingFlags.Public | BindingFlags.Instance).Where(field => field.GetCustomAttribute<ConfigSubObjectAttribute>() != null))
			{
				object? value = field.GetValue(this);
				if (value != null)
				{
					yield return value;
				}
			}
			foreach (PropertyInfo property in GetSubObjectProperties(GetType()))
			{
				object? value = property.GetValue(this);
				if (value != null)
				{
					yield return value;
				}
			}
			foreach (UnrealTargetPlatform platform in UnrealTargetPlatform.GetValidPlatforms())
			{
				UEBuildPlatform? buildPlatform;
				if (UEBuildPlatform.TryGetBuildPlatform(platform, out buildPlatform))
				{
					yield return buildPlatform.ArchitectureConfig;
				}
			}
		}

		/// <summary>
		/// Gets the host platform being built on
		/// </summary>
#pragma warning disable CA1822 // Mark members as static - used in module build files, leave as-is
		public UnrealTargetPlatform HostPlatform => BuildHostPlatform.Current.Platform;
#pragma warning restore CA1822

		/// <summary>
		/// Expose the bGenerateProjectFiles flag to targets, so we can modify behavior as appropriate for better intellisense
		/// </summary>
#pragma warning disable CA1822 // Mark members as static - used in module build files, leave as-is
		public bool bGenerateProjectFiles => ProjectFileGenerator.bGenerateProjectFiles;
#pragma warning restore CA1822

		/// <summary>
		/// Indicates whether target rules should be used to explicitly enable or disable plugins. Usually not needed for project generation unless project files indicate whether referenced plugins should be built or not.
		/// </summary>
#pragma warning disable CA1822 // Mark members as static - used in module build files, leave as-is
		public bool bShouldTargetRulesTogglePlugins => ((ProjectFileGenerator.Current != null) && ProjectFileGenerator.Current.ShouldTargetRulesTogglePlugins())
					|| ((ProjectFileGenerator.Current == null) && !ProjectFileGenerator.bGenerateProjectFiles);
#pragma warning restore CA1822

		/// <summary>
		/// Expose a setting for whether or not the engine is installed
		/// </summary>
		/// <returns>Flag for whether the engine is installed</returns>
#pragma warning disable CA1822 // Mark members as static - used in module build files, leave as-is
		public bool bIsEngineInstalled => Unreal.IsEngineInstalled();
#pragma warning restore CA1822

		/// <summary>
		/// Gets diagnostic messages about default settings which have changed in newer versions of the engine
		/// </summary>
		/// <param name="diagnostics">List of diagnostic messages</param>
		internal void GetBuildSettingsInfo(List<string> diagnostics)
		{
			// Resolve BuildSettingsVersion.Latest to the version it's assigned to
			BuildSettingsVersion latestVersion = BuildSettingsVersion.Latest;
			foreach (BuildSettingsVersion value in Enum.GetValues<BuildSettingsVersion>())
			{
				if ((int)value == (int)BuildSettingsVersion.Latest)
				{
					latestVersion = value;
					break;
				}
			}

			if (DefaultBuildSettings < latestVersion)
			{
				diagnostics.Add("[Upgrade]");
				diagnostics.Add("[Upgrade] Using backward-compatible build settings. The latest version of UE sets the following values by default, which may require code changes:");

				List<Tuple<string, string>> modifiedSettings = [];
				if (BuildSettingsVersion.V2 <= latestVersion && DefaultBuildSettings < BuildSettingsVersion.V2)
				{
					modifiedSettings.Add(Tuple.Create($"{nameof(bLegacyPublicIncludePaths)} = false", "Omits subfolders from public include paths to reduce compiler command line length. (Previously: true)."));
					modifiedSettings.Add(Tuple.Create($"{nameof(CppCompileWarningSettings.ShadowVariableWarningLevel)} = WarningLevel.Error", "Treats shadowed variable warnings as errors. (Previously: WarningLevel.Warning)."));
					modifiedSettings.Add(Tuple.Create($"{nameof(ModuleRules.PCHUsage)} = PCHUsageMode.UseExplicitOrSharedPCHs", "Set in build.cs files to enables IWYU-style PCH model. See https://docs.unrealengine.com/en-US/Programming/BuildTools/UnrealBuildTool/IWYU/index.html. (Previously: PCHUsageMode.UseSharedPCHs)."));
				}

				if (BuildSettingsVersion.V3 <= latestVersion && DefaultBuildSettings < BuildSettingsVersion.V3)
				{
					modifiedSettings.Add(Tuple.Create($"{nameof(bLegacyParentIncludePaths)} = false", "Omits module parent folders from include paths to reduce compiler command line length. (Previously: true)."));
				}

				if (BuildSettingsVersion.V4 <= latestVersion && DefaultBuildSettings < BuildSettingsVersion.V4)
				{
					modifiedSettings.Add(Tuple.Create($"{nameof(CppStandard)} = CppStandardVersion.Default", "Updates C++ Standard to C++20 (Previously: CppStandardVersion.Cpp17)."));
					modifiedSettings.Add(Tuple.Create($"{nameof(WindowsPlatform)}.{nameof(WindowsPlatform.bStrictConformanceMode)} = true", "Updates MSVC strict conformance mode to true (Previously: false)."));
				}

				if (BuildSettingsVersion.V5 <= latestVersion && DefaultBuildSettings < BuildSettingsVersion.V5)
				{
					modifiedSettings.Add(Tuple.Create($"{nameof(bValidateFormatStrings)} = true", "Enables compile-time validation of strings+args passed to UE_LOG. (Previously: false)."));
				}

				if (BuildSettingsVersion.V6 <= latestVersion && DefaultBuildSettings < BuildSettingsVersion.V6)
				{
					modifiedSettings.Add(Tuple.Create($"{nameof(WindowsPlatform)}.{nameof(WindowsPlatform.bStrictInlineConformance)} = true", "Updates MSVC inline conformance (Remove unreferenced COMDAT) to true (Previously: false)."));
					modifiedSettings.Add(Tuple.Create($"{nameof(CppCompileWarningSettings.UndefinedIdentifierWarningLevel)} = WarningLevel.Error", "Enables compile-time validation of undefined identifiers in conditional expressions. (Previously: Off)."));
				}

				if (BuildSettingsVersion.V7 <= latestVersion && DefaultBuildSettings < BuildSettingsVersion.V7)
				{
					modifiedSettings.Add(Tuple.Create($"{nameof(FPSemantics)} = {nameof(FPSemanticsMode.Precise)}", "Updates default floating point semantics for Editor and Program targets to Precise for Microsoft platforms. Game, Client, & Server default remains Imprecise. (Previously: Imprecise for MSVC, Precise for Clang)"));
					modifiedSettings.Add(Tuple.Create($"{nameof(CppCompileWarningSettings.ReturnTypeWarningLevel)} = WarningLevel.Error", "Validates that non-void functions return a value. (Previously: Off for VCClang, Error for MSVC and non-VC Clang)."));
					modifiedSettings.Add(Tuple.Create($"{nameof(CppCompileWarningSettings.DanglingWarningLevel)} = WarningLevel.Error", "Enables clang warnings related possible dangling references or pointers. (Previously: Off for VCClang, Error for non-VC Clang)."));
					modifiedSettings.Add(Tuple.Create($"{nameof(CppCompileWarningSettings.UnreachableCodeWarningLevel)} = WarningLevel.Error", "Enables compile-time validation of unreachable code. (Previously: Error for MSVC, Off for Clang)."));
				}

				if (modifiedSettings.Count > 0)
				{
					string formatString = $"[Upgrade]     {{0,-{modifiedSettings.Max(x => x.Item1.Length)}}}   => {{1}}";
					diagnostics.AddRange(modifiedSettings.Select(modifiedSetting => String.Format(formatString, modifiedSetting.Item1, modifiedSetting.Item2)));
				}
				diagnostics.Add($"[Upgrade] Suppress this message by setting 'DefaultBuildSettings = BuildSettingsVersion.{latestVersion};' in {File!.GetFileName()}, and explicitly overriding settings that differ from the new defaults.");
				diagnostics.Add("[Upgrade]");
			}

			if (IncludeOrderVersion <= (EngineIncludeOrderVersion)(EngineIncludeOrderVersion.Latest - 1) && ForcedIncludeOrder == null)
			{
				// Resolve EngineIncludeOrderVersion.Latest to the version it's assigned to
				EngineIncludeOrderVersion latestEngineIncludeOrder = EngineIncludeOrderVersion.Latest;
				foreach (EngineIncludeOrderVersion value in Enum.GetValues<EngineIncludeOrderVersion>())
				{
					if ((int)value == (int)EngineIncludeOrderVersion.Latest)
					{
						latestEngineIncludeOrder = value;
						break;
					}
				}

				diagnostics.Add("[Upgrade]");
				diagnostics.Add("[Upgrade] Using backward-compatible include order. The latest version of UE has changed the order of includes, which may require code changes. The current setting is:");
				diagnostics.Add($"[Upgrade]     IncludeOrderVersion = EngineIncludeOrderVersion.{IncludeOrderVersion}");
				diagnostics.Add($"[Upgrade] Suppress this message by setting 'IncludeOrderVersion = EngineIncludeOrderVersion.{latestEngineIncludeOrder};' in {File!.GetFileName()}.");
				diagnostics.Add("[Upgrade] Alternatively you can set this to 'EngineIncludeOrderVersion.Latest' to always use the latest include order. This will potentially cause compile errors when integrating new versions of the engine.");
				diagnostics.Add("[Upgrade]");
			}

			Logger.LogDebug("Using EngineIncludeOrderVersion.{Version} for target {Target}", IncludeOrderVersion, File!.GetFileName());

			if (CppStandardEngine < CppStandardVersion.EngineDefault)
			{
				diagnostics.Add("[Upgrade]");
				diagnostics.Add($"[Upgrade] The latest version of UE no longer supports CppStandardVersion.{CppStandardEngine} for engine code which may require code changes.");
				diagnostics.Add($"[Upgrade] Suppress this message by removing 'CppStandardEngine = CppStandardVersion.{CppStandardEngine};' in {File!.GetFileName()}.");
				diagnostics.Add("[Upgrade]");
			}

			if (CppStandard < CppStandardVersion.Default)
			{
				// Resolve CppStandardVersion.Default to the version it's assigned to
				CppStandardVersion defaultCppStandard = CppStandardVersion.Latest;
				foreach (CppStandardVersion value in Enum.GetValues<CppStandardVersion>())
				{
					if ((int)value == (int)CppStandardVersion.Default)
					{
						defaultCppStandard = value;
						break;
					}
				}

				diagnostics.Add("[Upgrade]");
				diagnostics.Add($"[Upgrade] The latest version of UE no longer {(CppStandard < CppStandardVersion.Minimum ? "supports" : "recommends")} CppStandardVersion.{CppStandard} which may require code changes.");
				diagnostics.Add($"[Upgrade] Suppress this message by setting 'CppStandard = CppStandardVersion.{defaultCppStandard};' in {File!.GetFileName()}.");
				diagnostics.Add("[Upgrade]");
			}
		}

		/// <summary>
		/// Prints diagnostic messages about default settings which have changed in newer versions of the engine
		/// </summary>
		public void PrintBuildSettingsInfoWarnings()
		{
			List<string> diagnostics = [];
			GetBuildSettingsInfo(diagnostics);
			diagnostics.ForEach(x => Logger.LogWarning("Warning: {Message}", x));
		}

		/// <summary>
		/// Determines whether a given platform, and its sdk, is available
		/// </summary>
#pragma warning disable CA1822 // Mark members as static - used in module build files, leave as-is
		public bool IsPlatformAvailable(UnrealTargetPlatform TargetPlatform)
		{
			return UEBuildPlatform.IsPlatformAvailable(TargetPlatform);
		}
#pragma warning restore CA1822

		/// <summary>
		/// Cached function for getting support configuration from type attributes
		/// </summary>
		/// <param name="t"></param>
		/// <returns></returns>
		internal static UnrealTargetConfiguration[]? GetSupportedConfigurationFromAttribute(Type t) => s_supportedConfigurations.GetOrAdd(t, static type => new Lazy<UnrealTargetConfiguration[]?>(() =>
		{
			object[] attributes = type.GetCustomAttributes(typeof(SupportedConfigurationsAttribute), false);
			if (attributes.Length > 0)
			{
				return attributes.OfType<SupportedConfigurationsAttribute>().SelectMany(x => x.Configurations).Distinct().ToArray();
			}
			if (type.BaseType != null)
			{
				return GetSupportedConfigurationFromAttribute(type.BaseType);
			}
			return null;
		})).Value;
		static readonly ConcurrentDictionary<Type, Lazy<UnrealTargetConfiguration[]?>> s_supportedConfigurations = new();

		/// <summary>
		/// Cached function for getting properties with ConfigSubObject attribute
		/// </summary>
		/// <param name="t"></param>
		/// <returns></returns>
		internal static List<PropertyInfo> GetSubObjectProperties(Type t) => s_subPropsCache.GetOrAdd(t, static type => new Lazy<List<PropertyInfo>>(() =>
		{
			List<PropertyInfo> list = new();

			if (type.BaseType != null)
			{
				list.AddRange(GetSubObjectProperties(type.BaseType));
			}

			PropertyInfo[] props = type.GetProperties(BindingFlags.Instance | BindingFlags.Public | BindingFlags.DeclaredOnly);
			foreach (PropertyInfo pi in props)
			{
				if (pi.IsDefined(typeof(ConfigSubObjectAttribute), inherit: false))
				{
					list.Add(pi);
				}
			}
			return list;
		})).Value;
		static readonly ConcurrentDictionary<Type, Lazy<List<PropertyInfo>>> s_subPropsCache = new();

	}
}
