// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Xml.Linq;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// ModuleRules extension for low level tests.
	/// </summary>
	public class TestModuleRules : ModuleRules
	{
		private static readonly XNamespace BuildGraphNamespace = XNamespace.Get("http://www.epicgames.com/BuildGraph");
		private static readonly XNamespace SchemaInstance = XNamespace.Get("http://www.w3.org/2001/XMLSchema-instance");
		private static readonly XNamespace SchemaLocation = XNamespace.Get("http://www.epicgames.com/BuildGraph ../../Build/Graph/Schema.xsd");
		private static readonly List<string> RestrictedFoldersNonPlatform = new List<string>() {
			RestrictedFolder.LimitedAccess.ToString(),
			RestrictedFolder.NotForLicensees.ToString(),
			RestrictedFolder.NoRedist.ToString(),
			RestrictedFolder.EpicInternal.ToString(),
			RestrictedFolder.CarefullyRedist.ToString()
		};

		/// <summary>
		/// Check if running in test mode.
		/// </summary>
		protected static bool InTestMode = Environment.GetCommandLineArgs().Contains("-Mode=Test");

		/// <summary>
		/// Associated tested module of this test module.
		/// </summary>
		public ModuleRules? TestedModule { get; private set; }

		/// <summary>
		/// Test metadata, used with BuildGraph only.
		/// </summary>
		protected static Metadata TestMetadata = new Metadata();

		/// <summary>
		/// Constructs a TestModuleRules object as its own test module.
		/// </summary>
		/// <param name="Target"></param>
		public TestModuleRules(ReadOnlyTargetRules Target) : base(Target)
		{
			SetupCommonProperties(Target);
		}

		/// <summary>
		/// Constructs a TestModuleRules object as its own test module.
		/// Sets common properties if using Catch2.
		/// </summary>
		public TestModuleRules(ReadOnlyTargetRules Target, bool InUsesCatch2) : base(Target)
		{
			if (InUsesCatch2)
			{
				SetupCommonProperties(Target);
			}
		}

		/// <summary>
		/// Constructs a TestModuleRules object with an associated tested module.
		/// </summary>
		public TestModuleRules(ModuleRules TestedModule) : base(TestedModule.Target)
		{
			this.TestedModule = TestedModule;

			Name = TestedModule.Name + "Tests";
			if (!String.IsNullOrEmpty(TestedModule.ShortName))
			{
				ShortName = TestedModule.ShortName + "Tests";
			}

			File = TestedModule.File;
			Directory = DirectoryReference.Combine(TestedModule.Directory, "Tests");

			Context = TestedModule.Context;

			PrivateDependencyModuleNames.AddRange(TestedModule.PrivateDependencyModuleNames);
			PublicDependencyModuleNames.AddRange(TestedModule.PublicDependencyModuleNames);

			DirectoriesForModuleSubClasses = new Dictionary<Type, DirectoryReference>
			{
				{ typeof(TestModuleRules), Directory }
			};

			// Tests can refer to tested module's Public and Private paths
			string ModulePublicDir = Path.Combine(TestedModule.ModuleDirectory, "Public");
			if (System.IO.Directory.Exists(ModulePublicDir))
			{
				PublicIncludePaths.Add(ModulePublicDir);
			}

			string ModulePrivateDir = Path.Combine(TestedModule.ModuleDirectory, "Private");
			if (System.IO.Directory.Exists(ModulePrivateDir))
			{
				PrivateIncludePaths.Add(ModulePrivateDir);
			}

			SetupCommonProperties(Target);
		}

		private void SetupCommonProperties(ReadOnlyTargetRules Target)
		{
			bIsTestModuleOverride = true;

			PCHUsage = PCHUsageMode.NoPCHs;
			PrecompileForTargets = PrecompileTargetsType.None;

			if (Target.Configuration == UnrealTargetConfiguration.Debug && Target.Platform == UnrealTargetPlatform.Linux)
			{
				OptimizeCode = CodeOptimization.Never;
			}

			bAllowConfidentialPlatformDefines = true;
			bLegalToDistributeObjectCode = true;

			// Required false for catch.hpp
			bUseUnity = false;

			// Disable exception handling so that tests can assert for exceptions
			bEnableObjCExceptions = false;
			bEnableExceptions = false;

			SetResourcesFolder("Resources");

			if (!PublicDependencyModuleNames.Contains("Catch2"))
			{
				PublicDependencyModuleNames.Add("Catch2");
			}

			if (!PrivateDependencyModuleNames.Contains("LowLevelTestsRunner"))
			{
				PrivateDependencyModuleNames.Add("LowLevelTestsRunner");
			}

			if (Target.Platform == UnrealTargetPlatform.IOS || Target.Platform == UnrealTargetPlatform.TVOS)
			{
				// Fix missing frameworks from ApplicationCore

				// Needed for CADisplayLink
				PublicFrameworks.Add("QuartzCore");

				// Needed for MTLCreateSystemDefaultDevice
				PublicWeakFrameworks.Add("Metal");
			}
		}

		/// <summary>
		/// Set test-specific resources folder relative to module directory.
		/// This will be copied to the binaries path during deployment.
		/// </summary>
		protected void SetResourcesFolder(string ResourcesRelativeFolder)
		{
			AdditionalPropertiesForReceipt.RemoveAll(Prop => Prop.Name == "ResourcesFolder");

			foreach (DirectoryReference Directory in GetAllModuleDirectories())
			{
				string TestResourcesDir = Path.Combine(Directory.FullName, ResourcesRelativeFolder);
				if (System.IO.Directory.Exists(TestResourcesDir))
				{
					AdditionalPropertiesForReceipt.Add("ResourcesFolder", TestResourcesDir);
				}
			}
		}

#pragma warning disable 8602
#pragma warning disable 8604

		/// <summary>
		/// Deprecated, test metadata now generated explicitly using -Mode-Test with -GenerateMetadata.
		/// </summary>
		/// <param name="TestMetadata"></param>
		[Obsolete("Use RunUBT -Mode=Test -GenerateMetadata instead")]
		protected void UpdateBuildGraphPropertiesFile(Metadata TestMetadata)
		{
		}

#pragma warning disable IDE0051 // Remove unused private member - used via reflection, see TestMode.cs
		/// <summary>
		/// Generates or updates metadata file for LowLevelTests.xml containing test flags: name, short name, target name, relative binaries path, supported platforms etc.
		/// Called by RunUBT.bat -Mode=Test -GenerateMetadata
		/// </summary>
		private static void UpdateBuildGraphMetadata(Metadata TestMetadata, string ModuleDirectory, string ModuleName, ILogger Log)
#pragma warning restore IDE0051
		{
			string BaseFolder = GetBaseFolder(ModuleDirectory);

			bool ModuleInRestrictedPath = IsRestrictedPath(ModuleDirectory);

			// All relevant properties
			string TestTargetName = ModuleName ?? "Launch";
			string TestBinariesPath = TryGetBinariesPath(ModuleDirectory);

			// Do not save full paths
			if (Path.IsPathRooted(TestBinariesPath))
			{
				TestBinariesPath = Path.GetRelativePath(Unreal.RootDirectory.FullName, TestBinariesPath);
			}

			// Platform-specific configurations
			string GeneratedPropertiesPlatformFile;

			string NonPublicPathPlatform;

			Dictionary<string, XDocument> SaveAtEnd = new Dictionary<string, XDocument>();

			// Generate peroperty file for each supported platform
			foreach (UnrealTargetPlatform ValidPlatform in TestMetadata.SupportedPlatforms)
			{
				bool IsRestrictedPlatformName = IsPlatformRestricted(ValidPlatform);
				if (IsRestrictedPlatformName)
				{
					NonPublicPathPlatform = Path.Combine(BaseFolder, "Restricted", "NotForLicensees", "Platforms", ValidPlatform.ToString(), "Build", "LowLevelTests", $"{TestMetadata.TestName}.xml");
				}
				else
				{
					NonPublicPathPlatform = Path.Combine(BaseFolder, "Restricted", "NotForLicensees", "Build", "LowLevelTests", $"{TestMetadata.TestName}.xml");
				}

				if (ModuleInRestrictedPath)
				{
					GeneratedPropertiesPlatformFile = NonPublicPathPlatform;
				}
				else
				{
					if (IsRestrictedPlatformName)
					{
						GeneratedPropertiesPlatformFile = Path.Combine(BaseFolder, "Platforms", ValidPlatform.ToString(), "Build", "LowLevelTests", $"{TestMetadata.TestName}.xml");
					}
					else
					{
						GeneratedPropertiesPlatformFile = Path.Combine(BaseFolder, "Build", "LowLevelTests", $"{TestMetadata.TestName}.xml");
					}
				}

				if (!System.IO.File.Exists(GeneratedPropertiesPlatformFile))
				{
					string? DirGenPropsPlatforms = Path.GetDirectoryName(GeneratedPropertiesPlatformFile);
					if (DirGenPropsPlatforms != null && !System.IO.Directory.Exists(DirGenPropsPlatforms))
					{
						System.IO.Directory.CreateDirectory(DirGenPropsPlatforms);
					}
					using (FileStream FileStream = System.IO.File.Create(GeneratedPropertiesPlatformFile))
					{
						new XDocument(new XElement(BuildGraphNamespace + "BuildGraph", new XAttribute(XNamespace.Xmlns + "xsi", SchemaInstance), new XAttribute(SchemaInstance + "schemaLocation", SchemaLocation))).Save(FileStream);
					}
				}

				MakeFileWriteable(GeneratedPropertiesPlatformFile);
				XElement Root;
				if (!SaveAtEnd.ContainsKey(GeneratedPropertiesPlatformFile))
				{
					XDocument XInitPlatformFile = XDocument.Load(GeneratedPropertiesPlatformFile);
					// Any manually edited elements to keep
					List<XElement> KeepElements = XInitPlatformFile.Root!.Elements().Where(e => e.Attribute("Name").Value == $"{TestMetadata.TestName}AfterSteps").ToList();
					XInitPlatformFile.Root!.Elements().Remove();
					foreach (XElement Element in KeepElements)
					{
						XInitPlatformFile.Root!.Add(Element);
					}
					SaveAtEnd.Add(GeneratedPropertiesPlatformFile, XInitPlatformFile);
				}

				Root = SaveAtEnd[GeneratedPropertiesPlatformFile].Root!;

				// Optional metadata, use Expand and set any non-default metadata
				Dictionary<string, string> ExpandArguments = new Dictionary<string, string>();

				if (!IsRestrictedPlatformName)
				{
					InsertOrUpdateTestOption(Root, $"Run{TestMetadata.TestName}Tests", $"Run {TestMetadata.TestShortName} Tests", "");
					InsertOrUpdateTestProperty(Root, $"TestNames", TestMetadata.TestName, true);
				}

				if (TestMetadata.Deactivated)
				{
					ExpandArguments.Add("Deactivated", Convert.ToString(TestMetadata.Deactivated));
				}
				ExpandArguments.Add("TestName", Convert.ToString(TestMetadata.TestName));
				ExpandArguments.Add("ShortName", Convert.ToString(TestMetadata.TestShortName));
				if (TestMetadata.StagesWithProjectFile)
				{
					ExpandArguments.Add("StagesWithProjectFile", Convert.ToString(TestMetadata.StagesWithProjectFile));
				}
				ExpandArguments.Add("TargetName", Convert.ToString(TestTargetName));
				ExpandArguments.Add("BinaryRelativePath", Convert.ToString(TestBinariesPath));
				string ReportTypeValue =
					(ValidPlatform == UnrealTargetPlatform.IOS || ValidPlatform == UnrealTargetPlatform.TVOS || ValidPlatform == UnrealTargetPlatform.Android) ? "console" : TestMetadata.ReportType;
				ExpandArguments.Add("ReportType", Convert.ToString(ReportTypeValue));
				if (!String.IsNullOrEmpty(TestMetadata.GauntletArgs))
				{
					ExpandArguments.Add("GauntletArgs", Convert.ToString(TestMetadata.InitialExtraArgs) + Convert.ToString(TestMetadata.GauntletArgs));
				}
				if(TestMetadata.PlatformGauntletArgs.ContainsKey(ValidPlatform))
				{
					ExpandArguments.Add("PlatformGauntletArgs", TestMetadata.PlatformGauntletArgs[ValidPlatform]);
				}
				if (!String.IsNullOrEmpty(TestMetadata.ExtraArgs))
				{
					ExpandArguments.Add("ExtraArgs", Convert.ToString(TestMetadata.ExtraArgs));
				}
				if (TestMetadata.HasAfterSteps)
				{
					ExpandArguments.Add("HasAfterSteps", Convert.ToString(TestMetadata.HasAfterSteps));
				}
				if (!TestMetadata.UsesCatch2)
				{
					ExpandArguments.Add("UsesCatch2", Convert.ToString(TestMetadata.UsesCatch2));
				}
				string TagsValue = TestMetadata.PlatformTags.ContainsKey(ValidPlatform) ? TestMetadata.PlatformTags[ValidPlatform] : String.Empty;
				if (!String.IsNullOrEmpty(TagsValue))
				{
					ExpandArguments.Add("Tags", TagsValue);
					
				}

				string ExtraCompilationArgsValue = TestMetadata.PlatformCompilationExtraArgs.ContainsKey(ValidPlatform) ? TestMetadata.PlatformCompilationExtraArgs[ValidPlatform] : String.Empty;
				if (!String.IsNullOrEmpty (ExtraCompilationArgsValue))
				{
					ExpandArguments.Add("ExtraCompilationArgs", ExtraCompilationArgsValue);
				}

				// By default all test supported platforms have run supported, generally only a few don't (e.g. iOS)
				bool RunUnsupportedPlatform = TestMetadata.PlatformsRunUnsupported.Contains(ValidPlatform);
				if (RunUnsupportedPlatform)
				{
					ExpandArguments.Add("RunUnsupported", "True");
				}

				string RunContainerizedValue = TestMetadata.PlatformRunContainerized.ContainsKey(ValidPlatform) ? "True" : "False";
				if (RunContainerizedValue == "True")
				{
					ExpandArguments.Add("RunContainerized", RunContainerizedValue);
				}

				AppendOrUpdateRunAllTestsNode(Root, "DeployAndTest", ValidPlatform.ToString(), ExpandArguments);

				if (IsRestrictedPlatformName)
				{
					// Create a General.xml file and add a TestPlatform* option
					string RestrictedPlatformFolderPath = Path.GetDirectoryName(GeneratedPropertiesPlatformFile)!;
					string RestrictedPlatformGeneral = Path.Combine(RestrictedPlatformFolderPath, "General.xml");
					if (!System.IO.File.Exists(RestrictedPlatformGeneral))
					{
						using (FileStream FileStream = System.IO.File.Create(RestrictedPlatformGeneral))
						{
							Log.LogInformation("Saving general metadata to {File}", RestrictedPlatformGeneral);
							XDocument GeneralProps = new XDocument(new XElement(BuildGraphNamespace + "BuildGraph", new XAttribute(XNamespace.Xmlns + "xsi", SchemaInstance), new XAttribute(SchemaInstance + "schemaLocation", SchemaLocation)));
							InsertOrUpdateTestOption(GeneralProps.Root, $"TestPlatform{ValidPlatform}", $"Run tests on {ValidPlatform}", false.ToString());
							GeneralProps.Save(FileStream);
						}
					}
				}
			}

			foreach (KeyValuePair<string, XDocument> KVP in SaveAtEnd)
			{
				Log.LogInformation("Saving metadata to {File}", KVP.Key);
				KVP.Value.Save(KVP.Key);
			}
		}

		private static string GetBaseFolder(string ModuleDirectory)
		{
			string RelativeModulePath = Path.GetRelativePath(Unreal.RootDirectory.FullName, ModuleDirectory);
			string[] BreadCrumbs = RelativeModulePath.Split(new char[] { '\\', '/' }, StringSplitOptions.RemoveEmptyEntries);
			if (BreadCrumbs.Length > 0)
			{
				return Path.Combine(Unreal.RootDirectory.FullName, BreadCrumbs[0]);
			}
			return Unreal.EngineDirectory.FullName;
		}

		private static bool IsPlatformRestricted(UnrealTargetPlatform Platform)
		{
			return RestrictedFolder.GetNames().Contains(Platform.ToString());
		}

		private static bool IsRestrictedPath(string ModuleDirectory)
		{
			return ModuleDirectory.Split(new char[] { '/', '\\' }).Intersect(RestrictedFoldersNonPlatform).Any();
		}

		private static string TryGetBinariesPath(string ModuleDirectory)
		{
			int SourceFolderIndex = ModuleDirectory.IndexOf("Source", StringComparison.Ordinal);
			if (SourceFolderIndex < 0)
			{
				int PluginFolderIndex = ModuleDirectory.IndexOf("Plugins", StringComparison.Ordinal);
				if (PluginFolderIndex >= 0)
				{
					return ModuleDirectory.Substring(0, PluginFolderIndex) + "Binaries";
				}
				throw new Exception("Could not detect source folder path for module from directory " + ModuleDirectory);
			}
			return ModuleDirectory.Substring(0, SourceFolderIndex) + "Binaries";
		}

		private static void AppendOrUpdateRunAllTestsNode(XElement Root, string MacroName, string Platform, Dictionary<string, string> ExpandArguments)
		{
			XElement? ExtendNode = Root.Elements().Where(element => element.Name.LocalName == "Extend").FirstOrDefault();
			if (ExtendNode == null)
			{
				ExtendNode = new XElement(BuildGraphNamespace + "Extend");
				ExtendNode.SetAttributeValue("Name", "RunAllTests");
				Root.Add(ExtendNode);
			}
			
			XElement? ExpandNode = ExtendNode.Elements().Where(element => element.Attribute("Name").Value == MacroName && element.Attribute("Platform").Value == Platform).FirstOrDefault();
			if (ExpandNode == null)
			{
				ExpandNode = new XElement(BuildGraphNamespace + "Expand");
				ExpandNode.SetAttributeValue("Name", MacroName);
				ExpandNode.SetAttributeValue("Platform", Platform);
				ExtendNode.Add(ExpandNode);
			}
			foreach (KeyValuePair<string, string> ArgumentAndValue in ExpandArguments)
			{
				ExpandNode!.SetAttributeValue(ArgumentAndValue.Key, ArgumentAndValue.Value);
			}
		}

		private static void InsertOrUpdateTestOption(XElement Root, string OptionName, string Description, string DefaultValue)
		{
			XElement? OptionElementWithName = Root.Elements(BuildGraphNamespace + "Option")
				.Where(prop => prop.Attribute("Name").Value == OptionName).FirstOrDefault();
			if (OptionElementWithName == null)
			{
				XElement ElementInsert = new XElement(BuildGraphNamespace + "Option");
				ElementInsert.SetAttributeValue("Name", OptionName);
				ElementInsert.SetAttributeValue("DefaultValue", DefaultValue);
				ElementInsert.SetAttributeValue("Description", Description);
				Root.Add(ElementInsert);
			}
			else
			{
				OptionElementWithName.SetAttributeValue("Description", Description);
				OptionElementWithName.SetAttributeValue("DefaultValue", DefaultValue);
			}
		}

		private static void InsertOrUpdateTestProperty(XElement Root, string PropertyName, string PropertyValue, bool Append)
		{
			XElement? PropertyElementWithName = Root.Elements(BuildGraphNamespace + "Property")
				.Where(prop => prop.Attribute("Name").Value == PropertyName).FirstOrDefault();
			if (PropertyElementWithName == null)
			{
				XElement ElementInsert = new XElement(BuildGraphNamespace + "Property");
				ElementInsert.SetAttributeValue("Name", PropertyName);
				ElementInsert.SetAttributeValue("Value", !Append ? PropertyValue : $"$({PropertyName});{PropertyValue}");
				Root.Add(ElementInsert);
			}
			else
			{
				PropertyElementWithName.SetAttributeValue("Value", !Append ? PropertyValue : $"$({PropertyName});{PropertyValue}");
			}
		}

#pragma warning restore 8604
#pragma warning restore 8602

		private static void MakeFileWriteable(string InFilePath)
		{
			System.IO.File.SetAttributes(InFilePath, System.IO.File.GetAttributes(InFilePath) & ~FileAttributes.ReadOnly);
		}

#pragma warning disable CA1034 // Nested types should not be visible - preserve API of APIs exposed to Rules files
#pragma warning disable 8618
		/// <summary>
		/// Test metadata class.
		/// </summary>
#pragma warning disable CA1724 // Type names should not match namespaces
		public class Metadata
#pragma warning restore CA1724
		{
			/// <summary>
			/// Test long name.
			/// </summary>
			public string TestName { get; set; }

			/// <summary>
			/// Test short name used for display in build system.
			/// </summary>
			public string TestShortName { get; set; }

			// Default to JSON test data output
			private string ReportTypePrivate = "testdata";
			/// <summary>
			/// Type of Catch2 report, defaults to console.
			/// </summary>
			public string ReportType
			{
				get => ReportTypePrivate;
				set => ReportTypePrivate = value;
			}

			/// <summary>
			/// Does this test use project files for staging additional files
			/// and cause the build to use BuildCookRun instead of a Compile step
			/// </summary>
			public bool StagesWithProjectFile { get; set; }

			/// <summary>
			/// Is this test deactivated?
			/// </summary>
			public bool Deactivated { get; set; }

			/// <summary>
			/// Deprecated, use GauntletArgs or ExtraArgs instead to help indicate arguments to launch the test under.
			/// </summary>
			public string InitialExtraArgs
			{
				get;
				[Obsolete("Deprecated in UE5.3 - use GauntletArgs or ExtraArgs")]
				set;
			}

			/// <summary>
			/// Any initial Gauntlet args to be passed to the test executable
			/// </summary>
			public string GauntletArgs { get; set; }

			/// <summary>
			/// Any extra args to be passed to the test executable as --extra-args
			/// </summary>
			public string ExtraArgs { get; set; }

			/// <summary>
			/// Whether there's a step that gets executed after the tests have finished.
			/// Typically used for cleanup of resources.
			/// </summary>
			public bool HasAfterSteps { get; set; }

			private bool UsesCatch2Private = true;
			/// <summary>
			/// Test built with a frakework other than Catch2
			/// </summary>
			public bool UsesCatch2
			{
				get => UsesCatch2Private;
				set => UsesCatch2Private = value;
			}

#pragma warning disable CA2227 // Collection properties should be read only - can't break user API
			/// <summary>
			/// Set of supported platforms.
			/// </summary>
			public HashSet<UnrealTargetPlatform> SupportedPlatforms
			{
				get;
				[Obsolete("Deprecated in UE5.8. Modify the existing collection")]
				set;
			} = [UnrealTargetPlatform.Win64];

			/// <summary>
			/// Per-platform tags.
			/// </summary>
			public Dictionary<UnrealTargetPlatform, string> PlatformTags
			{
				get;
				[Obsolete("Deprecated in UE5.8. Modify the existing collection")]
				set;
			} = [];

			/// <summary>
			/// Per-platform gauntlet args.
			/// </summary>
			public Dictionary<UnrealTargetPlatform, string> PlatformGauntletArgs
			{
				get;
				[Obsolete("Deprecated in UE5.8. Modify the existing collection")]
				set;
			} = [];

			/// <summary>
			/// Per-platform extra compilation arguments.
			/// </summary>
			public Dictionary<UnrealTargetPlatform, string> PlatformCompilationExtraArgs
			{
				get;
				[Obsolete("Deprecated in UE5.8. Modify the existing collection")]
				set;
			} = [];

			/// <summary>
			/// List of platforms that cannot run tests.
			/// </summary>
			public List<UnrealTargetPlatform> PlatformsRunUnsupported
			{
				get;
				[Obsolete("Deprecated in UE5.8. Modify the existing collection")]
				set;
			} = [UnrealTargetPlatform.Android, UnrealTargetPlatform.IOS, UnrealTargetPlatform.TVOS, UnrealTargetPlatform.VisionOS];

			/// <summary>
			/// Whether or not the test is run inside a Docker container for a given platform.
			/// </summary>
			public Dictionary<UnrealTargetPlatform, bool> PlatformRunContainerized
			{
				get;
				[Obsolete("Deprecated in UE5.8. Modify the existing collection")]
				set;
			} = [];
#pragma warning restore CA2227
		}
#pragma warning restore 8618
#pragma warning restore CA1034
	}
}