// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using System;
using System.IO;
using System.Reflection;
using System.Runtime.CompilerServices;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// TargetRules extension for low level tests.
	/// </summary>
	public class TestTargetRules : TargetRules
	{
		/// <summary>
		/// Configuration mapping to control build graph test metadata generation during project files generation.
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "LowLevelTestsSettings")]
		public bool bUpdateBuildGraphPropertiesFile { get; set; }

		/// <summary>
		/// Keeps track if low level tests executable must build with the Editor.
		/// </summary>
		public static bool bTestsRequireEditor { get; set; }

		/// <summary>
		/// Keeps track if low level tests executable must build with the Engine.
		/// </summary>
		public static bool? bTestsRequireEngine { get; set; }

		/// <summary>
		/// Keeps track if low level tests executable must build with the ApplicationCore.
		/// </summary>
		public static bool bTestsRequireApplicationCore { get; set; }

		/// <summary>
		/// Keeps track if low level tests executable must build with the CoreUObject.
		/// </summary>
		public static bool bTestsRequireCoreUObject { get; set; }

		/// <summary>
		/// Associated tested target of this test target, if defined.
		/// </summary>
		public TargetRules? TestedTarget { get; private set; }

		/// <summary>
		/// Test target override for bCompileAgainstApplicationCore.
		/// It is set to true if there is any reference to ApplicationCore in the dependency graph.
		/// </summary>
		public override bool bCompileAgainstApplicationCore
		{
			get => bTestsRequireApplicationCore;
			set => bTestsRequireApplicationCore = value;
		}

		/// <summary>
		/// If set to true, it will not compile against ApplicationCore even if "ApplicationCore" is in the dependency graph.
		/// </summary>
		public bool bNeverCompileAgainstApplicationCore { get; set; }

		/// <summary>
		/// Test target override for bCompileAgainstCoreUObject.
		/// It is set to true if there is any reference to CoreUObject in the dependency graph.
		/// </summary>
		public override bool bCompileAgainstCoreUObject
		{
			get => bTestsRequireCoreUObject;
			set => bTestsRequireCoreUObject = value;
		}

		/// <summary>
		/// If set to true, it will not compile against CoreUObject even if "CoreUObject" is in the dependency graph.
		/// </summary>
		public bool bNeverCompileAgainstCoreUObject { get; set; }

		/// <summary>
		/// Test target override for bCompileAgainstEngine.
		/// It is set to true if there is any reference to Engine in the dependency graph.
		/// </summary>
		public override bool bCompileAgainstEngine
		{
			get => bTestsRequireEngine??bCompileAgainstEnginePrivate;
			set => bCompileAgainstEnginePrivate = value;
		}

		/// <summary>
		/// If set to true, it will not compile against engine even if "Engine" is in the dependency graph.
		/// </summary>
		public bool bNeverCompileAgainstEngine { get; set; }

		/// <summary>
		/// Test target override for bCompileAgainstEditor.
		/// It is set to true if there is any reference to UnrealEd in the dependency graph.
		/// </summary>
		public override bool bCompileAgainstEditor
		{
			get => bTestsRequireEditor;
			set => bTestsRequireEditor = value;
		}

		/// <summary>
		/// If set to true, it will not compile against editor even if "UnrealEd" is in the dependency graph.
		/// </summary>
		public bool bNeverCompileAgainstEditor { get; set; }

		/// <summary>
		/// Whether to stub the platform file.
		/// </summary>
		public bool bUsePlatformFileStub
		{
			get => bUsePlatformFileStubPrivate;
			set
			{
				bUsePlatformFileStubPrivate = value;
				GlobalDefinitions.Remove("UE_LLT_USE_PLATFORM_FILE_STUB=0");
				GlobalDefinitions.Remove("UE_LLT_USE_PLATFORM_FILE_STUB=1");
				GlobalDefinitions.Add($"UE_LLT_USE_PLATFORM_FILE_STUB={Convert.ToInt32(bUsePlatformFileStubPrivate)}");
			}
		}
		private bool bUsePlatformFileStubPrivate { get; set; }

		/// <summary>
		/// Whether to mock engine default instances for materials, AI controller etc.
		/// </summary>
		public bool bMockEngineDefaults
		{
			get => bMockEngineDefaultsPrivate;
			set
			{
				bMockEngineDefaultsPrivate = value;
				GlobalDefinitions.Remove("UE_LLT_WITH_MOCK_ENGINE_DEFAULTS=0");
				GlobalDefinitions.Remove("UE_LLT_WITH_MOCK_ENGINE_DEFAULTS=1");
				GlobalDefinitions.Add($"UE_LLT_WITH_MOCK_ENGINE_DEFAULTS={Convert.ToInt32(bMockEngineDefaultsPrivate)}");
			}
		}
		private bool bMockEngineDefaultsPrivate { get; set; }

		/// <summary>
		/// Constructor that explicit targets can inherit from.
		/// </summary>
		/// <param name="target"></param>
		public TestTargetRules(TargetInfo target) : base(target)
		{
			SetupCommonProperties(target);

			ExeBinariesSubFolder = LaunchModuleName = Name + (ExplicitTestsTarget ? String.Empty : "Tests");

			if (ExplicitTestsTarget)
			{
				bBuildInSolutionByDefault = true;
				if (ProjectFile != null)
				{
					DirectoryReference samplesDirectory = DirectoryReference.Combine(Unreal.RootDirectory, "Samples");
					if (!ProjectFile.IsUnderDirectory(Unreal.EngineDirectory))
					{
						if (ProjectFile.IsUnderDirectory(samplesDirectory))
						{
							SolutionDirectory = Path.Combine("Samples", ProjectFile.Directory.GetDirectoryName(), "LowLevelTests");
						}
						else
						{
							SolutionDirectory = Path.Combine("Games", ProjectFile.Directory.GetDirectoryName(), "LowLevelTests");
						}
					}
					else
					{
						SolutionDirectory = "Programs/LowLevelTests";
					}
				}
				else
				{
					SolutionDirectory = "Programs/LowLevelTests";
				}

				// Default to true for explicit targets to reduce compilation times.
				// Selective module compilation will automatically detect if Engine is required based on Engine include files in tests.
				bNeverCompileAgainstEngine = true;
			}

			if (target.Platform == UnrealTargetPlatform.Win64)
			{
				string outputName = target.Name;
				if (target.Configuration != UndecoratedConfiguration)
				{
					outputName = outputName + "-" + target.Platform + "-" + target.Configuration;
				}
				if (File != null)
				{
					FileReference outputTestFile = FileReference.Combine(GetOutputDirectory(), "Binaries", target.Platform.ToString(), ExeBinariesSubFolder, $"{outputName}.exe.is_unreal_test");
					PostBuildSteps.Add("echo > " + outputTestFile.FullName);
				}
			}
		}

		private DirectoryReference GetOutputDirectory()
		{
			DirectoryReference outputDirectory;
			if (ProjectFile != null && File!.IsUnderDirectory(ProjectFile.Directory))
			{
				outputDirectory = UEBuildTarget.GetOutputDirectoryForExecutable(ProjectFile.Directory, File);
			}
			else
			{
				outputDirectory = UEBuildTarget.GetOutputDirectoryForExecutable(Unreal.EngineDirectory, File!);
			}
			return outputDirectory;
		}

		/// <summary>
		/// Constructs a valid TestTargetRules instance from an existent TargetRules instance.
		/// </summary>
		public static TestTargetRules Create(TargetRules rules, TargetInfo targetInfo)
		{
			Type testTargetRulesType = typeof(TestTargetRules);
			TestTargetRules testRules = (TestTargetRules)RuntimeHelpers.GetUninitializedObject(testTargetRulesType);

			// Initialize the logger before calling the constructor
			testRules.Logger = rules.Logger;

			ConstructorInfo? constructor = testTargetRulesType.GetConstructor(new Type[] { typeof(TargetRules), typeof(TargetInfo) })
				?? throw new BuildException("No constructor found on {0} which takes first argument of type TargetRules and second of type TargetInfo.", testTargetRulesType.Name);
			try
			{
				constructor.Invoke(testRules, new object[] { rules, targetInfo });
				CppCompileWarnings.ApplyTargetDefaults(testRules.CppCompileWarningSettings);
			}
			catch (Exception ex)
			{
				throw new BuildException(ex, "Unable to instantiate instance of '{0}' object type from compiled assembly '{1}'.  Unreal Build Tool creates an instance of your module's 'Rules' object in order to find out about your module's requirements.  The CLR exception details may provide more information:  {2}", testTargetRulesType.Name, Path.GetFileNameWithoutExtension(testTargetRulesType.Assembly?.Location), ex.ToString());
			}

			return testRules;
		}

		/// <summary>
		/// Constructor for TestTargetRules based on existing target a.k.a Implicit test target.
		/// TestTargetRules is setup as a program and is linked monolithically.
		/// It removes a lot of default compilation behavior in order to produce a minimal test environment.
		/// </summary>
		public TestTargetRules(TargetRules testedTarget, TargetInfo target) : base(target)
		{
			if (testedTarget is TestTargetRules)
			{
				throw new BuildException("TestedTarget can't be of type TestTargetRules.");
			}

			TestedTarget = testedTarget;

			TargetFiles = testedTarget.TargetFiles;

			ExeBinariesSubFolder = Name = testedTarget.Name + "Tests";
			TargetSourceFile = File = testedTarget.File;
			if (testedTarget.LaunchModuleName != null)
			{
				LaunchModuleName = testedTarget.LaunchModuleName + "Tests";
			}

			ManifestFileNames.Clear();
			ManifestFileNames.AddRange(testedTarget.ManifestFileNames);

			WindowsPlatform = testedTarget.WindowsPlatform;

			SetupCommonProperties(target);
			SetupImplicitTestProperties(testedTarget);
		}

		private void SetupCommonProperties(TargetInfo target)
		{
			IncludeOrderVersion = EngineIncludeOrderVersion.Latest;

			bIsTestTargetOverride = true;

			VSTestRunSettingsFile = FileReference.Combine(Unreal.EngineDirectory, "Source", "Programs", "LowLevelTests", "vstest.runsettings");

			DefaultBuildSettings = BuildSettingsVersion.Latest;

			Type = TargetType.Program;
			LinkType = TargetLinkType.Monolithic;

			bBuildInSolutionByDefault = false;

			bDeployAfterCompile = true;
			bIsBuildingConsoleApplication = true;

			// Disabling default true flags that aren't necessary for tests

			// Lean and Mean mode
			bBuildDeveloperTools = false;

			// No localization
			bCompileICU = false;

			// No need for shaders by default
			bForceBuildShaderFormats = false;

			// Do not compile against the engine, editor etc
			bCompileAgainstEnginePrivate = false;
			bCompileAgainstEditor = bTestsRequireEditor;
			bCompileAgainstCoreUObject = bTestsRequireCoreUObject;
			bCompileAgainstApplicationCore = bTestsRequireApplicationCore;
			bCompileCEF3 = false;

			// No mixing with Functional Test framework
			bForceDisableAutomationTests = true;

			bDebugBuildsActuallyUseDebugCRT = true;

			// Allow logging in shipping
			bUseLoggingInShipping = true;

			// Allow exception handling
			bForceEnableExceptions = true;

			bBuildWithEditorOnlyData = false;
			bBuildRequiresCookedData = true;
			bBuildDeveloperTools = false;

			bUsePlatformFileStub = (Platform == UnrealTargetPlatform.Android);

			// Useful for debugging test failures
			if (target.Configuration == UnrealTargetConfiguration.Debug)
			{
				bDebugBuildsActuallyUseDebugCRT = true;
			}

			if (!ExplicitTestsTarget && TestedTarget != null)
			{
				GlobalDefinitions.AddRange(TestedTarget.GlobalDefinitions);
			}

			GlobalDefinitions.Add("AUTOSDKS_ENABLED=0");
			GlobalDefinitions.Add("STATS=0");
			GlobalDefinitions.Add("TEST_FOR_VALID_FILE_SYSTEM_MEMORY=0");

			// LLT Globals
			GlobalDefinitions.Add("UE_LLT_WITH_MOCK_ENGINE_DEFAULTS=0");

			// Platform specific setup
			if (target.Platform == UnrealTargetPlatform.Android)
			{
				UndecoratedConfiguration = target.Configuration;

				GlobalDefinitions.Add("USE_ANDROID_INPUT=0");
				GlobalDefinitions.Add("USE_ANDROID_OPENGL=0");
				GlobalDefinitions.Add("USE_ANDROID_LAUNCH=0");

				bUseAndroidJni = false;

				// Addresses linker errors when building LowLevelTests for Android
				// TODO: This should be written to the intermediate directory, somewhere else not when TargetRules are being created
				string IntermediatePath = Path.Combine(Unreal.EngineDirectory.FullName, "Intermediate", "Android");
				FileReference versionScriptFile = new FileReference(Path.Combine(IntermediatePath, $"LLTWorkaroundScript-{Name}.ldscript"));
				string SymbolsLinkScriptContent = @"{
					global: _Znwm*; _Znam*; _ZdlPv*; _ZdaPv*;
					global: LowLevelTestsMain;
					global: JNI_OnLoad;
					global: Java_com_epicgames_unreal_tests_TestActivity_runTests;
					*VM*;
					*AndroidPath*;
					*Catch*;
					local: *;
				};";
				if (!Directory.Exists(IntermediatePath))
				{
					Directory.CreateDirectory(IntermediatePath);
				}
				FileReference.WriteAllTextIfDifferent(versionScriptFile, SymbolsLinkScriptContent);
				AdditionalLinkerArguments = " -Wl,--version-script=\"" + versionScriptFile + "\"";
			}
			else if (target.Platform == UnrealTargetPlatform.IOS)
			{
				bIsBuildingConsoleApplication = false;
			}

			if (target.Arguments != null)
			{
				string? selectedModules;
				if (target.Arguments.TryGetValue("-SelectModules=", out selectedModules))
				{
					SelectedTestModuleNames = [.. selectedModules.Split(',')];
				}
			}
		}

		private void SetupImplicitTestProperties(TargetRules testedTarget)
		{
			bool isEditorTestedTarget = (testedTarget.Type == TargetType.Editor);
			bBuildWithEditorOnlyData = bCompileAgainstEditor = isEditorTestedTarget;
			bBuildDeveloperTools = bCompileAgainstEngine && isEditorTestedTarget;

			bUsePlatformFileStub = bCompileAgainstEngine;
			bMockEngineDefaults = bCompileAgainstEngine;
			bCompileWithPluginSupport = bCompileAgainstEngine;
		}
	}
}
