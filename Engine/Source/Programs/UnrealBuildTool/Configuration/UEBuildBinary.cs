// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// All binary types generated by UBT
	/// </summary>
	enum UEBuildBinaryType
	{
		/// <summary>
		/// An executable
		/// </summary>
		Executable,

		/// <summary>
		/// A dynamic library (.dll, .dylib, or .so)
		/// </summary>
		DynamicLinkLibrary,

		/// <summary>
		/// A static library (.lib or .a)
		/// </summary>
		StaticLibrary,

		/// <summary>
		/// Object files
		/// </summary>
		Object,

		/// <summary>
		/// Precompiled header
		/// </summary>
		PrecompiledHeader,
	}

	/// <summary>
	/// A binary built by UBT.
	/// </summary>
	class UEBuildBinary
	{
		/// <summary>
		/// The type of binary to build
		/// </summary>
		public UEBuildBinaryType Type;

		/// <summary>
		/// Output directory for this binary
		/// </summary>
		public DirectoryReference OutputDir;

		/// <summary>
		/// The output file path. This must be set before a binary can be built using it.
		/// </summary>
		public List<FileReference> OutputFilePaths;

		/// <summary>
		/// Returns the OutputFilePath if there is only one entry in OutputFilePaths
		/// </summary>
		public FileReference OutputFilePath
		{
			get
			{
				if (OutputFilePaths.Count != 1)
				{
					throw new BuildException("Attempted to use UEBuildBinaryConfiguration.OutputFilePath property, but there are multiple (or no) OutputFilePaths. You need to handle multiple in the code that called this (size = {0})", OutputFilePaths.Count);
				}
				return OutputFilePaths[0];
			}
		}

		/// <summary>
		/// The intermediate directory for this binary. Modules should create separate intermediate directories below this. Must be set before a binary can be built using it.
		/// </summary>
		public DirectoryReference IntermediateDirectory;

		/// <summary>
		/// If true, build exports lib
		/// </summary>
		public bool bAllowExports = false;

		/// <summary>
		/// If true, create a separate import library
		/// </summary>
		public bool bCreateImportLibrarySeparately = false;

		/// <summary>
		/// If true, creates an additional console application. Hack for Windows, where it's not possible to conditionally inherit a parent's console Window depending on how
		/// the application is invoked; you have to link the same executable with a different subsystem setting.
		/// </summary>
		public bool bBuildAdditionalConsoleApp = false;

		/// <summary>
		/// If true, replaces the executable with a console application. Hack for Windows, where it's not possible to conditionally inherit a parent's console Window depending on how
		/// the application is invoked
		/// </summary>
		public bool bBuildConsoleAppOnly = false;

		/// <summary>
		/// 
		/// </summary>
		public bool bUsePrecompiled;

		/// <summary>
		/// The primary module that this binary was constructed for. For executables, this is typically the launch module.
		/// </summary>
		public readonly UEBuildModuleCPP PrimaryModule;

		/// <summary>
		/// List of modules to link together into this executable
		/// </summary>
		public readonly List<UEBuildModule> Modules = new List<UEBuildModule>();

		/// <summary>
		/// Cached list of dependent link libraries.
		/// </summary>
		private readonly Lazy<List<FileReference>> DependentLinkLibraries;

		/// <summary>
		/// Expanded list of runtime dependencies generated by calling PrepareRuntimeDependencies.
		/// </summary>
		public readonly List<ModuleRules.RuntimeDependency> RuntimeDependencies = new List<ModuleRules.RuntimeDependency>();

		/// <summary>
		/// If compiled objects will have unused exports stripped, only affects objects compiled into modular libraries.
		/// </summary>
		public bool bStripUnusedExports = false;

		/// <summary>
		/// List of all objects that would be linked.
		/// </summary>
		public readonly HashSet<FileItem> InputObjects = new();

		/// <summary>
		/// Create an instance of the class with the given configuration data
		/// </summary>
		/// <param name="Type"></param>
		/// <param name="OutputFilePaths"></param>
		/// <param name="IntermediateDirectory"></param>
		/// <param name="bAllowExports"></param>
		/// <param name="bBuildAdditionalConsoleApp"></param>
		/// <param name="bBuildConsoleAppOnly"></param>
		/// <param name="PrimaryModule"></param>
		/// <param name="bUsePrecompiled"></param>
		public UEBuildBinary(
				UEBuildBinaryType Type,
				IEnumerable<FileReference> OutputFilePaths,
				DirectoryReference IntermediateDirectory,
				bool bAllowExports,
				bool bBuildAdditionalConsoleApp,
				bool bBuildConsoleAppOnly,
				UEBuildModuleCPP PrimaryModule,
				bool bUsePrecompiled
			)
		{
			this.Type = Type;
			OutputDir = OutputFilePaths.First().Directory;
			this.OutputFilePaths = new List<FileReference>(OutputFilePaths);
			this.IntermediateDirectory = IntermediateDirectory;
			this.bAllowExports = bAllowExports;
			this.bBuildAdditionalConsoleApp = bBuildAdditionalConsoleApp;
			this.bBuildConsoleAppOnly = bBuildConsoleAppOnly;
			this.PrimaryModule = PrimaryModule;
			this.bUsePrecompiled = bUsePrecompiled;

			PrimaryModule.bCreatePerModuleFile = true;

			Modules.Add(PrimaryModule);

			// Cache the list of libraries in the dependent link environment between calls. We typically run this code path many times for each module.
			DependentLinkLibraries = new(() =>
			{
				List<FileReference> libs = new ();
				foreach (FileReference OutputFilePath in OutputFilePaths)
				{
					FileReference LibraryFileName;
					if (Type == UEBuildBinaryType.StaticLibrary || !PrimaryModule.Rules.Target.Platform.IsInGroup(UnrealPlatformGroup.Microsoft))
					{
						LibraryFileName = OutputFilePath;
					}
					else
					{
						LibraryFileName = FileReference.Combine(IntermediateDirectory, OutputFilePath.GetFileNameWithoutExtension() + ".lib");
					}
					libs.Add(LibraryFileName);
				}
				return libs;
			});
		}

		/// <summary>
		/// Expands all of the runtime dependencies.
		/// </summary>
		/// <param name="ExeDir">Output directory for the executable</param>
		public void PrepareRuntimeDependencies(DirectoryReference ExeDir)
		{
			RuntimeDependencies.Clear();
			foreach (UEBuildModule Module in Modules)
			{
				foreach (ModuleRules.RuntimeDependency Dependency in Module.Rules.RuntimeDependencies.Inner)
				{
					if (Dependency.SourcePath == null)
					{
						// Expand the target path
						string ExpandedPath = Module.ExpandPathVariables(Dependency.Path, OutputDir, ExeDir);
						if (FileFilter.FindWildcardIndex(ExpandedPath) == -1)
						{
							RuntimeDependencies.Add(new ModuleRules.RuntimeDependency(ExpandedPath, Dependency.Type));
						}
						else
						{
							RuntimeDependencies.AddRange(FileFilter.ResolveWildcard(ExpandedPath).Select(x => new ModuleRules.RuntimeDependency(x.FullName, Dependency.Type)));
						}
					}
					else
					{
						// Parse the source and target patterns
						FilePattern SourcePattern = new FilePattern(Unreal.EngineSourceDirectory, Module.ExpandPathVariables(Dependency.SourcePath, OutputDir, ExeDir));
						FilePattern TargetPattern = new FilePattern(Unreal.EngineSourceDirectory, Module.ExpandPathVariables(Dependency.Path, OutputDir, ExeDir));

						// Skip non-essential single files if they do not exist
						if (Dependency.Type == StagedFileType.DebugNonUFS && !SourcePattern.ContainsWildcards() && !FileReference.Exists(SourcePattern.GetSingleFile()))
						{
							continue;
						}

						// Resolve all the wildcards between the source and target paths
						Dictionary<FileReference, FileReference> Mapping;
						try
						{
							Mapping = FilePattern.CreateMapping(null, ref SourcePattern, ref TargetPattern);
						}
						catch (FilePatternException Ex)
						{
							ExceptionUtils.AddContext(Ex, "while creating runtime dependencies for module '{0}'", Module.Name);
							throw;
						}
						foreach (KeyValuePair<FileReference, FileReference> Pair in Mapping)
						{
							RuntimeDependencies.Add(new ModuleRules.RuntimeDependency(Pair.Key.FullName, Pair.Value.FullName, Dependency.Type));
						}
					}
				}
			}
		}

		/// <summary>
		/// Builds the binary.
		/// </summary>
		/// <param name="Target">Rules for the target being built</param>
		/// <param name="ToolChain">The toolchain which to use for building</param>
		/// <param name="CompileEnvironment">The environment to compile the binary in</param>
		/// <param name="LinkEnvironment">The environment to link the binary in</param>
		/// <param name="WorkingSet">The working set of source files</param>
		/// <param name="ExeDir">Directory containing the output executable</param>
		/// <param name="Graph">The graph being built</param>
		/// <param name="Logger">Logger for output</param>
		/// <returns>Set of built products</returns>
		public List<FileItem> Build(ReadOnlyTargetRules Target, UEToolChain ToolChain, CppCompileEnvironment CompileEnvironment, LinkEnvironment LinkEnvironment, ISourceFileWorkingSet WorkingSet, DirectoryReference ExeDir, IActionGraphBuilder Graph, ILogger Logger)
		{
			// Return nothing if we're using precompiled binaries. If we're not linking, we might want just one module to be compiled (eg. a foreign plugin), so allow any actions to run.
			if (bUsePrecompiled && !(Target.LinkType == TargetLinkType.Monolithic && Target.bDisableLinking))
			{
				return new List<FileItem>();
			}

			// Setup linking environment.
			LinkEnvironment BinaryLinkEnvironment = SetupBinaryLinkEnvironment(Target, ToolChain, LinkEnvironment, CompileEnvironment, WorkingSet, ExeDir, Graph, Logger);

			// If we're generating projects, we only need include paths and definitions, there is no need to run the linking logic.
			if (ProjectFileGenerator.bGenerateProjectFiles)
			{
				return BinaryLinkEnvironment.InputFiles;
			}

			// If linking is disabled, our build products are just the compiled object files
			if (Target.bDisableLinking)
			{
				if (Target.LinkType == TargetLinkType.Modular)
				{
					// Make sure we record these files as outputs for this particular module.
					Graph.SetOutputItemsForModule(PrimaryModule.Name, BinaryLinkEnvironment.InputFiles.ToArray());
				}

				return BinaryLinkEnvironment.InputFiles;
			}

			// Generate import libraries as a separate step
			List<FileItem> OutputFiles = new List<FileItem>();
			if (bCreateImportLibrarySeparately)
			{
				// Mark the link environment as cross-referenced.
				BinaryLinkEnvironment.bIsCrossReferenced = true;
			}

			// Create the import library if needed
			OutputFiles.AddRange(ToolChain.LinkImportLibrary(BinaryLinkEnvironment, Graph));

			// Override the build to be a console app (i.e. only build a console app)
			if (bBuildConsoleAppOnly)
			{
				BinaryLinkEnvironment.bIsBuildingConsoleApplication = true;
				BinaryLinkEnvironment.bCodeCoverage = CompileEnvironment.bCodeCoverage;
				BinaryLinkEnvironment.WindowsEntryPointOverride = "WinMainCRTStartup";       // For WinMain() instead of "main()" for Launch module
				BinaryLinkEnvironment.OutputFilePaths = BinaryLinkEnvironment.OutputFilePaths.Select(Path => GetAdditionalConsoleAppPath(Path)).ToList();
			}
			else if (bBuildAdditionalConsoleApp)
			{
				// Produce additional binary but link it as a console app
				LinkEnvironment ConsoleAppLinkEnvironment = new LinkEnvironment(BinaryLinkEnvironment);
				ConsoleAppLinkEnvironment.bIsBuildingConsoleApplication = true;
				ConsoleAppLinkEnvironment.bCodeCoverage = CompileEnvironment.bCodeCoverage;
				ConsoleAppLinkEnvironment.WindowsEntryPointOverride = "WinMainCRTStartup";       // For WinMain() instead of "main()" for Launch module
				ConsoleAppLinkEnvironment.OutputFilePaths = ConsoleAppLinkEnvironment.OutputFilePaths.Select(Path => GetAdditionalConsoleAppPath(Path)).ToList();

				// Link the console app executable
				FileItem[] ConsoleAppOutputFiles = ToolChain.LinkAllFiles(ConsoleAppLinkEnvironment, false, Graph);
				OutputFiles.AddRange(ConsoleAppOutputFiles);

				foreach (FileItem Executable in ConsoleAppOutputFiles)
				{
					OutputFiles.AddRange(ToolChain.PostBuild(Target, Executable, ConsoleAppLinkEnvironment, Graph));
				}
			}

			// If target type is monolithic and we are building a dll it means that this link step will output final binaries which is a dynamic library
			// In this case we also want additional files (.lib etc) to end up in the same directory as the binary
			if (Target.LinkType == TargetLinkType.Monolithic && BinaryLinkEnvironment.bIsBuildingDLL)
			{
				BinaryLinkEnvironment.LocalShadowDirectory = BinaryLinkEnvironment.OutputFilePath.Directory;
			}

			// Link the binary.
			FileItem[] Executables = ToolChain.LinkAllFiles(BinaryLinkEnvironment, false, Graph);
			OutputFiles.AddRange(Executables);

			// Save all the output items for this binary. This is used for hot-reload, and excludes any items added in PostBuild (such as additional files copied into the app).
			if (Target.LinkType == TargetLinkType.Modular)
			{
				Graph.SetOutputItemsForModule(PrimaryModule.Name, OutputFiles.ToArray());
			}

			foreach (FileItem Executable in Executables)
			{
				OutputFiles.AddRange(ToolChain.PostBuild(Target, Executable, BinaryLinkEnvironment, Graph));
			}

			return OutputFiles;
		}

		/// <summary>
		/// Gets all the runtime dependencies Copies all the runtime dependencies from any modules in 
		/// </summary>
		/// <param name="OutRuntimeDependencies">The output list of runtime dependencies, mapping target file to type</param>
		/// <param name="TargetFileToSourceFile">Map of target files to source files that need to be copied</param>
		public void CollectRuntimeDependencies(List<RuntimeDependency> OutRuntimeDependencies, Dictionary<FileReference, FileReference> TargetFileToSourceFile)
		{
			foreach (ModuleRules.RuntimeDependency Dependency in RuntimeDependencies)
			{
				FileReference DepFile = new FileReference(Dependency.Path);
				if (Dependency.SourcePath == null)
				{
					OutRuntimeDependencies.Add(new RuntimeDependency(DepFile, Dependency.Type));
				}
				else
				{
					FileReference? ExistingSourceFile;
					if (!TargetFileToSourceFile.TryGetValue(DepFile, out ExistingSourceFile))
					{
						TargetFileToSourceFile[DepFile] = new FileReference(Dependency.SourcePath);
						OutRuntimeDependencies.Add(new RuntimeDependency(DepFile, Dependency.Type));
					}
					else if (ExistingSourceFile.FullName != Dependency.SourcePath)
					{
						throw new BuildException("Runtime dependency '{0}' is configured to be staged from '{1}' and '{2}'", Dependency.Path, Dependency.SourcePath, ExistingSourceFile);
					}
				}
			}
		}

		/// <summary>
		/// Called to allow the binary to modify the link environment of a different binary containing 
		/// a module that depends on a module in this binary.
		/// </summary>
		/// <param name="DependentLinkEnvironment">The link environment of the dependency</param>
		public void SetupDependentLinkEnvironment(LinkEnvironment DependentLinkEnvironment)
		{
			DependentLinkEnvironment.Libraries.AddRange(DependentLinkLibraries.Value);
		}

		/// <summary>
		/// Called to allow the binary to find hot reloadable modules.
		/// </summary>
		/// <returns>Enumerable of modules</returns>
		public IEnumerable<UEBuildModule> FindHotReloadModules()
		{
			return Modules.Where(x => x.Rules.Context.bCanHotReload);
		}

		/// <summary>
		/// Generates a list of all modules referenced by this binary
		/// </summary>
		/// <param name="bIncludeDynamicallyLoaded">True if dynamically loaded modules (and all of their dependent modules) should be included.</param>
		/// <param name="bForceCircular">True if circular dependencies should be process</param>
		/// <returns>List of all referenced modules</returns>
		public IEnumerable<UEBuildModule> GetAllDependencyModules(bool bIncludeDynamicallyLoaded, bool bForceCircular)
		{
			if (Modules.Count == 1 && bIncludeDynamicallyLoaded && bForceCircular)
			{
				UEBuildModule First = Modules[0];
				if (First.DependenciesForStableTopologicalSort != null)
				{
					return First.DependenciesForStableTopologicalSort;
				}
			}

			List<UEBuildModule> ReferencedModules = new List<UEBuildModule>();

			using Pooled<HashSet<UEBuildModule>> rented = HashSetPool<UEBuildModule>.Rent();
			HashSet<UEBuildModule> IgnoreReferencedModules = rented.Value;

			foreach (UEBuildModule Module in Modules)
			{
				if (IgnoreReferencedModules.Add(Module))
				{
					Module.GetAllDependencyModules(ReferencedModules, IgnoreReferencedModules, bIncludeDynamicallyLoaded, bForceCircular, bOnlyDirectDependencies: false);

					ReferencedModules.Add(Module);
				}
			}

			return ReferencedModules;
		}

		/// <summary>
		/// Generates a list of all modules referenced by this binary
		/// </summary>
		/// <param name="ReferencedBy">Map of module to the module that referenced it</param>
		/// <returns>List of all referenced modules</returns>
		public void FindModuleReferences(Dictionary<UEBuildModule, UEBuildModule?> ReferencedBy)
		{
			List<UEBuildModule> ReferencedModules = new List<UEBuildModule>();
			foreach (UEBuildModule Module in Modules)
			{
				ReferencedModules.Add(Module);
				ReferencedBy.Add(Module, null);
			}

			List<UEBuildModule> DirectlyReferencedModules = new List<UEBuildModule>();

			using Pooled<HashSet<UEBuildModule>> rented = HashSetPool<UEBuildModule>.Rent();
			HashSet<UEBuildModule> VisitedModules = rented.Value;

			for (int Idx = 0; Idx < ReferencedModules.Count; Idx++)
			{
				UEBuildModule SourceModule = ReferencedModules[Idx];

				// Find all the direct references from this module
				DirectlyReferencedModules.Clear();
				SourceModule.GetAllDependencyModules(DirectlyReferencedModules, VisitedModules, false, false, true);

				// Set up the references for all the new modules
				foreach (UEBuildModule DirectlyReferencedModule in DirectlyReferencedModules)
				{
					if (!ReferencedBy.ContainsKey(DirectlyReferencedModule))
					{
						ReferencedBy.Add(DirectlyReferencedModule, SourceModule);
						ReferencedModules.Add(DirectlyReferencedModule);
					}
				}
			}
		}

		/// <summary>
		/// Sets whether to create a separate import library to resolve circular dependencies for this binary
		/// </summary>
		/// <param name="bInCreateImportLibrarySeparately">True to create a separate import library</param>
		public void SetCreateImportLibrarySeparately(bool bInCreateImportLibrarySeparately)
		{
			bCreateImportLibrarySeparately = bInCreateImportLibrarySeparately;
		}

		/// <summary>
		/// Adds a module to the binary.
		/// </summary>
		/// <param name="Module">The module to add</param>
		public void AddModule(UEBuildModule Module)
		{
			if (!Modules.Contains(Module))
			{
				Modules.Add(Module);
			}
		}

		/// <summary>
		/// Gets all build products produced by this binary
		/// </summary>
		/// <param name="Target">The target being built</param>
		/// <param name="ToolChain">The platform toolchain</param>
		/// <param name="BuildProducts">Mapping of produced build product to type</param>
		/// <param name="bCreateDebugInfo">Whether debug info is enabled for this binary</param>
		public void GetBuildProducts(ReadOnlyTargetRules Target, UEToolChain ToolChain, Dictionary<FileReference, BuildProductType> BuildProducts, bool bCreateDebugInfo)
		{
			// Add all the precompiled outputs
			foreach (UEBuildModule Module in Modules)
			{
				Module.GetBuildProducts(ToolChain, BuildProducts);
			}

			// Add all the binary outputs
			if (!Target.bDisableLinking)
			{
				// Get the type of build products we're creating
				BuildProductType OutputType = BuildProductType.RequiredResource;
				switch (Type)
				{
					case UEBuildBinaryType.Executable:
						OutputType = BuildProductType.Executable;
						break;
					case UEBuildBinaryType.DynamicLinkLibrary:
						OutputType = BuildProductType.DynamicLibrary;
						break;
					case UEBuildBinaryType.StaticLibrary:
						OutputType = BuildProductType.BuildResource;
						break;
				}

				// Add the primary build products
				string[] DebugExtensions = UEBuildPlatform.GetBuildPlatform(Target.Platform).GetDebugInfoExtensions(Target, Type);

				if (Type == UEBuildBinaryType.Executable && bBuildConsoleAppOnly)
				{
					foreach (FileReference OutputFilePath in OutputFilePaths)
					{
						AddBuildProductAndDebugFiles(GetAdditionalConsoleAppPath(OutputFilePath), OutputType, DebugExtensions, BuildProducts, ToolChain, bCreateDebugInfo);
					}
				}
				else
				{
					foreach (FileReference OutputFilePath in OutputFilePaths)
					{
						AddBuildProductAndDebugFiles(OutputFilePath, OutputType, DebugExtensions, BuildProducts, ToolChain, bCreateDebugInfo);
					}
				}

				// Add the console app, if there is one
				if (Type == UEBuildBinaryType.Executable && !bBuildConsoleAppOnly && bBuildAdditionalConsoleApp)
				{
					foreach (FileReference OutputFilePath in OutputFilePaths)
					{
						AddBuildProductAndDebugFiles(GetAdditionalConsoleAppPath(OutputFilePath), OutputType, DebugExtensions, BuildProducts, ToolChain, bCreateDebugInfo);
					}
				}

				// Add any additional build products from the modules in this binary, including additional bundle resources/dylibs on Mac.
				List<string> Libraries = new List<string>();
				List<UEBuildBundleResource> BundleResources = new List<UEBuildBundleResource>();
				GatherAdditionalResources(Libraries, BundleResources);

				// Add any extra files from the toolchain
				ToolChain.ModifyBuildProducts(Target, this, Libraries, BundleResources, BuildProducts);
			}
		}

		/// <summary>
		/// Adds a build product and its associated debug file to a receipt.
		/// </summary>
		/// <param name="OutputFile">Build product to add</param>
		/// <param name="OutputType">The type of built product</param>
		/// <param name="DebugExtensions">Extensions for the matching debug file (may be null).</param>
		/// <param name="BuildProducts">Map of build products to their type</param>
		/// <param name="ToolChain">The toolchain used to build these binaries</param>
		/// <param name="bCreateDebugInfo">Whether creating debug info is enabled</param>
		static void AddBuildProductAndDebugFiles(FileReference OutputFile, BuildProductType OutputType, string[] DebugExtensions, Dictionary<FileReference, BuildProductType> BuildProducts, UEToolChain ToolChain, bool bCreateDebugInfo)
		{
			BuildProducts.Add(OutputFile, OutputType);

			foreach (string DebugExtension in DebugExtensions)
			{
				if (!String.IsNullOrEmpty(DebugExtension) && ToolChain.ShouldAddDebugFileToReceipt(OutputFile, OutputType) && bCreateDebugInfo)
				{
					// @todo this could be cleaned up if we replaced Platform.GetDebugExtensions() with ToolChain.GetDebugFiles(OutputFile)
					// would need care in MacToolchain tho, so too risky for now
					BuildProducts.Add(ToolChain.GetDebugFile(OutputFile, DebugExtension), BuildProductType.SymbolFile);
				}
			}
		}

		/// <summary>
		/// Enumerates resources which the toolchain may need may produced additional build products from. Some platforms (eg. Mac, Linux) can link directly 
		/// against .so/.dylibs, but they are also copied to the output folder by the toolchain.
		/// </summary>
		/// <param name="Libraries">List to which libraries required by this module are added</param>
		/// <param name="BundleResources">List of bundle resources required by this module</param>
		public void GatherAdditionalResources(List<string> Libraries, List<UEBuildBundleResource> BundleResources)
		{
			foreach (UEBuildModule Module in Modules)
			{
				Module.GatherAdditionalResources(Libraries, BundleResources);
			}
		}

		/// <summary>
		/// Helper function to get the console app BinaryName-Cmd.exe filename based on the binary filename.
		/// </summary>
		/// <param name="BinaryPath">Full path to the binary exe.</param>
		/// <returns></returns>
		public static FileReference GetAdditionalConsoleAppPath(FileReference BinaryPath)
		{
			DirectoryReference Directory = BinaryPath.Directory;
			if (Directory.FullName.EndsWith(".app/Contents/MacOS", StringComparison.Ordinal))
			{
				Directory = Directory.ParentDirectory!.ParentDirectory!.ParentDirectory!;
			}

			return FileReference.Combine(Directory, BinaryPath.GetFileNameWithoutExtension() + "-Cmd" + BinaryPath.GetExtension());
		}

		/// <summary>
		/// Checks whether the binary output paths are appropriate for the distribution
		/// level of its direct module dependencies
		/// </summary>
		public bool CheckRestrictedFolders(List<DirectoryReference> RootDirectories, Dictionary<UEBuildModule, Dictionary<RestrictedFolder, DirectoryReference>> ModuleRestrictedFolderCache, ILogger Logger)
		{
			// Find all the modules we depend on
			Dictionary<UEBuildModule, UEBuildModule?> ModuleReferencedBy = new Dictionary<UEBuildModule, UEBuildModule?>();
			FindModuleReferences(ModuleReferencedBy);

			// Loop through each of the output binaries and check them separately
			bool bResult = true;
			foreach (FileReference OutputFilePath in OutputFilePaths)
			{
				// Find the base directory for this binary
				DirectoryReference? BaseDir = RootDirectories.FirstOrDefault(x => OutputFilePath.IsUnderDirectory(x));
				if (BaseDir == null)
				{
					continue;
				}

				// Find the permitted restricted folder references under the base directory
				List<RestrictedFolder> BinaryFolders = RestrictedFolders.FindPermittedRestrictedFolderReferences(BaseDir, OutputFilePath.Directory);

				List<RestrictedFolder> AliasedBinaryFolders = new List<RestrictedFolder>();
				foreach (RestrictedFolder BinaryFolder in BinaryFolders)
				{
					string? Alias;
					if (PrimaryModule.AliasRestrictedFolders.TryGetValue(BinaryFolder.ToString(), out Alias))
					{
						foreach (RestrictedFolder Folder in RestrictedFolder.GetValues())
						{
							if (Folder.ToString().Equals(Alias, StringComparison.Ordinal))
							{
								AliasedBinaryFolders.Add(Folder);
							}
						}
					}
				}
				BinaryFolders.AddRange(AliasedBinaryFolders);

				// Check all the dependent modules
				foreach (UEBuildModule Module in ModuleReferencedBy.Keys)
				{
					// Find the restricted folders for this module
					Dictionary<RestrictedFolder, DirectoryReference>? ModuleRestrictedFolders;
					if (!ModuleRestrictedFolderCache.TryGetValue(Module, out ModuleRestrictedFolders))
					{
						ModuleRestrictedFolders = Module.FindRestrictedFolderReferences(RootDirectories);
						ModuleRestrictedFolderCache.Add(Module, ModuleRestrictedFolders);
					}

					// Write errors for any missing paths in the output files
					foreach (KeyValuePair<RestrictedFolder, DirectoryReference> Pair in ModuleRestrictedFolders)
					{
						if (!BinaryFolders.Contains(Pair.Key))
						{
							List<string> ReferenceChain = new List<string>();
							for (UEBuildModule? ReferencedModule = Module; ReferencedModule != null; ReferencedModule = ModuleReferencedBy[ReferencedModule])
							{
								ReferenceChain.Insert(0, ReferencedModule.Name);
							}
							Logger.LogError("Output binary \"{OutputFilePath}\" is not in a {Folder} folder, but references \"{ReferenceFile}\" via {ReferenceChain}.", OutputFilePath, Pair.Key.ToString(), Pair.Value, String.Join(" -> ", ReferenceChain));
							bResult = false;
						}
					}
				}
			}
			return bResult;
		}

		/// <summary>
		/// Write information about this binary to a JSON file
		/// </summary>
		/// <param name="Writer">Writer for this binary's data</param>
		public virtual void ExportJson(JsonWriter Writer)
		{
			Writer.WriteValue("File", OutputFilePath.FullName);
			Writer.WriteValue("Type", Type.ToString());

			Writer.WriteArrayStart("Modules");
			foreach (UEBuildModule Module in Modules)
			{
				Writer.WriteValue(Module.Name);
			}
			Writer.WriteArrayEnd();
		}

		// UEBuildBinary interface.

		private static bool IsBuildingDll(UEBuildBinaryType Type)
		{
			return Type == UEBuildBinaryType.DynamicLinkLibrary;
		}

		private static bool IsBuildingLibrary(UEBuildBinaryType Type)
		{
			return Type == UEBuildBinaryType.StaticLibrary;
		}

		public CppCompileEnvironment CreateBinaryCompileEnvironment(CppCompileEnvironment GlobalCompileEnvironment)
		{
			CppCompileEnvironment BinaryCompileEnvironment = new CppCompileEnvironment(GlobalCompileEnvironment);
			BinaryCompileEnvironment.bIsBuildingDLL = IsBuildingDll(Type);
			BinaryCompileEnvironment.bIsBuildingLibrary = IsBuildingLibrary(Type);
			return BinaryCompileEnvironment;
		}

		private LinkEnvironment SetupBinaryLinkEnvironment(ReadOnlyTargetRules Target, UEToolChain ToolChain, LinkEnvironment LinkEnvironment, CppCompileEnvironment CompileEnvironment, ISourceFileWorkingSet WorkingSet, DirectoryReference ExeDir, IActionGraphBuilder Graph, ILogger Logger)
		{
			LinkEnvironment BinaryLinkEnvironment = new LinkEnvironment(LinkEnvironment);
			List<UEBuildBinary> BinaryDependencies = new List<UEBuildBinary>();

			using Pooled<HashSet<UEBuildModule>> rented = HashSetPool<UEBuildModule>.Rent();
			HashSet<UEBuildModule> LinkEnvironmentVisitedModules = rented.Value;

			CppCompileEnvironment BinaryCompileEnvironment = CreateBinaryCompileEnvironment(CompileEnvironment);

			if (BinaryCompileEnvironment.bUseSharedBuildEnvironment && Target.ProjectFile != null && IntermediateDirectory.IsUnderDirectory(Target.ProjectFile.Directory))
			{
				BinaryCompileEnvironment.bUseSharedBuildEnvironment = false;
			}

			using Pooled<HashSet<FileItem>> rented2 = HashSetPool<FileItem>.Rent();
			HashSet<FileItem> InputFilesLookup = rented2.Value;

			List<UEBuildModule> CompiledModules = [];
			foreach (UEBuildModule Module in UEBuildModule.StableTopologicalSort(Modules))
			{
				if (Module.Binary == null || Module.Binary == this)
				{
					CompiledModules.Add(Module);
				}
				else
				{
					BinaryDependencies.Add(Module.Binary);

					// Allow the module to modify the link environment for the binary.
					Module.SetupPrivateLinkEnvironment(this, BinaryLinkEnvironment, BinaryDependencies, LinkEnvironmentVisitedModules, ExeDir, Logger);
				}
			}

			List<FileItem>[] LinkInputFilesArray = new List<FileItem>[CompiledModules.Count];

			// Compile call is expensive so do it in parallel
			Parallel2.For(0, CompiledModules.Count, index =>
			{
				LinkInputFilesArray[index] = CompiledModules[index].Compile(Target, ToolChain, BinaryCompileEnvironment, WorkingSet, Graph, Logger);
			});

			for (int index = 0, e = CompiledModules.Count; index != e; ++index)
			{
				UEBuildModule Module = CompiledModules[index];

				// Add the module's AutoRTFM external mapping files to the compile environment
				BinaryCompileEnvironment.AutoRTFMExternalMappingFiles.UnionWith(Module.PublicAutoRTFMExternalMappingFiles);

				List<FileItem> LinkInputFiles = LinkInputFilesArray[index];

				InputObjects.UnionWith(LinkInputFiles.Where(x => x.HasExtension(".obj") || x.HasExtension(".o")));

				// Save the module outputs. In monolithic builds, this is just the object files.
				if (Target.LinkType == TargetLinkType.Monolithic)
				{
					Graph.SetOutputItemsForModule(Module.Name, LinkInputFiles.ToArray());
				}

				// Sometimes the same link input file is added multiple times, so make sure to filter out the dups
				foreach (FileItem LinkInputFile in LinkInputFiles)
				{
					if (InputFilesLookup.Add(LinkInputFile))
					{
						BinaryLinkEnvironment.InputFiles.Add(LinkInputFile);
					}
				}

				// Project root path should only be added if the binary needs it, to prevent violating shared environment actions
				if (Target.ProjectFile != null && BinaryLinkEnvironment.InputFiles.Any(x => x.Location.IsUnderDirectory(Target.ProjectFile.Directory)))
				{
					BinaryLinkEnvironment.RootPaths[CppRootPathFolder.Project] = Target.ProjectFile.Directory;
				}

				// Force a reference to initialize module for this binary
				if (Module.Rules.bRequiresImplementModule ?? true)
				{
					BinaryLinkEnvironment.IncludeFunctions.Add($"IMPLEMENT_MODULE_{Module.Name}");
				}

				// Allow the module to modify the link environment for the binary.
				Module.SetupPrivateLinkEnvironment(this, BinaryLinkEnvironment, BinaryDependencies, LinkEnvironmentVisitedModules, ExeDir, Logger);
			}

			// Allow the binary dependencies to modify the link environment.
			foreach (UEBuildBinary BinaryDependency in BinaryDependencies)
			{
				BinaryDependency.SetupDependentLinkEnvironment(BinaryLinkEnvironment);
			}

			// Gather debug visualizers for every relevant module 
			foreach (UEBuildModule VisitedModule in LinkEnvironmentVisitedModules)
			{
				VisitedModule.LinkDebuggerVisualizers(BinaryLinkEnvironment.DebuggerVisualizerFiles, ToolChain);
			}

			// Set the link output file.
			BinaryLinkEnvironment.OutputFilePaths = [.. OutputFilePaths];

			// Remember the link type
			BinaryLinkEnvironment.LinkType = Target.LinkType;

			// Set whether the link is allowed to have exports.
			BinaryLinkEnvironment.bHasExports = bAllowExports;

			// Set the output folder for intermediate files
			BinaryLinkEnvironment.IntermediateDirectory = IntermediateDirectory;

			// Put the non-executable output files (PDB, import library, etc) in the same directory as the production
			BinaryLinkEnvironment.OutputDirectory = OutputFilePaths[0].Directory;

			// Setup link output type
			BinaryLinkEnvironment.bIsBuildingDLL = IsBuildingDll(Type);
			BinaryLinkEnvironment.bIsBuildingLibrary = IsBuildingLibrary(Type);

			// Compile any resource files
			CompileResourceFiles(ToolChain, BinaryCompileEnvironment, BinaryLinkEnvironment, [.. CompiledModules.SelectMany(x => x.ResourceFiles)], Graph);

			// Setup object export stripping
			if (Target.bStripExports)
			{
				bStripUnusedExports = true;

				string Name = OutputFilePaths.First().GetFileNameWithoutExtension();
				string Ext = ToolChain.GetExtraLinkFileExtension();
				FileReference extraObj = FileReference.Combine(IntermediateDirectory!, $"{Name}.exports.{Ext}");
				BinaryLinkEnvironment.InputFiles.Add(FileItem.GetItemByFileReference(extraObj));
			}

			// Code coverage inherited from compile environment
			BinaryLinkEnvironment.bCodeCoverage = CompileEnvironment.bCodeCoverage;

			// Provide list of runtime dependencies to link environment.
			BinaryLinkEnvironment.RuntimeDependencies = RuntimeDependencies;

			// Allow the platform to modify the binary link environment for platform-specific resources etc.
			UEBuildPlatform.GetBuildPlatform(Target.Platform).ModifyBinaryLinkEnvironment(BinaryLinkEnvironment, BinaryCompileEnvironment, Target, ToolChain, Graph);

			return BinaryLinkEnvironment;
		}

		private void CompileResourceFiles(UEToolChain ToolChain, CppCompileEnvironment BinaryCompileEnvironment, LinkEnvironment BinaryLinkEnvironment, IEnumerable<FileItem> InputFiles, IActionGraphBuilder Graph)
		{
			// @todo: This should be in some Windows code somewhere...
			if (!BinaryLinkEnvironment.Platform.IsInGroup(UnrealPlatformGroup.Windows))
			{
				return;
			}

			// If we don't have any resource file, use the default or compile a custom one for this module
			IEnumerable<FileItem> DefaultResourceFiles = [];
			// Figure out if this binary has any custom resource files. Hacky check to ignore the resource file in the Launch module, since it contains dialogs that the engine needs and always needs to be included.
			FileItem PCLaunchResource = FileItem.GetItemByFileReference(FileReference.Combine(Unreal.EngineSourceDirectory, "Runtime", "Launch", "Resources", "Windows", "PCLaunch.rc"));
			if (!InputFiles.Any(x => x != PCLaunchResource))
			{
				if (BinaryLinkEnvironment.DefaultResourceFiles.Count > 0 && (BinaryLinkEnvironment.bIsBuildingDLL || BinaryLinkEnvironment.bIsBuildingLibrary))
				{
					// If no resource files are found, use the precompiled default resource file for the target
					if (!InputFiles.Any())
					{
						BinaryLinkEnvironment.InputFiles.AddRange(BinaryLinkEnvironment.DefaultConvertedResourceFiles);
						return;
					}
					// If this module has other resource files, add the shared compiled resource files to be converted as well
					DefaultResourceFiles = BinaryLinkEnvironment.DefaultResourceFiles;
				}
				else
				{
					// Otherwise compile the default resource file per-binary, so that it gets the correct ORIGINAL_FILE_NAME macro.
					FileItem DefaultResourceFile = FileItem.GetItemByFileReference(FileReference.Combine(Unreal.EngineDirectory, "Build", "Windows", "Resources", "Default.rc2"));
					if (!BinaryLinkEnvironment.bIsBuildingDLL && !BinaryLinkEnvironment.bIsBuildingLibrary)
					{
						FileItem ExecutableResourceFile = FileItem.GetItemByFileReference(FileReference.Combine(IntermediateDirectory, $"{OutputFilePaths[0].GetFileNameWithoutAnyExtensions()}-{DefaultResourceFile.Location.GetFileName()}"));
						Graph.CreateCopyAction(DefaultResourceFile, ExecutableResourceFile);
						DefaultResourceFile = ExecutableResourceFile;
					}
					InputFiles = [.. InputFiles, DefaultResourceFile];
				}
			}

			// Create a compile environment for resource files
			CppCompileEnvironment ResourceCompileEnvironment = new(BinaryCompileEnvironment);

			// Set the original file name macro; used in Default.rc2 to set the binary metadata fields.
			ResourceCompileEnvironment.Definitions.Add($"ORIGINAL_FILE_NAME=\"{OutputFilePaths[0].GetFileName()}\"");
			
			// Compile the resource files
			List<FileItem> ObjectFiles = ToolChain.CompileRCFiles(ResourceCompileEnvironment, InputFiles, IntermediateDirectory, Graph).ObjectFiles;

			// Optionally convert the resource files depending on toolchain settings
			BinaryLinkEnvironment.InputFiles.AddRange(ToolChain.ConvertRCFiles(BinaryLinkEnvironment, [.. ObjectFiles, .. DefaultResourceFiles], IntermediateDirectory, Graph).ObjectFiles);
		}

		/// <summary>
		/// ToString implementation
		/// </summary>
		/// <returns>Returns the OutputFilePath for this binary</returns>
		public override string ToString()
		{
			return OutputFilePath.FullName;
		}
	}
}
