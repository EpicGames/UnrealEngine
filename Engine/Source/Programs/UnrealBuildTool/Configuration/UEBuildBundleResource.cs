// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool
{
	/// <summary>
	/// 
	/// </summary>
	public class UEBuildBundleResource
	{
		/// <summary>
		/// 
		/// </summary>
		/// <param name="BundleResource"></param>
		public UEBuildBundleResource(ModuleRules.BundleResource BundleResource)
		{
			ResourcePath = BundleResource.ResourcePath;
			BundleContentsSubdir = BundleResource.BundleContentsSubdir;
			bShouldLog = BundleResource.bShouldLog;
		}

		/// <summary>
		/// 
		/// </summary>
		public string? ResourcePath = null;

		/// <summary>
		/// 
		/// </summary>
		public string? BundleContentsSubdir = null;

		/// <summary>
		/// 
		/// </summary>
		public bool bShouldLog = true;
	}
}
