// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using Microsoft.Extensions.Logging;

namespace UnrealBuildTool
{
	/// <summary>
	/// Base class to handle deploy of a target for a given platform
	/// </summary>
	abstract class UEBuildDeploy
	{
		protected readonly ILogger Logger;

		public UEBuildDeploy(ILogger InLogger)
		{
			Logger = InLogger;
		}

		protected bool CopyFilesRecursively(string FromDir, string ToDir)
		{
			DirectoryInfo FromDirInfo = new DirectoryInfo(FromDir);

			if (!FromDirInfo.Exists)
			{
				Logger.LogError("Cannot copy files, source directory not found: {FromDir}", FromDirInfo.FullName);
				return false;
			}

			DirectoryInfo[] DirsToCopy = FromDirInfo.GetDirectories();

			Directory.CreateDirectory(ToDir);

			foreach (FileInfo FromFile in FromDirInfo.GetFiles())
			{
				try
				{
					// Ensure we can overrwrite dest file as non-read only.
					string ToDestFile = Path.Combine(ToDir, FromFile.Name);
					if (File.Exists(ToDestFile))
					{
						File.SetAttributes(ToDestFile, File.GetAttributes(ToDestFile) & ~FileAttributes.ReadOnly);
					}

					FromFile.CopyTo(ToDestFile, true);
				}
				catch (Exception ex)
				{
					Logger.LogError(ex, "Error while copying {FromFile} from {FromDir} to {ToDir}: {Ex}", FromFile, FromDir, ToDir, ex);
					return false;
				}
			}

			foreach (DirectoryInfo SubDir in DirsToCopy)
			{
				CopyFilesRecursively(SubDir.FullName, Path.Combine(ToDir, SubDir.Name));
			}
			return true;
		}

		/// <summary>
		/// Prepare the target for deployment
		/// </summary>
		/// <param name="Receipt">Receipt for the target being deployed</param>
		/// <returns>True if successful, false if not</returns>
		public virtual bool PrepTargetForDeployment(TargetReceipt Receipt)
		{
			bool Result = true;
			if (Receipt.IsTestTarget && Receipt.Launch != null)
			{
				foreach (ReceiptProperty TestResourcesFolder in Receipt.AdditionalProperties.FindAll(Prop => Prop.Name == "ResourcesFolder"))
				{
					DirectoryInfo ResourceDirectoryInfo = new DirectoryInfo(TestResourcesFolder.Value);
					if (ResourceDirectoryInfo.Exists)
					{
						Result &= CopyFilesRecursively(TestResourcesFolder.Value, Path.Combine(Receipt.Launch.Directory.FullName, ResourceDirectoryInfo.Name));
					}
				}
			}
			return Result;
		}
	}
}
