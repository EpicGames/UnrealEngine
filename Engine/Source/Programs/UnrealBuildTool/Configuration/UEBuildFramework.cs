// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.IO;
using System.IO.Compression;
using System.Xml;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// Represents a Mac/IOS framework
	/// </summary>
	class UEBuildFramework
	{
		/// <summary>
		/// The name of this framework
		/// </summary>
		public readonly string Name;

		/// <summary>
		/// Path to a zip file containing the framework. May be null.
		/// </summary>
		public readonly FileReference? ZipFile;

		/// <summary>
		/// Path where the zip should be extracted. May be null.
		/// </summary>
		public readonly DirectoryReference? ZipOutputDirectory;

		/// <summary>
		/// Base path to the framework on disk.
		/// </summary>
		readonly DirectoryReference? FrameworkDirectory;

		/// <summary>
		/// 
		/// </summary>
		public readonly string? CopyBundledAssets;

		/// <summary>
		/// Link the framework libraries into the executable
		/// </summary>
		public readonly bool bLinkFramework = true;

		/// <summary>
		/// Copy the framework to the target's Framework directory
		/// </summary>
		public readonly bool bCopyFramework = false;

		/// <summary>
		/// File created after the framework has been extracted. Used to add dependencies into the action graph, used by Xcode as well
		/// </summary>
		public FileItem? ExtractedTokenFile => ZipOutputDirectory == null ? null : FileItem.GetItemByFileReference(new FileReference(ZipOutputDirectory!.FullName + ".extracted"));
		/// <summary>
		/// For legacy xcode projects, we unzip in UBT (isntead of xcode), so we track if we've made an action for this framework yet (if two
		/// modules use the same framework, we only want to unzip it once. Other than time waste, there could be conflicts
		/// </summary>
		public bool bHasMadeUnzipAction;

		/// <summary>
		/// List of variants contained in this XCFramework. Only non null for XCFrameworks
		/// </summary>
		readonly List<XCFrameworkVariantEntry>? XCFrameworkVariants;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="Name">The framework name</param>
		/// <param name="CopyBundledAssets"></param>
		public UEBuildFramework(string Name, string? CopyBundledAssets = null)
		{
			this.Name = Name;
			this.CopyBundledAssets = CopyBundledAssets;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="Name">The framework name</param>
		/// <param name="ZipFile">Path to the zip file for this framework/xcframework</param>
		/// <param name="OutputDirectory">Path for the extracted zip file</param>
		/// <param name="CopyBundledAssets"></param>
		/// <param name="bLinkFramework">Link the framework into the executable</param>
		/// <param name="bCopyFramework">Copy the framework to the target's Framework directory</param>
		/// <param name="Logger">Logger for diagnostic output</param>
		public UEBuildFramework(string Name, FileReference? ZipFile, DirectoryReference OutputDirectory, string? CopyBundledAssets, bool bLinkFramework, bool bCopyFramework, ILogger Logger)
		{
			this.Name = Name;
			this.ZipFile = ZipFile;
			ZipOutputDirectory = OutputDirectory;
			FrameworkDirectory = OutputDirectory;
			this.CopyBundledAssets = CopyBundledAssets;
			this.bCopyFramework = bCopyFramework;
			this.bLinkFramework = bLinkFramework;
			if (this.ZipFile != null && this.ZipFile.FullName.EndsWith(".xcframework.zip", System.StringComparison.Ordinal))
			{
				XCFrameworkVariants = LoadXCFrameworkVariantsFromZipFile(Logger);
			}
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="Name">The framework name</param>
		/// <param name="FrameworkDirectory">Path for the framework/xcframework on disk</param>
		/// <param name="CopyBundledAssets"></param>
		/// <param name="bLinkFramework">Link the framework into the executable</param>
		/// <param name="bCopyFramework">Copy the framework to the target's Framework directory</param>
		/// <param name="Logger">Logger for diagnostic output</param>
		public UEBuildFramework(string Name, DirectoryReference FrameworkDirectory, string? CopyBundledAssets, bool bLinkFramework, bool bCopyFramework, ILogger Logger)
		{
			this.Name = Name;
			this.FrameworkDirectory = FrameworkDirectory;
			this.CopyBundledAssets = CopyBundledAssets;
			this.bCopyFramework = bCopyFramework;
			this.bLinkFramework = bLinkFramework;
			if (this.FrameworkDirectory.FullName.EndsWith(".xcframework", System.StringComparison.Ordinal))
			{
				XCFrameworkVariants = LoadXCFrameworkVariants(Logger);
			}
		}

		/// <summary>
		/// Gets the framework directory for given build settings
		/// </summary>
		/// <param name="Platform"></param>
		/// <param name="Architecture"></param>
		/// <param name="Logger">Logger for diagnostic output</param>
		public DirectoryReference? GetFrameworkDirectory(UnrealTargetPlatform? Platform, UnrealArch? Architecture, ILogger Logger)
		{
			if (XCFrameworkVariants != null && Platform != null && Architecture != null)
			{
				XCFrameworkVariantEntry? FrameworkVariant = XCFrameworkVariants.Find(x => x.Matches(Platform.Value, Architecture.Value));
				if (FrameworkVariant != null)
				{
					return DirectoryReference.Combine(FrameworkDirectory!, FrameworkVariant.LibraryIdentifier);
				}
				Logger.LogWarning("Variant for platform \"{Platform}\" with architecture {Architecture} not found in XCFramework {Name}.", Platform.ToString(), Architecture, Name);
			}
			return FrameworkDirectory;
		}

		/// <summary>
		/// Loads XCFramework variants description from Info.plist file inside XCFramework structure 
		/// <param name="Logger">Logger for diagnostic output</param>
		/// </summary>
		List<XCFrameworkVariantEntry>? LoadXCFrameworkVariants(ILogger Logger)
		{
			XmlDocument PlistDoc = new XmlDocument();
			PlistDoc.Load(FileReference.Combine(FrameworkDirectory!, "Info.plist").FullName);
			return LoadXCFrameworkVariants(PlistDoc, Logger);
		}

		/// <summary>
		/// Loads XCFramework variants description from Info.plist file inside the zipped XCFramework structure 
		/// <param name="Logger">Logger for diagnostic output</param>
		/// </summary>
		List<XCFrameworkVariantEntry>? LoadXCFrameworkVariantsFromZipFile(ILogger Logger)
		{
			using (ZipArchive Archive =  System.IO.Compression.ZipFile.OpenRead(ZipFile!.FullName))
			{
				ZipArchiveEntry? Entry  = Archive.GetEntry($"{Name}.xcframework/Info.plist");
				if (Entry == null)
				{
					Logger.LogError("Failed find Info.plist in XCFramework {Name}", Name);
					return null;
				}
				else
				{
					Stream InfoPlist = Entry.Open();
					XmlDocument PlistDoc = new XmlDocument();
					PlistDoc.Load(InfoPlist);
					return LoadXCFrameworkVariants(PlistDoc, Logger);
				}				
			}
		}

		/// <summary>
		/// Loads XCFramework variants description from Info.plist inside XCFramework structure loaded in a XmlDocument 
		/// <param name="PlistDoc">XmlDocument containing Info.plist that defines the xcframework settings</param>
		/// <param name="Logger">Logger for diagnostic output</param>
		/// </summary>
		List<XCFrameworkVariantEntry>? LoadXCFrameworkVariants(XmlDocument PlistDoc, ILogger Logger)
		{
			// Check the plist type
			XmlNode? CFBundlePackageType = PlistDoc.SelectSingleNode("/plist/dict[key='CFBundlePackageType']/string[1]");
			if (CFBundlePackageType == null || CFBundlePackageType.NodeType != XmlNodeType.Element || CFBundlePackageType.InnerText != "XFWK")
			{
				Logger.LogWarning("CFBundlePackageType is not set to XFWK in Info.plist for XCFramework {Name}", Name);
				return null;
			}

			// Load Framework variants data from dictionary nodes
			XmlNodeList? FrameworkVariantDicts = PlistDoc.SelectNodes("/plist/dict[key='AvailableLibraries']/array/dict");
			if (FrameworkVariantDicts == null)
			{
				Logger.LogWarning("Invalid Info.plist file for XCFramework {Name}. It will be used as a regular Framework", Name);
				return null;
			}

			List<XCFrameworkVariantEntry> Variants = new List<XCFrameworkVariantEntry>();
			foreach (XmlNode VariantDict in FrameworkVariantDicts)
			{
				XCFrameworkVariantEntry? Variant = XCFrameworkVariantEntry.Parse(VariantDict, Logger);
				if (Variant == null)
				{
					Logger.LogWarning("Failed to load variant from Info.plist file for XCFramework {Name}", Name);
				}
				else
				{
					Logger.LogTrace("Found {LibraryIdentifier} variant in XCFramework {Name}", Variant.LibraryIdentifier, Name);
					Variants.Add(Variant);
				}
			}
			return Variants;
		}

		/// <summary>
		/// Represents a XCFramework variant description
		/// </summary>		
		private class XCFrameworkVariantEntry
		{
			/// <summary>
			/// Identifier of this framework variant. Is in the form platform-architectures-platformvariant.
			/// Some examples would be ios-arm64, ios-arm64_x86_64-simulator
			/// It also represents the relative path inside the XCFramefork where the Framework for this variant resides
			/// </summary>
			public readonly string LibraryIdentifier;

			/// <summary>
			/// Relative path where framework lives after applying LibraryIdentifier path
			/// </summary>
			public readonly string LibraryPath;

			/// <summary>
			/// List of supported architectures for this Framework variants
			/// </summary>
			public readonly List<string> SupportedArchitectures;

			/// <summary>
			/// Platform this Framework variant is intended for. Possible values are ios, macos, watchos, tvos
			/// </summary>
			public readonly string SupportedPlatform;

			/// <summary>
			/// Platform variant for this Framework variant. It can be empty or any other value representing a platform variane, like maccatalyst or simulator 
			/// </summary>
			public readonly string? SupportedPlatformVariant;

			/// <summary>
			/// Constructor
			/// </summary>
			/// <param name="LibraryIdentifier"></param>
			/// <param name="LibraryPath"></param>
			/// <param name="SupportedArchitectures"></param>
			/// <param name="SupportedPlatform"></param>
			/// <param name="SupportedPlatformVariant"></param>
			XCFrameworkVariantEntry(string LibraryIdentifier, string LibraryPath, List<string> SupportedArchitectures, string SupportedPlatform, string? SupportedPlatformVariant)
			{
				this.LibraryIdentifier = LibraryIdentifier;
				this.LibraryPath = LibraryPath;
				this.SupportedArchitectures = SupportedArchitectures;
				this.SupportedPlatform = SupportedPlatform;
				this.SupportedPlatformVariant = SupportedPlatformVariant;
			}

			/// <summary>
			/// Returns wether this variant is a match for given platform and architecture 
			/// </summary>
			/// <param name="Platform"></param>
			/// <param name="Architecture"></param>
			public bool Matches(UnrealTargetPlatform Platform, UnrealArch Architecture)
			{
				if ((Platform == UnrealTargetPlatform.IOS && SupportedPlatform == "ios") ||
				   (Platform == UnrealTargetPlatform.Mac && SupportedPlatform == "macos") ||
				   (Platform == UnrealTargetPlatform.TVOS && SupportedPlatform == "tvos"))
				{
					if (Architecture == UnrealArch.IOSSimulator || Architecture == UnrealArch.TVOSSimulator)
					{
						// When using -simulator we don't have the actual architecture. Assume arm64 as other parts of UBT already are doing
						return (SupportedPlatformVariant == "simulator") && SupportedArchitectures.Contains("arm64");
					}
					else
					{
						return (SupportedPlatformVariant == null) && SupportedArchitectures.Contains(Architecture.AppleName);
					}
				}
				return false;
			}

			/// <summary>
			/// Creates a XCFrameworkVariantEntry by parsing the content from the XCFramework Info.plist file  
			/// </summary>
			/// <param name="DictNode">XmlNode loaded form Info.plist containing the representation of a &lt;dict&gt; that contains the variant description</param>
			/// <param name="Logger">Logger for diagnostic output</param>
			public static XCFrameworkVariantEntry? Parse(XmlNode DictNode, ILogger Logger)
			{
				// Entries from a <dict> in a plist file always contains a <key> node and a value node
				if ((DictNode.ChildNodes.Count % 2) != 0)
				{
					Logger.LogDebug("Invalid dict in Info.plist file for XCFramework");
					return null;
				}

				string? LibraryIdentifier = null;
				string? LibraryPath = null;
				List<string>? SupportedArchitectures = null;
				string? SupportedPlatform = null;
				string? SupportedPlatformVariant = null;

				for (int i = 0; i < DictNode.ChildNodes.Count; i += 2)
				{
					XmlNode? Key = DictNode.ChildNodes[i];
					XmlNode? Value = DictNode.ChildNodes[i + 1];

					if (Value != null && Key != null && Key.Name == "key")
					{
						switch (Key.InnerText)
						{
							case "LibraryIdentifier":
								LibraryIdentifier = ParseString(Value, Logger);
								break;
							case "LibraryPath":
								LibraryPath = ParseString(Value, Logger);
								break;
							case "SupportedPlatform":
								SupportedPlatform = ParseString(Value, Logger);
								break;
							case "SupportedPlatformVariant":
								SupportedPlatformVariant = ParseString(Value, Logger);
								break;
							case "SupportedArchitectures":
								SupportedArchitectures = ParseListOfStrings(Value, Logger);
								break;
							default:
								break;
						}
					}
				}

				// All fields but SupportedPlatformVariant are allowed to be present in the framework variant descriprion
				if (LibraryIdentifier == null || LibraryPath == null || SupportedArchitectures == null || SupportedPlatform == null)
				{
					Logger.LogDebug("Missing data in Info.plist file for XCFramework");
					return null;
				}
				return new XCFrameworkVariantEntry(LibraryIdentifier, LibraryPath, SupportedArchitectures, SupportedPlatform, SupportedPlatformVariant);
			}

			/// <summary>
			/// Parses the content from a string value node in a plist file
			/// </summary>
			/// <param name="ValueNode">XmlNode we expect to contain a &lt;string&gt; value</param>			
			/// <param name="Logger">Logger for diagnostic output</param>
			static string? ParseString(XmlNode ValueNode, ILogger Logger)
			{
				if (ValueNode.Name != "string")
				{
					Logger.LogDebug("Unexpected tag \"{Tag}\" while expecting \"string\" in Info.plist file for XCFramework", ValueNode.Name);
					return null;
				}
				return ValueNode.InnerText;
			}

			/// <summary>
			/// Parses the content from an array value node in a plist file that has several string entries
			/// </summary>
			/// <param name="ValueNode">XmlNode we expect to contain a &lt;array&gt; value with several &lt;string&gt; entries</param>			
			/// <param name="Logger">Logger for diagnostic output</param>
			static List<string>? ParseListOfStrings(XmlNode ValueNode, ILogger Logger)
			{
				if (ValueNode.Name != "array")
				{
					Logger.LogDebug("Unexpected tag \"{Name}\" while expecting \"array\" in Info.plist file for XCFramework", ValueNode.Name);
					return null;
				}

				List<string> ListOfStrings = new List<string>();
				foreach (XmlNode ChildNode in ValueNode.ChildNodes)
				{
					string? ParsedString = ParseString(ChildNode, Logger);
					if (ParsedString != null)
					{
						ListOfStrings.Add(ParsedString);
					}
				}
				return ListOfStrings;
			}
		}
	}
}
