// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.Linq;
using System.Reflection;
using System.Text;
using EpicGames.Core;
using Microsoft.CodeAnalysis;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// A unit of code compilation and linking.
	/// </summary>
	abstract class UEBuildModule
	{
		/// <summary>
		/// The rules for this module
		/// </summary>
		public readonly ModuleRules Rules;

		/// <summary>
		/// The directory for this module's object files
		/// </summary>
		public readonly DirectoryReference IntermediateDirectory;

		/// <summary>
		/// The directory for this module's generated files that are architecture independent
		/// </summary>
		public readonly DirectoryReference IntermediateDirectoryNoArch;

		/// <summary>
		/// The name that uniquely identifies the module.
		/// </summary>
		public string Name => Rules.Name;

		/// <summary>
		/// Path to the module directory
		/// </summary>
		public DirectoryReference ModuleDirectory => Rules.Directory;

		/// <summary>
		/// Paths to all potential module source directories (with platform extension directories added in)
		/// </summary>
		public DirectoryItem[] ModuleDirectories;

		/// <summary>
		/// The name of the .Build.cs file this module was created from, if any
		/// </summary>
		public FileReference RulesFile => Rules.File;

		/// <summary>
		/// The binary the module will be linked into for the current target.  Only set after UEBuildBinary.BindModules is called.
		/// </summary>
		public UEBuildBinary Binary = null!;

		/// <summary>
		/// The name of the _API define for this module (for functions)
		/// </summary>
		protected readonly string ModuleApiDefine;

		/// <summary>
		/// The name of the _NON_ATTRIBUTED_API define for this module (for variables)
		/// </summary>
		protected readonly string ModuleNonAttributedDefine;

		/// <summary>
		/// Set of all the public definitions
		/// </summary>
		public readonly HashSet<string> PublicDefinitions;

		/// <summary>
		/// Set of all public include paths
		/// </summary>
		public readonly HashSet<DirectoryReference> PublicIncludePaths;

		/// <summary>
		/// Set of all internal include paths
		/// </summary>
		public readonly HashSet<DirectoryReference> InternalIncludePaths;

		/// <summary>
		/// Nested public include paths which used to be added automatically, but are now only added for modules with bNestedPublicIncludePaths set.
		/// </summary>
		public readonly HashSet<DirectoryReference> LegacyPublicIncludePaths = new HashSet<DirectoryReference>();

		/// <summary>
		/// Parent include paths which used to be added automatically, but are now only added for modules with bLegacyParentIncludePaths set.
		/// </summary>
		public readonly HashSet<DirectoryReference> LegacyParentIncludePaths = new HashSet<DirectoryReference>();

		/// <summary>
		/// Set of all private include paths
		/// </summary>
		public readonly HashSet<DirectoryReference> PrivateIncludePaths;

		/// <summary>
		/// Set of all system include paths
		/// </summary>
		public readonly HashSet<DirectoryReference> PublicSystemIncludePaths;

		/// <summary>
		/// Set of all additional libraries
		/// </summary>
		public readonly HashSet<FileReference> PublicLibraries = new HashSet<FileReference>();

		/// <summary>
		/// Set of all system libraries
		/// </summary>
		public readonly HashSet<string> PublicSystemLibraries = new HashSet<string>();

		/// <summary>
		/// Set of all public system library paths
		/// </summary>
		public readonly HashSet<DirectoryReference> PublicSystemLibraryPaths;

		/// <summary>
		/// Set of additional frameworks
		/// </summary>
		public readonly HashSet<string> PublicFrameworks;

		/// <summary>
		/// 
		/// </summary>
		public readonly HashSet<string> PublicWeakFrameworks;

		/// <summary>
		/// 
		/// </summary>
		protected readonly HashSet<UEBuildFramework> PublicAdditionalFrameworks;

		/// <summary>
		/// 
		/// </summary>
		protected readonly HashSet<UEBuildBundleResource> PublicAdditionalBundleResources;

		/// <summary>
		/// Names of modules with header files that this module's public interface needs access to.
		/// </summary>
		public List<UEBuildModule>? PublicIncludePathModules;

		/// <summary>
		/// Names of modules that this module's public interface depends on.
		/// </summary>
		public List<UEBuildModule>? PublicDependencyModules;

		/// <summary>
		/// Names of DLLs that this module should delay load
		/// </summary>
		public HashSet<string> PublicDelayLoadDLLs;

		/// <summary>
		/// Names of modules with header files that this module's private implementation needs access to.
		/// </summary>
		public List<UEBuildModule>? PrivateIncludePathModules;

		/// <summary>
		/// Names of modules that this module's private implementation depends on.
		/// </summary>
		public List<UEBuildModule>? PrivateDependencyModules;

		/// <summary>
		/// Extra modules this module may require at run time
		/// </summary>
		public List<UEBuildModule>? DynamicallyLoadedModules;

		/// <summary>
		/// Set of modules that have referenced this module
		/// </summary>
		public SortedSet<UEBuildModule>? ReferenceStackParentModules;

		/// <summary>
		/// Set of all allowed restricted folder references
		/// </summary>
		private readonly HashSet<DirectoryReference> RestrictedFoldersAllowList;

		/// <summary>
		/// Set of aliased restricted folder references
		/// </summary>
		public readonly Dictionary<string, string> AliasRestrictedFolders;

		/// <summary>
		/// Per-architecture lists of dependencies for linking to ignore (useful when building for multiple architectures, and a lib only is needed for one architecture), it's up to the Toolchain to use this
		/// </summary>
		public Dictionary<string, List<UnrealArch>> DependenciesToSkipPerArchitecture;

		/// <summary>
		/// The Verse source code directory associated with this module if any
		/// </summary>
		public virtual DirectoryReference? VerseDirectory => null;

		/// <summary>
		/// If this module has Verse code associated with it (convenience function)
		/// </summary>
		public bool bHasVerse => VerseDirectory != null;

		/// <summary>
		/// If this module or any of its dependencies has Verse code associated with it
		/// </summary>
		public bool bDependsOnVerse = false;

		/// <summary>
		/// Set of resource file paths
		/// </summary>
		public HashSet<FileItem> ResourceFiles = new();

		/// <summary>
		/// Set of debug visualizer paths
		/// </summary>
		public HashSet<FileItem> NatvisFiles = new();

		/// <summary>
		/// AutoRTFM function-map definition files
		/// </summary>
		public HashSet<FileItem> PublicAutoRTFMExternalMappingFiles = new();

		public HashSet<UEBuildModule>? DependenciesForStableTopologicalSort { get; private set; }

		static readonly string[] ApiDefinitions = ["", "DLLEXPORT", "DLLIMPORT"];

		private readonly string[] ModuleApiDefinitionStrings;
		private readonly string[] ModuleNonAttributedApiDefinitionStrings;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="Rules">Rules for this module</param>
		/// <param name="IntermediateDirectory">Intermediate directory for this module</param>
		/// <param name="IntermediateDirectoryNoArch">Intermediate directory for this module for files that are architecture independent</param>
		/// <param name="Logger">Logger for output</param>
		public UEBuildModule(ModuleRules Rules, DirectoryReference IntermediateDirectory, DirectoryReference IntermediateDirectoryNoArch, ILogger Logger)
		{
			this.Rules = Rules;
			this.IntermediateDirectory = IntermediateDirectory;
			this.IntermediateDirectoryNoArch = IntermediateDirectoryNoArch;

			ModuleApiDefine = Name.ToUpperInvariant() + "_API";
			ModuleNonAttributedDefine = Name.ToUpperInvariant() + "_NON_ATTRIBUTED_API";

			HashSet<string> PublicPreBuildLibraries = HashSetFromOptionalEnumerableStringParameter(Rules.PublicPreBuildLibraries);
			PublicDefinitions = [.. HashSetFromOptionalEnumerableStringParameter(Rules.PublicDefinitions).Select(TrimDefinition)];
			PublicIncludePaths = CreateDirectoryHashSet(Rules.PublicIncludePaths);
			InternalIncludePaths = CreateDirectoryHashSet(Rules.InternalIncludePaths);
			PublicSystemIncludePaths = CreateDirectoryHashSet(Rules.PublicSystemIncludePaths);
			PublicSystemLibraryPaths = CreateDirectoryHashSet(Rules.PublicSystemLibraryPaths);
			HashSet<string> PublicAdditionalLibraries = HashSetFromOptionalEnumerableStringParameter(Rules.PublicAdditionalLibraries.Union(PublicPreBuildLibraries));
			PublicSystemLibraries = HashSetFromOptionalEnumerableStringParameter(Rules.PublicSystemLibraries);
			PublicFrameworks = HashSetFromOptionalEnumerableStringParameter(Rules.PublicFrameworks);
			PublicWeakFrameworks = HashSetFromOptionalEnumerableStringParameter(Rules.PublicWeakFrameworks);
			PublicAutoRTFMExternalMappingFiles = [.. Rules.AutoRTFMExternalMappingFiles.Select(FileItem.GetItemByPath)];

			foreach (string LibraryName in PublicAdditionalLibraries)
			{
				FileItem Library = FileItem.GetItemByPath(LibraryName);
				if (Library.Exists)
				{
					// if the library path is fully qualified we just add it, this is the preferred method of adding a library
					FileReference Location = Library.Location;
					PublicLibraries.Add(Location);
				}
				else if (PublicPreBuildLibraries.Contains(LibraryName))
				{
					Logger.LogDebug("Library '{LibraryName}' was not resolvable to a file when used in Module '{Name}'.  Be sure to add either a TargetRules.PreBuildSteps entry or a TargetRules.PreBuildTargets entry to assure it is built for your target.", LibraryName, Name);
					PublicLibraries.Add(Library.Location);
				}
				else
				{
					// the library path does not seem to be resolvable as is, lets warn about it as dependency checking will not work for it
					LogWarningOrThrowError(Logger, Rules.Target.DefaultWarningLevel, "Library '{0}' was not resolvable to a file when used in Module '{1}', assuming it is a filename and will search library paths for it. This is slow and dependency checking will not work for it. Please update reference to be fully qualified alternatively use PublicSystemLibraryPaths if you do intended to use this slow path to suppress this warning. ", LibraryName, Name);
					PublicSystemLibraries.Add(LibraryName);
				}
			}

			PublicAdditionalFrameworks = new HashSet<UEBuildFramework>();
			if (Rules.PublicAdditionalFrameworks != null)
			{
				foreach (ModuleRules.Framework FrameworkRules in Rules.PublicAdditionalFrameworks)
				{
					bool bLinkFramework = FrameworkRules.Mode == ModuleRules.Framework.FrameworkMode.Link || FrameworkRules.Mode == ModuleRules.Framework.FrameworkMode.LinkAndCopy;
					bool bCopyFramework = FrameworkRules.Mode == ModuleRules.Framework.FrameworkMode.Copy || FrameworkRules.Mode == ModuleRules.Framework.FrameworkMode.LinkAndCopy;

					UEBuildFramework Framework;
					if (FrameworkRules.IsZipFile())
					{
						FileReference ZipFile = FileReference.Combine(ModuleDirectory, FrameworkRules.Path);
						// If FrameworkPath ends in .zip, it needs to be extracted
						Framework = new UEBuildFramework(
							Name: FrameworkRules.Name,
							ZipFile: ZipFile,
							OutputDirectory: DirectoryReference.Combine(Unreal.EngineDirectory, "Intermediate", "UnzippedFrameworks", FrameworkRules.Name, ZipFile.GetFileNameWithoutExtension()),
							CopyBundledAssets: FrameworkRules.CopyBundledAssets,
							bLinkFramework, bCopyFramework, Logger);
					}
					else
					{
						// Framework on disk
						Framework = new UEBuildFramework(
							Name: FrameworkRules.Name,
							FrameworkDirectory: DirectoryReference.Combine(ModuleDirectory, FrameworkRules.Path),
							CopyBundledAssets: FrameworkRules.CopyBundledAssets,
							bLinkFramework, bCopyFramework, Logger);
					}
					PublicAdditionalFrameworks.Add(Framework);
				}
			}

			PublicAdditionalBundleResources = Rules.AdditionalBundleResources == null ? new HashSet<UEBuildBundleResource>() : new HashSet<UEBuildBundleResource>(Rules.AdditionalBundleResources.Select(x => new UEBuildBundleResource(x)));
			PublicDelayLoadDLLs = HashSetFromOptionalEnumerableStringParameter(Rules.PublicDelayLoadDLLs);
			if (Rules.bUsePrecompiled)
			{
				PrivateIncludePaths = new HashSet<DirectoryReference>();
			}
			else
			{
				PrivateIncludePaths = CreateDirectoryHashSet(Rules.PrivateIncludePaths);
			}

			RestrictedFoldersAllowList = new HashSet<DirectoryReference>(Rules.AllowedRestrictedFolders.Select(x => DirectoryReference.Combine(ModuleDirectory, x)));

			AliasRestrictedFolders = new Dictionary<string, string>(Rules.AliasRestrictedFolders);

			DependenciesToSkipPerArchitecture = Rules.DependenciesToSkipPerArchitecture;

			// get the module directories from the module
			ModuleDirectories = Rules.GetAllModuleDirectories().Select(dirRef => DirectoryItem.GetItemByDirectoryReference(dirRef)).ToArray();

			// Add any additional debug visualizers 
			foreach (string natVisPath in HashSetFromOptionalEnumerableStringParameter(Rules.PublicDebugVisualizerPaths))
			{
				FileItem natVisItem = FileItem.GetItemByPath(natVisPath);
				if (!natVisItem.HasExtension(".natvis") && !natVisItem.HasExtension(".natstepfilter") && !natVisItem.HasExtension(".natjmc"))
				{
					Log.TraceWarningTask(RulesFile, $"Referenced Debug Visualizer '{natVisItem}' is not a .natvis, .natstepfilter, or .natjmc file");
				}
				else if (!natVisItem.Exists)
				{
					Log.TraceWarningTask(RulesFile, $"Referenced Debug Visualizer '{natVisItem}' does not exist");
				}
				else
				{
					NatvisFiles.Add(natVisItem);
				}
			}

			// Add any debug visualizers found in the module directories
			foreach (DirectoryItem directory in ModuleDirectories)
			{
				NatvisFiles.UnionWith(directory.EnumerateFiles().Where(x => x.HasExtension(".natvis") || x.HasExtension(".natstepfilter") || x.HasExtension(".natjmc")));
			}

			ModuleApiDefinitionStrings = new string[3];
			ModuleNonAttributedApiDefinitionStrings = new string[3];

			// We use a C-style attribute here because of where these macros are defined in UE. C++ attributes must come at the beginning of the function
			// definition, like `[[my_attr]] void Foo();`, but these macros are used like `void FOO_API Foo();`, and if we used the C++ attribute name to
			// disable AutoRTFM on this function the compiler would spit nails. But it'll happily consume the C-style attribute at that location and do
			// the right thing, so we use that instead.
			// Caution: Hard coded constant '1' in this string must be kept in sync with Engine/Source/Runtime/AutoRTFM/Public/AutoRTFM/Constants.h.
			Debug.Assert(Rules.Target.StaticAnalyzer == StaticAnalyzer.None || Rules.Target.StaticAnalyzer == StaticAnalyzer.Default);
			bool bNonClangSA = Rules.Target.StaticAnalyzer == StaticAnalyzer.Default && Rules.Target.Platform.IsInGroup(UnrealPlatformGroup.Microsoft) && !Rules.Target.WindowsPlatform.Compiler.IsClang();
			string AutoRTFMDisable = (Rules.Target.bUseAutoRTFMCompiler && Rules.bDisableAutoRTFMInstrumentation && Rules.bInjectAutoRTFMAttributeDisables && !bNonClangSA) ?
				"__attribute__((autortfm(/*autortfm_mode_disable*/ 1)))" : String.Empty;

			// We precalculate these because they are reused over and over again and causing tons of allocations
			for (int i=0; i!=3; ++i)
			{
				string ApiDefinition = ApiDefinitions[i];

				// Add a spacer if and only if we've actually used import/export above.
				string Spacer = ApiDefinition.Length == 0 ? String.Empty : " ";
				// Add the import or export declaration for the module
				ModuleApiDefinitionStrings[i] = $"{ModuleApiDefine}={ApiDefinition}{Spacer}{AutoRTFMDisable}";
				ModuleNonAttributedApiDefinitionStrings[i] = $"{ModuleNonAttributedDefine}={ApiDefinition}";
			}
		}

		/// <summary>
		/// Log a warning or throw an error message
		/// </summary>
		/// <param name="Logger"></param>
		/// <param name="Level"></param>
		/// <param name="Format"></param>
		/// <param name="Args"></param>
		static void LogWarningOrThrowError(ILogger Logger, WarningLevel Level, string Format, params object[] Args)
		{
			if (Level == WarningLevel.Error)
			{
				throw new BuildException(Format, Args);
			}
			else if (Level == WarningLevel.Warning)
			{
				Logger.LogWarning("{Message}", String.Format(Format, Args));
			}
		}

		/// <summary>
		/// Determines if a file is part of the given module
		/// </summary>
		/// <param name="Location">Path to the file</param>
		/// <returns>True if the file is part of this module</returns>
		public virtual bool ContainsFile(FileReference Location)
		{
			return ModuleDirectories.Any(x => Location.IsUnderDirectory(x.Location));
		}

		/// <summary>
		/// Returns a list of this module's dependencies.
		/// </summary>
		/// <returns>An enumerable containing the dependencies of the module.</returns>
		public HashSet<UEBuildModule> GetDependencies(bool bWithIncludePathModules, bool bWithDynamicallyLoadedModules)
		{
			HashSet<UEBuildModule> Modules = new HashSet<UEBuildModule>();
			Modules.UnionWith(PublicDependencyModules ?? new());
			Modules.UnionWith(PrivateDependencyModules ?? new());
			if (bWithIncludePathModules)
			{
				Modules.UnionWith(PublicIncludePathModules ?? new());
				Modules.UnionWith(PrivateIncludePathModules ?? new());
			}
			if (bWithDynamicallyLoadedModules)
			{
				Modules.UnionWith(DynamicallyLoadedModules ?? new());
			}
			return Modules;
		}

		/// <summary>
		/// Returns a list of this module's frameworks.
		/// </summary>
		/// <returns>A List containing the frameworks this module requires.</returns>
		public List<string> GetPublicFrameworks()
		{
			return new List<string>(PublicFrameworks);
		}

		/// <summary>
		/// Returns a list of this module's immediate dependencies.
		/// </summary>
		/// <returns>An enumerable containing the dependencies of the module.</returns>
		public IEnumerable<UEBuildModule> GetDirectDependencyModules()
		{
			return [
				.. PublicDependencyModules ?? [],
				.. PrivateDependencyModules ?? [],
				.. DynamicallyLoadedModules ?? []
			];
		}

		/// <summary>
		/// Converts an optional string list parameter to a well-defined hash set.
		/// </summary>
		protected HashSet<DirectoryReference> CreateDirectoryHashSet(List<string> InEnumerableStrings)
		{
			HashSet<DirectoryReference> Directories = new (InEnumerableStrings.Count);
			foreach (string InputString in InEnumerableStrings)
			{
				DirectoryReference Dir = new DirectoryReference(ExpandPathVariables(InputString, null, null));
				if (DirectoryLookupCache.DirectoryExists(Dir))
				{
					Directories.Add(Dir);
				}
				else
				{
					Log.TraceWarningTask(RulesFile, $"Referenced directory '{Dir}' does not exist.");
				}
			}
			return Directories;
		}

		/// <summary>
		/// Converts an optional string list parameter to a well-defined hash set.
		/// </summary>
		protected HashSet<string> HashSetFromOptionalEnumerableStringParameter(IEnumerable<string> InEnumerableStrings)
		{
			_ = InEnumerableStrings.TryGetNonEnumeratedCount(out int estimatedCount);
			HashSet<string> hashSet = new(estimatedCount);
			foreach (string s in InEnumerableStrings.Where(str => !String.IsNullOrWhiteSpace(str)))
			{
				hashSet.Add(ExpandPathVariables(s, null, null));
			}
			return hashSet;
		}

		/// <summary>
		/// Determines whether this module has a circular dependency on the given module
		/// </summary>
		public bool HasCircularDependencyOn(UEBuildModule Module)
		{
			return Rules.CircularlyReferencedDependentModules.Contains(Module.Name);
		}

		/// <summary>
		/// Add precompile build products from this module to the target
		/// </summary>
		/// <param name="ToolChain"></param>
		/// <param name="BuildProducts"></param>
		public virtual void GetBuildProducts(UEToolChain ToolChain, Dictionary<FileReference, BuildProductType> BuildProducts)
		{
			if (Rules.bPrecompile)
			{
				foreach (FileItem NatvisSourceFile in NatvisFiles)
				{
					FileItem? Item = ToolChain.LinkDebuggerVisualizer(NatvisSourceFile, IntermediateDirectory);
					if (Item != null)
					{
						BuildProducts.Add(Item.Location, BuildProductType.BuildResource);
					}
				}
			}
		}

		/// <summary>
		/// Enumerates additional build products which may be produced by this module. Some platforms (eg. Mac, Linux) can link directly against .so/.dylibs, but they 
		/// are also copied to the output folder by the toolchain.
		/// </summary>
		/// <param name="Libraries">List to which libraries required by this module are added</param>
		/// <param name="BundleResources">List of bundle resources required by this module</param>
		public void GatherAdditionalResources(List<string> Libraries, List<UEBuildBundleResource> BundleResources)
		{
			Libraries.AddRange(PublicLibraries.Select(x => x.FullName));
			Libraries.AddRange(PublicSystemLibraries);
			BundleResources.AddRange(PublicAdditionalBundleResources);
		}

		/// <summary>
		/// Determines the distribution level of a module based on its directory and includes.
		/// </summary>
		/// <param name="RootDirectories">The set of additional paths to check, if available</param>
		/// <returns>Map of the restricted folder types to the first found instance</returns>
		public Dictionary<RestrictedFolder, DirectoryReference> FindRestrictedFolderReferences(List<DirectoryReference> RootDirectories)
		{
			Dictionary<RestrictedFolder, DirectoryReference> References = new Dictionary<RestrictedFolder, DirectoryReference>();
			if (!Rules.bLegalToDistributeObjectCode)
			{
				// Find all the directories that this module references
				HashSet<DirectoryReference> ReferencedDirs = new HashSet<DirectoryReference>();
				GetReferencedDirectories(ReferencedDirs);

				// Remove all the allow listed folders
				ReferencedDirs.ExceptWith(RestrictedFoldersAllowList);
				ReferencedDirs.ExceptWith(PublicDependencyModules!.SelectMany(x => x.RestrictedFoldersAllowList));
				ReferencedDirs.ExceptWith(PrivateDependencyModules!.SelectMany(x => x.RestrictedFoldersAllowList));

				// Add flags for each of them
				foreach (DirectoryReference ReferencedDir in ReferencedDirs)
				{
					// Find the base directory containing this reference
					DirectoryReference? BaseDir = RootDirectories.FirstOrDefault(x => ReferencedDir.IsUnderDirectory(x));
					// @todo platplug does this need to check platform extension engine directories? what are ReferencedDir's here?
					if (BaseDir == null)
					{
						continue;
					}

					// Add references to each of the restricted folders
					List<RestrictedFolder> Folders = RestrictedFolders.FindRestrictedFolders(BaseDir, ReferencedDir);
					foreach (RestrictedFolder Folder in Folders)
					{
						if (!References.ContainsKey(Folder))
						{
							References.Add(Folder, ReferencedDir);
						}
					}
				}
			}
			return References;
		}

		/// <summary>
		/// Finds all the directories that this folder references when building
		/// </summary>
		/// <param name="Directories">Set of directories to add to</param>
		protected virtual void GetReferencedDirectories(HashSet<DirectoryReference> Directories)
		{
			Directories.Add(ModuleDirectory);

			foreach (DirectoryReference PublicIncludePath in PublicIncludePaths)
			{
				Directories.Add(PublicIncludePath);
			}
			foreach (DirectoryReference InternalIncludePath in InternalIncludePaths)
			{
				Directories.Add(InternalIncludePath);
			}
			foreach (DirectoryReference PrivateIncludePath in PrivateIncludePaths)
			{
				Directories.Add(PrivateIncludePath);
			}
			foreach (DirectoryReference PublicSystemIncludePath in PublicSystemIncludePaths)
			{
				Directories.Add(PublicSystemIncludePath);
			}
			foreach (DirectoryReference PublicSystemLibraryPath in PublicSystemLibraryPaths)
			{
				Directories.Add(PublicSystemLibraryPath);
			}
		}

		/// <summary>
		/// Find all the modules which affect the private compile environment.
		/// </summary>
		/// <param name="ModuleToIncludePathsOnlyFlag"></param>
		protected void FindModulesInPrivateCompileEnvironment(Dictionary<UEBuildModule, bool> ModuleToIncludePathsOnlyFlag)
		{
			// Add in all the modules that are only in the private compile environment
			foreach (UEBuildModule PrivateDependencyModule in PrivateDependencyModules!)
			{
				PrivateDependencyModule.FindModulesInPublicCompileEnvironment(ModuleToIncludePathsOnlyFlag);
			}
			foreach (UEBuildModule PrivateIncludePathModule in PrivateIncludePathModules!)
			{
				PrivateIncludePathModule.FindIncludePathModulesInPublicCompileEnvironment(ModuleToIncludePathsOnlyFlag);
			}

			// Add the modules in the public compile environment
			FindModulesInPublicCompileEnvironment(ModuleToIncludePathsOnlyFlag);
		}

		/// <summary>
		/// Find all the modules which affect the public compile environment. 
		/// </summary>
		/// <param name="ModuleToIncludePathsOnlyFlag"></param>
		protected void FindModulesInPublicCompileEnvironment(Dictionary<UEBuildModule, bool> ModuleToIncludePathsOnlyFlag)
		{
			//
			bool bModuleIncludePathsOnly;
			if (ModuleToIncludePathsOnlyFlag.TryGetValue(this, out bModuleIncludePathsOnly) && !bModuleIncludePathsOnly)
			{
				return;
			}

			ModuleToIncludePathsOnlyFlag[this] = false;

			foreach (UEBuildModule DependencyModule in PublicDependencyModules!)
			{
				DependencyModule.FindModulesInPublicCompileEnvironment(ModuleToIncludePathsOnlyFlag);
			}

			// Now add an include paths from modules with header files that we need access to, but won't necessarily be importing
			foreach (UEBuildModule IncludePathModule in PublicIncludePathModules!)
			{
				IncludePathModule.FindIncludePathModulesInPublicCompileEnvironment(ModuleToIncludePathsOnlyFlag);
			}
		}

		/// <summary>
		/// Find all the modules which affect the public compile environment. Searches through 
		/// </summary>
		/// <param name="ModuleToIncludePathsOnlyFlag"></param>
		protected void FindIncludePathModulesInPublicCompileEnvironment(Dictionary<UEBuildModule, bool> ModuleToIncludePathsOnlyFlag)
		{
			// Add this module to the list
			if (ModuleToIncludePathsOnlyFlag.TryAdd(this, true))
			{
				// Include any of its public include path modules in the compile environment too
				foreach (UEBuildModule IncludePathModule in PublicIncludePathModules!)
				{
					IncludePathModule.FindIncludePathModulesInPublicCompileEnvironment(ModuleToIncludePathsOnlyFlag);
				}
			}
		}

		private static void AddIncludePaths(HashSet<DirectoryReference> IncludePaths, HashSet<DirectoryReference> IncludePathsToAdd)
		{
			// Need to check whether directories exist to avoid bloating compiler command line with generated code directories
			IncludePaths.UnionWith(IncludePathsToAdd);
		}

		/// <summary>
		/// Trims whitespace from a definition to match strict rules which does not allow space before and after '='.
		/// </summary>
		/// <param name="Def"></param>
		/// <returns>Trimmed string, or the current instance if unchanged</returns>
		private static string TrimDefinition(string Def)
		{
			int IndexOfAssign = Def.IndexOf('=', StringComparison.Ordinal);
			if (IndexOfAssign != -1)
			{
				ReadOnlySpan<char> DefSpan = Def.AsSpan();
				ReadOnlySpan<char> Name = DefSpan[..IndexOfAssign];
				ReadOnlySpan<char> NameTrim = Name.Trim();
				ReadOnlySpan<char> Value = DefSpan[(IndexOfAssign + 1)..];
				ReadOnlySpan<char> ValueTrim = Value.Trim();
				return Name.Length != NameTrim.Length || Value.Length != ValueTrim.Length
					? $"{NameTrim}={ValueTrim}"
					: Def;
			}
			return Def.Trim();
		}

		/// <summary>
		/// Add definitions from source to target. TrimDefinition is called in the constructor to match strict rules which does not allow space before and after '='.
		/// </summary>
		protected static void AddDefinitions(List<string> Target, HashSet<string> Source)
		{
			Target.AddRange(Source);
		}

		/// <summary>
		/// Sets up the environment for compiling any module that includes the public interface of this module.
		/// </summary>
		public virtual void AddModuleToCompileEnvironment(
			UEBuildModule? SourceModule,
			UEBuildBinary? SourceBinary,
			HashSet<DirectoryReference> IncludePaths,
			HashSet<DirectoryReference> SystemIncludePaths,
			HashSet<DirectoryReference> ModuleInterfacePaths,
			List<string> Definitions,
			List<UEBuildFramework> AdditionalFrameworks,
			HashSet<FileItem> AutoRTFMExternalMappingFiles,
			List<FileItem> AdditionalPrerequisites,
			bool bLegacyPublicIncludePaths,
			bool bLegacyParentIncludePaths
			)
		{
			// Add this module's public include paths and definitions
			AddIncludePaths(IncludePaths, PublicIncludePaths);

			// Add the module's parent directory to the include path, so we can root #includes from generated source files to it. Not recommended (Use BuildSetting.V3 or later)
			if (bLegacyParentIncludePaths)
			{
				AddIncludePaths(IncludePaths, LegacyParentIncludePaths);
				IncludePaths.Add(ModuleDirectory.ParentDirectory!);
			}

			// Add this module's legacy public include paths. Not recommended (Use BuildSetting.V2 or later)
			if (bLegacyPublicIncludePaths)
			{
				AddIncludePaths(IncludePaths, LegacyPublicIncludePaths);
			}

			// Add this module's internal include paths, only if the scope is the same as the SourceModule's scope or if the source module is an engine module
			if (SourceModule != null && (Rules.Context.Scope == SourceModule.Rules.Context.Scope || SourceModule.Rules.bTreatAsEngineModule))
			{
				AddIncludePaths(IncludePaths, InternalIncludePaths);
			}

			// Add this module's public system include paths
			SystemIncludePaths.UnionWith(PublicSystemIncludePaths);

			// Add this module's public definitions
			AddDefinitions(Definitions, PublicDefinitions);

			if (Rules.Type == ModuleRules.ModuleType.CPlusPlus)
			{
				int ApiDefinitionIndex = 0;

				if (Rules.Target.LinkType == TargetLinkType.Monolithic)
				{
					if (Rules.Target.bShouldCompileAsDLL && (Rules.Target.bHasExports || Rules.ModuleSymbolVisibility == ModuleRules.SymbolVisibility.VisibileForDll))
					{
						ApiDefinitionIndex = 1;
					}
				}
				else if (Binary == null || (Rules.Target.bMergeModules ? (SourceBinary == null || SourceModule != this) : SourceBinary != Binary))
				{
					ApiDefinitionIndex = 2;
				}
				else if (!(!Binary.bAllowExports || Rules.Target.bMergeModules)
					|| (Rules.Target.bMergeModules && Rules.ModuleSymbolVisibility == ModuleRules.SymbolVisibility.VisibileForDll))
				{
					ApiDefinitionIndex = 1;
				}

				// Add the import or export declaration for the module
				Definitions.Add(ModuleApiDefinitionStrings[ApiDefinitionIndex]);
				Definitions.Add(ModuleNonAttributedApiDefinitionStrings[ApiDefinitionIndex]);
			}

			// Add the additional frameworks so that the compiler can know about their #include paths
			AdditionalFrameworks.AddRange(PublicAdditionalFrameworks);

			// Add any generated type library headers
			if (Rules.TypeLibraries.Count > 0)
			{
				IncludePaths.Add(IntermediateDirectory);
				foreach (ModuleRules.TypeLibrary TypeLibrary in Rules.TypeLibraries)
				{
					AdditionalPrerequisites.Add(FileItem.GetItemByFileReference(FileReference.Combine(IntermediateDirectory, TypeLibrary.Header)));
					if (!String.IsNullOrEmpty(TypeLibrary.Include))
					{
						AdditionalPrerequisites.Add(FileItem.GetItemByFileReference(FileReference.Combine(IntermediateDirectory, TypeLibrary.Include)));
					}
				}
			}

			AutoRTFMExternalMappingFiles.UnionWith(PublicAutoRTFMExternalMappingFiles);
		}

		/// <summary>
		/// Sets up the environment for compiling this module.
		/// </summary>
		protected virtual void SetupPrivateCompileEnvironment(
			HashSet<DirectoryReference> IncludePaths,
			HashSet<DirectoryReference> SystemIncludePaths,
			HashSet<DirectoryReference> ModuleInterfacePaths,
			List<string> Definitions,
			List<UEBuildFramework> AdditionalFrameworks,
			HashSet<FileItem> AutoRTFMExternalMappingFiles,
			List<FileItem> AdditionalPrerequisites,
			bool bWithLegacyPublicIncludePaths,
			bool bWithLegacyParentIncludePaths
			)
		{
			// Add this module's private include paths and definitions.
			IncludePaths.UnionWith(PrivateIncludePaths);

			// Find all the modules that are part of the public compile environment for this module.
			Dictionary<UEBuildModule, bool> ModuleToIncludePathsOnlyFlag = new Dictionary<UEBuildModule, bool>();
			FindModulesInPrivateCompileEnvironment(ModuleToIncludePathsOnlyFlag);

			// Now set up the compile environment for the modules in the original order that we encountered them
			foreach (UEBuildModule Module in ModuleToIncludePathsOnlyFlag.Keys)
			{
				Module.AddModuleToCompileEnvironment(this, Binary, IncludePaths, SystemIncludePaths, ModuleInterfacePaths, Definitions, AdditionalFrameworks, AutoRTFMExternalMappingFiles, AdditionalPrerequisites, bWithLegacyPublicIncludePaths, bWithLegacyParentIncludePaths);
			}
		}

		/// <summary>
		/// Expand path variables within the context of this module
		/// </summary>
		/// <param name="Path">Path to expand variables within</param>
		/// <param name="BinaryOutputDir">Directory containing the binary that links this module. May be mull.</param>
		/// <param name="TargetOutputDir">Directory containing the output executable. May be null.</param>
		/// <returns>The path with variables expanded</returns>
		public string ExpandPathVariables(string Path, DirectoryReference? BinaryOutputDir, DirectoryReference? TargetOutputDir)
		{
			if (Path.StartsWith("$(", StringComparison.Ordinal))
			{
				int StartIdx = 2;
				for (int EndIdx = StartIdx; EndIdx < Path.Length; EndIdx++)
				{
					if (Path[EndIdx] == ')')
					{
						if (MatchVariableName(Path, StartIdx, EndIdx, "EngineDir"))
						{
							Path = Unreal.EngineDirectory + Path.Substring(EndIdx + 1);
						}
						else if (MatchVariableName(Path, StartIdx, EndIdx, "ProjectDir"))
						{
							if (Rules.Target.ProjectFile == null)
							{
								Path = Unreal.EngineDirectory + Path.Substring(EndIdx + 1);
							}
							else
							{
								Path = Rules.Target.ProjectFile.Directory + Path.Substring(EndIdx + 1);
							}
						}
						else if (MatchVariableName(Path, StartIdx, EndIdx, "ModuleDir"))
						{
							Path = Rules.ModuleDirectory + Path.Substring(EndIdx + 1);
						}
						else if (MatchVariableName(Path, StartIdx, EndIdx, "PluginDir"))
						{
							Path = Rules.PluginDirectory + Path.Substring(EndIdx + 1);
						}
						else if (BinaryOutputDir != null && MatchVariableName(Path, StartIdx, EndIdx, "BinaryOutputDir"))
						{
							Path = BinaryOutputDir.FullName + Path.Substring(EndIdx + 1);
						}
						else if (TargetOutputDir != null && MatchVariableName(Path, StartIdx, EndIdx, "TargetOutputDir"))
						{
							Path = TargetOutputDir.FullName + Path.Substring(EndIdx + 1);
						}
						else
						{
							string Name = Path.Substring(StartIdx, EndIdx - StartIdx);
							string? Value = Environment.GetEnvironmentVariable(Name);
							if (String.IsNullOrEmpty(Value))
							{
								throw new BuildException("Environment variable '{0}' is not defined (referenced by {1})", Name, Rules.File);
							}
							Path = Value + Path.Substring(EndIdx + 1);
						}
						break;
					}
				}
			}
			return Path;
		}

		/// <summary>
		/// Match a variable name within a path
		/// </summary>
		/// <param name="Path">The path variable</param>
		/// <param name="StartIdx">Start index of the substring to match</param>
		/// <param name="EndIdx">End index of the substring to match</param>
		/// <param name="Name">Variable name to compare against</param>
		/// <returns>True if the variable name matches</returns>
		private static bool MatchVariableName(string Path, int StartIdx, int EndIdx, string Name)
		{
			return Name.Length == EndIdx - StartIdx && String.Compare(Path, StartIdx, Name, 0, EndIdx - StartIdx, StringComparison.Ordinal) == 0;
		}

		/// <summary>
		/// Expand path variables within the context of this module
		/// </summary>
		/// <param name="Paths">Path to expand variables within</param>
		/// <param name="BinaryDir">Directory containing the binary that links this module. May be mull.</param>
		/// <param name="ExeDir">Directory containing the output executable. May be null.</param>
		/// <returns>The path with variables expanded</returns>
		private IEnumerable<string> ExpandPathVariables(IEnumerable<string> Paths, DirectoryReference? BinaryDir, DirectoryReference? ExeDir)
		{
			foreach (string Path in Paths)
			{
				yield return ExpandPathVariables(Path, BinaryDir, ExeDir);
			}
		}

		/// <summary>
		/// Sets up the environment for linking any module that includes the public interface of this module.
		/// </summary>
		protected virtual void SetupPublicLinkEnvironment(
			UEBuildBinary? SourceBinary,
			List<FileReference> Libraries,
			List<DirectoryReference> SystemLibraryPaths,
			List<string> SystemLibraries,
			List<string> RuntimeLibraryPaths,
			List<string> Frameworks,
			List<string> WeakFrameworks,
			List<UEBuildFramework> AdditionalFrameworks,
			List<UEBuildBundleResource> AdditionalBundleResources,
			List<string> DelayLoadDLLs,
			List<UEBuildBinary> BinaryDependencies,
			Dictionary<string, HashSet<UnrealArch>> DependenciesToSkip,
			HashSet<UEBuildModule> VisitedModules,
			DirectoryReference ExeDir,
			CppRootPaths RootPaths,
			ILogger Logger
			)
		{
			// There may be circular dependencies in compile dependencies, so we need to avoid reentrance.
			if (VisitedModules.Add(this))
			{
				// Add this module's binary to the binary dependencies.
				if (Binary != null
					&& Binary != SourceBinary
					&& !BinaryDependencies.Contains(Binary))
				{
					BinaryDependencies.Add(Binary);
				}

				RootPaths.AddExtraPath(Rules.ExtraRootPath);

				// If this module belongs to a static library that we are not currently building, recursively add the link environment settings for all of its dependencies too.
				// Keep doing this until we reach a module that is not part of a static library (or external module, since they have no associated binary).
				// Static libraries do not contain the symbols for their dependencies, so we need to recursively gather them to be linked into other binary types.
				bool bIsBuildingAStaticLibrary = (SourceBinary != null && SourceBinary.Type == UEBuildBinaryType.StaticLibrary);
				bool bIsModuleBinaryAStaticLibrary = (Binary != null && Binary.Type == UEBuildBinaryType.StaticLibrary);
				if (!bIsBuildingAStaticLibrary && bIsModuleBinaryAStaticLibrary)
				{
					// Gather all dependencies and recursively call SetupPublicLinkEnvironmnet
					foreach (UEBuildModule DependencyModule in PrivateDependencyModules!.Concat(PublicDependencyModules!))
					{
						bool bIsExternalModule = (DependencyModule as UEBuildModuleExternal != null);
						bool bIsInStaticLibrary = (DependencyModule.Binary != null && DependencyModule.Binary.Type == UEBuildBinaryType.StaticLibrary);
						if (bIsExternalModule || bIsInStaticLibrary)
						{
							DependencyModule.SetupPublicLinkEnvironment(SourceBinary, Libraries, SystemLibraryPaths, SystemLibraries, RuntimeLibraryPaths, Frameworks, WeakFrameworks,
								AdditionalFrameworks, AdditionalBundleResources, DelayLoadDLLs, BinaryDependencies, DependenciesToSkip, VisitedModules, ExeDir, RootPaths, Logger);
						}
					}
				}

				// Add this module's public include library paths and additional libraries.
				Libraries.AddRange(PublicLibraries);
				SystemLibraryPaths.AddRange(PublicSystemLibraryPaths);
				SystemLibraries.AddRange(PublicSystemLibraries);
				RuntimeLibraryPaths.AddRange(ExpandPathVariables(Rules.PublicRuntimeLibraryPaths, SourceBinary?.OutputDir, ExeDir));
				Frameworks.AddRange(PublicFrameworks);
				WeakFrameworks.AddRange(PublicWeakFrameworks);
				AdditionalBundleResources.AddRange(PublicAdditionalBundleResources);
				AdditionalFrameworks.AddRange(PublicAdditionalFrameworks);
				DelayLoadDLLs.AddRange(PublicDelayLoadDLLs);

				// merge in to the outgoing dictionary
				lock (DependenciesToSkipPerArchitecture)
				{
					foreach (KeyValuePair<string, List<UnrealArch>> Pair in DependenciesToSkipPerArchitecture)
					{
						if (!DependenciesToSkip.ContainsKey(Pair.Key))
						{
							DependenciesToSkip[Pair.Key] = new HashSet<UnrealArch>();
						}
						DependenciesToSkip[Pair.Key] = DependenciesToSkip[Pair.Key].Union(Pair.Value).ToHashSet();
					}
				}
			}
		}

		/// <summary>
		/// Sets up the environment for linking this module.
		/// </summary>
		public virtual void SetupPrivateLinkEnvironment(
			UEBuildBinary SourceBinary,
			LinkEnvironment LinkEnvironment,
			List<UEBuildBinary> BinaryDependencies,
			HashSet<UEBuildModule> VisitedModules,
			DirectoryReference ExeDir,
			ILogger Logger
			)
		{
			// Add the private rpaths
			LinkEnvironment.RuntimeLibraryPaths.AddRange(ExpandPathVariables(Rules.PrivateRuntimeLibraryPaths, SourceBinary.OutputDir, ExeDir));

			// Allow the module's public dependencies to add library paths and additional libraries to the link environment.
			SetupPublicLinkEnvironment(SourceBinary, LinkEnvironment.Libraries, LinkEnvironment.SystemLibraryPaths, LinkEnvironment.SystemLibraries, LinkEnvironment.RuntimeLibraryPaths, LinkEnvironment.Frameworks, LinkEnvironment.WeakFrameworks,
				LinkEnvironment.AdditionalFrameworks, LinkEnvironment.AdditionalBundleResources, LinkEnvironment.DelayLoadDLLs, BinaryDependencies, LinkEnvironment.DependenciesToSkipPerArchitecture, VisitedModules, ExeDir, LinkEnvironment.RootPaths, Logger);

			// Also allow the module's public and private dependencies to modify the link environment.
			foreach (UEBuildModule DependencyModule in PrivateDependencyModules!.Concat(PublicDependencyModules!))
			{
				DependencyModule.SetupPublicLinkEnvironment(SourceBinary, LinkEnvironment.Libraries, LinkEnvironment.SystemLibraryPaths, LinkEnvironment.SystemLibraries, LinkEnvironment.RuntimeLibraryPaths, LinkEnvironment.Frameworks, LinkEnvironment.WeakFrameworks,
					LinkEnvironment.AdditionalFrameworks, LinkEnvironment.AdditionalBundleResources, LinkEnvironment.DelayLoadDLLs, BinaryDependencies, LinkEnvironment.DependenciesToSkipPerArchitecture, VisitedModules, ExeDir, LinkEnvironment.RootPaths, Logger);
			}

			// Add all the additional properties
			LinkEnvironment.AdditionalProperties.AddRange(Rules.AdditionalPropertiesForReceipt.Inner);

			// this is a link-time property that needs to be accumulated (if any modules contributing to this module is ignoring, all are ignoring)
			LinkEnvironment.bIgnoreUnresolvedSymbols |= Rules.bIgnoreUnresolvedSymbols;
		}

		/// <summary>
		/// Compiles the module, and returns a list of files output by the compiler.
		/// </summary>
		public virtual List<FileItem> Compile(ReadOnlyTargetRules Target, UEToolChain ToolChain, CppCompileEnvironment CompileEnvironment, ISourceFileWorkingSet WorkingSet, IActionGraphBuilder Graph, ILogger Logger)
		{
			if (Rules.TypeLibraries.Count == 0)
			{
				return [];
			}

			CppCompileEnvironment ModuleCompileEnvironment = new CppCompileEnvironment(CompileEnvironment);
			IEnumerable<DirectoryReference> AllIncludePaths = [.. ModuleCompileEnvironment.UserIncludePaths, .. ModuleCompileEnvironment.SystemIncludePaths];
			if (Target.ProjectFile != null && (Target.BuildEnvironment != TargetBuildEnvironment.Shared || AllIncludePaths.Any(x => x.IsUnderDirectory(Target.ProjectFile.Directory))))
			{
				ModuleCompileEnvironment.RootPaths[CppRootPathFolder.Project] = Target.ProjectFile.Directory;
			}

			ModuleCompileEnvironment.RootPaths.AddExtraPath(Rules.ExtraRootPath);
			ModuleCompileEnvironment.RootPaths.AddExtraPath(("System32", Environment.SystemDirectory)); // Needed for cmd.exe

			// Generate type libraries for Windows
			foreach (ModuleRules.TypeLibrary TypeLibrary in Rules.TypeLibraries)
			{
				FileReference OutputFile = FileReference.Combine(IntermediateDirectory, TypeLibrary.Header);
				FileReference? OutputInclude = TypeLibrary.Include != null ? FileReference.Combine(IntermediateDirectory, TypeLibrary.Include) : null;
				ToolChain.GenerateTypeLibraryHeader(ModuleCompileEnvironment, TypeLibrary, OutputFile, OutputInclude, Graph);
			}

			return [];
		}

		public IEnumerable<FileItem> CopyDebuggerVisualizers(UEToolChain ToolChain, IActionGraphBuilder Graph)
		{
			List<FileItem> Results = new();
			foreach (FileItem NatvisSourceFile in NatvisFiles)
			{
				FileItem? Item = ToolChain.CopyDebuggerVisualizer(NatvisSourceFile, IntermediateDirectory, Graph);
				if (Item != null)
				{
					Results.Add(Item);
				}
			}
			return Results;
		}

		public void LinkDebuggerVisualizers(List<FileItem> OutFiles, UEToolChain ToolChain)
		{
			foreach (FileItem NatvisSourceFile in NatvisFiles)
			{
				FileItem? Item = ToolChain.LinkDebuggerVisualizer(NatvisSourceFile, IntermediateDirectory);
				if (Item != null)
				{
					OutFiles.Add(Item);
				}
			}
		}

		// Object interface.
		public override string ToString()
		{
			return Name;
		}

		/// <summary>
		/// Finds the modules referenced by this module which have not yet been bound to a binary
		/// </summary>
		/// <returns>List of unbound modules</returns>
		public List<UEBuildModule> GetUnboundReferences()
		{
			List<UEBuildModule> Modules = new List<UEBuildModule>();
			Modules.AddRange(PrivateDependencyModules!.Where(x => x.Binary == null));
			Modules.AddRange(PublicDependencyModules!.Where(x => x.Binary == null));
			return Modules;
		}

		/// <summary>
		/// Gets all of the modules referenced by this module
		/// </summary>
		/// <param name="ReferencedModules">Hash of all referenced modules with their addition index.</param>
		/// <param name="IgnoreReferencedModules">Hashset used to ignore modules which are already added to the list</param>
		/// <param name="bIncludeDynamicallyLoaded">True if dynamically loaded modules (and all of their dependent modules) should be included.</param>
		/// <param name="bForceCircular">True if circular dependencies should be processed</param>
		/// <param name="bOnlyDirectDependencies">True to return only this module's direct dependencies</param>
		public void GetAllDependencyModules(List<UEBuildModule> ReferencedModules, HashSet<UEBuildModule> IgnoreReferencedModules, bool bIncludeDynamicallyLoaded, bool bForceCircular, bool bOnlyDirectDependencies)
		{
			Stack<UEBuildModule> stack = new();
			stack.Push(this);

			void QueueReferencedModules(List<UEBuildModule> Modules)
			{
				foreach (UEBuildModule DependencyModule in Modules)
				{
					// Don't follow circular back-references!
					if (bForceCircular || !HasCircularDependencyOn(DependencyModule))
					{
						if (IgnoreReferencedModules.Add(DependencyModule))
						{
							if (!bOnlyDirectDependencies)
							{
								stack.Push(DependencyModule);
							}

							ReferencedModules.Add(DependencyModule);
						}
					}
				}
			}

			while (stack.Count != 0)
			{
				UEBuildModule module = stack.Pop();

				QueueReferencedModules(module.PrivateDependencyModules!);
				QueueReferencedModules(module.PublicDependencyModules!);
				if (bIncludeDynamicallyLoaded)
				{
					QueueReferencedModules(module.DynamicallyLoadedModules!);
				}
			}
		}

		/// <summary>
		/// Gathers all the module dependencies a PCH would have
		/// </summary>
		/// <param name="outReferenced">The collected modules</param>
		/// <param name="bIncludePrivateModules">Whether to include private modules.</param>
		public void GetAllDependencyModulesForPCH(HashSet<UEBuildModule> outReferenced, bool bIncludePrivateModules)
		{
			using Pooled<HashSet<UEBuildModule>> rented = HashSetPool<UEBuildModule>.Rent();
			HashSet<UEBuildModule> IgnoreReferencedModules = rented.Value;
			IgnoreReferencedModules.Add(this);
			InternalGetAllDependencyModulesForPCH(outReferenced, IgnoreReferencedModules, bIncludePrivateModules);
		}

		/// <summary>
		/// Internal function that gathers all the module dependencies a PCH would have
		/// </summary>
		/// <param name="ReferencedModules">Hash of all referenced modules with their addition index.</param>
		/// <param name="IgnoreReferencedModules">Hashset used to ignore modules which are already added to the list</param>
		/// <param name="bIncludePrivateModules">Whether to include private modules.</param>
		private void InternalGetAllDependencyModulesForPCH(HashSet<UEBuildModule> ReferencedModules, HashSet<UEBuildModule> IgnoreReferencedModules, bool bIncludePrivateModules)
		{
			//HashSet<UEBuildModule> AllDependencyModules = new(
			//	((bIncludePrivateModules && PrivateDependencyModules != null) ? PrivateDependencyModules.Count : 0) +
			//	(PublicDependencyModules != null ? PublicDependencyModules.Count : 0) +
			//	(PublicIncludePathModules != null ? PublicIncludePathModules!.Count : 0)
			//	);

			using Pooled<HashSet<UEBuildModule>> rented = HashSetPool<UEBuildModule>.Rent(((bIncludePrivateModules && PrivateDependencyModules != null) ? PrivateDependencyModules.Count : 0) +
				(PublicDependencyModules != null ? PublicDependencyModules.Count : 0) +
				(PublicIncludePathModules != null ? PublicIncludePathModules!.Count : 0));
			HashSet<UEBuildModule> AllDependencyModules = rented.Value;

			if (bIncludePrivateModules && PrivateDependencyModules != null)
			{
				AllDependencyModules.UnionWith(PrivateDependencyModules);
			}
			if (PublicDependencyModules != null)
			{
				AllDependencyModules.UnionWith(PublicDependencyModules!);
			}
			if (PublicIncludePathModules != null)
			{
				AllDependencyModules.UnionWith(PublicIncludePathModules);
			}

			foreach (UEBuildModule DependencyModule in AllDependencyModules)
			{
				// Don't follow circular back-references!
				if (!HasCircularDependencyOn(DependencyModule))
				{
					if (IgnoreReferencedModules.Add(DependencyModule))
					{
						// Recurse into dependent modules first
						DependencyModule.InternalGetAllDependencyModulesForPCH(ReferencedModules, IgnoreReferencedModules, false);

						ReferencedModules.Add(DependencyModule);
					}
				}
			}
		}

		public delegate UEBuildModule CreateModuleDelegate(string Name, string ReferenceChain);

		/// <summary>
		/// Creates all the modules required for this target
		/// </summary>
		/// <param name="CreateModule">Delegate to create a module with a given name</param>
		/// <param name="ReferenceChain">Chain of references before reaching this module</param>
		/// <param name="Logger">Logger for output</param>
		public void RecursivelyCreateModules(CreateModuleDelegate CreateModule, StringBuilder ReferenceChain, ILogger Logger)
		{
			// Name of this reference
			string ThisRefName = (RulesFile == null) ? Name : RulesFile.GetFileName();

			// Set the reference chain for anything referenced by this module
			//string NextReferenceChain = String.Format("{0} -> {1}", ReferenceChain, ThisRefName);
			int OriginalReferenceChainLength = ReferenceChain.Length;
			ReferenceChain.Append(" -> ").Append(ThisRefName);

			// Check if this module is invalid for the current target, based on Module attributes
			if (Rules.CppCompileWarningSettings.ModuleUnsupportedWarningLevel != WarningLevel.Off)
			{
				if (!ModuleRules.IsValidForTarget(Rules, Rules.Target, out string? InvalidReason))
				{
					if (Rules.CppCompileWarningSettings.ModuleUnsupportedWarningLevel == WarningLevel.Warning)
					{
						Logger.LogWarning("Warning: Referenced module '{Module}' not supported '{InvalidReason}' via {ReferenceChain}", ThisRefName, InvalidReason, ReferenceChain.ToString());
					}
					else if (Rules.CppCompileWarningSettings.ModuleUnsupportedWarningLevel == WarningLevel.Error)
					{
						Logger.LogError("Error: Referenced module '{Module}' not supported '{InvalidReason}' via {ReferenceChain}", ThisRefName, InvalidReason, ReferenceChain.ToString());
					}
				}
			}

			// Check if this module is invalid for the current target, based on Plugin attributes
			if (Rules.CppCompileWarningSettings.PluginModuleUnsupportedWarningLevel != WarningLevel.Off && Rules.Plugin != null)
			{
				if (!Rules.Plugin.Descriptor.IsModuleCompiledInConfiguration(Rules.Name, Rules.Target.Platform, Rules.Target.Configuration, Rules.Target.Name, Rules.Target.Type, Rules.Target.bBuildDeveloperTools, Rules.Target.bBuildRequiresCookedData, Rules.Target.Architectures, out string? InvalidReason))
				{
					if (Rules.CppCompileWarningSettings.PluginModuleUnsupportedWarningLevel == WarningLevel.Warning)
					{
						Logger.LogWarning("Warning: Referenced plugin '{Plugin}' module '{Module}' not supported '{InvalidReason}' via {ReferenceChain}", Rules.Plugin.File.GetFileName(), ThisRefName, InvalidReason, ReferenceChain.ToString());
					}
					else if (Rules.CppCompileWarningSettings.PluginModuleUnsupportedWarningLevel == WarningLevel.Error)
					{
						Logger.LogError("Error: Referenced plugin '{Plugin}' module '{Module}' not supported '{InvalidReason}' via {ReferenceChain}", Rules.Plugin.File.GetFileName(), ThisRefName, InvalidReason, ReferenceChain.ToString());
					}
				}
			}

			// Create all the referenced modules. This path can be recursive, so we check against PublicDependencyModules to ensure we don't recurse through the 
			// same module twice (it produces better errors if something fails).
			if (PublicDependencyModules == null)
			{
				// Log dependencies if required
				if (Logger.IsEnabled((LogLevel)LogEventType.VeryVerbose))
				{
					Logger.LogTrace("Module {Name} dependencies:", Name);
					LogDependencyNameList("Public:", Rules.PublicDependencyModuleNames, Logger);
					LogDependencyNameList("Private:", Rules.PrivateDependencyModuleNames, Logger);
					LogDependencyNameList("Dynamic:", Rules.DynamicallyLoadedModuleNames, Logger);
					LogDependencyNameList("Public Include Paths:", Rules.PublicIncludePathModuleNames, Logger);
					LogDependencyNameList("Private Include Paths:", Rules.PrivateIncludePathModuleNames, Logger);
				}

				ReferenceStackParentModules ??= new(Comparer<UEBuildModule>.Create(static (a, b) => String.Compare(a.Name, b.Name, StringComparison.Ordinal)));

				// Create all the dependency modules - pass through the reference stack so we can check for cycles
				RecursivelyCreateModulesByName(Rules.PublicDependencyModuleNames, ref PublicDependencyModules, ref bDependsOnVerse, CreateModule, ReferenceChain, Logger);
				if (Rules.Target.IsTestTarget)
				{
					// Move the test runner dependency to last position to give it the opportunity to build its special dependencies such as CoreUObject, ApplicationCore etc.
					if (Rules.PrivateDependencyModuleNames.Contains("LowLevelTestsRunner"))
					{
						Rules.PrivateDependencyModuleNames.Remove("LowLevelTestsRunner");
						Rules.PrivateDependencyModuleNames.Add("LowLevelTestsRunner");
					}
				}
				RecursivelyCreateModulesByName(Rules.PrivateDependencyModuleNames, ref PrivateDependencyModules, ref bDependsOnVerse, CreateModule, ReferenceChain, Logger);

				// Recursively create all the public include path modules
				HashSet<string> PublicIncludePathModuleNames = new();
				PublicIncludePathModuleNames.UnionWith(Rules.PublicIncludePathModuleNames!);
				PublicIncludePathModuleNames.UnionWith(Rules.PublicDependencyModuleNames!);
				RecursivelyCreateIncludePathModulesByName(PublicIncludePathModuleNames, ref PublicIncludePathModules, ref bDependsOnVerse, CreateModule, ReferenceChain);

				// Create the private include path modules
				RecursivelyCreateIncludePathModulesByName(Rules.PrivateIncludePathModuleNames, ref PrivateIncludePathModules, ref bDependsOnVerse, CreateModule, ReferenceChain);

				// Dynamic loads aren't considered a reference chain so start with an empty stack
				RecursivelyCreateModulesByName(Rules.DynamicallyLoadedModuleNames, ref DynamicallyLoadedModules, ref bDependsOnVerse, CreateModule, ReferenceChain, Logger);

				PublicDependencyModules?.ForEach(x => x.ReferenceStackParentModules?.Add(this));
				PrivateDependencyModules?.ForEach(x => x.ReferenceStackParentModules?.Add(this));
			}

			// Reset reference chain
			ReferenceChain.Length = OriginalReferenceChainLength;
		}

		record CircularReference
		{
			public List<UEBuildModule> ReferenceStack { get; }
			public Dictionary<UEBuildModule, UEBuildModule> GuiltyModules { get; } = new();
			public bool IsError => !GuiltyModules.Any();

			public CircularReference(List<UEBuildModule> referenceStack) => ReferenceStack = referenceStack;

			public void AddGuiltyModule(UEBuildModule guiltyModule, UEBuildModule victimModule) => GuiltyModules.TryAdd(guiltyModule, victimModule);
		}

		/// <summary>
		/// Public entry point to validate a module
		/// </summary>
		/// <param name="logger"></param>
		public virtual bool ValidateModule(ILogger logger)
		{
			bool anyErrors = false;

			// Report any [Obsolete("...")] messages from referenced modules.
			IEnumerable<ObsoleteAttribute> obsoleteAttributes = Rules.GetType().GetCustomAttributes<ObsoleteAttribute>();
			if (obsoleteAttributes.Any())
			{
				logger.LogWarning(KnownLogEvents.Validation, "Referenced Module '{Name}' is obsolete: '{Message}'", Name, String.Join(", ", obsoleteAttributes.Select(x => x.Message ?? "<unknown reason>")));
			}

			// Check if this module is invalid for the current target, based on Module attributes
			if (Rules.CppCompileWarningSettings.ModuleUnsupportedWarningLevel == WarningLevel.Error && !ModuleRules.IsValidForTarget(Rules.GetType(), Rules.Target, out string? _))
			{
				anyErrors = true;
			}
			if (Rules.CppCompileWarningSettings.PluginModuleUnsupportedWarningLevel == WarningLevel.Error && Rules.Plugin != null)
			{
				if (!Rules.Plugin.Descriptor.IsModuleCompiledInConfiguration(Rules.Name, Rules.Target.Platform, Rules.Target.Configuration, Rules.Target.Name, Rules.Target.Type, Rules.Target.bBuildDeveloperTools, Rules.Target.bBuildRequiresCookedData, Rules.Target.Architectures, out string? _))
				{
					anyErrors = true;
				}
			}

			anyErrors = ValidateCircularReferences(logger) || anyErrors;

			return anyErrors;
		}

		/// <summary>
		/// Validate a module for circular references
		/// </summary>
		/// <param name="logger"></param>
		/// <returns>If any circular references are found</returns>
		bool ValidateCircularReferences(ILogger logger)
		{
			List<CircularReference> circularReferences = [];

			using Pooled<HashSet<UEBuildModule>> rented = HashSetPool<UEBuildModule>.Rent();
			HashSet<UEBuildModule> processed = rented.Value;
			CheckForCircularReferences(this, processed, [], circularReferences);

			IEnumerable<CircularReference> allowed = circularReferences.Where(x => !x.IsError).OrderBy(x => x.ReferenceStack.Count);
			IEnumerable<CircularReference> errors = circularReferences.Where(x => x.IsError).OrderBy(x => x.ReferenceStack.Count);

			if (errors.Any())
			{
				logger.LogInformation(KnownLogEvents.Validation, "Circular dependency for '{RulesFile}' detected:", RulesFile.MakeRelativeTo(Unreal.EngineDirectory));
				foreach (CircularReference circularReference in errors)
				{
					logger.LogInformation(KnownLogEvents.Validation, "* {ReferenceStack}", String.Join(" -> ", circularReference.ReferenceStack));
				}
				foreach (FileReference rulesFile in errors.SelectMany(x => x.ReferenceStack).Select(x => x.RulesFile).Distinct())
				{
					logger.LogError(KnownLogEvents.CircularDependency, "Circular dependency in '{Module}' possibly due to '{RulesFile}'", Name, rulesFile);
				}
				logger.LogInformation(KnownLogEvents.Validation, "Break this loop by moving dependencies into a separate module or using Private/PublicIncludePathModuleNames to reference declarations");
			}

			if (allowed.Any())
			{
				logger.LogTrace(KnownLogEvents.Validation, "Allowed circular dependency for '{RulesFile}' detected:", RulesFile.MakeRelativeTo(Unreal.EngineDirectory));
				foreach (CircularReference circularReference in allowed)
				{
					logger.LogTrace(KnownLogEvents.Validation, "* {ReferenceStack} (allowed via {GuiltyModules})", String.Join(" -> ", circularReference.ReferenceStack), String.Join(", ", circularReference.GuiltyModules.Select(x => $"{x.Key} -> {x.Value}")));
				}
			}
			return errors.Any();
		}

		/// <summary>
		/// Recursively check a module for circular references
		/// </summary>
		/// <param name="module">The module currently being checked</param>
		/// <param name="processed">Set of modules already processed</param>
		/// <param name="parentStack">The current parent reference stack (in reverse order)</param>
		/// <param name="circularReferences">List of found circular reference stacks</param>
		void CheckForCircularReferences(UEBuildModule module, HashSet<UEBuildModule> processed, List<UEBuildModule> parentStack, List<CircularReference> circularReferences)
		{
			// If this module has no references or has been processed already, nothing to do
			if (ReferenceStackParentModules == null || !processed.Add(this))
			{
				return;
			}

			// Check for a circular reference
			if (ReferenceStackParentModules.Contains(module))
			{
				List<UEBuildModule> referenceStack = new(parentStack);
				referenceStack.Add(this);
				referenceStack.Add(module);
				referenceStack.Reverse();

				CircularReference circularReference = new(referenceStack);

				for (int i = 0; i < referenceStack.Count - 1; i++)
				{
					UEBuildModule referringModule = referenceStack.ElementAt(i);
					UEBuildModule targetModule = referenceStack.ElementAt(i + 1);

					// If module has confessed its guilt this is not an error
					if (referringModule.HasCircularDependencyOn(targetModule))
					{
						circularReference.AddGuiltyModule(referringModule, targetModule);
					}
					else if (targetModule.HasCircularDependencyOn(referringModule))
					{
						circularReference.AddGuiltyModule(targetModule, referringModule);
					}
				}
				circularReferences.Add(circularReference);
				return;
			}

			// Push this module to the stack
			parentStack.Add(this);

			// Check very module that references this module until all are processed
			foreach (UEBuildModule ParentModule in ReferenceStackParentModules)
			{
				ParentModule.CheckForCircularReferences(module, processed, parentStack, circularReferences);
			}

			// Pop this module from the stack
			parentStack.RemoveAt(parentStack.Count - 1);
		}

		private static void LogDependencyNameList(string Title, List<string> DependencyNameList, ILogger logger)
		{
			logger.LogTrace("  {Title}", Title);
			foreach (string name in DependencyNameList)
			{
				logger.LogTrace("    {Name}", name);
			}
		}

		private static void RecursivelyCreateModulesByName(List<string> ModuleNames, ref List<UEBuildModule>? Modules, ref bool bDependsOnVerse, CreateModuleDelegate CreateModule, StringBuilder ReferenceChain, ILogger Logger)
		{
			// Check whether the module list is already set. We set this immediately (via the ref) to avoid infinite recursion.
			if (Modules == null)
			{
				Modules = new List<UEBuildModule>();
				for (int i = 0; i < ModuleNames.Count; i++)
				{
					string ModuleName = ModuleNames[i];
					UEBuildModule Module = CreateModule(ModuleName, ReferenceChain.ToString());
					if (!Modules.Contains(Module))
					{
						Module.RecursivelyCreateModules(CreateModule, ReferenceChain, Logger);
						Modules.Add(Module);
						bDependsOnVerse |= Module.bDependsOnVerse;
					}
				}
			}
		}

		private static void RecursivelyCreateIncludePathModulesByName(IEnumerable<string> ModuleNames, ref List<UEBuildModule>? Modules, ref bool bDependsOnVerse, CreateModuleDelegate CreateModule, StringBuilder ReferenceChain)
		{
			// Check whether the module list is already set. We set this immediately (via the ref) to avoid infinite recursion.
			if (Modules == null)
			{
				string ReferenceChainString = ReferenceChain.ToString();
				Modules = new List<UEBuildModule>();
				foreach (string ModuleName in ModuleNames)
				{
					UEBuildModule Module = CreateModule(ModuleName, ReferenceChainString);
					if (!Modules.Contains(Module))
					{
						// Name of this reference
						string ModuleRefName = (Module.RulesFile == null) ? Module.Name : Module.RulesFile.GetFileName();
						// Set the reference chain for anything referenced by this module
						int OriginalReferenceChainLength = ReferenceChain.Length;
						ReferenceChain.Append(" -> ").Append(ModuleRefName).Append(" (public include)");

						HashSet<string> PublicIncludePathModuleNames = new();
						PublicIncludePathModuleNames.UnionWith(Module.Rules.PublicIncludePathModuleNames);
						PublicIncludePathModuleNames.UnionWith(Module.Rules.PublicDependencyModuleNames);
						RecursivelyCreateIncludePathModulesByName(PublicIncludePathModuleNames, ref Module.PublicIncludePathModules, ref Module.bDependsOnVerse, CreateModule, ReferenceChain);
						Modules.Add(Module);
						bDependsOnVerse |= Module.bDependsOnVerse;

						ReferenceChain.Length = OriginalReferenceChainLength;
					}
				}
			}
		}

		/// <summary>
		/// Returns valueless API defines (like MODULE_API)
		/// </summary>
		public IEnumerable<string> GetEmptyApiMacros()
		{
			if (Rules.Type == ModuleRules.ModuleType.CPlusPlus)
			{
				return new[] { ModuleApiDefine + "=" };
			}

			return Array.Empty<string>();
		}

		/// <summary>
		/// Write information about this binary to a JSON file
		/// </summary>
		/// <param name="BinaryOutputDir">The output directory for the binary containing this module</param>
		/// <param name="TargetOutputDir">The output directory for the target executable</param>
		/// <param name="Writer">Writer for this binary's data</param>
		public virtual void ExportJson(DirectoryReference? BinaryOutputDir, DirectoryReference? TargetOutputDir, JsonWriter Writer)
		{
			Writer.WriteValue("Name", Name);
			Writer.WriteValue("Type", Rules.Type.ToString());
			Writer.WriteValue("Directory", ModuleDirectory.FullName);
			Writer.WriteValue("Rules", RulesFile.FullName);
			Writer.WriteValue("PCHUsage", Rules.PCHUsage.ToString());

			ExportJsonStringArray(Writer, "ForceIncludeModules", Rules.ForceIncludeFiles);

			if (Rules.PrivatePCHHeaderFile != null)
			{
				Writer.WriteValue("PrivatePCH", FileReference.Combine(ModuleDirectory, Rules.PrivatePCHHeaderFile).FullName);
			}

			if (Rules.SharedPCHHeaderFile != null)
			{
				Writer.WriteValue("SharedPCH", FileReference.Combine(ModuleDirectory, Rules.SharedPCHHeaderFile).FullName);
			}
			Writer.WriteValue("ChainSharedPCH", Rules.Target.bChainPCHs);

			ExportJsonModuleArray(Writer, "PublicDependencyModules", PublicDependencyModules);
			ExportJsonModuleArray(Writer, "PublicIncludePathModules", PublicIncludePathModules);
			ExportJsonModuleArray(Writer, "PrivateDependencyModules", PrivateDependencyModules);
			ExportJsonModuleArray(Writer, "PrivateIncludePathModules", PrivateIncludePathModules);
			ExportJsonModuleArray(Writer, "DynamicallyLoadedModules", DynamicallyLoadedModules);

			ExportJsonStringArray(Writer, "PublicSystemIncludePaths", PublicSystemIncludePaths.Select(x => x.FullName));
			ExportJsonStringArray(Writer, "PublicIncludePaths", PublicIncludePaths.Select(x => x.FullName));
			ExportJsonStringArray(Writer, "InternalIncludePaths", PublicIncludePaths.Select(x => x.FullName));
			ExportJsonStringArray(Writer, "PrivateIncludePaths", PrivateIncludePaths.Select(x => x.FullName));
			ExportJsonStringArray(Writer, "PublicLibraries", PublicLibraries.Select(x => x.FullName));
			ExportJsonStringArray(Writer, "PublicSystemLibraries", PublicSystemLibraries);
			ExportJsonStringArray(Writer, "PublicSystemLibraryPaths", PublicSystemLibraryPaths.Select(x => x.FullName));
			ExportJsonStringArray(Writer, "PublicFrameworks", PublicFrameworks);
			ExportJsonStringArray(Writer, "PublicWeakFrameworks", PublicWeakFrameworks);
			ExportJsonStringArray(Writer, "PublicDelayLoadDLLs", PublicDelayLoadDLLs);
			ExportJsonStringArray(Writer, "PublicDefinitions", PublicDefinitions);

			ExportJsonStringArray(Writer, "AutoRTFMExternalMappingFiles", PublicAutoRTFMExternalMappingFiles.Select(x => x.FullName));

			Writer.WriteArrayStart("CircularlyReferencedModules");
			foreach (string ModuleName in Rules.CircularlyReferencedDependentModules)
			{
				Writer.WriteValue(ModuleName);
			}
			Writer.WriteArrayEnd();

			// Don't add runtime dependencies for modules that aren't being linked in. They may reference BinaryOutputDir, which is invalid.
			if (Binary != null)
			{
				Writer.WriteArrayStart("RuntimeDependencies");
				foreach (ModuleRules.RuntimeDependency RuntimeDependency in Rules.RuntimeDependencies.Inner)
				{
					Writer.WriteObjectStart();
					Writer.WriteValue("Path", ExpandPathVariables(RuntimeDependency.Path, BinaryOutputDir, TargetOutputDir));
					if (RuntimeDependency.SourcePath != null)
					{
						Writer.WriteValue("SourcePath", ExpandPathVariables(RuntimeDependency.SourcePath, BinaryOutputDir, TargetOutputDir));
					}
					Writer.WriteValue("Type", RuntimeDependency.Type.ToString());
					Writer.WriteObjectEnd();
				}
				Writer.WriteArrayEnd();
			}
		}

		/// <summary>
		/// Write an array of module names to a JSON writer
		/// </summary>
		/// <param name="Writer">Writer for the array data</param>
		/// <param name="ArrayName">Name of the array property</param>
		/// <param name="Modules">Sequence of modules to write. May be null.</param>
		private static void ExportJsonModuleArray(JsonWriter Writer, string ArrayName, IEnumerable<UEBuildModule>? Modules)
		{
			Writer.WriteArrayStart(ArrayName);
			if (Modules != null)
			{
				foreach (UEBuildModule Module in Modules)
				{
					Writer.WriteValue(Module.Name);
				}
			}
			Writer.WriteArrayEnd();
		}

		/// <summary>
		/// Write an array of strings to a JSON writer
		/// </summary>
		/// <param name="Writer">Writer for the array data</param>
		/// <param name="ArrayName">Name of the array property</param>
		/// <param name="Strings">Sequence of strings to write. May be null.</param>
		private static void ExportJsonStringArray(JsonWriter Writer, string ArrayName, IEnumerable<string>? Strings)
		{
			Writer.WriteArrayStart(ArrayName);
			if (Strings != null)
			{
				foreach (string String in Strings)
				{
					Writer.WriteValue(String);
				}
			}
			Writer.WriteArrayEnd();
		}

		public static void CacheDependenciesForStableTopologicalSort(IEnumerable<UEBuildModule> Modules)
		{
			Parallel2.ForEach(Modules, module =>
			{
				if (module.DependenciesForStableTopologicalSort == null && module.PrivateDependencyModules != null)
				{
					HashSet<UEBuildModule> Deps = [];
					module.GetAllDependencyModules([], Deps, true, true, false);
					module.DependenciesForStableTopologicalSort = Deps;
				}
			});
		}

		/// <summary>
		/// Returns a copy of Nodes sorted by dependency.  Independent or circularly-dependent nodes should
		/// remain in their same relative order within the original Nodes sequence.
		/// </summary>
		/// <param name="NodeList">The list of nodes to sort.</param>
		public static UEBuildModule[] StableTopologicalSort(List<UEBuildModule> NodeList)
		{
			int nodeCount = NodeList.Count;
			if (nodeCount <= 1)
			{
				return [.. NodeList];
			}

			// Use default equality (same as the old code)
			Dictionary<UEBuildModule, int> indexOf = new(nodeCount);
			for (int i = 0; i < nodeCount; i++)
			{
				indexOf[NodeList[i]] = i;
			}

			int words = (nodeCount + 63) >> 6;
			ulong[] depBits = new ulong[(long)nodeCount * words];

			for (int i = 0; i < nodeCount; i++)
			{
				int rowBase = i * words;
				foreach (UEBuildModule d in NodeList[i].DependenciesForStableTopologicalSort!)
				{
					if (indexOf.TryGetValue(d, out int j) && j != i)
					{
						depBits[rowBase + (j >> 6)] |= 1UL << (j & 63);
					}
				}
			}

			List<int> outIdx = new(nodeCount);
			for (int i = 0; i < nodeCount; i++)
			{
				int insertAt = outIdx.Count;

				// Precompute these once per i
				int baseI = i * words;
				int wI = i >> 6;
				ulong maskI = 1UL << (i & 63);

				for (int pos = 0; pos < outIdx.Count; pos++)
				{
					int j = outIdx[pos];
					int baseJ = j * words;

					// earlierDependsOnNode? (does j depend on i)
					if ((depBits[baseJ + wI] & maskI) != 0)
					{
						// Only if needed, check mutual dep (does i depend on j)
						int wJ = j >> 6;
						ulong maskJ = 1UL << (j & 63);

						if ((depBits[baseI + wJ] & maskJ) == 0)
						{
							insertAt = pos;
							break;
						}
					}
				}
				outIdx.Insert(insertAt, i);
			}

			UEBuildModule[] ordered = new UEBuildModule[nodeCount];
			int counter = 0;
			foreach (int idx in outIdx)
			{
				ordered[counter++] = NodeList[idx];
			}

			return ordered;
		}
	};
}
