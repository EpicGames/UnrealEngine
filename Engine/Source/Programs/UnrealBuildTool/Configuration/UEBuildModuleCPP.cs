// Copyright Epic Games, Inc. All Rights Reserved.

//#define LOG_PCH_PARSING

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using EpicGames.Core;
using Microsoft.CodeAnalysis;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// A module that is compiled from C++ code.
	/// </summary>
	class UEBuildModuleCPP : UEBuildModule
	{
		/// <summary>
		/// Stores a list of all source files, of different types
		/// </summary>
		public class InputFileCollection
		{
			public readonly List<FileItem> HeaderFiles = new List<FileItem>();
			public readonly List<FileItem> ISPCHeaderFiles = new List<FileItem>();

			public readonly List<FileItem> IXXFiles = new List<FileItem>();
			public readonly List<FileItem> CPPFiles = new List<FileItem>();
			public readonly List<FileItem> CFiles = new List<FileItem>();
			public readonly List<FileItem> CCFiles = new List<FileItem>();
			public readonly List<FileItem> CXXFiles = new List<FileItem>();
			public readonly List<FileItem> MFiles = new List<FileItem>();
			public readonly List<FileItem> MMFiles = new List<FileItem>();
			public readonly List<FileItem> SwiftFiles = new List<FileItem>();
			public readonly List<FileItem> RCFiles = new List<FileItem>();
			public readonly List<FileItem> ISPCFiles = new List<FileItem>();
			public readonly List<FileItem> ProtoFiles = new List<FileItem>();

			public IEnumerable<FileItem> AllHeaderFiles => HeaderFiles.Concat(ISPCHeaderFiles);
			public IEnumerable<FileItem> AllCompiledFiles => IXXFiles.Concat(CPPFiles).Concat(CFiles).Concat(CCFiles).Concat(CXXFiles).Concat(MFiles).Concat(MMFiles).Concat(SwiftFiles).Concat(RCFiles).Concat(ISPCFiles);
		}

		/// <summary>
		/// Instance where a CPP file doesn't include the expected header as the first include.
		/// </summary>
		public readonly struct InvalidIncludeDirective
		{
			public FileReference CppFile { get; }
			public FileReference HeaderFile { get; }

			public InvalidIncludeDirective(FileReference cppFile, FileReference headerFile)
			{
				CppFile = cppFile;
				HeaderFile = headerFile;
			}
		}

		/// <summary>
		/// If UHT found any associated UObjects in this module's source files
		/// </summary>
		public bool bHasUObjects;

		/// <summary>
		/// If any proto files were found in the module's source files
		/// </summary>
		public bool bHasProto;

		/// <summary>
		/// The directory for this module's generated code
		/// </summary>
		public readonly DirectoryReference? GeneratedCodeDirectory;

		/// <summary>
		/// The directory for this module's generated UHT code
		/// </summary>
		public readonly DirectoryReference? GeneratedCodeDirectoryUHT;

		/// <summary>
		/// The directory for this module's generated VNI code
		/// </summary>
		public readonly DirectoryReference? GeneratedCodeDirectoryVNI;

		/// <summary>
		/// Module directory. This is just here to reduce number of string creation since it is used in 
		/// </summary>
		private readonly DirectoryReference ModuleInterfaceDir;

		/// <summary>
		/// Global override to force all include paths to be always added
		/// </summary>
		public static bool bForceAddGeneratedCodeIncludePath;

		/// <summary>
		/// Paths containing *.gen.cpp files for this module.  If this is null then this module doesn't have any generated code.
		/// </summary>
		public List<string>? GeneratedCppDirectories;

		/// <summary>
		/// List of invalid include directives. These are buffered up and output before we start compiling.
		/// </summary>
		public List<InvalidIncludeDirective>? InvalidIncludeDirectives;

		/// <summary>
		/// Set of source directories referenced during a build
		/// </summary>
		HashSet<DirectoryReference>? SourceDirectories;

		/// <summary>
		/// Verse source code directory associated with this module
		/// </summary>
		readonly DirectoryItem? AssociatedVerseDirectory;

		/// <summary>
		/// Create PerModuleInline source file. Defaults to PrimaryModule in a binary but could be another one if primary module does not depend on core
		/// </summary>
		public bool bCreatePerModuleFile;

		/// <summary>
		/// The Verse source code directory associated with this module if any
		/// </summary>
		public override DirectoryReference? VerseDirectory => AssociatedVerseDirectory?.Location;

		/// <inheritdoc/>
		public override void GetBuildProducts(UEToolChain ToolChain, Dictionary<FileReference, BuildProductType> BuildProducts)
		{
			base.GetBuildProducts(ToolChain, BuildProducts);

			if (Rules.bPrecompile)
			{
				DirectoryItem? GeneratedCodeDirectoryItem = GeneratedCodeDirectory != null ? DirectoryItem.GetItemByDirectoryReference(GeneratedCodeDirectory) : null;
				if (GeneratedCodeDirectoryItem?.Exists == true)
				{
					Queue<DirectoryItem> DirQueue = new();
					DirQueue.Enqueue(GeneratedCodeDirectoryItem);
					while (DirQueue.TryDequeue(out DirectoryItem? dir))
					{
						foreach (FileItem GeneratedCodeFile in dir.EnumerateFiles())
						{
							// Exclude timestamp files, since they're always updated and cause collisions between builds
							if (!GeneratedCodeFile.Name.Equals("Timestamp", StringComparison.OrdinalIgnoreCase) && !GeneratedCodeFile.HasExtension(".cpp"))
							{
								BuildProducts.Add(GeneratedCodeFile.Location, BuildProductType.BuildResource);
							}
						}
						foreach (DirectoryItem subdir in dir.EnumerateDirectories())
						{
							DirQueue.Enqueue(subdir);
						}
					}
				}
				foreach (string filename in Rules.FilesToGenerate.Keys)
				{
					FileReference generatedItem = FileReference.Combine(IntermediateDirectory, "Gen", filename);
					if (generatedItem.HasExtension(".h"))
					{
						BuildProducts.Add(generatedItem, BuildProductType.BuildResource);
					}
				}
				if (Rules.Target.LinkType == TargetLinkType.Monolithic)
				{
					BuildProducts.Add(PrecompiledManifestLocation, BuildProductType.BuildResource);

					if (FileReference.Exists(PrecompiledManifestLocation)) // May be single file compile; skipped modulePrecompiledManifestLocation{
					{
						PrecompiledManifest ModuleManifest = PrecompiledManifest.Read(PrecompiledManifestLocation);
						foreach (FileReference OutputFile in ModuleManifest.OutputFiles)
						{
							BuildProducts.TryAdd(OutputFile, BuildProductType.BuildResource);
						}
					}
				}
			}
		}

		protected override void GetReferencedDirectories(HashSet<DirectoryReference> Directories)
		{
			base.GetReferencedDirectories(Directories);

			if (!Rules.bUsePrecompiled)
			{
				if (SourceDirectories == null)
				{
					throw new BuildException("GetReferencedDirectories() should not be called before building.");
				}
				Directories.UnionWith(SourceDirectories);
			}
		}

		/// <summary>
		/// List of allowed circular dependencies. Please do NOT add new modules here; refactor to allow the modules to be decoupled instead.
		/// </summary>
		static readonly KeyValuePair<string, string>[] CircularDependenciesAllowList =
		[
			new("AIModule", "AITestSuite"),
			new("AnimGraph", "BlueprintGraph"),
			new("AnimGraph", "ContentBrowser"),
			new("AnimGraph", "GraphEditor"),
			new("AnimGraph", "Kismet"),
			new("AnimGraph", "KismetCompiler"),
			new("AnimGraph", "PropertyEditor"),
			new("AnimGraph", "UnrealEd"),
			new("ApplicationCore", "RenderCore"),
			new("ApplicationCore", "RHI"),
			new("AssetTools", "Kismet"),
			new("AssetTools", "Landscape"),
			new("AssetTools", "MaterialEditor"),
			new("AssetTools", "UnrealEd"),
			new("AudioEditor", "DetailCustomizations"),
			new("AudioMixer", "NonRealtimeAudioRenderer"),
			new("AudioMixer", "SoundFieldRendering"),
			new("BlueprintGraph", "CinematicCamera"),
			new("BlueprintGraph", "GraphEditor"),
			new("BlueprintGraph", "Kismet"),
			new("BlueprintGraph", "KismetCompiler"),
			new("BlueprintGraph", "UnrealEd"),
			new("ConfigEditor", "PropertyEditor"),
			new("Documentation", "ContentBrowser"),
			new("Documentation", "SourceControl"),
			new("Engine", "AudioMixer"),
			new("Engine", "CinematicCamera"),
			new("Engine", "CollisionAnalyzer"),
			new("Engine", "GameplayTags"),
			new("Engine", "Kismet"),
			new("Engine", "Landscape"),
			new("Engine", "LogVisualizer"),
			new("Engine", "MaterialShaderQualitySettings"),
			new("Engine", "MaterialUtilities"),
			new("Engine", "UMG"),
			new("Engine", "UnrealEd"),
			new("FoliageEdit", "ViewportInteraction"),
			new("FoliageEdit", "VREditor"),
			new("FunctionalTesting", "UnrealEd"),
			new("GameplayAbilitiesEditor", "BlueprintGraph"),
			new("GameplayDebugger", "AIModule"),
			new("GameplayDebugger", "GameplayTasks"),
			new("GameplayTasks", "UnrealEd"),
			new("GraphEditor", "Kismet"),
			new("GraphEditor", "Persona"),
			new("HierarchicalLODOutliner", "UnrealEd"),
			new("Kismet", "BlueprintGraph"),
			new("Kismet", "KismetCompiler"),
			new("Kismet", "Merge"),
			new("Kismet", "SubobjectEditor"),
			new("Kismet", "UMGEditor"),
			new("KismetWidgets", "BlueprintGraph"),
			new("Landscape", "MaterialUtilities"),
			new("Landscape", "MeshUtilities"),
			new("Landscape", "UnrealEd"),
			new("LandscapeEditor", "ViewportInteraction"),
			new("LandscapeEditor", "VREditor"),
			new("LocalizationDashboard", "LocalizationService"),
			new("LocalizationDashboard", "MainFrame"),
			new("LocalizationDashboard", "TranslationEditor"),
			new("MaterialUtilities", "Landscape"),
			new("MovieSceneTools", "Sequencer"),
			new("NavigationSystem", "UnrealEd"),
			new("PacketHandler", "ReliabilityHandlerComponent"),
			new("PixelInspectorModule", "UnrealEd"),
			new("Sequencer", "MovieSceneCaptureDialog"),
			new("Sequencer", "MovieSceneTools"),
			new("Sequencer", "UniversalObjectLocatorEditor"),
			new("Sequencer", "ViewportInteraction"),
			new("SourceControl", "UnrealEd"),
			new("UMGEditor", "Blutility"),
			new("UnrealEd", "AddContentDialog"),
			new("UnrealEd", "AnimationBlueprintLibrary"),
			new("UnrealEd", "AudioEditor"),
			new("UnrealEd", "AutomationController"),
			new("UnrealEd", "ClothingSystemEditor"),
			new("UnrealEd", "CommonMenuExtensions"),
			new("UnrealEd", "CurveEditor"),
			new("UnrealEd", "DataLayerEditor"),
			new("UnrealEd", "Documentation"),
			new("UnrealEd", "EditorInteractiveToolsFramework"),
			new("UnrealEd", "EditorWidgets"),
			new("UnrealEd", "FoliageEdit"),
			new("UnrealEd", "GameProjectGeneration"),
			new("UnrealEd", "GraphEditor"),
			new("UnrealEd", "HierarchicalLODUtilities"),
			new("UnrealEd", "Kismet"),
			new("UnrealEd", "LevelEditor"),
			new("UnrealEd", "LevelSequence"),
			new("UnrealEd", "LocalizationService"),
			new("UnrealEd", "MaterialBaking"),
			new("UnrealEd", "MaterialEditor"),
			new("UnrealEd", "MaterialShaderQualitySettings"),
			new("UnrealEd", "MaterialUtilities"),
			new("UnrealEd", "MeshBuilder"),
			new("UnrealEd", "MeshPaint"),
			new("UnrealEd", "MovieSceneTracks"),
			new("UnrealEd", "PropertyEditor"),
			new("UnrealEd", "StatsViewer"),
			new("UnrealEd", "StatusBar"),
			new("UnrealEd", "SubobjectDataInterface"),
			new("UnrealEd", "SubobjectEditor"),
			new("UnrealEd", "ToolMenusEditor"),
			new("UnrealEd", "UATHelper"),
			new("UnrealEd", "UncontrolledChangelists"),
			new("UnrealEd", "UniversalObjectLocatorEditor"),
			new("UnrealEd", "ViewportInteraction"),
			new("UnrealEd", "VirtualizationEditor"),
			new("UnrealEd", "VREditor"),
			new("UnrealEd", "WidgetRegistration"),
			new("WebBrowser", "WebBrowserTexture"),
		];

		public UEBuildModuleCPP(ModuleRules Rules, DirectoryReference IntermediateDirectory, DirectoryReference IntermediateDirectoryNoArch, DirectoryReference? GeneratedCodeDirectory, ILogger Logger)
			: base(Rules, IntermediateDirectory, IntermediateDirectoryNoArch, Logger)
		{
			this.GeneratedCodeDirectory = GeneratedCodeDirectory;

			if (GeneratedCodeDirectory != null)
			{
				GeneratedCodeDirectoryUHT = DirectoryReference.Combine(GeneratedCodeDirectory!, "UHT");
				GeneratedCodeDirectoryVNI = DirectoryReference.Combine(GeneratedCodeDirectory!, "VNI");
			}

			ModuleInterfaceDir = UEToolChain.GetModuleInterfaceDir(IntermediateDirectory);

			// Check for a Verse directory next to the rules file
			DirectoryItem RulesDirectory = DirectoryItem.GetItemByDirectoryReference(Rules.File.Directory);
			if (RulesDirectory.TryGetDirectory("Verse", out DirectoryItem? MaybeVerseDirectory))
			{
				if (IsValidVerseDirectory(MaybeVerseDirectory.Location))
				{
					AssociatedVerseDirectory = MaybeVerseDirectory;
					bDependsOnVerse = true;
				}
			}

			// Check for a Proto directory next to the rules file
			if (RulesDirectory.TryGetDirectory("Proto", out DirectoryItem? MaybeProtoDirectory))
			{
				if (IsValidProtoDirectory(MaybeProtoDirectory.Location))
				{
					bHasProto = true;
				}
			}

			if (Rules.bAddDefaultIncludePaths)
			{
				AddDefaultIncludePaths();
			}
		}

		public override bool ValidateModule(ILogger logger)
		{
			bool anyErrors = false;

			{
				IEnumerable<string> InvalidDependencies = Rules.DynamicallyLoadedModuleNames.Intersect(Rules.PublicDependencyModuleNames.Concat(Rules.PrivateDependencyModuleNames));
				if (InvalidDependencies.Any())
				{
					logger.LogError("{Module} module should not be dependent on modules which are also dynamically loaded: {Invalid}", Name, String.Join(", ", InvalidDependencies));
					anyErrors = true;
				}
			}

			if (Rules.bValidateCircularDependencies || Rules.bTreatAsEngineModule)
			{
				foreach (string CircularlyReferencedModuleName in Rules.CircularlyReferencedDependentModules)
				{
					if (CircularlyReferencedModuleName != "BlueprintContext" &&
						!CircularDependenciesAllowList.Any(x =>
							x.Key == Name && x.Value == CircularlyReferencedModuleName))
					{
						logger.LogWarning(
							"Found reference between '{Source}' and '{Target}'. Support for circular references is being phased out; please do not introduce new ones.",
							Name, CircularlyReferencedModuleName);
					}
				}
			}

			CppStandardVersion CppStandard = Rules.CppStandard ?? (Rules.bTreatAsEngineModule ? Rules.Target.CppStandardEngine : Rules.Target.CppStandard);
			if (CppStandard < CppStandardVersion.Minimum && !Rules.Target.bGenerateProjectFiles)
			{
				logger.LogError("{Module} CppStandard CppStandardVersion.{CppStandard} is no longer supported.", Name, CppStandard);
				anyErrors = true;
			}

			// Make sure that engine modules use shared PCHs or have an explicit private PCH
			if (Rules.PCHUsage == ModuleRules.PCHUsageMode.NoSharedPCHs && Rules.PrivatePCHHeaderFile == null)
			{
				if (Rules.Target.ProjectFile == null || !Rules.File.IsUnderDirectory(Rules.Target.ProjectFile.Directory))
				{
					logger.LogWarning("{Module} module has shared PCHs disabled, but does not have a private PCH set", Name);
				}
			}

			// If we can't use a shared PCH, check there's a private PCH set
			if (Rules.PCHUsage != ModuleRules.PCHUsageMode.NoPCHs && Rules.PCHUsage != ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs && Rules.PrivatePCHHeaderFile == null)
			{
				logger.LogWarning("{Module} module must specify an explicit precompiled header for PCHUsage {PCHUsage} (eg. PrivatePCHHeaderFile = \"Private/{Module}PrivatePCH.h\")", Name, Rules.PCHUsage, Name);
			}

			return base.ValidateModule(logger) || anyErrors;
		}

		/// <summary>
		/// Determines if a file is part of the given module
		/// </summary>
		/// <param name="Location">Path to the file</param>
		/// <returns>True if the file is part of this module</returns>
		public override bool ContainsFile(FileReference Location)
		{
			if (base.ContainsFile(Location))
			{
				return true;
			}
			if (GeneratedCodeDirectory != null && Location.IsUnderDirectory(GeneratedCodeDirectory))
			{
				return true;
			}
			return false;
		}

		/// <summary>
		/// Add the default include paths for this module to its settings
		/// </summary>
		private void AddDefaultIncludePaths()
		{
			// Add the module's parent directory to the public include paths, so other modules may include headers from it explicitly.
			foreach (DirectoryItem ModuleDir in ModuleDirectories)
			{
				// Add the parent directory to the legacy parent include paths.
				LegacyParentIncludePaths.Add(ModuleDir.Location.ParentDirectory!);

				// Add the base directory to the legacy include paths.
				LegacyPublicIncludePaths.Add(ModuleDir.Location);

				// Add the 'classes' directory, if it exists
				if (ModuleDir.TryGetDirectory("Classes", out DirectoryItem? ClassesDirectory) && ClassesDirectory.ContainsFiles(SearchOption.AllDirectories))
				{
					PublicIncludePaths.Add(ClassesDirectory.Location);
				}

				// Add all the public directories
				if (ModuleDir.TryGetDirectory("Public", out DirectoryItem? PublicDirectory) && PublicDirectory.ContainsFiles(SearchOption.AllDirectories))
				{
					PublicIncludePaths.Add(PublicDirectory.Location);

					IReadOnlySet<string> ExcludeNames = UEBuildPlatform.GetBuildPlatform(Rules.Target.Platform).GetExcludedFolderNames();
					EnumerateLegacyIncludePaths(PublicDirectory, ExcludeNames, LegacyPublicIncludePaths);
				}

				// Add the 'internal' directory, if it exists
				if (ModuleDir.TryGetDirectory("Internal", out DirectoryItem? InternalDirectory) && InternalDirectory.ContainsFiles(SearchOption.AllDirectories))
				{
					InternalIncludePaths.Add(InternalDirectory.Location);
				}

				// Add the base private directory for this module
				if (ModuleDir.TryGetDirectory("Private", out DirectoryItem? PrivateDirectory) && PrivateDirectory.ContainsFiles(SearchOption.AllDirectories))
				{
					PrivateIncludePaths.Add(PrivateDirectory.Location);
				}
			}
		}

		/// <summary>
		/// Enumerates legacy include paths under a given base directory
		/// </summary>
		/// <param name="BaseDirectory">The directory to start from. This directory is not added to the output list.</param>
		/// <param name="ExcludeNames">Set of folder names to exclude from the search.</param>
		/// <param name="LegacyPublicIncludePaths">List populated with the discovered directories</param>
		static void EnumerateLegacyIncludePaths(DirectoryItem BaseDirectory, IReadOnlySet<string> ExcludeNames, HashSet<DirectoryReference> LegacyPublicIncludePaths)
		{
			foreach (DirectoryItem SubDirectory in BaseDirectory.EnumerateDirectories())
			{
				if (!ExcludeNames.Contains(SubDirectory.Name))
				{
					LegacyPublicIncludePaths.Add(SubDirectory.Location);
					EnumerateLegacyIncludePaths(SubDirectory, ExcludeNames, LegacyPublicIncludePaths);
				}
			}
		}

		/// <summary>
		/// Path to the precompiled manifest location
		/// </summary>
		public virtual FileReference PrecompiledManifestLocation => UnrealArchitectureConfig.ForPlatform(Rules.Target.Platform).Mode <= UnrealArchitectureMode.OneTargetPerArchitecture
			? FileReference.Combine(IntermediateDirectory, $"{Name}.precompiled")
			: FileReference.Combine(IntermediateDirectoryNoArch, $"{Name}.precompiled");

		/// <summary>
		/// Sets up the environment for compiling any module that includes the public interface of this module.
		/// </summary>
		public override void AddModuleToCompileEnvironment(
			UEBuildModule? SourceModule,
			UEBuildBinary? SourceBinary,
			HashSet<DirectoryReference> IncludePaths,
			HashSet<DirectoryReference> SystemIncludePaths,
			HashSet<DirectoryReference> ModuleInterfacePaths,
			List<string> Definitions,
			List<UEBuildFramework> AdditionalFrameworks,
			HashSet<FileItem> AutoRTFMExternalMappingFiles,
			List<FileItem> AdditionalPrerequisites,
			bool bLegacyPublicIncludePaths,
			bool bLegacyParentIncludePaths
			)
		{
			// TODO: CACHE THESE THINGS... THIS IS CALLED A MILLION TIMES BECAUSE OF SHARED PCH
			// Change HashSets to DirectoryItem instead

			if (GeneratedCodeDirectory != null)
			{
				// This directory may not exist for this module (or ever exist, if it doesn't contain any generated headers), but we want the project files
				// to search it so we can pick up generated code definitions after UHT is run for the first time.
				bool bForceAddIncludePath = bForceAddGeneratedCodeIncludePath || ProjectFileGenerator.bGenerateProjectFiles;

				if (bHasUObjects || bForceAddIncludePath)
				{
					IncludePaths.Add(GeneratedCodeDirectoryUHT!);
				}

				if (bHasVerse || bForceAddGeneratedCodeIncludePath) // bHasVerse should be up-to-date already from ctor.. so does not need to always be added when generating project files
				{
					IncludePaths.Add(GeneratedCodeDirectoryVNI!);
				}

				foreach ((string Subdirectory, Action<ILogger, DirectoryReference> _) in Rules.GenerateHeaderFuncs)
				{
					IncludePaths.Add(DirectoryReference.Combine(GeneratedCodeDirectory, Subdirectory));
				}
			}

			ModuleInterfacePaths.Add(ModuleInterfaceDir);

			base.AddModuleToCompileEnvironment(SourceModule, SourceBinary, IncludePaths, SystemIncludePaths, ModuleInterfacePaths, Definitions, AdditionalFrameworks, AutoRTFMExternalMappingFiles, AdditionalPrerequisites, bLegacyPublicIncludePaths, bLegacyParentIncludePaths);
		}

		// UEBuildModule interface.
		public override List<FileItem> Compile(ReadOnlyTargetRules Target, UEToolChain ToolChain, CppCompileEnvironment BinaryCompileEnvironment, ISourceFileWorkingSet WorkingSet, IActionGraphBuilder Graph, ILogger Logger)
		{
			//UEBuildPlatform BuildPlatform = UEBuildPlatform.GetBuildPlatform(BinaryCompileEnvironment.Platform);

			List<FileItem> LinkInputFiles = base.Compile(Target, ToolChain, BinaryCompileEnvironment, WorkingSet, Graph, Logger);

			CppCompileEnvironment ModuleCompileEnvironment = CreateModuleCompileEnvironment(Target, BinaryCompileEnvironment, Logger);

			// If the module is precompiled, read the object files from the manifest
			if (Rules.bUsePrecompiled && Target.LinkType == TargetLinkType.Monolithic)
			{
				if (!FileReference.Exists(PrecompiledManifestLocation))
				{
					Logger.LogError("Missing precompiled manifest for '{Name}', '{Manifest}. This module can not be referenced in a monolithic precompiled build, remove this reference or migrate to a fully compiled source build.", Name, PrecompiledManifestLocation);
					Logger.LogInformation("This module was most likely not flagged during a release for being included in a precompiled build - set 'PrecompileForTargets = PrecompileTargetsType.Any;' in {Name}.Build.cs to override.", Name);
					if (Rules.Plugin != null)
					{
						Logger.LogInformation("As it is part of the plugin '{PluginName}', also check if its 'Type' is correct.", Rules.Plugin.Name);
					}
					if (ReferenceStackParentModules != null)
					{
						Logger.LogInformation("Dependent modules '{OtherModules}'", String.Join(' ', ReferenceStackParentModules.OrderBy(x => x.Name).Select(x => x.Name)));
					}
					throw new BuildLogEventException("Missing precompiled manifest for '{Name}', '{Manifest}", Name, PrecompiledManifestLocation);
				}

				PrecompiledManifest Manifest = PrecompiledManifest.Read(PrecompiledManifestLocation);

				// Validate the precompiled objects were not built with a newer MSVC toolchain than the one currently selected.
				// A newer compiler can introduce intrinsics (e.g. __std_search_1) that don't exist in older runtimes, causing LNK2019.
				if (Target.Platform.IsInGroup(UnrealPlatformGroup.Microsoft) && Target.WindowsPlatform.Compiler.IsMSVC()
					&& Manifest.ToolChainVersion != null && Manifest.ToolChainVersion.Components.Count >= 2
					&& Target.WindowsPlatform.Environment != null && Target.WindowsPlatform.Environment.ToolChainVersion.Components.Count >= 2)
				{
					VersionNumber ManifestFamily = new VersionNumber(Manifest.ToolChainVersion.GetComponent(0), Manifest.ToolChainVersion.GetComponent(1));
					VersionNumber CurrentFamily  = new VersionNumber(Target.WindowsPlatform.Environment.ToolChainVersion.GetComponent(0), Target.WindowsPlatform.Environment.ToolChainVersion.GetComponent(1));

					if (ManifestFamily > CurrentFamily)
					{
						throw new BuildLogEventException(
							"Module '{Name}' was precompiled with MSVC toolchain {ManifestVersion} but the currently selected toolchain is {CurrentVersion}. " +
							"The precompiled objects may reference intrinsics that do not exist in older runtimes, which will cause linker errors. " +
							"Please use a newer toolchain, or recompile the engine from source.",
							Name, Manifest.ToolChainVersion, Target.WindowsPlatform.Environment.ToolChainVersion);
					}
				}

				foreach (FileReference OutputFile in Manifest.OutputFiles)
				{
					FileItem ObjectFile = FileItem.GetItemByFileReference(OutputFile);
					if (!ObjectFile.Exists)
					{
						throw new BuildLogEventException("Missing object file {OutputFile} listed in {Manifest}", OutputFile, PrecompiledManifestLocation);
					}
					LinkInputFiles.Add(ObjectFile);
				}
				return LinkInputFiles;
			}

			// Add all the module source directories to the makefile
			foreach (DirectoryItem ModuleDirectory in ModuleDirectories)
			{
				Graph.AddSourceDir(ModuleDirectory);
			}

			// Find all the input files
			Dictionary<DirectoryItem, FileItem[]> DirectoryToSourceFiles = new Dictionary<DirectoryItem, FileItem[]>();
			InputFileCollection InputFiles;

			if (Rules.Target.IsTestTarget)
			{
				InputFiles = FindInputFilesTests(Target.Platform, DirectoryToSourceFiles, Logger);
			}
			else
			{
				InputFiles = FindInputFiles(Target.Platform, DirectoryToSourceFiles, Logger);
			}

			{
				Dictionary<string, List<FileItem>> nameToPath = new();
				foreach (FileItem InputFile in InputFiles.AllCompiledFiles)
				{
					string name = InputFile.Name.ToUpperInvariant();
					if (!nameToPath.ContainsKey(name))
					{
						nameToPath.Add(name, new List<FileItem>());
					}
					nameToPath[name].Add(InputFile);
				}
				IEnumerable<KeyValuePair<string, List<FileItem>>> fileConflicts = nameToPath.Where(item => item.Value.Count > 1);
				if (fileConflicts.Any())
				{
					Logger.LogInformation("Input filename conflicts:");
					foreach (KeyValuePair<string, List<FileItem>> item in fileConflicts)
					{
						item.Value.ForEach(x => Logger.LogInformation("* {Path}", x));
					}
					throw new BuildException("Multiple input files found with duplicate filenames, this is is not allowed as intermediate output files for non-unity builds will conflict");
				}
			}

			foreach (KeyValuePair<DirectoryItem, FileItem[]> Pair in DirectoryToSourceFiles)
			{
				Graph.AddSourceFiles(Pair.Key, Pair.Value);
			}

			Graph.AddHeaderFiles(InputFiles.AllHeaderFiles.ToArray());

			// We are building with IWYU and thismodule does not support it, early out
			if (Target.bIWYU && Rules.IWYUSupport == IWYUSupport.None)
			{
				return new List<FileItem>();
			}

			// Process all of the header file dependencies for this module
			CheckFirstIncludeMatchesEachCppFile(Target, ModuleCompileEnvironment, InputFiles.HeaderFiles, InputFiles.CPPFiles);

			// Should we force a precompiled header to be generated for this module?  Usually, we only bother with a
			// precompiled header if there are at least several source files in the module (after combining them for unity
			// builds.)  But for game modules, it can be convenient to always have a precompiled header to single-file
			// changes to code is really quick to compile.
			int MinFilesUsingPrecompiledHeader = Target.MinFilesUsingPrecompiledHeader;
			if (Rules.MinFilesUsingPrecompiledHeaderOverride != 0)
			{
				MinFilesUsingPrecompiledHeader = Rules.MinFilesUsingPrecompiledHeaderOverride;
			}
			else if (!Rules.bTreatAsEngineModule && Target.bForcePrecompiledHeaderForGameModules)
			{
				// This is a game module with only a small number of source files, so go ahead and force a precompiled header
				// to be generated to make incremental changes to source files as fast as possible for small projects.
				MinFilesUsingPrecompiledHeader = 1;
			}

			// Set up the environment with which to compile the CPP files
			CppCompileEnvironment CppCompileEnvironment = new(ModuleCompileEnvironment);

			// Compile any module interfaces
			if (InputFiles.IXXFiles.Count > 0 && Target.bEnableCppModules)
			{
				CppCompileEnvironment IxxCompileEnvironment = new(ModuleCompileEnvironment);

				// Write all the definitions to a separate file for the ixx compile
				CreateHeaderForDefinitions(IxxCompileEnvironment, IntermediateDirectory, "ixx", Graph);

				CPPOutput ModuleOutput = ToolChain.CompileAllCPPFiles(IxxCompileEnvironment, InputFiles.IXXFiles, IntermediateDirectory, Name, Graph);

				LinkInputFiles.AddRange(ModuleOutput.ObjectFiles);
				CppCompileEnvironment.AdditionalPrerequisites.AddRange(ModuleOutput.CompiledModuleInterfaces);
			}

			// Configure the precompiled headers for this module
			CppCompileEnvironment = SetupPrecompiledHeaders(Target, ToolChain, CppCompileEnvironment, LinkInputFiles, Graph);
			if (CppCompileEnvironment.PerArchPrecompiledHeaderFiles != null)
			{
				foreach (UnrealArch Arch in CppCompileEnvironment.PerArchPrecompiledHeaderFiles.Keys)
				{
					Logger.LogDebug("Module '{ModuleName}' uses PCH '{PCHIncludeFilename}' for Architecture '{Arch}'", Name, Arch, CppCompileEnvironment.PerArchPrecompiledHeaderFiles[Arch]);
				}
			}
			else if (CppCompileEnvironment.PrecompiledHeaderFile != null)
			{
				Logger.LogDebug("Module '{ModuleName}' uses PCH '{PCHIncludeFilename}'", Name, CppCompileEnvironment.PrecompiledHeaderFile);
			}

			// Generate ISPC headers before normal c++ actions but after pch since we don't want pch actions to wait for ispc (ispc headers are not allowed in pch)
			if (InputFiles.ISPCFiles.Count > 0)
			{
				IEnumerable<FileItem> ISPCHeaders = ToolChain.GenerateAllISPCHeaders(CppCompileEnvironment, InputFiles.ISPCFiles, IntermediateDirectory, Graph).GeneratedHeaderFiles;
				CppCompileEnvironment.AdditionalPrerequisites.AddRange(ISPCHeaders);
				ModuleCompileEnvironment.AdditionalPrerequisites.AddRange(ISPCHeaders);
				CppCompileEnvironment.UserIncludePaths.Add(IntermediateDirectory);
				ModuleCompileEnvironment.UserIncludePaths.Add(IntermediateDirectory);
			}

			if (bHasProto)
			{
				DirectoryReference ProtoDirectory = DirectoryReference.Combine(Rules.File.Directory, "Proto");
				ProtocOutput OutputFiles = ProtoExecution.CompileProtoFiles(InputFiles.ProtoFiles, ProtoDirectory, IntermediateDirectory, Graph);
				CppCompileEnvironment.AdditionalPrerequisites.AddRange(OutputFiles.HeaderFiles);
				ModuleCompileEnvironment.AdditionalPrerequisites.AddRange(OutputFiles.HeaderFiles);
				CppCompileEnvironment.UserIncludePaths.Add(IntermediateDirectory);
				ModuleCompileEnvironment.UserIncludePaths.Add(IntermediateDirectory);

				CppCompileEnvironment ProtoCompileEnvironment = new CppCompileEnvironment(CppCompileEnvironment);
				ProtoCompileEnvironment.PrecompiledHeaderAction = PrecompiledHeaderAction.None;
				ProtoCompileEnvironment.bUseUnity = false;
				ProtoCompileEnvironment.bWarningsAsErrors = false;
				// Disable dynamic analysis on generated proto code as it causes issues when the proto/grpc libs are not
				// compiled the same way. This can cause false positive in some situations, for example with address
				// sanitize enabled (see: https://github.com/protocolbuffers/protobuf/issues/13115).
				ProtoCompileEnvironment.bDisableDynamicAnalysis = true;
				// Disable warnings that are common in generated proto cpp
				ProtoCompileEnvironment.CppCompileWarnings.DeprecationWarningLevel = WarningLevel.Off;
				ProtoCompileEnvironment.CppCompileWarnings.ShortenSizeTToIntWarningLevel = WarningLevel.Off;
				ProtoCompileEnvironment.CppCompileWarnings.UnusedParameterWarningLevel = WarningLevel.Off;
				ProtoCompileEnvironment.CppCompileWarnings.DeprecationWarningLevel = WarningLevel.Off;

				CPPOutput Output = ToolChain.CompileAllCPPFiles(ProtoCompileEnvironment, OutputFiles.CppFiles, IntermediateDirectory, Name, Graph);

				LinkInputFiles.AddRange(Output.ObjectFiles);
			}

			// Write all the definitions to a separate file
			CreateHeaderForDefinitions(CppCompileEnvironment, IntermediateDirectory, null, Graph);

			// Create shared rsp for the normal cpp files
			FileReference SharedResponseFile = FileReference.Combine(IntermediateDirectory, $"{Rules.ShortName ?? Name}.Shared{UEToolChain.ResponseExt}");
			CppCompileEnvironment = ToolChain.CreateSharedResponseFile(CppCompileEnvironment, SharedResponseFile, Graph);

			// Mapping of source file to unity file. We output this to intermediate directories for other tools (eg. live coding) to use.
			Dictionary<FileItem, FileItem> SourceFileToUnityFile = new Dictionary<FileItem, FileItem>();

			List<FileItem> CPPFiles = new List<FileItem>(InputFiles.CPPFiles);
			List<FileItem> GeneratedFileItems = new List<FileItem>();
			CppCompileEnvironment GeneratedCPPCompileEnvironment = CppCompileEnvironment;
			bool bMergeUnityFiles = !(Target.DisableMergingModuleAndGeneratedFilesInUnityFiles?.Contains(Name) ?? false) && Rules.bMergeUnityFiles;

			// Compile all the generated CPP files
			if (GeneratedCppDirectories != null && !CppCompileEnvironment.bHackHeaderGenerator)
			{
				Dictionary<string, FileItem> GeneratedFiles = new Dictionary<string, FileItem>();
				foreach (string GeneratedDir in GeneratedCppDirectories)
				{
					string Prefix = Path.GetFileName(GeneratedDir) + '/'; // "UHT/" or "VNI/"

					DirectoryItem DirItem = DirectoryItem.GetItemByPath(GeneratedDir);
					foreach (FileItem File in DirItem.EnumerateFiles())
					{
						string FileName = File.Name;
						if (FileName.EndsWith(".gen.cpp", System.StringComparison.Ordinal))
						{
							string Key = Prefix + FileName.Substring(0, FileName.Length - ".gen.cpp".Length);
							GeneratedFiles.Add(Key, File);
						}
						else if (GeneratedCPPCompileEnvironment.FileMatchesExtraGeneratedCPPTypes(FileName))
						{
							GeneratedFiles.Add(FileName, File);
						}
					}
				}

				if (GeneratedFiles.Count > 0)
				{
					// Remove any generated files from the compile list if they are inlined
					foreach (FileItem CPPFileItem in CPPFiles)
					{
						IList<string> ListOfInlinedGenCpps = ModuleCompileEnvironment.MetadataCache.GetListOfInlinedGeneratedCppFiles(CPPFileItem);
						foreach (string ListOfInlinedGenCppsItem in ListOfInlinedGenCpps)
						{
							string Prefix = "UHT/";
							string Key = Prefix + ListOfInlinedGenCppsItem;
							if (GeneratedFiles.Remove(Key, out FileItem? FoundGenCppFile))
							{
								if (!CppCompileEnvironment.FileInlineSourceMap.ContainsKey(CPPFileItem))
								{
									CppCompileEnvironment.FileInlineSourceMap[CPPFileItem] = [];
								}
								CppCompileEnvironment.FileInlineSourceMap[CPPFileItem].Add(FoundGenCppFile);
							}
							else
							{
								Logger.LogError("'{CPPFileItem}' is looking for a generated cpp with named '{HeaderFile}.gen.cpp' (Found {FoundCount}) time)", CPPFileItem.AbsolutePath, ListOfInlinedGenCppsItem, ListOfInlinedGenCpps.Count);
							}
						}
					}
					ModuleCompileEnvironment.FileInlineSourceMap = CppCompileEnvironment.FileInlineSourceMap;

					WarningLevel NonInlinedGenCppWarningLevel = Rules.CppCompileWarningSettings.NonInlinedGenCppWarningLevel;
					if (NonInlinedGenCppWarningLevel >= WarningLevel.Warning)
					{
						Dictionary<string, FileItem> CPPFilesLookup = new Dictionary<string, FileItem>();
						foreach (FileItem CPPFile in CPPFiles)
						{
							CPPFilesLookup.Add(CPPFile.Name, CPPFile);
						}

						foreach (string Name in GeneratedFiles.Keys)
						{
							if (!Name.StartsWith("UHT/", StringComparison.Ordinal))
							{
								continue;
							}

							string NameWithoutPrefix = Name.Substring(4);
							string NameWithoutPrefixWithSuffix = $"{NameWithoutPrefix}.cpp";
							if (CPPFilesLookup.TryGetValue(NameWithoutPrefixWithSuffix, out FileItem? Item))
							{
								LogLevel LogLevel = NonInlinedGenCppWarningLevel switch
								{
									WarningLevel.Warning => LogLevel.Warning,
									WarningLevel.Error => LogLevel.Error,
									_ => throw new BuildException("Unhandled warning level"),
								};
								Logger.Log(LogLevel, "{CPPFileItem}(1): {LogLevel}: .gen.cpp not inlined. Add '#include UE_INLINE_GENERATED_CPP_BY_NAME({CPPFileName})' after header includes.", Item.AbsolutePath, LogLevel, NameWithoutPrefix);
							}
						}
					}

					// Create a compile environment for the generated files. We can disable creating debug info here to improve link times.
					if (GeneratedCPPCompileEnvironment.bCreateDebugInfo && Target.bDisableDebugInfoForGeneratedCode)
					{
						GeneratedCPPCompileEnvironment = new CppCompileEnvironment(GeneratedCPPCompileEnvironment);
						GeneratedCPPCompileEnvironment.bCreateDebugInfo = false;
						bMergeUnityFiles = false;
					}

					if (Target.StaticAnalyzer != StaticAnalyzer.None && !Target.bStaticAnalyzerIncludeGenerated)
					{
						GeneratedCPPCompileEnvironment = new CppCompileEnvironment(GeneratedCPPCompileEnvironment);
						GeneratedCPPCompileEnvironment.bDisableStaticAnalysis = true;
						bMergeUnityFiles = false;
					}

					// Always force include the PCH, even if PCHs are disabled, for generated code. Legacy code can rely on PCHs being included to compile correctly, and this used to be done by UHT manually including it.
					if (Target.bForceIncludePCHHeadersForGenCppFilesWhenPCHIsDisabled && GeneratedCPPCompileEnvironment.bHasPrecompiledHeader == false && Rules.PrivatePCHHeaderFile != null && Rules.PCHUsage != ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs)
					{
						FileItem PrivatePchFileItem = FileItem.GetItemByFileReference(FileReference.Combine(ModuleDirectory, Rules.PrivatePCHHeaderFile));
						if (!PrivatePchFileItem.Exists)
						{
							throw new BuildException("Unable to find private PCH file '{0}', referenced by '{1}'", PrivatePchFileItem.Location, RulesFile);
						}

						GeneratedCPPCompileEnvironment = new CppCompileEnvironment(GeneratedCPPCompileEnvironment);
						GeneratedCPPCompileEnvironment.ForceIncludeFiles.Add(PrivatePchFileItem);
						bMergeUnityFiles = false;
					}

					// Compile all the generated files
					foreach (FileItem GeneratedCppFileItem in GeneratedFiles.Values)
					{
						GeneratedFileItems.Add(GeneratedCppFileItem);
					}
				}
			}

			if (GeneratedCPPCompileEnvironment == CppCompileEnvironment)
			{
				GeneratedCPPCompileEnvironment = new(CppCompileEnvironment);
			}

			if (Target.WindowsPlatform.bEnableInstrumentation)
			{
				CppCompileEnvironment.UserIncludePaths.Add(DirectoryReference.Combine(Unreal.EngineSourceDirectory, "Runtime", "Instrumentation", "Public"));
				CppCompileEnvironment.UserIncludePaths.Add(DirectoryReference.Combine(Unreal.EngineSourceDirectory, "Runtime", "Instrumentation", "Internal"));
				CppCompileEnvironment.UserIncludePaths.Add(DirectoryReference.Combine(Unreal.EngineSourceDirectory, "Runtime", "Core", "Public"));
				CppCompileEnvironment.UserIncludePaths.Add(DirectoryReference.Combine(Unreal.EngineSourceDirectory, "Runtime", "Core", "Internal"));
			}

			// The operator new/delete overrides need to exist in all modules.
			// We put these overrides in a separate file in order to be able to ensure only one file is added per binary for the primary module
			if (Target.LinkType == TargetLinkType.Modular && bCreatePerModuleFile)
			{
				FileReference NewDeleteOverrides = FileReference.Combine(IntermediateDirectory, "PerModuleInline.gen.cpp");
				List<string> Content =
				[
					"// Generated by UnrealBuildTool (UEBuildModuleCPP.cs)",
					"#if !defined(PER_MODULE_INLINE_FILE) && defined(CORE_API)",
					"#define PER_MODULE_INLINE_FILE \"HAL/PerModuleInline.inl\"",
					"#endif",
					"#if defined(PER_MODULE_INLINE_FILE) && !defined(SUPPRESS_PER_MODULE_INLINE_FILE)",
					"#include PER_MODULE_INLINE_FILE",
					"#endif",
				];

				if (Target.WindowsPlatform.bEnableInstrumentation)
				{
					// Give the ability for Core to hotpatch itself for the per module functions to save another set of JMPs.
					if (Binary.Modules.Any(x => x.Name == "Core"))
					{
						Content.Add("#define USE_INSTRUMENTATION_PERMODULE_HOTPATCH 1");
					}
					Content.Add("#include \"Instrumentation/PerModuleInline.inl\"");
				}

				FileItem PerModuleFile = Graph.CreateIntermediateTextFile(NewDeleteOverrides, Content, false);
				CPPFiles.Add(PerModuleFile);
			}

			// Generate files provided through module rules.
			foreach (KeyValuePair<string, IEnumerable<string>> kv in Rules.FilesToGenerate)
			{
				FileReference generatedFileRef = FileReference.Combine(IntermediateDirectory, "Gen", kv.Key);
				FileItem generatedFile = Graph.CreateIntermediateTextFile(generatedFileRef, kv.Value, false);
				if (generatedFileRef.HasExtension(".cpp"))
				{
					CPPFiles.Add(generatedFile);
				}
			}

			// Engine modules will always use unity build mode unless MinSourceFilesForUnityBuildOverride is specified in
			// the module rules file.  By default, game modules only use unity of they have enough source files for that
			// to be worthwhile.  If you have a lot of small game modules, consider specifying MinSourceFilesForUnityBuildOverride=0
			// in the modules that you don't typically iterate on source files in very frequently.
			int MinSourceFilesForUnityBuild = 0;
			if (Rules.MinSourceFilesForUnityBuildOverride != 0)
			{
				MinSourceFilesForUnityBuild = Rules.MinSourceFilesForUnityBuildOverride;
			}
			else if (Target.ProjectFile != null && RulesFile.IsUnderDirectory(DirectoryReference.Combine(Target.ProjectFile.Directory, "Source")))
			{
				// Game modules with only a small number of source files are usually better off having faster iteration times
				// on single source file changes, so we forcibly disable unity build for those modules
				MinSourceFilesForUnityBuild = Target.MinGameModuleSourceFilesForUnityBuild;
			}

			// Set up the NumIncludedBytesPerUnityCPP for this particular module
			int NumIncludedBytesPerUnityCPP = Rules.GetNumIncludedBytesPerUnityCPP();

			// Should we use unity build mode for this module?
			bool bModuleUsesUnityBuild = false;
			if (Target.bUseUnityBuild || Target.bForceUnityBuild)
			{
				int FileCount = CPPFiles.Count;

				// if we are merging the generated cpp files then that needs to be part of the count
				if (bMergeUnityFiles)
				{
					FileCount += GeneratedFileItems.Count;
				}

				if (Target.bForceUnityBuild)
				{
					Logger.LogTrace("Module '{ModuleName}' using unity build mode (bForceUnityBuild enabled for this module)", Name);
					bModuleUsesUnityBuild = true;
				}
				else if (!Rules.bUseUnity)
				{
					Logger.LogTrace("Module '{ModuleName}' not using unity build mode (bUseUnity disabled for this module)", Name);
					bModuleUsesUnityBuild = false;
				}
				else if (FileCount < MinSourceFilesForUnityBuild)
				{
					Logger.LogTrace("Module '{ModuleName}' not using unity build mode (module with fewer than {NumFiles} source files)", Name, MinSourceFilesForUnityBuild);
					bModuleUsesUnityBuild = false;
				}
				else
				{
					Logger.LogTrace("Module '{ModuleName}' using unity build mode", Name);
					bModuleUsesUnityBuild = true;
				}
				bModuleUsesUnityBuild = bModuleUsesUnityBuild && Rules.UnityFileTypes.HasFlag(ModuleRules.UnityFileType.Cpp);
			}
			else
			{
				Logger.LogTrace("Module '{ModuleName}' not using unity build mode", Name);
			}

			CppCompileEnvironment.bUseUnity = bModuleUsesUnityBuild;
			GeneratedCPPCompileEnvironment.bUseUnity = CppCompileEnvironment.bUseUnity;

			// Create and register a special action that can be used to compile single files (even when unity is enabled) for generated files
			if (GeneratedFileItems.Any())
			{
				CppCompileEnvironment SingleGeneratedCPPCompileEnvironment = new CppCompileEnvironment(GeneratedCPPCompileEnvironment);
				SingleGeneratedCPPCompileEnvironment.PrecompiledHeaderAction = PrecompiledHeaderAction.None;
				SingleGeneratedCPPCompileEnvironment.bUseUnity = false;
				foreach (DirectoryReference Directory in GeneratedFileItems.Select(x => x.Location.Directory).Distinct())
				{
					ToolChain.CreateSpecificFileAction(SingleGeneratedCPPCompileEnvironment, Directory, IntermediateDirectory, Graph);
				}
			}

			// Create and register a special action that can be used to compile single files (even when unity is enabled) for normal files
			if (ModuleDirectories.Any())
			{
				CppCompileEnvironment SingleCompileEnvironment = new CppCompileEnvironment(CppCompileEnvironment);
				SingleCompileEnvironment.PrecompiledHeaderAction = PrecompiledHeaderAction.None;
				SingleCompileEnvironment.bUseUnity = false;
				foreach (DirectoryItem Directory in ModuleDirectories)
				{
					ToolChain.CreateSpecificFileAction(SingleCompileEnvironment, Directory.Location, IntermediateDirectory, Graph);
				}
			}

			// Compile Generated CPP Files, always use unity for .gen.cpp even if disabled for the current module
			if (bModuleUsesUnityBuild && bMergeUnityFiles)
			{
				CPPFiles.AddRange(GeneratedFileItems);
			}
			else if (Rules.Target.bAlwaysUseUnityForGeneratedFiles && !Rules.Target.bEnableCppModules)
			{
				GeneratedCPPCompileEnvironment.bUseUnity = true;
				Unity.GenerateUnitySource(Target, GeneratedFileItems, new List<FileItem>(), GeneratedCPPCompileEnvironment, WorkingSet, (Rules.ShortName ?? Name) + ".gen", IntermediateDirectory, Graph, SourceFileToUnityFile,
					out List<FileItem> NormalGeneratedFiles, out List<FileItem> AdaptiveGeneratedFiles, NumIncludedBytesPerUnityCPP);
				ModuleCompileEnvironment.FileInlineSourceMap = new(GeneratedCPPCompileEnvironment.FileInlineSourceMap);
				CppCompileEnvironment.FileInlineSourceMap = new(GeneratedCPPCompileEnvironment.FileInlineSourceMap);
				LinkInputFiles.AddRange(CompileFilesWithToolChain(Target, ToolChain, GeneratedCPPCompileEnvironment, ModuleCompileEnvironment, NormalGeneratedFiles, AdaptiveGeneratedFiles, Graph).ObjectFiles);
			}
			else
			{
				Unity.GetAdaptiveFiles(Target, GeneratedFileItems, InputFiles.HeaderFiles, GeneratedCPPCompileEnvironment, WorkingSet, IntermediateDirectory, Graph,
					out List<FileItem> NormalGeneratedFiles, out List<FileItem> AdaptiveGeneratedFiles);
				LinkInputFiles.AddRange(CompileFilesWithToolChain(Target, ToolChain, GeneratedCPPCompileEnvironment, ModuleCompileEnvironment, NormalGeneratedFiles, AdaptiveGeneratedFiles, Graph).ObjectFiles);
			}

			// Compile Swift - which may generate a header that the cpp files need below
			if (InputFiles.SwiftFiles.Count > 0 && Rules.SwiftInteropHeader != null)
			{
				CppCompileEnvironment SwiftCompileEnvironment = new CppCompileEnvironment(ModuleCompileEnvironment);
				// currently nothing is using this header, but it could be helpful to have in swift
				CreateHeaderForDefinitions(SwiftCompileEnvironment, IntermediateDirectory, "swift", Graph);
				SwiftCompileEnvironment.ForceIncludeFiles.Clear();
				SwiftCompileEnvironment.ForceIncludeFiles.Add(FileItem.GetItemByPath(Rules.SwiftInteropHeader));

				// build the swift files and bridging headers to expose swift to cpp
				CPPOutput Output = ToolChain.CompileAllCPPFiles(SwiftCompileEnvironment, InputFiles.SwiftFiles, IntermediateDirectory, Name, Graph);

				// link the .o's
				LinkInputFiles.AddRange(Output.ObjectFiles);

				// and add the .h's as compile dependencies to make sure they are generated before we compile
				IReadOnlyList<FileItem> HeaderFiles = Output.GeneratedHeaderFiles;

				// every cpp file in the module will depend on this header, because any file could need it (this means heavy swift
				// changes will require rebuilding the whole module a lot)
				// modify both ModuleCompileEnvironment and CompileEnvironment because they are branched already, and are
				// both used when compiling files that may need the headers
				ModuleCompileEnvironment.AdditionalPrerequisites.AddRange(HeaderFiles);
				CppCompileEnvironment.AdditionalPrerequisites.AddRange(HeaderFiles);
				if (HeaderFiles.Count > 0)
				{
					// @todo: this could have different per-arch headers- should be rare but possible
					ModuleCompileEnvironment.UserIncludePaths.Add(HeaderFiles[0].Location.Directory);
					CppCompileEnvironment.UserIncludePaths.Add(HeaderFiles[0].Location.Directory);
				}
			}

			// Compile CPP files
			if (CPPFiles.Count > 0)
			{
				if (bModuleUsesUnityBuild)
				{
					Unity.GenerateUnitySource(Target, CPPFiles, InputFiles.HeaderFiles, CppCompileEnvironment, WorkingSet, Rules.ShortName ?? Name, IntermediateDirectory, Graph, SourceFileToUnityFile,
						out List<FileItem> NormalFiles, out List<FileItem> AdaptiveFiles, NumIncludedBytesPerUnityCPP);
					ModuleCompileEnvironment.FileInlineSourceMap = new(CppCompileEnvironment.FileInlineSourceMap);
					LinkInputFiles.AddRange(CompileFilesWithToolChain(Target, ToolChain, CppCompileEnvironment, ModuleCompileEnvironment, NormalFiles, AdaptiveFiles, Graph).ObjectFiles);
				}
				else
				{
					Unity.GetAdaptiveFiles(Target, CPPFiles, InputFiles.HeaderFiles, CppCompileEnvironment, WorkingSet, IntermediateDirectory, Graph,
						out List<FileItem> NormalFiles, out List<FileItem> AdaptiveFiles);
					if (!NormalFiles.Where(file => !file.HasExtension(".gen.cpp") && !GeneratedCPPCompileEnvironment.FileMatchesExtraGeneratedCPPTypes(file.FullName)).Any())
					{
						NormalFiles = CPPFiles;
						AdaptiveFiles.RemoveAll(new HashSet<FileItem>(NormalFiles).Contains);
					}
					LinkInputFiles.AddRange(CompileFilesWithToolChain(Target, ToolChain, CppCompileEnvironment, ModuleCompileEnvironment, NormalFiles, AdaptiveFiles, Graph).ObjectFiles);
				}
			}

			// Compile ISPC files directly
			if (InputFiles.ISPCFiles.Count > 0)
			{
				CppCompileEnvironment IspcCompileEnvironment = new(ModuleCompileEnvironment);
				LinkInputFiles.AddRange(ToolChain.CompileAllISPCFiles(IspcCompileEnvironment, InputFiles.ISPCFiles, IntermediateDirectory, Graph).ObjectFiles);
			}

			// Compile C files. Do not use a PCH here, because a C++ PCH is not compatible with C source files.
			if (InputFiles.CFiles.Count > 0)
			{
				CppCompileEnvironment CCompileEnvironment = new CppCompileEnvironment(ModuleCompileEnvironment);
				CCompileEnvironment.bUseUnity = bModuleUsesUnityBuild && Rules.UnityFileTypes.HasFlag(ModuleRules.UnityFileType.C);
				CreateHeaderForDefinitions(CCompileEnvironment, IntermediateDirectory, "c", Graph);

				if (CCompileEnvironment.bUseUnity)
				{
					Unity.GenerateUnitySource(Target, InputFiles.CFiles, InputFiles.HeaderFiles, CCompileEnvironment, WorkingSet, Rules.ShortName ?? Name, IntermediateDirectory, Graph, SourceFileToUnityFile,
						out List<FileItem> NormalFiles, out List<FileItem> AdaptiveFiles, NumIncludedBytesPerUnityCPP);
					LinkInputFiles.AddRange(CompileFilesWithToolChain(Target, ToolChain, CCompileEnvironment, ModuleCompileEnvironment, NormalFiles, AdaptiveFiles, Graph).ObjectFiles);
				}
				else
				{
					LinkInputFiles.AddRange(ToolChain.CompileAllCPPFiles(CCompileEnvironment, InputFiles.CFiles, IntermediateDirectory, Name, Graph).ObjectFiles);
				}
			}

			// Compile CC files
			if (InputFiles.CCFiles.Count > 0)
			{
				CppCompileEnvironment CCCompileEnvironment = new CppCompileEnvironment(CppCompileEnvironment);
				CCCompileEnvironment.bUseUnity = bModuleUsesUnityBuild && Rules.UnityFileTypes.HasFlag(ModuleRules.UnityFileType.CC);
				CreateHeaderForDefinitions(CCCompileEnvironment, IntermediateDirectory, "cc", Graph);

				if (CCCompileEnvironment.bUseUnity)
				{
					Unity.GenerateUnitySource(Target, InputFiles.CCFiles, InputFiles.HeaderFiles, CCCompileEnvironment, WorkingSet, Rules.ShortName ?? Name, IntermediateDirectory, Graph, SourceFileToUnityFile,
						out List<FileItem> NormalFiles, out List<FileItem> AdaptiveFiles, NumIncludedBytesPerUnityCPP);
					LinkInputFiles.AddRange(CompileFilesWithToolChain(Target, ToolChain, CCCompileEnvironment, CppCompileEnvironment, NormalFiles, AdaptiveFiles, Graph).ObjectFiles);
				}
				else
				{
					LinkInputFiles.AddRange(ToolChain.CompileAllCPPFiles(CCCompileEnvironment, InputFiles.CCFiles, IntermediateDirectory, Name, Graph).ObjectFiles);
				}
			}

			// Compile CXX files
			if (InputFiles.CXXFiles.Count > 0)
			{
				CppCompileEnvironment CXXCompileEnvironment = new CppCompileEnvironment(CppCompileEnvironment);
				CXXCompileEnvironment.bUseUnity = bModuleUsesUnityBuild && Rules.UnityFileTypes.HasFlag(ModuleRules.UnityFileType.CXX);
				CreateHeaderForDefinitions(CXXCompileEnvironment, IntermediateDirectory, "cxx", Graph);

				if (CXXCompileEnvironment.bUseUnity)
				{
					Unity.GenerateUnitySource(Target, InputFiles.CXXFiles, InputFiles.HeaderFiles, CXXCompileEnvironment, WorkingSet, Rules.ShortName ?? Name, IntermediateDirectory, Graph, SourceFileToUnityFile,
						out List<FileItem> NormalFiles, out List<FileItem> AdaptiveFiles, NumIncludedBytesPerUnityCPP);
					LinkInputFiles.AddRange(CompileFilesWithToolChain(Target, ToolChain, CXXCompileEnvironment, CppCompileEnvironment, NormalFiles, AdaptiveFiles, Graph).ObjectFiles);
				}
				else
				{
					LinkInputFiles.AddRange(ToolChain.CompileAllCPPFiles(CXXCompileEnvironment, InputFiles.CXXFiles, IntermediateDirectory, Name, Graph).ObjectFiles);
				}
			}

			// Compile M files. Do not use a PCH here, because a C++ PCH is not compatible with Objective-C source files.
			if (InputFiles.MFiles.Count > 0)
			{
				CppCompileEnvironment MCompileEnvironment = new CppCompileEnvironment(ModuleCompileEnvironment);
				MCompileEnvironment.bUseUnity = bModuleUsesUnityBuild && Rules.UnityFileTypes.HasFlag(ModuleRules.UnityFileType.M);
				CreateHeaderForDefinitions(MCompileEnvironment, IntermediateDirectory, "m", Graph);

				if (MCompileEnvironment.bUseUnity)
				{
					Unity.GenerateUnitySource(Target, InputFiles.MFiles, InputFiles.HeaderFiles, MCompileEnvironment, WorkingSet, Rules.ShortName ?? Name, IntermediateDirectory, Graph, SourceFileToUnityFile,
						out List<FileItem> NormalFiles, out List<FileItem> AdaptiveFiles, NumIncludedBytesPerUnityCPP);
					LinkInputFiles.AddRange(CompileFilesWithToolChain(Target, ToolChain, MCompileEnvironment, ModuleCompileEnvironment, NormalFiles, AdaptiveFiles, Graph).ObjectFiles);
				}
				else
				{
					LinkInputFiles.AddRange(ToolChain.CompileAllCPPFiles(MCompileEnvironment, InputFiles.MFiles, IntermediateDirectory, Name, Graph).ObjectFiles);
				}
			}

			// Compile MM files
			if (InputFiles.MMFiles.Count > 0)
			{
				CppCompileEnvironment MMCompileEnvironment = new CppCompileEnvironment(CppCompileEnvironment);
				MMCompileEnvironment.bUseUnity = bModuleUsesUnityBuild && Rules.UnityFileTypes.HasFlag(ModuleRules.UnityFileType.MM);
				CreateHeaderForDefinitions(MMCompileEnvironment, IntermediateDirectory, "mm", Graph);

				if (MMCompileEnvironment.bUseUnity)
				{
					Unity.GenerateUnitySource(Target, InputFiles.MMFiles, InputFiles.HeaderFiles, MMCompileEnvironment, WorkingSet, Rules.ShortName ?? Name, IntermediateDirectory, Graph, SourceFileToUnityFile,
						out List<FileItem> NormalFiles, out List<FileItem> AdaptiveFiles, NumIncludedBytesPerUnityCPP);
					LinkInputFiles.AddRange(CompileFilesWithToolChain(Target, ToolChain, MMCompileEnvironment, CppCompileEnvironment, NormalFiles, AdaptiveFiles, Graph).ObjectFiles);
				}
				else
				{
					LinkInputFiles.AddRange(ToolChain.CompileAllCPPFiles(MMCompileEnvironment, InputFiles.MMFiles, IntermediateDirectory, Name, Graph).ObjectFiles);
				}
			}

			// RC files for all modules in a binary will be compiled in UEBuildBinary.cs using the BinaryCompileEnvironment
			ResourceFiles.UnionWith(InputFiles.RCFiles);

			// Write the compiled manifest
			if (Rules.bPrecompile && Target.LinkType == TargetLinkType.Monolithic)
			{
				DirectoryReference.CreateDirectory(PrecompiledManifestLocation.Directory);

				PrecompiledManifest Manifest = new PrecompiledManifest();
				Manifest.OutputFiles.AddRange(LinkInputFiles.Select(x => x.Location));

				// Store MSVC toolchain version to detect ABI mismatches
				if (Target.Platform.IsInGroup(UnrealPlatformGroup.Microsoft) && Target.WindowsPlatform.Compiler.IsMSVC())
				{
					Manifest.ToolChainVersion = Target.WindowsPlatform.Environment?.ToolChainVersion;
				}

				Manifest.WriteIfModified(PrecompiledManifestLocation);
			}

			// Write a mapping of unity object file to standalone object file for live coding
			if (Rules.Target.bWithLiveCoding)
			{
				StringWriter StringWriter = new();
				using (JsonWriter Writer = new JsonWriter(StringWriter))
				{
					Writer.WriteObjectStart();
					Writer.WriteObjectStart("RemapUnityFiles");
					foreach (IGrouping<FileItem, KeyValuePair<FileItem, FileItem>> UnityGroup in SourceFileToUnityFile.GroupBy(x => x.Value))
					{
						Writer.WriteArrayStart(UnityGroup.Key.Location.GetFileName() + ".obj");
						foreach (FileItem SourceFile in UnityGroup.Select(x => x.Key))
						{
							Writer.WriteValue(SourceFile.Location.GetFileName() + ".obj");
						}
						Writer.WriteArrayEnd();
					}
					Writer.WriteObjectEnd();
					Writer.WriteObjectEnd();
				}

				FileReference UnityManifestFile = FileReference.Combine(IntermediateDirectory, "LiveCodingInfo.json");
				Graph.CreateIntermediateTextFile(UnityManifestFile, StringWriter.ToString());
			}

			// IWYU needs to build all headers separate from cpp files to produce proper recommendations for includes
			if (Target.bIncludeHeaders)
			{
				if (Target.bHeadersOnly)
				{
					LinkInputFiles.Clear();
				}

				// Collect the headers that should be built
				List<FileItem> HeaderFileItems = GetCompilableHeaders(InputFiles, CppCompileEnvironment);
				if (HeaderFileItems.Count > 0)
				{
					CppCompileEnvironment HeaderCompileEnvironment = new(CppCompileEnvironment);
					HeaderCompileEnvironment.bUseUnity = false;

					DirectoryReference HeaderIntermediateDirectory = DirectoryReference.Combine(IntermediateDirectory, "H");
					CreateHeaderForDefinitions(HeaderCompileEnvironment, HeaderIntermediateDirectory, "h", Graph);

					// Duplicate named headers are allowed, so adjust the IntermediateDirectory to compensate for this by using an index
					int DirectoryIndex = 0;
					foreach (DirectoryItem ParentDir in HeaderFileItems.Select(x => x.Directory!).Distinct().OrderBy(x => x.FullName))
					{
						DirectoryReference HeaderSubIntermediateDirectory = DirectoryReference.Combine(HeaderIntermediateDirectory, (DirectoryIndex++).ToString());
						List<FileItem> HeadersToCompile = new();
						foreach (FileItem Header in HeaderFileItems.Where(x => x.Directory == ParentDir))
						{
							string IncludeFileString = Header.AbsolutePath;
							if (HeaderCompileEnvironment.RootPaths.GetVfsOverlayPath(Header.Location, out string? VfsPath))
							{
								IncludeFileString = VfsPath;
							}
							else if (Header.Location.IsUnderDirectory(Unreal.RootDirectory))
							{
								IncludeFileString = Header.Location.MakeRelativeTo(Unreal.EngineSourceDirectory);
							}

							List<string> GeneratedHeaderCppContents = UEBuildModuleCPP.GenerateHeaderCpp(Header.Name, IncludeFileString);
							FileItem GeneratedHeaderCpp = FileItem.GetItemByFileReference(FileReference.Combine(HeaderSubIntermediateDirectory, $"{Header.Name}.cpp"));
							Graph.CreateIntermediateTextFile(GeneratedHeaderCpp, GeneratedHeaderCppContents);
							HeadersToCompile.Add(GeneratedHeaderCpp);
						}

						// Add the compile actions
						LinkInputFiles.AddRange(ToolChain.CompileAllCPPFiles(HeaderCompileEnvironment, HeadersToCompile, HeaderSubIntermediateDirectory, Name, Graph).ObjectFiles);
					}
				}
			}

			return LinkInputFiles;
		}

		List<FileItem> GetCompilableHeaders(InputFileCollection InputFiles, CppCompileEnvironment CompileEnvironment)
		{
			if (Rules.IWYUSupport == IWYUSupport.None)
			{
				return new List<FileItem>();
			}

			// Find FileItems for module's pch files
			FileItem? PrivatePchFileItem = null;
			if (Rules.PrivatePCHHeaderFile != null)
			{
				PrivatePchFileItem = FileItem.GetItemByFileReference(FileReference.Combine(ModuleDirectory, Rules.PrivatePCHHeaderFile));
			}

			FileItem? SharedPchFileItem = null;
			if (Rules.SharedPCHHeaderFile != null)
			{
				SharedPchFileItem = FileItem.GetItemByFileReference(FileReference.Combine(ModuleDirectory, Rules.SharedPCHHeaderFile));
			}

			Dictionary<string, FileItem> NameToFileItem = new();

			HashSet<FileItem> CollidingFiles = new();

			// Find all the platform folders to exclude from the list of valid modules
			IReadOnlySet<string> ExcludeFolders = UEBuildPlatform.GetBuildPlatform(Rules.Target.Platform).GetExcludedFolderNames();

			// Collect the headers that should be built
			List<FileItem> HeaderFileItems = new();
			foreach (FileItem HeaderFileItem in InputFiles.HeaderFiles)
			{
				// Skip headers for excluded platforms
				if (HeaderFileItem.Location.ContainsAnyNames(ExcludeFolders, Unreal.EngineDirectory))
				{
					continue;
				}

				// We don't want to build pch files in iwyu, skip those.
				if (HeaderFileItem == PrivatePchFileItem || HeaderFileItem == SharedPchFileItem)
				{
					continue;
				}

				// If file is skipped by header units it means they can't be compiled by themselves and we must skip them too
				if (CompileEnvironment.MetadataCache.GetHeaderUnitType(HeaderFileItem) != HeaderUnitType.Valid)
				{
					continue;
				}

				if (!NameToFileItem.TryAdd(HeaderFileItem.Name, HeaderFileItem))
				{
					CollidingFiles.Add(NameToFileItem[HeaderFileItem.Name]);
					CollidingFiles.Add(HeaderFileItem);
				}

				HeaderFileItems.Add(HeaderFileItem);
			}

			if (CollidingFiles.Count != 0)
			{
				CompileEnvironment.CollidingNames = CollidingFiles;
			}

			return HeaderFileItems;
		}

		static readonly ConcurrentDictionary<string, Lazy<HashSet<string>>> s_sharedPchModuleDependenciesCache = [];

		/// <summary>
		/// Gets the list of module dependencies that this module declares and then tries to optimize the list to what is actually used by the header.
		/// </summary>
		/// <param name="CompileEnvironment">Compile environment for this PCH</param>
		/// <param name="PCHHeaderFile">PCH header file</param>
		/// <param name="Logger">Logger for output</param>
		/// <returns>Returns an optimized list of modules this module's shared PCH uses</returns>
		private HashSet<UEBuildModule> GetSharedPCHModuleDependencies(CppCompileEnvironment CompileEnvironment, FileItem PCHHeaderFile, ILogger Logger)
		{
			Dictionary<UEBuildModule, bool> ModuleToIncludePathsOnlyFlag = new();
			FindModulesInPublicCompileEnvironment(ModuleToIncludePathsOnlyFlag);

			string key = PCHHeaderFile.FullName + CompileEnvironment.Platform;
			HashSet<string> moduleNames = s_sharedPchModuleDependenciesCache.GetOrAdd(key, (k) => new Lazy<HashSet<string>>(() =>
			{
				HashSet<UEBuildModule> modules = GetSharedPCHModuleDependenciesUncached(CompileEnvironment, PCHHeaderFile, Logger, ModuleToIncludePathsOnlyFlag);
				return modules.Select(m => m.Name).ToHashSet();
			})).Value;

			HashSet<UEBuildModule> modules = [];
			foreach (UEBuildModule module in ModuleToIncludePathsOnlyFlag.Keys)
			{
				if (moduleNames.Contains(module.Name))
				{
					modules.Add(module);
				}
			}

			// This can actually diff because Engine Shared Pch in editor builds can reach UnrealEd includes which is bad
			if (modules.Count != moduleNames.Count)
			{
				throw new Exception($"This should never happen. {PCHHeaderFile} is found with different number of module dependencies");
			}

			return modules;

			//return GetSharedPCHModuleDependenciesUncached(CompileEnvironment, PCHHeaderFile, Logger, ModuleToIncludePathsOnlyFlag);
		}

		private HashSet<UEBuildModule> GetSharedPCHModuleDependenciesUncached(CppCompileEnvironment CompileEnvironment, FileItem PCHHeaderFile, ILogger Logger, Dictionary<UEBuildModule, bool> ModuleToIncludePathsOnlyFlag)
		{
			HashSet<UEBuildModule> AllModuleDeps = ModuleToIncludePathsOnlyFlag.Keys.ToHashSet();
			HashSet<UEBuildModule> ModuleDeps = new HashSet<UEBuildModule>(AllModuleDeps) { this };

			using Pooled<HashSet<string>> rented = HashSetPool<string>.Rent();
			HashSet<string> VisitedIncludes = rented.Value;

			HashSet<FileItem> FoundIncludeFileItems = new();

			using Pooled<HashSet<string>> rented3 = HashSetPool<string>.Rent();
			HashSet<string> IncludesNotFound = rented3.Value;

			// Go through user directories and create bloom filter for files and dirs so we can early out quickly
			List<(DirectoryItem, BloomFilter, BloomFilter)> UserIncludePaths = new(CompileEnvironment.UserIncludePaths.Count);
			foreach (DirectoryReference UserInclude in CompileEnvironment.UserIncludePaths)
			{
				DirectoryItem dirItem = DirectoryItem.GetItemByDirectoryReference(UserInclude);
				BloomFilter dirFilter = new();
				BloomFilter fileFilter = new();
				foreach (DirectoryItem dir in dirItem.EnumerateDirectories())
				{
					dirFilter.Add(dir.Name);
				}

				foreach (FileItem file in dirItem.EnumerateFiles())
				{
					fileFilter.Add(file.Name);
				}

				UserIncludePaths.Add((dirItem, dirFilter, fileFilter));
			}

			// First, traverse _all_ includes we can reach from the pch header file and store them in FoundIncludeFileItems
			// This can be cached per-platform since we ignore things like ifdef DEBUG, SHIPPING etc.
			// Reason we want to cache this is because

			Stack<(FileItem, List<string>)> stack = new();

			stack.Push((PCHHeaderFile, CompileEnvironment.MetadataCache.GetHeaderIncludes(PCHHeaderFile)));

			while (stack.Count != 0)
			{
				(FileItem FileToSearch, List<string> HeaderIncludes) = stack.Pop();
				foreach (string HeaderInclude in HeaderIncludes)
				{
					// Skip .generated.h. They should always be in the same module as the header including them.. so no need to track them
					if (HeaderInclude.Contains(".generated.h", StringComparison.Ordinal))
					{
						continue;
					}

					string TransformedHeaderInclude = HeaderInclude;
					if (TransformedHeaderInclude.Contains("COMPILED_PLATFORM_HEADER(", StringComparison.Ordinal))
					{
						string PlatformName = UEBuildPlatform.GetBuildPlatform(CompileEnvironment.Platform).GetPlatformName();
						TransformedHeaderInclude = TransformedHeaderInclude.Replace("COMPILED_PLATFORM_HEADER(", PlatformName + "/" + PlatformName, StringComparison.Ordinal).TrimEnd(')');
					}

					if (VisitedIncludes.Add(TransformedHeaderInclude))
					{
						FileItem? SearchForFileItem(DirectoryItem dir)
						{
							FileReference FileRef = FileReference.Combine(dir.Location, TransformedHeaderInclude);
							DirectoryItem dirItem = DirectoryItem.GetItemByDirectoryReference(FileRef.Directory);
							dirItem.TryGetFile(FileRef.GetFileName(), out FileItem? fileItem);
							return fileItem;
						}
						;

						// check to see if the file is a relative path first
						FileItem? IncludeFileItem = SearchForFileItem(FileToSearch.Directory);

						// search through the include paths if the file isn't relative
						if (IncludeFileItem == null)
						{
							foreach ((DirectoryItem IncludePath, BloomFilter dirFilter, BloomFilter fileFilter) in UserIncludePaths)
							{
								ReadOnlySpan<char> RelativePath = TransformedHeaderInclude.AsSpan();
								int firstSlash = RelativePath.IndexOf('/');
								if (firstSlash != -1)
								{
									if (!dirFilter.MightContain(RelativePath.Slice(0, firstSlash)))
									{
										continue;
									}
								}
								else
								{
									if (!fileFilter.MightContain(RelativePath))
									{
										continue;
									}
								}

								IncludeFileItem = SearchForFileItem(IncludePath);
								if (IncludeFileItem != null)
								{
									break;
								}
							}
						}

						if (IncludeFileItem != null)
						{
							FoundIncludeFileItems.Add(IncludeFileItem);
							// Logger.LogDebug("{0} SharedPCH - '{1}' included '{2}'.", Name, FileToSearch.Location, IncludeFileItem.Location);
							stack.Push((IncludeFileItem, CompileEnvironment.MetadataCache.GetHeaderIncludes(IncludeFileItem)));
						}
						else
						{
							if (!TransformedHeaderInclude.Contains('.', StringComparison.Ordinal))
							{
#if LOG_PCH_PARSING
								Logger.LogDebug("{0} SharedPCH - Skipping '{1}' found in '{2}' because it doesn't appear to be a module header.", Name, TransformedHeaderInclude, FileToSearch.Location);
#endif
							}
							else if (TransformedHeaderInclude.EndsWith(".generated.h", StringComparison.Ordinal))
							{
#if LOG_PCH_PARSING
								Logger.LogDebug("{0} SharedPCH - Skipping '{1}' found in '{2}' because it appears to be a generated UHT header.", Name, TransformedHeaderInclude, FileToSearch.Location);
#endif
							}
							else
							{
#if LOG_PCH_PARSING
								Logger.LogDebug("{0} SharedPCH - Could not find include directory for '{1}' found in '{2}'.", Name, TransformedHeaderInclude, FileToSearch.Location);
#endif
								IncludesNotFound.Add(TransformedHeaderInclude);
							}
						}
					}
				}
			}

			bool FoundAllModules = true;
			HashSet<UEBuildModule> OptModules = new();
			foreach (FileItem IncludeFile in FoundIncludeFileItems)
			{
				if (CompileEnvironment.MetadataCache.UsesAPIDefine(IncludeFile) || CompileEnvironment.MetadataCache.ContainsReflectionMarkup(IncludeFile))
				{
					UEBuildModule? FoundModule = ModuleDeps.FirstOrDefault(Module => Module.ContainsFile(IncludeFile.Location));
					if (FoundModule != null)
					{
#if LOG_PCH_PARSING
						if (ModuleToIncludePathsOnlyFlag[FoundModule])
						{
							Logger.LogDebug("{0} SharedPCH - '{1}' is exporting types but '{2}' isn't declared as a public dependency.", Name, IncludeFile.Location, FoundModule);
						}
#endif

						OptModules.Add(FoundModule);
					}
					else
					{
#if LOG_PCH_PARSING
						Logger.LogDebug("{0} SharedPCH - '{1}' is exporting types but the module this file belongs to isn't declared as a public dependency or include.", Name, IncludeFile.Location);
#endif
						FoundAllModules = false;
					}
				}
				else
				{
#if LOG_PCH_PARSING
					Logger.LogDebug("{0} SharedPCH - '{1}' is not exporting types so we are ignoring the dependency", Name, IncludeFile.Location);
#endif
				}
			}

			if (!FoundAllModules)
			{
#if LOG_PCH_PARSING
				Logger.LogDebug("{0} SharedPCH - Is missing public dependencies. To be safe, this shared PCH will fall back to use all the module dependencies. Note that this could affect compile times.", Name);
#endif
				return AllModuleDeps;
			}
			else
			{
				return OptModules;
			}
		}

		/// <summary>
		/// Create a shared PCH template for this module, which allows constructing shared PCH instances in the future
		/// </summary>
		/// <param name="Target">The target which owns this module</param>
		/// <param name="BaseCompileEnvironment">Base compile environment for this target</param>
		/// <param name="Logger">Logger for output</param>
		/// <returns>Template for shared PCHs</returns>
		public PrecompiledHeaderTemplate CreateSharedPCHTemplate(UEBuildTarget Target, CppCompileEnvironment BaseCompileEnvironment, ILogger Logger)
		{
			CppCompileEnvironment CompileEnvironment = CreateSharedPCHCompileEnvironment(Target, BaseCompileEnvironment);
			FileItem HeaderFile = FileItem.GetItemByFileReference(FileReference.Combine(ModuleDirectory, Rules.SharedPCHHeaderFile!));

			DirectoryReference PrecompiledHeaderDir;
			if (Rules.bUsePrecompiled)
			{
				PrecompiledHeaderDir = DirectoryReference.Combine(Target.ProjectIntermediateDirectory, Name);
			}
			else
			{
				PrecompiledHeaderDir = IntermediateDirectory;
			}

			return new PrecompiledHeaderTemplate(this, CompileEnvironment, HeaderFile, PrecompiledHeaderDir, GetSharedPCHModuleDependencies(CompileEnvironment, HeaderFile, Logger));
		}

		static HashSet<string> GetImmutableDefinitions(List<string> Definitions)
		{
			HashSet<string> ImmutableDefinitions = new();
			foreach (string Definition in Definitions)
			{
				if (Definition.Contains("UE_IS_ENGINE_MODULE", StringComparison.Ordinal) ||
					Definition.Contains("UE_VALIDATE_FORMAT_STRINGS", StringComparison.Ordinal) ||
					Definition.Contains("UE_VALIDATE_INTERNAL_API", StringComparison.Ordinal) ||
					Definition.Contains("UE_VALIDATE_EXPERIMENTAL_API", StringComparison.Ordinal) ||
					Definition.Contains("UE_DEPRECATED_FORGAME", StringComparison.Ordinal) ||
					Definition.Contains("UE_DEPRECATED_FORENGINE", StringComparison.Ordinal) ||
					Definition.Contains("UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_", StringComparison.Ordinal))
				{
					continue;
				}
				ImmutableDefinitions.Add(Definition);
			}
			return ImmutableDefinitions;
		}

		/// <summary>
		/// Creates a precompiled header action to generate a new pch file 
		/// </summary>
		/// <param name="ToolChain">The toolchain to generate the PCH</param>
		/// <param name="HeaderFile"></param>
		/// <param name="ModuleCompileEnvironment"></param>
		/// <param name="Graph">List of actions to be executed. Additional actions will be added to this list.</param>
		/// <returns>The created PCH instance.</returns>
		private PrecompiledHeaderInstance CreatePrivatePCH(UEToolChain? ToolChain, FileItem HeaderFile, CppCompileEnvironment ModuleCompileEnvironment, IActionGraphBuilder Graph)
		{
			// Create the wrapper file, which sets all the definitions needed to compile it
			FileItem DefinitionsFileItem = CreateHeaderForDefinitions(ModuleCompileEnvironment, IntermediateDirectory, null, Graph)!;
			FileReference WrapperLocation = FileReference.Combine(IntermediateDirectory, String.Format("PCH.{0}.h", Name));
			FileItem WrapperFile = CreatePCHWrapperFile(WrapperLocation, DefinitionsFileItem!, HeaderFile, Graph);

			// Create a new C++ environment that is used to create the PCH.
			CppCompileEnvironment CompileEnvironment = new CppCompileEnvironment(ModuleCompileEnvironment);
			CompileEnvironment.Definitions.Clear();
			CompileEnvironment.PrecompiledHeaderAction = PrecompiledHeaderAction.Create;
			CompileEnvironment.PrecompiledHeaderIncludeFilename = WrapperFile.Location;
			CompileEnvironment.bOptimizeCode = ModuleCompileEnvironment.bOptimizeCode;
			CompileEnvironment.bCodeCoverage = ModuleCompileEnvironment.bCodeCoverage;

			// Create the action to compile the PCH file.
			CPPOutput Output;
			if (ToolChain == null)
			{
				Output = new CPPOutput();
			}
			else
			{
				Output = ToolChain.CompileAllCPPFiles(CompileEnvironment, new List<FileItem>() { WrapperFile }, IntermediateDirectory, Name, Graph);
			}

			return new PrecompiledHeaderInstance(WrapperFile, DefinitionsFileItem, CompileEnvironment, Output, GetImmutableDefinitions(ModuleCompileEnvironment.Definitions));
		}

		/// <summary>
		/// Generates a precompiled header instance from the given template, or returns an existing one if it already exists
		/// </summary>
		/// <param name="ToolChain">The toolchain being used to build this module</param>
		/// <param name="Template">The PCH template</param>
		/// <param name="ModuleCompileEnvironment">Compile environment for the current module</param>
		/// <param name="Graph">List of actions to be executed. Additional actions will be added to this list.</param>
		/// <returns>Instance of a PCH</returns>
		public PrecompiledHeaderInstance FindOrCreateSharedPCH(UEToolChain? ToolChain, PrecompiledHeaderTemplate Template, CppCompileEnvironment ModuleCompileEnvironment, IActionGraphBuilder Graph)
		{
			lock (Template.Instances)
			{
				PrecompiledHeaderInstance? Instance = Template.Instances.Find(x => IsCompatibleForSharedPCH(x.CompileEnvironment, ModuleCompileEnvironment));
				if (Instance == null)
				{
					List<string> Definitions = Template.BaseCompileEnvironment.Definitions;

					// Modify definitions if we need to create a new shared pch for the include order
					if (ModuleCompileEnvironment.IncludeOrderVersion != Template.BaseCompileEnvironment.IncludeOrderVersion)
					{
						Definitions = new List<string>(Definitions);
						foreach (string OldDefine in EngineIncludeOrderHelper.GetDeprecationDefines(Template.BaseCompileEnvironment.IncludeOrderVersion))
						{
							Definitions.Remove(OldDefine);
						}
						Definitions.AddRange(EngineIncludeOrderHelper.GetDeprecationDefines(ModuleCompileEnvironment.IncludeOrderVersion));
					}

					// Modify definitions if we need to create a new shared pch for project modules
					if (ModuleCompileEnvironment.bTreatAsEngineModule != Template.BaseCompileEnvironment.bTreatAsEngineModule)
					{
						Definitions = new List<string>(Definitions);
						Definitions.RemoveAll(x => x.Contains("UE_IS_ENGINE_MODULE", StringComparison.Ordinal));
						Definitions.RemoveAll(x => x.Contains("UE_DEPRECATED_FORGAME", StringComparison.Ordinal));
						Definitions.RemoveAll(x => x.Contains("UE_DEPRECATED_FORENGINE", StringComparison.Ordinal));

						Definitions.Add($"UE_IS_ENGINE_MODULE={(Rules.bTreatAsEngineModule || Rules.Target.bWarnAboutMonolithicHeadersIncluded ? "1" : "0")}");
						Definitions.Add($"UE_DEPRECATED_FORGAME={(Rules.bTreatAsEngineModule ? "UE_EMPTY_FUNCTION" : "UE_DEPRECATED")}");
						Definitions.Add($"UE_DEPRECATED_FORENGINE={(Rules.bTreatAsEngineModule || Rules.Target.bDisableEngineDeprecations ? "UE_EMPTY_FUNCTION" : "UE_DEPRECATED")}");
					}

					// Modify definitions if we need to create a new shared pch for validating format strings
					if (ModuleCompileEnvironment.bValidateFormatStrings != Template.BaseCompileEnvironment.bValidateFormatStrings)
					{
						Definitions = new List<string>(Definitions);
						Definitions.RemoveAll(x => x.Contains("UE_VALIDATE_FORMAT_STRINGS", StringComparison.Ordinal));
						Definitions.Add($"UE_VALIDATE_FORMAT_STRINGS={(ModuleCompileEnvironment.bValidateFormatStrings ? "1" : "0")}");
					}

					// Modify definitions if we need to create a new shared pch for validating internal api usage
					if (ModuleCompileEnvironment.bValidateInternalApi != Template.BaseCompileEnvironment.bValidateInternalApi)
					{
						Definitions = new List<string>(Definitions);
						Definitions.RemoveAll(x => x.Contains("UE_VALIDATE_INTERNAL_API", StringComparison.Ordinal));
						Definitions.Add($"UE_VALIDATE_INTERNAL_API={(ModuleCompileEnvironment.bValidateInternalApi ? "1" : "0")}");
					}

					// Modify definitions if we need to create a new shared pch for validating experimental api usage
					if (ModuleCompileEnvironment.bValidateExperimentalApi != Template.BaseCompileEnvironment.bValidateExperimentalApi)
					{
						Definitions = new List<string>(Definitions);
						Definitions.RemoveAll(x => x.Contains("UE_VALIDATE_EXPERIMENTAL_API", StringComparison.Ordinal));
						Definitions.Add($"UE_VALIDATE_EXPERIMENTAL_API={(ModuleCompileEnvironment.bValidateExperimentalApi ? "1" : "0")}");
					}

					// Create a suffix to distinguish this shared PCH variant from any others.
					string Variant = GetSuffixForSharedPCH(ModuleCompileEnvironment, Template.BaseCompileEnvironment);

					FileReference SharedDefinitionsLocation = FileReference.Combine(Template.OutputDir, $"SharedDefinitions.{Template.Module.Name}{Variant}.h");
					List<string> NewDefinitions = new();
					StringBuilder Writer = new StringBuilder();
					WriteDefinitions($"Shared Definitions for {Template.Module.Name}{Variant}", [], Definitions, [], Writer);
					FileItem SharedDefinitionsFileItem = Graph.CreateIntermediateTextFile(SharedDefinitionsLocation, Writer.ToString(), AllowAsync: false);

					// Create the wrapper file, which sets all the definitions needed to compile it
					FileReference WrapperLocation = FileReference.Combine(Template.OutputDir, $"SharedPCH.{Template.Module.Name}{Variant}.h");
					FileItem WrapperFile = CreatePCHWrapperFile(WrapperLocation, SharedDefinitionsFileItem, Template.HeaderFile, Graph);

					// Create the compile environment for this PCH
					CppCompileEnvironment CompileEnvironment = new CppCompileEnvironment(Template.BaseCompileEnvironment);
					CompileEnvironment.Definitions.Clear();
					CompileEnvironment.PrecompiledHeaderAction = PrecompiledHeaderAction.Create;
					CompileEnvironment.PrecompiledHeaderIncludeFilename = WrapperFile.Location;
					CopySettingsForSharedPCH(ModuleCompileEnvironment, CompileEnvironment);

					// Setup PCH chaining
					PrecompiledHeaderInstance? ParentPCHInstance = null;
					if (Rules.Target.bChainPCHs)
					{
						// Find all the dependencies of this module
						using Pooled<HashSet<UEBuildModule>> rented = HashSetPool<UEBuildModule>.Rent();
						HashSet<UEBuildModule> ReferencedModules = rented.Value;
						Template.Module.GetAllDependencyModules(new List<UEBuildModule>(), ReferencedModules, bIncludeDynamicallyLoaded: false, bForceCircular: false, bOnlyDirectDependencies: false);

						PrecompiledHeaderTemplate? ParentTemplate = CompileEnvironment.SharedPCHs
							.Skip(CompileEnvironment.SharedPCHs.IndexOf(Template) + 1)
							.FirstOrDefault(x => ReferencedModules.Contains(x.Module) && x.IsValidFor(CompileEnvironment));
						if (ParentTemplate != null)
						{
							ParentPCHInstance = FindOrCreateSharedPCH(ToolChain, ParentTemplate, ModuleCompileEnvironment, Graph);
							CompileEnvironment.ParentPCHInstance = ParentPCHInstance;
						}
					}

					// Create the PCH
					CPPOutput Output;
					if (ToolChain == null)
					{
						Output = new CPPOutput();
					}
					else
					{
						Output = ToolChain.CompileAllCPPFiles(CompileEnvironment, new List<FileItem>() { WrapperFile }, Template.OutputDir, "Shared", Graph);
					}
					Instance = new PrecompiledHeaderInstance(WrapperFile, SharedDefinitionsFileItem, CompileEnvironment, Output, GetImmutableDefinitions(Template.BaseCompileEnvironment.Definitions), ParentPCHInstance);
					Template.Instances.Add(Instance);
				}

				Instance.Modules.Add(this);
				return Instance;
			}
		}

		/// <summary>
		/// Determines if a module compile environment is compatible with the given shared PCH compile environment
		/// </summary>
		/// <param name="ModuleCompileEnvironment">The module compile environment</param>
		/// <param name="CompileEnvironment">The shared PCH compile environment</param>
		/// <returns>True if the two compile environments are compatible</returns>
		internal static bool IsCompatibleForSharedPCH(CppCompileEnvironment ModuleCompileEnvironment, CppCompileEnvironment CompileEnvironment)
		{
			if (ModuleCompileEnvironment.bTreatAsEngineModule != CompileEnvironment.bTreatAsEngineModule)
			{
				return false;
			}

			if (ModuleCompileEnvironment.bRequiresPlatformSDK != CompileEnvironment.bRequiresPlatformSDK)
			{
				return false;
			}

			if (ModuleCompileEnvironment.bOptimizeCode != CompileEnvironment.bOptimizeCode)
			{
				return false;
			}
			else if (ModuleCompileEnvironment.bOptimizeCode && ModuleCompileEnvironment.OptimizationLevel != CompileEnvironment.OptimizationLevel)
			{
				return false;
			}

			if (ModuleCompileEnvironment.FPSemantics != CompileEnvironment.FPSemantics)
			{
				return false;
			}

			if (ModuleCompileEnvironment.bUseRTTI != CompileEnvironment.bUseRTTI)
			{
				return false;
			}

			if (ModuleCompileEnvironment.bEnableExceptions != CompileEnvironment.bEnableExceptions)
			{
				return false;
			}

			if (ModuleCompileEnvironment.CppStandard != CompileEnvironment.CppStandard)
			{
				return false;
			}

			if (ModuleCompileEnvironment.IncludeOrderVersion != CompileEnvironment.IncludeOrderVersion)
			{
				return false;
			}

			if (ModuleCompileEnvironment.bValidateFormatStrings != CompileEnvironment.bValidateFormatStrings)
			{
				return false;
			}

			if (ModuleCompileEnvironment.bValidateInternalApi != CompileEnvironment.bValidateInternalApi)
			{
				return false;
			}

			if (ModuleCompileEnvironment.bValidateExperimentalApi != CompileEnvironment.bValidateExperimentalApi)
			{
				return false;
			}

			if (ModuleCompileEnvironment.bEnableAutoRTFMInstrumentation != CompileEnvironment.bEnableAutoRTFMInstrumentation)
			{
				return false;
			}

			return true;
		}

		/// <summary>
		/// Gets the unique suffix for a shared PCH
		/// </summary>
		/// <param name="CompileEnvironment">The shared PCH compile environment</param>
		/// <param name="BaseCompileEnvironment">The base compile environment</param>
		/// <returns>The unique suffix for the shared PCH</returns>
		private static string GetSuffixForSharedPCH(CppCompileEnvironment CompileEnvironment, CppCompileEnvironment BaseCompileEnvironment)
		{
			string Variant = "";
			if (CompileEnvironment.bTreatAsEngineModule != BaseCompileEnvironment.bTreatAsEngineModule)
			{
				if (CompileEnvironment.bTreatAsEngineModule)
				{
					Variant += ".Engine";
				}
				else
				{
					Variant += ".Project";
				}
			}
			if (CompileEnvironment.bRequiresPlatformSDK != BaseCompileEnvironment.bRequiresPlatformSDK)
			{
				if (CompileEnvironment.bRequiresPlatformSDK)
				{
					Variant += ".Plat";
				}
				else
				{
					Variant += ".NonPlat";
				}
			}
			if (CompileEnvironment.bOptimizeCode != BaseCompileEnvironment.bOptimizeCode)
			{
				if (CompileEnvironment.bOptimizeCode)
				{
					Variant += $".Opt{CompileEnvironment.OptimizationLevel}";
				}
				else
				{
					Variant += ".NonOpt";
				}
			}
			else if (CompileEnvironment.bOptimizeCode && CompileEnvironment.OptimizationLevel != BaseCompileEnvironment.OptimizationLevel)
			{
				Variant += $".Opt{CompileEnvironment.OptimizationLevel}";
			}
			if (CompileEnvironment.FPSemantics != BaseCompileEnvironment.FPSemantics)
			{
				Variant += $".FP{CompileEnvironment.FPSemantics}";
			}
			if (CompileEnvironment.bUseRTTI != BaseCompileEnvironment.bUseRTTI)
			{
				if (CompileEnvironment.bUseRTTI)
				{
					Variant += ".RTTI";
				}
				else
				{
					Variant += ".NonRTTI";
				}
			}
			if (CompileEnvironment.bEnableAutoRTFMInstrumentation != BaseCompileEnvironment.bEnableAutoRTFMInstrumentation)
			{
				if (CompileEnvironment.bEnableAutoRTFMInstrumentation)
				{
					Variant += ".AutoRTFM";
				}
				else
				{
					Variant += ".NonAutoRTFM";
				}
			}
			if (CompileEnvironment.bEnableExceptions != BaseCompileEnvironment.bEnableExceptions)
			{
				if (CompileEnvironment.bEnableExceptions)
				{
					Variant += ".Exceptions";
				}
				else
				{
					Variant += ".NoExceptions";
				}
			}
			if (CompileEnvironment.bValidateFormatStrings != BaseCompileEnvironment.bValidateFormatStrings)
			{
				if (CompileEnvironment.bValidateFormatStrings)
				{
					Variant += ".ValFmtStr";
				}
				else
				{
					Variant += ".NoValFmtStr";
				}
			}

			if (CompileEnvironment.bValidateInternalApi != BaseCompileEnvironment.bValidateInternalApi)
			{
				if (CompileEnvironment.bValidateInternalApi)
				{
					Variant += ".ValApi";
				}
				else
				{
					Variant += ".NoValApi";
				}
			}

			if (CompileEnvironment.bValidateExperimentalApi != BaseCompileEnvironment.bValidateExperimentalApi)
			{
				if (CompileEnvironment.bValidateExperimentalApi)
				{
					Variant += ".ValExpApi";
				}
				else
				{
					Variant += ".NoValExpApi";
				}
			}

			if (CompileEnvironment.bDeterministic != BaseCompileEnvironment.bDeterministic)
			{
				if (CompileEnvironment.bDeterministic)
				{
					Variant += ".Determ";
				}
				else
				{
					Variant += ".NonDeterm";
				}
			}

			// Always add the cpp standard into the filename to prevent variants when the engine standard is mismatched with a project
			switch (CompileEnvironment.CppStandard)
			{
				case CppStandardVersion.Cpp20: Variant += ".Cpp20"; break;
				case CppStandardVersion.Cpp23: Variant += ".Cpp23"; break;
				case CppStandardVersion.Latest: Variant += ".CppLatest"; break;
			}

			if (CompileEnvironment.IncludeOrderVersion != BaseCompileEnvironment.IncludeOrderVersion)
			{
				if (CompileEnvironment.IncludeOrderVersion != EngineIncludeOrderVersion.Latest)
				{
					Variant += ".InclOrder" + CompileEnvironment.IncludeOrderVersion.ToString();
				}
			}

			return Variant;
		}

		/// <summary>
		/// Copy settings from the module's compile environment into the environment for the shared PCH
		/// </summary>
		/// <param name="ModuleCompileEnvironment">The module compile environment</param>
		/// <param name="CompileEnvironment">The shared PCH compile environment</param>
		private static void CopySettingsForSharedPCH(CppCompileEnvironment ModuleCompileEnvironment, CppCompileEnvironment CompileEnvironment)
		{
			CompileEnvironment.bOptimizeCode = ModuleCompileEnvironment.bOptimizeCode;
			CompileEnvironment.OptimizationLevel = ModuleCompileEnvironment.OptimizationLevel;
			CompileEnvironment.FPSemantics = ModuleCompileEnvironment.FPSemantics;
			CompileEnvironment.bCodeCoverage = ModuleCompileEnvironment.bCodeCoverage;
			CompileEnvironment.bUseRTTI = ModuleCompileEnvironment.bUseRTTI;
			CompileEnvironment.bEnableExceptions = ModuleCompileEnvironment.bEnableExceptions;
			CompileEnvironment.bTreatAsEngineModule = ModuleCompileEnvironment.bTreatAsEngineModule;
			CompileEnvironment.CppStandardEngine = ModuleCompileEnvironment.CppStandardEngine;
			CompileEnvironment.CppStandard = ModuleCompileEnvironment.CppStandard;
			CompileEnvironment.IncludeOrderVersion = ModuleCompileEnvironment.IncludeOrderVersion;
			CompileEnvironment.bUseAutoRTFMCompiler = ModuleCompileEnvironment.bUseAutoRTFMCompiler;
			CompileEnvironment.bDisableAutoRTFMInstrumentation = ModuleCompileEnvironment.bDisableAutoRTFMInstrumentation;
			CompileEnvironment.bEnableAutoRTFMVerification = ModuleCompileEnvironment.bEnableAutoRTFMVerification;
			CompileEnvironment.bAutoRTFMVerify = ModuleCompileEnvironment.bAutoRTFMVerify;
			CompileEnvironment.bAutoRTFMClosedStaticLinkage = ModuleCompileEnvironment.bAutoRTFMClosedStaticLinkage;
			CompileEnvironment.bAutoRTFMRedirectionInstrumentation = ModuleCompileEnvironment.bAutoRTFMRedirectionInstrumentation;
			CompileEnvironment.bValidateFormatStrings = ModuleCompileEnvironment.bValidateFormatStrings;
			CompileEnvironment.bValidateInternalApi = ModuleCompileEnvironment.bValidateInternalApi;
			CompileEnvironment.bValidateExperimentalApi = ModuleCompileEnvironment.bValidateExperimentalApi;
			CompileEnvironment.bRequiresPlatformSDK = ModuleCompileEnvironment.bRequiresPlatformSDK;
			CompileEnvironment.bUseUnity = ModuleCompileEnvironment.bUseUnity; // Used by cache bucket. We want shared pch in same bucket
		}

		/// <summary>
		/// Compiles the provided source files. Will possibly update environment settings for adaptive files if any are provided.
		/// </summary>
		private CPPOutput CompileFilesWithToolChain(
			ReadOnlyTargetRules Target,
			UEToolChain ToolChain,
			CppCompileEnvironment CompileEnvironment,
			CppCompileEnvironment ModuleCompileEnvironment,
			List<FileItem> NormalFiles,
			List<FileItem> AdaptiveFiles,
			IActionGraphBuilder Graph)
		{
			bool isEngine = Rules.File.IsUnderDirectory(Unreal.EngineDirectory) || Rules.bTreatAsEngineModule || Rules.PrivatePCHHeaderFile == null;
			bool bAdaptiveUnityDisablesPCH = isEngine ? Target.bAdaptiveUnityDisablesPCH : Target.bAdaptiveUnityDisablesPCHForProject;
			string adaptiveUnityDisablesPCHName = isEngine ? nameof(Target.bAdaptiveUnityDisablesPCH) : nameof(Target.bAdaptiveUnityDisablesPCHForProject);

			CPPOutput OutputFiles = new CPPOutput();

			if (NormalFiles.Count > 0)
			{
				OutputFiles = ToolChain.CompileAllCPPFiles(CompileEnvironment, NormalFiles, IntermediateDirectory, Name, Graph);
			}

			if (AdaptiveFiles.Count > 0)
			{
				if (CompileEnvironment.PrecompiledHeaderAction == PrecompiledHeaderAction.Include)
				{
					if (Target.bAdaptiveUnityCreatesDedicatedPCH)
					{
						Graph.AddDiagnostic("[Adaptive Build] Creating dedicated PCH for each excluded file. Set bAdaptiveUnityCreatesDedicatedPCH to false in BuildConfiguration.xml to change this behavior.");
					}
					else if (bAdaptiveUnityDisablesPCH)
					{
						Graph.AddDiagnostic($"[Adaptive Build] Disabling PCH for excluded files. Set {adaptiveUnityDisablesPCHName} to false in BuildConfiguration.xml to change this behavior.");
					}
				}

				if (Target.bAdaptiveUnityDisablesOptimizations)
				{
					Graph.AddDiagnostic("[Adaptive Build] Disabling optimizations for excluded files. Set bAdaptiveUnityDisablesOptimizations to false in BuildConfiguration.xml to change this behavior.");
				}
				else if (Target.AdaptiveUnityDisablesOptimizationsConfigurations.Contains(Target.Configuration))
				{
					Graph.AddDiagnostic($"[Adaptive Build] Disabling optimizations for excluded files in {Target.Configuration} configuration. Adjust {nameof(TargetRules.AdaptiveUnityDisablesOptimizationsConfigurations)} in BuildConfiguration.xml to change this behavior.");
				}
				if (Target.bAdaptiveUnityEnablesEditAndContinue)
				{
					Graph.AddDiagnostic("[Adaptive Build] Enabling Edit & Continue for excluded files. Set bAdaptiveUnityEnablesEditAndContinue to false in BuildConfiguration.xml to change this behavior.");
				}

				Graph.AddDiagnostic($"[Adaptive Build] Excluded from {Name} unity file: " + String.Join(", ", AdaptiveFiles.Select(File => Path.GetFileName(File.AbsolutePath))));

				// Create the new compile environment. Always turn off PCH due to different compiler settings.
				CppCompileEnvironment AdaptiveUnityEnvironment = new CppCompileEnvironment(ModuleCompileEnvironment);
				AdaptiveUnityEnvironment.AdditionalPrerequisites = new(AdaptiveUnityEnvironment.AdditionalPrerequisites.Concat(CompileEnvironment.AdditionalPrerequisites).Distinct());
				AdaptiveUnityEnvironment.UserIncludePaths.UnionWith(CompileEnvironment.SharedUserIncludePaths);
				AdaptiveUnityEnvironment.bDisableStaticAnalysis = CompileEnvironment.bDisableStaticAnalysis;
				if (Target.bAdaptiveUnityDisablesOptimizations)
				{
					AdaptiveUnityEnvironment.bOptimizeCode = false;
				}
				if (Target.AdaptiveUnityDisablesOptimizationsConfigurations.Contains(Target.Configuration))
				{
					AdaptiveUnityEnvironment.bOptimizeCode = false;
				}
				if (Target.bAdaptiveUnityEnablesEditAndContinue)
				{
					AdaptiveUnityEnvironment.bSupportEditAndContinue = true;
				}

				// Create a per-file PCH
				CPPOutput AdaptiveOutput;
				if (CompileEnvironment.PrecompiledHeaderAction == PrecompiledHeaderAction.Include && Target.bAdaptiveUnityCreatesDedicatedPCH)
				{
					AdaptiveOutput = CompileAdaptiveNonUnityFilesWithDedicatedPCH(ToolChain, AdaptiveUnityEnvironment, AdaptiveFiles, IntermediateDirectory, Name, Graph);
				}
				else if (CompileEnvironment.PrecompiledHeaderAction == PrecompiledHeaderAction.Include && bAdaptiveUnityDisablesPCH)
				{
					AdaptiveOutput = CompileAdaptiveNonUnityFilesWithoutPCH(ToolChain, AdaptiveUnityEnvironment, AdaptiveFiles, IntermediateDirectory, Name, Graph);
				}
				else if (AdaptiveUnityEnvironment.bOptimizeCode != CompileEnvironment.bOptimizeCode || AdaptiveUnityEnvironment.bSupportEditAndContinue != CompileEnvironment.bSupportEditAndContinue)
				{
					AdaptiveOutput = CompileAdaptiveNonUnityFiles(ToolChain, AdaptiveUnityEnvironment, AdaptiveFiles, IntermediateDirectory, Name, Graph);
				}
				else
				{
					AdaptiveOutput = CompileAdaptiveNonUnityFiles(ToolChain, CompileEnvironment, AdaptiveFiles, IntermediateDirectory, Name, Graph);
				}

				// Merge output
				OutputFiles.ObjectFiles.AddRange(AdaptiveOutput.ObjectFiles);
			}

			return OutputFiles;
		}

		static CPPOutput CompileAdaptiveNonUnityFiles(UEToolChain ToolChain, CppCompileEnvironment CompileEnvironment, List<FileItem> Files, DirectoryReference IntermediateDirectory, string ModuleName, IActionGraphBuilder Graph)
		{
			// Write all the definitions out to a separate file
			CreateHeaderForDefinitions(CompileEnvironment, IntermediateDirectory, "Adaptive", Graph);

			// Compile the files
			return ToolChain.CompileAllCPPFiles(CompileEnvironment, Files, IntermediateDirectory, ModuleName, Graph);
		}

		static CPPOutput CompileAdaptiveNonUnityFilesWithoutPCH(UEToolChain ToolChain, CppCompileEnvironment CompileEnvironment, List<FileItem> Files, DirectoryReference IntermediateDirectory, string ModuleName, IActionGraphBuilder Graph)
		{
			// Disable precompiled headers
			CompileEnvironment.PrecompiledHeaderAction = PrecompiledHeaderAction.None;

			// Write all the definitions out to a separate file
			CreateHeaderForDefinitions(CompileEnvironment, IntermediateDirectory, "Adaptive", Graph);

			// Compile the files
			return ToolChain.CompileAllCPPFiles(CompileEnvironment, Files, IntermediateDirectory, ModuleName, Graph);
		}

		static CPPOutput CompileAdaptiveNonUnityFilesWithDedicatedPCH(UEToolChain ToolChain, CppCompileEnvironment CompileEnvironment, List<FileItem> Files, DirectoryReference IntermediateDirectory, string ModuleName, IActionGraphBuilder Graph)
		{
			CPPOutput Output = new CPPOutput();
			// Pre-allocate as much space as we're likely to need for a single include directive, to save reallocations.
			// The default capacity of 16 chars is too small for many includes.
			StringBuilder TempStorageForCopyIncludes = new StringBuilder(512);
			// 16KB of headers is a reasonable estimate to save reallocation.
			StringBuilder WrapperContents = new StringBuilder(1024 * 16);
			foreach (FileItem File in Files)
			{
				WrapperContents.Clear();
				// Build the contents of the wrapper file
				{
					string FileString = File.AbsolutePath;
					if (File.Location.IsUnderDirectory(Unreal.RootDirectory))
					{
						FileString = File.Location.MakeRelativeTo(Unreal.EngineSourceDirectory);
					}
					FileString = FileString.Replace('\\', '/');
					WriteDefinitions($"Dedicated PCH for {FileString}", [], CompileEnvironment.Definitions, [], WrapperContents);
					WrapperContents.AppendLine();
					using (StreamReader Reader = new StreamReader(File.Location.FullName))
					{
						CppIncludeParser.CopyIncludeDirectives(Reader, WrapperContents, TempStorageForCopyIncludes);
					}
				}

				// Write the PCH header
				FileReference DedicatedPchLocation = FileReference.Combine(IntermediateDirectory, String.Format("PCH.Dedicated.{0}.h", File.Location.GetFileNameWithoutExtension()));
				FileItem DedicatedPchFile = Graph.CreateIntermediateTextFile(DedicatedPchLocation, WrapperContents.ToString());

				// Create a new C++ environment to compile the PCH
				CppCompileEnvironment PchEnvironment = new CppCompileEnvironment(CompileEnvironment);
				PchEnvironment.Definitions.Clear();
				PchEnvironment.UserIncludePaths.Add(File.Location.Directory); // Need to be able to include headers in the same directory as the source file
				PchEnvironment.PrecompiledHeaderAction = PrecompiledHeaderAction.Create;
				PchEnvironment.PrecompiledHeaderIncludeFilename = DedicatedPchFile.Location;

				// Create the action to compile the PCH file.
				CPPOutput PchOutput = ToolChain.CompileAllCPPFiles(PchEnvironment, new List<FileItem>() { DedicatedPchFile }, IntermediateDirectory, ModuleName, Graph);
				Output.ObjectFiles.AddRange(PchOutput.ObjectFiles);

				// Create a new C++ environment to compile the original file
				CppCompileEnvironment FileEnvironment = new CppCompileEnvironment(CompileEnvironment);
				FileEnvironment.Definitions.Clear();
				FileEnvironment.PrecompiledHeaderAction = PrecompiledHeaderAction.Include;
				FileEnvironment.PrecompiledHeaderIncludeFilename = DedicatedPchFile.Location;
				FileEnvironment.PCHInstance = new PrecompiledHeaderInstance(DedicatedPchFile, DedicatedPchFile, PchEnvironment, PchOutput, GetImmutableDefinitions(CompileEnvironment.Definitions));

				// Create the action to compile the PCH file.
				CPPOutput FileOutput = ToolChain.CompileAllCPPFiles(FileEnvironment, new List<FileItem>() { File }, IntermediateDirectory, ModuleName, Graph);
				Output.ObjectFiles.AddRange(FileOutput.ObjectFiles);
			}
			return Output;
		}

		/// <summary>
		/// Configure precompiled headers for this module
		/// </summary>
		/// <param name="Target">The target being built</param>
		/// <param name="ToolChain">The toolchain to build with</param>
		/// <param name="CompileEnvironment">The current compile environment</param>
		/// <param name="LinkInputFiles">List of files that will be linked for the target</param>
		/// <param name="Graph">List of build actions</param>
		CppCompileEnvironment SetupPrecompiledHeaders(ReadOnlyTargetRules Target, UEToolChain? ToolChain, CppCompileEnvironment CompileEnvironment, List<FileItem> LinkInputFiles, IActionGraphBuilder Graph)
		{
			if (Target.bUsePCHFiles && Rules.PCHUsage != ModuleRules.PCHUsageMode.NoPCHs)
			{
				// If this module doesn't need a shared PCH, configure that
				if (Rules.PrivatePCHHeaderFile != null && (Rules.PCHUsage == ModuleRules.PCHUsageMode.NoSharedPCHs || Rules.PCHUsage == ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs))
				{
					PrecompiledHeaderInstance Instance = CreatePrivatePCH(ToolChain, FileItem.GetItemByFileReference(FileReference.Combine(ModuleDirectory, Rules.PrivatePCHHeaderFile)), CompileEnvironment, Graph);

					CompileEnvironment = new CppCompileEnvironment(CompileEnvironment);
					CompileEnvironment.Definitions.Clear();
					CompileEnvironment.PrecompiledHeaderAction = PrecompiledHeaderAction.Include;
					CompileEnvironment.PrecompiledHeaderIncludeFilename = Instance.HeaderFile.Location;
					CompileEnvironment.PCHInstance = Instance;

					LinkInputFiles.AddRange(Instance.Output.ObjectFiles);
				}

				// Try to find a suitable shared PCH for this module
				if (CompileEnvironment.bHasPrecompiledHeader == false && CompileEnvironment.SharedPCHs.Count > 0 && !CompileEnvironment.bIsBuildingLibrary && Rules.PCHUsage != ModuleRules.PCHUsageMode.NoSharedPCHs)
				{
					// Find the first shared PCH module we can use that doesn't reference this module
					using Pooled<HashSet<UEBuildModule>> rented = HashSetPool<UEBuildModule>.Rent();
					HashSet<UEBuildModule> AllDependencies = rented.Value;
					GetAllDependencyModulesForPCH(AllDependencies, true);

					PrecompiledHeaderTemplate? Template = CompileEnvironment.SharedPCHs.FirstOrDefault(x => !x.ModuleDependencies.Contains(this) && AllDependencies.Contains(x.Module) && x.IsValidFor(CompileEnvironment));

					if (Template != null)
					{
						PrecompiledHeaderInstance Instance = FindOrCreateSharedPCH(ToolChain, Template, CompileEnvironment, Graph);

						FileReference PrivateDefinitionsFile = FileReference.Combine(IntermediateDirectory, $"Definitions.{Rules.ShortName ?? Name}.h");

						FileItem PrivateDefinitionsFileItem;
						{
							// Only add new definitions that are not already existing in the shared pch
							List<string> NewDefinitions = [];
							List<string> Undefinitions = [];
							bool ModuleApiUndef = false;
							bool ModuleNonAttributedUndef = false;
							foreach (string Definition in CompileEnvironment.Definitions)
							{
								if (Instance.ImmutableDefinitions.Contains(Definition))
								{
									continue;
								}

								NewDefinitions.Add(Definition);

								// Remove the module _API definition for cases where there are circular dependencies between the shared PCH module and modules using it
								if (!ModuleApiUndef && Definition.StartsWith(ModuleApiDefine, StringComparison.Ordinal))
								{
									ModuleApiUndef = true;
									Undefinitions.Add(ModuleApiDefine);
								}

								// Remove the module _NON_ATTRIBUTED_API definition for cases where there are circular dependencies between the shared PCH module and modules using it
								if (!ModuleNonAttributedUndef && Definition.StartsWith(ModuleNonAttributedDefine, StringComparison.Ordinal))
								{
									ModuleNonAttributedUndef = true;
									Undefinitions.Add(ModuleNonAttributedDefine);
								}
							}

							StringBuilder Writer = new();
							WriteDefinitions($"Shared PCH Definitions for {Name}", new string[] { Instance.DefinitionsFile.Name }, NewDefinitions, Undefinitions, Writer);
							PrivateDefinitionsFileItem = Graph.CreateIntermediateTextFile(PrivateDefinitionsFile, Writer.ToString());
						}

						CompileEnvironment = new CppCompileEnvironment(CompileEnvironment);
						CompileEnvironment.Definitions.Clear();
						CompileEnvironment.ForceIncludeFiles.Insert(0, PrivateDefinitionsFileItem);
						CompileEnvironment.PrecompiledHeaderAction = PrecompiledHeaderAction.Include;
						CompileEnvironment.PrecompiledHeaderIncludeFilename = Instance.HeaderFile.Location;
						CompileEnvironment.UserIncludePaths.Add(Instance.DefinitionsFile.Directory.Location);
						CompileEnvironment.PCHInstance = Instance;

						LinkInputFiles.AddRange(Instance.Output.ObjectFiles);
					}
				}
			}
			return CompileEnvironment;
		}

		/// <summary>
		/// Creates a header file containing all the preprocessor definitions for a compile environment, and force-include it. We allow a more flexible syntax for preprocessor definitions than
		/// is typically allowed on the command line (allowing function macros or double-quote characters, for example). Ensuring all definitions are specified in a header files ensures consistent
		/// behavior.
		/// </summary>
		/// <param name="CompileEnvironment">The compile environment</param>
		/// <param name="IntermediateDirectory">Directory to create the intermediate file</param>
		/// <param name="HeaderSuffix">Suffix for the included file</param>
		/// <param name="Graph">The action graph being built</param>
		static FileItem? CreateHeaderForDefinitions(CppCompileEnvironment CompileEnvironment, DirectoryReference IntermediateDirectory, string? HeaderSuffix, IActionGraphBuilder Graph)
		{
			if (CompileEnvironment.Definitions.Count > 0)
			{
				string PrivateDefinitionsName = "Definitions.h";

				if (!String.IsNullOrEmpty(HeaderSuffix))
				{
					PrivateDefinitionsName = $"Definitions.{HeaderSuffix}.h";
				}

				FileReference PrivateDefinitionsFile = FileReference.Combine(IntermediateDirectory, PrivateDefinitionsName);
				{
					StringBuilder Writer = new();
					WriteDefinitions("Definitions", Array.Empty<string>(), CompileEnvironment.Definitions, Array.Empty<string>(), Writer);
					CompileEnvironment.Definitions.Clear();

					FileItem PrivateDefinitionsFileItem = Graph.CreateIntermediateTextFile(PrivateDefinitionsFile, Writer.ToString());
					CompileEnvironment.ForceIncludeFiles.Insert(0, PrivateDefinitionsFileItem);
					return PrivateDefinitionsFileItem;
				}
			}
			return null;
		}

		/// <summary>
		/// Create a header file containing the module definitions, which also includes the PCH itself. Including through another file is necessary on 
		/// Clang, since we get warnings about #pragma once otherwise, but it also allows us to consistently define the preprocessor state on all 
		/// platforms.
		/// </summary>
		/// <param name="OutputFile">The output file to create</param>
		/// <param name="DefinitionsFile">File containing definitions required by the PCH</param>
		/// <param name="IncludedFile">The PCH file to include</param>
		/// <param name="Graph">The action graph builder</param>
		/// <returns>FileItem for the created file</returns>
		static FileItem CreatePCHWrapperFile(FileReference OutputFile, FileItem DefinitionsFile, FileItem IncludedFile, IActionGraphBuilder Graph)
		{
			// Build the contents of the wrapper file
			StringBuilder WrapperContents = new StringBuilder();
			{
				WrapperContents.AppendLine("// Generated by UnrealBuildTool (UEBuildModuleCPP.cs) : PCH for {0}", IncludedFile.Name);
				WrapperContents.AppendLine("#pragma once");
				WrapperContents.AppendLine("#include \"{0}\"", DefinitionsFile.Name);
				WrapperContents.AppendLine("#include \"{0}\"", IncludedFile.Name);
				WrapperContents.AppendLine("#ifdef __ISPC_ALIGN__");
				WrapperContents.AppendLine("#error ispc.generated.h files are not allowed in precompiled headers (This is a build time optimization to reduce action dependencies)");
				WrapperContents.AppendLine("#endif");
			}

			// Create the item
			FileItem WrapperFile = Graph.CreateIntermediateTextFile(OutputFile, WrapperContents.ToString(), AllowAsync: false);

			// Touch it if the included file is newer, to make sure our timestamp dependency checking is accurate.
			if (IncludedFile.LastWriteTimeUtc > WrapperFile.LastWriteTimeUtc)
			{
				File.SetLastWriteTimeUtc(WrapperFile.AbsolutePath, DateTime.UtcNow);
				WrapperFile.ResetCachedInfo();
			}
			return WrapperFile;
		}

		/// <summary>
		/// Write a list of macro definitions to an output file
		/// </summary>
		/// <param name="Context">Additional context to add to generated comment</param>
		/// <param name="Includes">List of includes</param>
		/// <param name="Definitions">List of definitions</param>
		/// <param name="Undefinitions">List of definitions to undefine</param>
		/// <param name="Writer">Writer to receive output</param>
		static void WriteDefinitions(string Context, IEnumerable<string> Includes, IEnumerable<string> Definitions, IEnumerable<string> Undefinitions, StringBuilder Writer)
		{
			Writer.AppendLine($"// Generated by UnrealBuildTool (UEBuildModuleCPP.cs) : {Context}");
			Writer.AppendLine("#pragma once");
			foreach (string Include in Includes)
			{
				Writer.Append("#include \"").Append(Include).Append('\"').AppendLine();
			}
			foreach (string Undefinition in Undefinitions)
			{
				Writer.Append("#undef ").Append(Undefinition).AppendLine();
			}

			using Pooled<HashSet<string>> rented = HashSetPool<string>.Rent();
			HashSet<string> Processed = rented.Value;
			foreach (string Definition in Definitions.Where(x => Processed.Add(x)))
			{
				int EqualsIdx = Definition.IndexOf('=', StringComparison.Ordinal);
				if (EqualsIdx == -1)
				{
					Writer.Append("#define ").Append(Definition).AppendLine(" 1");
				}
				else
				{
					Writer.Append("#define ").Append(Definition.AsSpan(0, EqualsIdx)).Append(' ').Append(Definition.AsSpan(EqualsIdx + 1)).AppendLine();
				}
			}
		}

		/// <summary>
		/// Checks that the first header included by the source files in this module all include the same header
		/// </summary>
		/// <param name="Target">The target being compiled</param>
		/// <param name="ModuleCompileEnvironment">Compile environment for the module</param>
		/// <param name="HeaderFiles">All header files for this module</param>
		/// <param name="CppFiles">List of C++ source files</param>
		private void CheckFirstIncludeMatchesEachCppFile(ReadOnlyTargetRules Target, CppCompileEnvironment ModuleCompileEnvironment, List<FileItem> HeaderFiles, List<FileItem> CppFiles)
		{
			if (Rules.PCHUsage == ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs)
			{
				if (InvalidIncludeDirectives == null)
				{
					// Find headers used by the source file.
					Dictionary<string, FileReference> NameToHeaderFile = new Dictionary<string, FileReference>();
					foreach (FileItem HeaderFile in HeaderFiles)
					{
						NameToHeaderFile[HeaderFile.Location.GetFileNameWithoutExtension()] = HeaderFile.Location;
					}

					// Find the directly included files for each source file, and make sure it includes the matching header if possible

					List<InvalidIncludeDirective>? list = null;
					if (Rules != null && Rules.IWYUSupport != IWYUSupport.None && Target.bEnforceIWYU)
					{
						foreach (FileItem CppFile in CppFiles)
						{
							string? FirstInclude = ModuleCompileEnvironment.MetadataCache.GetFirstInclude(CppFile);
							if (FirstInclude != null)
							{
								string IncludeName = Path.GetFileNameWithoutExtension(FirstInclude);
								string ExpectedName = CppFile.Location.GetFileNameWithoutExtension();
								if (!String.Equals(IncludeName, ExpectedName, StringComparison.OrdinalIgnoreCase))
								{
									FileReference? HeaderFile;
									if (NameToHeaderFile.TryGetValue(ExpectedName, out HeaderFile))
									{
										if (list == null)
										{
											list = new();
										}

										list.Add(new InvalidIncludeDirective(CppFile.Location, HeaderFile));
									}
								}
							}
						}
					}

					if (list != null)
					{
						InvalidIncludeDirectives = list;
					}
				}
			}
		}

		private void CompileEnvironmentDebugInfoSettings(ReadOnlyTargetRules Target, CppCompileEnvironment Result)
		{
			// If bCreateDebugInfo is disabled for the whole target do not adjust any settings
			if (!Result.bCreateDebugInfo)
			{
				return;
			}

			if (Target.bPGOProfile)
			{
				Result.bDebugLineTablesOnly = true;
			}

			if (Rules.Target.ProjectFile != null && Rules.File.IsUnderDirectory(Rules.Target.ProjectFile.Directory))
			{
				if (Target.DebugInfoLineTablesOnly.HasFlag(DebugInfoMode.ProjectPlugins) && Rules.Plugin != null)
				{
					Result.bDebugLineTablesOnly = true;
				}
				else if (Target.DebugInfoLineTablesOnly.HasFlag(DebugInfoMode.Project) && Rules.Plugin == null)
				{
					Result.bDebugLineTablesOnly = true;
				}

				if (Target.DebugInfoNoInlineLineTables.HasFlag(DebugInfoMode.ProjectPlugins) && Rules.Plugin != null)
				{
					Result.bDebugNoInlineLineTables = true;
				}
				else if (Target.DebugInfoNoInlineLineTables.HasFlag(DebugInfoMode.Project) && Rules.Plugin == null)
				{
					Result.bDebugNoInlineLineTables = true;
				}

				if (Target.DebugInfoSimpleTemplateNames.HasFlag(DebugInfoMode.ProjectPlugins) && Rules.Plugin != null)
				{
					Result.bDebugSimpleTemplateNames = true;
				}
				else if (Target.DebugInfoSimpleTemplateNames.HasFlag(DebugInfoMode.Project) && Rules.Plugin == null)
				{
					Result.bDebugSimpleTemplateNames = true;
				}
			}
			else
			{
				if (Target.DebugInfoLineTablesOnly.HasFlag(DebugInfoMode.EnginePlugins) && Rules.Plugin != null)
				{
					Result.bDebugLineTablesOnly = true;
				}
				else if (Target.DebugInfoLineTablesOnly.HasFlag(DebugInfoMode.Engine) && Rules.Plugin == null)
				{
					Result.bDebugLineTablesOnly = true;
				}

				if (Target.DebugInfoNoInlineLineTables.HasFlag(DebugInfoMode.EnginePlugins) && Rules.Plugin != null)
				{
					Result.bDebugNoInlineLineTables = true;
				}
				else if (Target.DebugInfoNoInlineLineTables.HasFlag(DebugInfoMode.Engine) && Rules.Plugin == null)
				{
					Result.bDebugNoInlineLineTables = true;
				}

				if (Target.DebugInfoSimpleTemplateNames.HasFlag(DebugInfoMode.EnginePlugins) && Rules.Plugin != null)
				{
					Result.bDebugSimpleTemplateNames = true;
				}
				else if (Target.DebugInfoSimpleTemplateNames.HasFlag(DebugInfoMode.Engine) && Rules.Plugin == null)
				{
					Result.bDebugSimpleTemplateNames = true;
				}
			}

			if (Rules.Plugin != null && Rules.Target.DebugInfoLineTablesOnlyPlugins.Contains(Rules.Plugin.Name))
			{
				Result.bDebugLineTablesOnly = true;
			}
			else if (Rules.Target.DebugInfoLineTablesOnlyModules.Contains(Name))
			{
				Result.bDebugLineTablesOnly = true;
			}

			// Note that these rules are the opposite of DebugInfoLineTablesOnly* - they selectively reintroduce line tables for 
			// inlined functions even if the blanket rules above would remove them.
			if (Rules.Plugin != null && Rules.Target.DebugInfoInlineLineTablesPlugins.Contains(Rules.Plugin.Name))
			{
				Result.bDebugNoInlineLineTables = false;
			}
			else if (Rules.Target.DebugInfoInlineLineTablesModules.Contains(Name))
			{
				Result.bDebugNoInlineLineTables = false;
			}

			if (Rules.Plugin != null && Rules.Target.DebugInfoSimpleTemplateNamesPlugins.Contains(Rules.Plugin.Name))
			{
				Result.bDebugSimpleTemplateNames = true;
			}
			else if (Rules.Target.DebugInfoSimpleTemplateNamesModules.Contains(Name))
			{
				Result.bDebugSimpleTemplateNames = true;
			}

			if (Result.bDebugLineTablesOnly)
			{
				// Don't disable debug info if only line tables are requested
				return;
			}

			// Disable debug info for modules if requested
			if (!Target.bUsePDBFiles || !Target.Platform.IsInGroup(UnrealPlatformGroup.Microsoft))
			{
				if (Rules.Target.ProjectFile != null && Rules.File.IsUnderDirectory(Rules.Target.ProjectFile.Directory))
				{
					if (!Target.DebugInfo.HasFlag(DebugInfoMode.ProjectPlugins) && Rules.Plugin != null)
					{
						Result.bCreateDebugInfo = false;
					}
					else if (!Target.DebugInfo.HasFlag(DebugInfoMode.Project) && Rules.Plugin == null)
					{
						Result.bCreateDebugInfo = false;
					}
				}
				else
				{
					if (!Target.DebugInfo.HasFlag(DebugInfoMode.EnginePlugins) && Rules.Plugin != null)
					{
						Result.bCreateDebugInfo = false;
					}
					else if (!Target.DebugInfo.HasFlag(DebugInfoMode.Engine) && Rules.Plugin == null)
					{
						Result.bCreateDebugInfo = false;
					}
				}

				if (Rules.Plugin != null && Rules.Target.DisableDebugInfoPlugins.Contains(Rules.Plugin.Name))
				{
					Result.bCreateDebugInfo = false;
				}
				else if (Rules.Target.DisableDebugInfoModules.Contains(Name))
				{
					Result.bCreateDebugInfo = false;
				}
			}
		}

		/// <summary>
		/// Determine whether optimization should be enabled for a given target
		/// </summary>
		/// <param name="Setting">The optimization setting from the rules file</param>
		/// <param name="Configuration">The active target configuration</param>
		/// <param name="bIsEngineModule">Whether the current module is an engine module</param>
		/// <param name="bCodeCoverage">Whether the current module should be compiled with code coverage support</param>
		/// <returns>True if optimization should be enabled</returns>
		public static bool ShouldEnableOptimization(ModuleRules.CodeOptimization Setting, UnrealTargetConfiguration Configuration, bool bIsEngineModule, bool bCodeCoverage)
		{
			if (bCodeCoverage)
			{
				return false;
			}
			switch (Setting)
			{
				case ModuleRules.CodeOptimization.Never:
					return false;
				case ModuleRules.CodeOptimization.Default:
				case ModuleRules.CodeOptimization.InNonDebugBuilds:
					return Configuration != UnrealTargetConfiguration.Debug && (Configuration != UnrealTargetConfiguration.DebugGame || bIsEngineModule);
				case ModuleRules.CodeOptimization.InShippingBuildsOnly:
					return (Configuration == UnrealTargetConfiguration.Shipping);
				default:
					return true;
			}
		}

		public CppCompileEnvironment CreateCompileEnvironmentForIntellisense(ReadOnlyTargetRules Target, CppCompileEnvironment BaseCompileEnvironment, ILogger Logger, NullActionGraphBuilder? builder = null)
		{
			builder ??= new NullActionGraphBuilder(Logger);
			CppCompileEnvironment CompileEnvironment = CreateModuleCompileEnvironment(Target, BaseCompileEnvironment, Logger);
			CompileEnvironment = SetupPrecompiledHeaders(Target, null, CompileEnvironment, new List<FileItem>(), builder);
			CreateHeaderForDefinitions(CompileEnvironment, IntermediateDirectory, null, builder);
			return CompileEnvironment;
		}

		/// <summary>
		/// Creates a compile environment from a base environment based on the module settings.
		/// </summary>
		/// <param name="Target">Rules for the target being built</param>
		/// <param name="BaseCompileEnvironment">An existing environment to base the module compile environment on.</param>
		/// <param name="Logger">Logger for output</param>
		/// <returns>The new module compile environment.</returns>
		public CppCompileEnvironment CreateModuleCompileEnvironment(ReadOnlyTargetRules Target, CppCompileEnvironment BaseCompileEnvironment, ILogger Logger)
		{
			CppCompileEnvironment Result = new CppCompileEnvironment(BaseCompileEnvironment);

			// Override compile environment
			Result.bUseUnity = Rules.bUseUnity;
			Result.bCodeCoverage = Target.bCodeCoverage;
			Result.bOptimizeCode = ShouldEnableOptimization(Rules.OptimizeCode, Target.Configuration, Rules.bTreatAsEngineModule, Result.bCodeCoverage);
			Result.bUseRTTI |= Rules.bUseRTTI;
			Result.bVcRemoveUnreferencedComdat &= Rules.bVcRemoveUnreferencedComdat;
			Result.bEnableBufferSecurityChecks = Rules.bEnableBufferSecurityChecks;
			Result.MinSourceFilesForUnityBuildOverride = Rules.MinSourceFilesForUnityBuildOverride;
			Result.MinFilesUsingPrecompiledHeaderOverride = Rules.MinFilesUsingPrecompiledHeaderOverride;
			Result.bBuildLocallyWithSNDBS = Rules.bBuildLocallyWithSNDBS;
			Result.bEnableExceptions |= Rules.bEnableExceptions;
			Result.bEnableObjCExceptions |= Rules.bEnableObjCExceptions;
			Result.bEnableObjCAutomaticReferenceCounting = Rules.bEnableObjCAutomaticReferenceCounting;
			Result.bWarningsAsErrors |= Rules.bWarningsAsErrors;
			Result.CppCompileWarnings = CppCompileWarnings.CreateShallowCopy(Rules.CppCompileWarningSettings);
			Result.bDisableStaticAnalysis = Rules.bDisableStaticAnalysis || (Target.bStaticAnalyzerProjectOnly && Rules.bTreatAsEngineModule);
			Result.bStaticAnalyzerExtensions = Rules.bStaticAnalyzerExtensions;
			Result.StaticAnalyzerRulesets = Rules.StaticAnalyzerRulesets;
			Result.StaticAnalyzerCheckers = Rules.StaticAnalyzerCheckers;
			Result.StaticAnalyzerDisabledCheckers = Rules.StaticAnalyzerDisabledCheckers;
			Result.StaticAnalyzerAdditionalCheckers = Rules.StaticAnalyzerAdditionalCheckers;
			Result.StaticAnalyzerPVSDisabledErrors = Rules.StaticAnalyzerPVSDisabledErrors;
			Result.bTreatAsEngineModule = Rules.bTreatAsEngineModule;
			Result.IncludeOrderVersion = Rules.IncludeOrderVersion;
			Result.bValidateFormatStrings = Rules.bValidateFormatStrings;
			Result.bValidateInternalApi = Rules.bValidateInternalApi;
			Result.bValidateExperimentalApi = Rules.bValidateExperimentalApi;
			Result.bUseAutoRTFMCompiler = Target.bUseAutoRTFMCompiler;
			Result.FPassPlugins = Rules.FPassPlugins;
			Result.OptimizationLevel = Rules.OptimizationLevel;
			Result.FPSemantics = Rules.FPSemantics;

			if (Target.Platform.IsInGroup(UnrealPlatformGroup.Apple))
			{
				Result.bRequiresPlatformSDK = Rules.bRequiresPlatformSDK;
			}

			CompileEnvironmentDebugInfoSettings(Target, Result);

			// Only copy the AutoRTFM flags if we are using the AutoRTFM compiler
			if (Result.bUseAutoRTFMCompiler)
			{
				Result.bDisableAutoRTFMInstrumentation = Rules.bDisableAutoRTFMInstrumentation;
				Result.bEnableAutoRTFMVerification = Target.bUseAutoRTFMVerifier;
				Result.bAutoRTFMVerify = Target.bAutoRTFMVerify;
				Result.bAutoRTFMClosedStaticLinkage = Target.bAutoRTFMClosedStaticLinkage;
				Result.bAutoRTFMRedirectionInstrumentation = Target.bAutoRTFMRedirectionInstrumentation;
			}

			// If the module overrides the C++ language version, override it on the compile environment
			if (Rules.CppStandard != null)
			{
				Result.CppStandard = (CppStandardVersion)Rules.CppStandard;
			}
			// Otherwise, for engine modules use the engine standard (typically CppStandardVersion.EngineDefault).
			else if (Rules.bTreatAsEngineModule)
			{
				Result.CppStandard = Result.CppStandardEngine;
			}

			if (Result.CppStandard != Result.CppStandardEngine)
			{
				// SharedPCH is disallowed for modules that compile against an older CppStandard than the engine
				if (Rules.PCHUsage == ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs)
				{
					if (Rules.PrivatePCHHeaderFile != null)
					{
						Logger.LogDebug("  CppStandard {CppStandard} cannot use PCHUsage {PCHUsage}, however PrivatePCHHeaderFile is set. Overriding to NoSharedPCHs for {Name}", Result.CppStandard, Rules.PCHUsage, Name);
						Rules.PCHUsage = ModuleRules.PCHUsageMode.NoSharedPCHs;
					}
					else
					{
						Logger.LogDebug("  CppStandard {CppStandard} cannot use PCHUsage {PCHUsage}. Overriding to NoPCHs for {Name}", Result.CppStandard, Rules.PCHUsage, Name);
						Rules.PCHUsage = ModuleRules.PCHUsageMode.NoPCHs;
					}
				}
				else if (Rules.PCHUsage == ModuleRules.PCHUsageMode.UseSharedPCHs)
				{
					Logger.LogDebug("  CppStandard {CppStandard} cannot use PCHUsage {PCHUsage}. Overriding to NoPCHs for {Name}", Result.CppStandard, Rules.PCHUsage, Name);
					Rules.PCHUsage = ModuleRules.PCHUsageMode.NoPCHs;
				}
			}

			// If the module overrides the C language version, override it on the compile environment
			if (Rules.CStandard != null)
			{
				Result.CStandard = (CStandardVersion)Rules.CStandard;
			}

			// If the module overrides the x64 minimum arch, override it on the compile environment
			if (Rules.MinCpuArchX64 != null)
			{
				Result.MinCpuArchX64 = (MinimumCpuArchitectureX64)Rules.MinCpuArchX64;
			}

			if (Rules.bMinCpuArchAPX != null)
			{
				Result.bMinCpuArchAPX = (bool)Rules.bMinCpuArchAPX;
			}

			// Set the macro used to check whether monolithic headers can be used
			if ((Rules.bTreatAsEngineModule || Target.bWarnAboutMonolithicHeadersIncluded) && (Rules.IWYUSupport == IWYUSupport.None || !Target.bEnforceIWYU))
			{
				Result.Definitions.Add("SUPPRESS_MONOLITHIC_HEADER_WARNINGS=1");
			}

			// Add a macro for when we're compiling an engine module, to enable additional compiler diagnostics through code.
			Result.Definitions.Add($"UE_IS_ENGINE_MODULE={(Rules.bTreatAsEngineModule || Target.bWarnAboutMonolithicHeadersIncluded ? "1" : "0")}");

			if (!Rules.bTreatAsEngineModule)
			{
				Result.Definitions.Add("UE_DEPRECATED_FORGAME=UE_DEPRECATED");
				Result.Definitions.Add($"UE_DEPRECATED_FORENGINE={(Rules.Target.bDisableEngineDeprecations ? "UE_EMPTY_FUNCTION" : "UE_DEPRECATED")}");
			}

			Result.Definitions.Add($"UE_VALIDATE_FORMAT_STRINGS={(Rules.bValidateFormatStrings ? "1" : "0")}");
			Result.Definitions.Add($"UE_VALIDATE_INTERNAL_API={(Rules.bValidateInternalApi ? "1" : "0")}");
			Result.Definitions.Add($"UE_VALIDATE_EXPERIMENTAL_API={(Rules.bValidateExperimentalApi ? "1" : "0")}");

			Result.Definitions.AddRange(EngineIncludeOrderHelper.GetDeprecationDefines(Rules.IncludeOrderVersion));

			DirectoryReference? ProjectDirectory = Target.ProjectFile?.Directory;

			// If module intermediate files are under the project folder we can set UE_PROJECT_NAME and friends.
			// This makes it possible to create modular builds where you don't have to provide the project name on cmd line
			if (ProjectDirectory != null && IntermediateDirectory.IsUnderDirectory(ProjectDirectory))
			{
				string ProjectName = Target.ProjectFile!.GetFileNameWithoutExtension();
				Result.Definitions.Add(String.Format("UE_PROJECT_NAME={0}", ProjectName));
				Result.Definitions.Add(String.Format("UE_TARGET_NAME={0}", Target.OriginalName));
			}

			// Add the module's public and private definitions.
			AddDefinitions(Result.Definitions, PublicDefinitions);

			Result.Definitions.AddRange(Rules.PrivateDefinitions);

			if (Rules.Name == "BuildSettings")
			{
				Result.Definitions.Add(String.Format("UE_WITH_DEBUG_INFO={0}", BaseCompileEnvironment.bCreateDebugInfo ? "1" : "0"));

				StringBuilder sb = new();
				foreach ((uint id, DirectoryReference vfs, DirectoryReference local) in BaseCompileEnvironment.RootPaths)
				{
					sb.Append(vfs.FullName.Replace("\\", "\\\\", StringComparison.Ordinal)).Append(';');
					sb.Append(local.FullName.Replace("\\", "\\\\", StringComparison.Ordinal)).Append(';');
				}
				Result.Definitions.Add($"UE_VFS_PATHS=\"{sb}\"");

				if (Target.BuildSettingsVariables.Count > 0)
				{
					sb.Clear();
					foreach ((string key, string value) in Target.BuildSettingsVariables.OrderBy(kv => kv.Key))
					{
						if (key.Contains(';', StringComparison.Ordinal) || value.Contains(';', StringComparison.Ordinal))
						{
							throw new Exception("BuildSettingsDefinitions can't contain keys or values with character ';'");
						}

						sb.Append(key).Append(';').Append(value).Append(';');
					}

					Result.Definitions.Add($"UE_BUILDSETTINGS_VARIABLES=\"{sb}\"");
				}
			}

			// Add the project definitions
			if (!Rules.bTreatAsEngineModule)
			{
				Result.Definitions.AddRange(Rules.Target.ProjectDefinitions);
			}

			// Setup the compile environment for the module.
			SetupPrivateCompileEnvironment(Result.UserIncludePaths, Result.SystemIncludePaths, Result.ModuleInterfacePaths, Result.Definitions, Result.AdditionalFrameworks, Result.AutoRTFMExternalMappingFiles, Result.AdditionalPrerequisites, Rules.bLegacyPublicIncludePaths, Rules.bLegacyParentIncludePaths);

			foreach (string ForceIncludeFile in Rules.ForceIncludeFiles)
			{
				if (Path.IsPathFullyQualified(ForceIncludeFile))
				{
					Result.ForceIncludeFiles.Add(FileItem.GetItemByPath(ForceIncludeFile));
				}
				else
				{
					FileReference? Path = Result.UserIncludePaths.Concat(Result.SystemIncludePaths).Select(x => FileReference.Combine(x, ForceIncludeFile)).FirstOrDefault(x => FileReference.Exists(x));
					if (Path == null)
					{
						Logger.LogWarning("Unable to resolve force include path '{Path}'. Please verify that it can be found in the include directories for module '{Module}'", ForceIncludeFile, Name);
						continue;
					}
					Result.ForceIncludeFiles.Add(FileItem.GetItemByFileReference(Path));
				}
			}

			Result.RootPaths.AddExtraPath(Rules.ExtraRootPath);

			// Inherit extra root paths from dependencies
			if (PrivateDependencyModules != null)
			{
				foreach (UEBuildModule Dependent in PrivateDependencyModules)
				{
					Result.RootPaths.AddExtraPath(Dependent.Rules.ExtraRootPath);
				}
			}

			// Project root path should only be added if the module needs it, to prevent violating shared environment actions
			if (ProjectDirectory != null && (!Result.bUseSharedBuildEnvironment || Result.AllIncludePath.Any(x => x.IsUnderDirectory(ProjectDirectory))))
			{
				Result.RootPaths[CppRootPathFolder.Project] = ProjectDirectory;
			}

			return Result;
		}

		/// <summary>
		/// Creates a compile environment for a shared PCH from a base environment based on the module settings.
		/// </summary>
		/// <param name="Target">The target being built</param>
		/// <param name="BaseCompileEnvironment">An existing environment to base the module compile environment on.</param>
		/// <returns>The new shared PCH compile environment.</returns>
		public CppCompileEnvironment CreateSharedPCHCompileEnvironment(UEBuildTarget Target, CppCompileEnvironment BaseCompileEnvironment)
		{
			CppCompileEnvironment CompileEnvironment = new CppCompileEnvironment(BaseCompileEnvironment);

			// Use the default optimization setting for
			CompileEnvironment.bOptimizeCode = ShouldEnableOptimization(ModuleRules.CodeOptimization.Default, Target.Configuration, Rules.bTreatAsEngineModule, Rules.bCodeCoverage);
			CompileEnvironment.bCodeCoverage = Rules.bCodeCoverage;
			CompileEnvironment.bTreatAsEngineModule = Rules.bTreatAsEngineModule;
			CompileEnvironment.bValidateFormatStrings = Rules.bValidateFormatStrings;
			CompileEnvironment.bValidateInternalApi = Rules.bValidateInternalApi;
			CompileEnvironment.bValidateExperimentalApi = Rules.bValidateExperimentalApi;

			// Override compile environment
			CompileEnvironment.bIsBuildingDLL = !Target.ShouldCompileMonolithic();
			CompileEnvironment.bIsBuildingLibrary = false;

			// Add a macro for when we're compiling an engine module, to enable additional compiler diagnostics through code.
			CompileEnvironment.Definitions.Add($"UE_IS_ENGINE_MODULE={(Rules.bTreatAsEngineModule || Rules.Target.bWarnAboutMonolithicHeadersIncluded ? "1" : "0")}");

			if (!Rules.bTreatAsEngineModule)
			{
				CompileEnvironment.Definitions.Add("UE_DEPRECATED_FORGAME=UE_DEPRECATED");
				CompileEnvironment.Definitions.Add($"UE_DEPRECATED_FORENGINE={(Rules.Target.bDisableEngineDeprecations ? "UE_EMPTY_FUNCTION" : "UE_DEPRECATED")}");
			}

			CompileEnvironment.Definitions.Add($"UE_VALIDATE_FORMAT_STRINGS={(Rules.bValidateFormatStrings ? "1" : "0")}");
			CompileEnvironment.Definitions.Add($"UE_VALIDATE_INTERNAL_API={(Rules.bValidateInternalApi ? "1" : "0")}");
			CompileEnvironment.Definitions.Add($"UE_VALIDATE_EXPERIMENTAL_API={(Rules.bValidateExperimentalApi ? "1" : "0")}");

			CompileEnvironment.Definitions.AddRange(EngineIncludeOrderHelper.GetDeprecationDefines(Rules.IncludeOrderVersion));

			// Add the module's private definitions.
			CompileEnvironment.Definitions.AddRange(PublicDefinitions);

			// Find all the modules that are part of the public compile environment for this module.
			Dictionary<UEBuildModule, bool> ModuleToIncludePathsOnlyFlag = new Dictionary<UEBuildModule, bool>();
			FindModulesInPublicCompileEnvironment(ModuleToIncludePathsOnlyFlag);

			// Set up additional root paths
			if (Target.ProjectFile != null && (!Target.bUseSharedBuildEnvironment || RulesFile.IsUnderDirectory(Target.ProjectFile.Directory)))
			{
				CompileEnvironment.RootPaths[CppRootPathFolder.Project] = Target.ProjectFile.Directory;
			}

			CompileEnvironment.RootPaths.AddExtraPath(Rules.ExtraRootPath);

			// Now set up the compile environment for the modules in the original order that we encountered them
			foreach (UEBuildModule Module in ModuleToIncludePathsOnlyFlag.Keys)
			{
				Module.AddModuleToCompileEnvironment(this, null, CompileEnvironment.UserIncludePaths, CompileEnvironment.SystemIncludePaths, CompileEnvironment.ModuleInterfacePaths, CompileEnvironment.Definitions, CompileEnvironment.AdditionalFrameworks, CompileEnvironment.AutoRTFMExternalMappingFiles, CompileEnvironment.AdditionalPrerequisites, Rules.bLegacyPublicIncludePaths, Rules.bLegacyParentIncludePaths);
			}
			return CompileEnvironment;
		}

		/// <summary>
		/// Similar with FindInputFiles but for test targets.
		/// If SelectedTestModuleNames is non-empty it will include tests exclusively from this list.
		/// </summary>
		/// <param name="Platform">The platform the module is being built for</param>
		/// <param name="DirectoryToSourceFiles">Map of directory to source files inside it</param>
		/// <param name="Logger">Logger for output</param>
		/// <returns>Set of source files that should be built for a test target.</returns>
		public InputFileCollection FindInputFilesTests(UnrealTargetPlatform Platform, Dictionary<DirectoryItem, FileItem[]> DirectoryToSourceFiles, ILogger Logger)
		{
			IReadOnlySet<string> ExcludedNames = UEBuildPlatform.GetBuildPlatform(Platform).GetExcludedFolderNames();
			InputFileCollection InputFiles = new InputFileCollection();

			SourceDirectories = new HashSet<DirectoryReference>();
			foreach (DirectoryItem Dir in ModuleDirectories)
			{
				FindInputFilesFromDirectoryRecursiveTests(Dir, ExcludedNames, SourceDirectories, DirectoryToSourceFiles, InputFiles, Logger);
			}

			return InputFiles;
		}

		/// <summary>
		/// Finds all the source files that should be built for this module
		/// </summary>
		/// <param name="Platform">The platform the module is being built for</param>
		/// <param name="DirectoryToSourceFiles">Map of directory to source files inside it</param>
		/// <param name="Logger">Logger for output</param>
		/// <returns>Set of source files that should be built</returns>
		public InputFileCollection FindInputFiles(UnrealTargetPlatform Platform, Dictionary<DirectoryItem, FileItem[]> DirectoryToSourceFiles, ILogger Logger)
		{
			IReadOnlySet<string> ExcludedNames = UEBuildPlatform.GetBuildPlatform(Platform).GetExcludedFolderNames();

			InputFileCollection InputFiles = new InputFileCollection();

			SourceDirectories = new HashSet<DirectoryReference>();
			foreach (DirectoryItem Dir in ModuleDirectories)
			{
				FindInputFilesFromDirectoryRecursive(Dir, ExcludedNames, SourceDirectories, DirectoryToSourceFiles, InputFiles, Logger);
			}

			return InputFiles;
		}

		/// <summary>
		/// Finds all the source files that should be built for this module when using test mode
		/// </summary>
		/// <param name="BaseDirectory">Directory to search from</param>
		/// <param name="ExcludedNames">Set of excluded directory names (eg. other platforms)</param>
		/// <param name="SourceDirectories">Set of all non-empty source directories.</param>
		/// <param name="DirectoryToSourceFiles">Map from directory to source files inside it</param>
		/// <param name="InputFiles">Collection of source files, categorized by type</param>
		/// <param name="Logger">Logger for output</param>
		public void FindInputFilesFromDirectoryRecursiveTests(DirectoryItem BaseDirectory, IReadOnlySet<string> ExcludedNames, HashSet<DirectoryReference> SourceDirectories, Dictionary<DirectoryItem, FileItem[]> DirectoryToSourceFiles, InputFileCollection InputFiles, ILogger Logger)
		{
			bool bIgnoreFileFound;
			FileItem[] SourceFiles = FindInputFilesFromDirectory(BaseDirectory, InputFiles, out bIgnoreFileFound, Logger);

			if (bIgnoreFileFound)
			{
				return;
			}

			foreach (DirectoryItem SubDirectory in BaseDirectory.EnumerateDirectories())
			{
				if (ExcludedNames.Contains(SubDirectory.Name))
				{
					continue;
				}

				if (SubDirectory.Name == "Tests" && Rules.Target.SelectedTestModuleNames.Count > 0 && IsModuleDirectory(SubDirectory.GetParentDirectoryItem()))
				{
					// If it's one of the selected test module names, make sure to include them exclusively
					if (!Rules.Target.SelectedTestModuleNames.Any(STM => SubDirectory.FullName.EndsWith($"{Path.DirectorySeparatorChar}{STM}{Path.DirectorySeparatorChar}Tests", StringComparison.Ordinal)))
					{
						continue;
					}
					else
					{
						FindInputFilesFromDirectoryRecursiveTests(SubDirectory, ExcludedNames, SourceDirectories, DirectoryToSourceFiles, InputFiles, Logger);
					}
				}
				else
				{
					FindInputFilesFromDirectoryRecursiveTests(SubDirectory, ExcludedNames, SourceDirectories, DirectoryToSourceFiles, InputFiles, Logger);
				}
			}

			if (SourceFiles.Length > 0)
			{
				SourceDirectories.Add(BaseDirectory.Location);
			}
			DirectoryToSourceFiles.Add(BaseDirectory, SourceFiles);
		}

		private static bool IsModuleDirectory(DirectoryItem? InDirectory)
		{
			if (InDirectory == null)
			{
				return false;
			}
			return Directory.GetFiles(InDirectory.FullName, "*.Build.cs").Length > 0;
		}

		/// <summary>
		/// Finds all the source files that should be built for this module
		/// </summary>
		/// <param name="BaseDirectory">Directory to search from</param>
		/// <param name="ExcludedNames">Set of excluded directory names (eg. other platforms)</param>
		/// <param name="SourceDirectories">Set of all non-empty source directories.</param>
		/// <param name="DirectoryToSourceFiles">Map from directory to source files inside it</param>
		/// <param name="InputFiles">Collection of source files, categorized by type</param>
		/// <param name="Logger">Logger for output</param>
		static void FindInputFilesFromDirectoryRecursive(DirectoryItem BaseDirectory, IReadOnlySet<string> ExcludedNames, HashSet<DirectoryReference> SourceDirectories, Dictionary<DirectoryItem, FileItem[]> DirectoryToSourceFiles, InputFileCollection InputFiles, ILogger Logger)
		{
			bool bIgnoreFileFound;
			FileItem[] SourceFiles = FindInputFilesFromDirectory(BaseDirectory, InputFiles, out bIgnoreFileFound, Logger);

			if (bIgnoreFileFound)
			{
				return;
			}

			foreach (DirectoryItem SubDirectory in BaseDirectory.EnumerateDirectories())
			{
				if (!ExcludedNames.Contains(SubDirectory.Name))
				{
					FindInputFilesFromDirectoryRecursive(SubDirectory, ExcludedNames, SourceDirectories, DirectoryToSourceFiles, InputFiles, Logger);
				}
			}

			if (SourceFiles.Length > 0)
			{
				SourceDirectories.Add(BaseDirectory.Location);
			}
			DirectoryToSourceFiles.Add(BaseDirectory, SourceFiles);
		}

		// List of known extensions that are checked in FindInputFilesFromDirectory, so we can log out only if a hidden file
		// is skipped, only if it has an extension that would normally be compiled
		static readonly string[] KnownInputFileExtensions =
		{
			".h", ".ipsh",
			".cpp", ".cxx", ".ixx", ".c", ".cc",
			".m", ".mm", ".swift",
			".rc", ".ispc",
			".proto",
		};

		/// <summary>
		/// Finds the input files that should be built for this module, from a given directory
		/// </summary>
		/// <param name="BaseDirectory"></param>
		/// <param name="InputFiles"></param>
		/// <param name="bIgnoreFileFound"></param>
		/// <param name="Logger">Logger for output</param>
		/// <returns>Array of source files</returns>
		static FileItem[] FindInputFilesFromDirectory(DirectoryItem BaseDirectory, InputFileCollection InputFiles, out bool bIgnoreFileFound, ILogger Logger)
		{
			bIgnoreFileFound = false;
			List<FileItem> SourceFiles = new List<FileItem>();
			foreach (FileItem InputFile in BaseDirectory.EnumerateFiles())
			{
				if (InputFile.Name == ".ubtignore")
				{
					bIgnoreFileFound = true;
				}
				// some text editors will leave temp files that start with . next to a, say, .cpp file, and we do not want to compile them,
				// but skip files like .DS_Store that don't have an extension (so, no more .'s after the first)
				else if (InputFile.Name.StartsWith(".", StringComparison.Ordinal) && InputFile.Name.IndexOf('.', 1) != -1)
				{
					if (KnownInputFileExtensions.Contains(InputFile.Location.GetExtension()))
					{
						Logger.LogInformation("Hidden file '{HiddenFile}' found, skipping.", InputFile.FullName);
					}
				}
				else if (InputFile.HasExtension(".h"))
				{
					InputFiles.HeaderFiles.Add(InputFile);
				}
				else if (InputFile.HasExtension(".isph"))
				{
					InputFiles.ISPCHeaderFiles.Add(InputFile);
				}
				else if (InputFile.HasExtension(".cpp"))
				{
					SourceFiles.Add(InputFile);
					InputFiles.CPPFiles.Add(InputFile);
				}
				else if (InputFile.HasExtension(".ixx"))
				{
					SourceFiles.Add(InputFile);
					InputFiles.IXXFiles.Add(InputFile);
				}
				else if (InputFile.HasExtension(".c"))
				{
					SourceFiles.Add(InputFile);
					InputFiles.CFiles.Add(InputFile);
				}
				else if (InputFile.HasExtension(".cc"))
				{
					SourceFiles.Add(InputFile);
					InputFiles.CCFiles.Add(InputFile);
				}
				else if (InputFile.HasExtension(".cxx"))
				{
					SourceFiles.Add(InputFile);
					InputFiles.CXXFiles.Add(InputFile);
				}
				else if (InputFile.HasExtension(".m"))
				{
					SourceFiles.Add(InputFile);
					InputFiles.MFiles.Add(InputFile);
				}
				else if (InputFile.HasExtension(".mm"))
				{
					SourceFiles.Add(InputFile);
					InputFiles.MMFiles.Add(InputFile);
				}
				else if (InputFile.HasExtension(".swift"))
				{
					SourceFiles.Add(InputFile);
					InputFiles.SwiftFiles.Add(InputFile);
				}
				else if (InputFile.HasExtension(".rc"))
				{
					SourceFiles.Add(InputFile);
					InputFiles.RCFiles.Add(InputFile);
				}
				else if (InputFile.HasExtension(".ispc"))
				{
					SourceFiles.Add(InputFile);
					InputFiles.ISPCFiles.Add(InputFile);
				}
				else if (InputFile.HasExtension(".proto"))
				{
					InputFiles.ProtoFiles.Add(InputFile);
				}
				// if you add any extension checks here, make sure to update KnownInputFileExtensions
			}
			return SourceFiles.ToArray();
		}

		public static List<string> GenerateHeaderCpp(string HeaderName, string IncludeFileString)
		{
			HeaderName = HeaderName.Substring(0, HeaderName.Length - 2).Replace(".", "_", StringComparison.Ordinal).Replace("-", "_", StringComparison.Ordinal);

			List<string> GeneratedHeaderCppContents = new();
			GeneratedHeaderCppContents.Add("// This file is automatically generated at compile-time to include a user created header file.");
			GeneratedHeaderCppContents.Add("#define UE_DIRECT_HEADER_COMPILE 1");
			GeneratedHeaderCppContents.Add($"#define __COMPILING_{HeaderName}");
			GeneratedHeaderCppContents.Add($"#include \"{IncludeFileString.Replace('\\', '/')}\" // IWYU pragma: associated");
			return GeneratedHeaderCppContents;
		}

		/// <summary>
		/// Gets a set of source files for the given directory. Used to detect when the makefile is out of date.
		/// </summary>
		/// <param name="Directory"></param>
		/// <param name="Logger">Logger for output</param>
		/// <returns>Array of source files</returns>
		public static FileItem[] GetSourceFiles(DirectoryItem Directory, ILogger Logger)
		{
			bool bIgnoreFileFound;
			FileItem[] Files = FindInputFilesFromDirectory(Directory, new InputFileCollection(), out bIgnoreFileFound, Logger);
			if (bIgnoreFileFound)
			{
				return Array.Empty<FileItem>();
			}
			return Files;
		}

		/// <summary>
		/// Gets a set of source files and headers for the given directory. Used to detect when the makefile is out of date due to a new directory
		/// </summary>
		/// <param name="Directory"></param>
		/// <param name="Logger">Logger for output</param>
		/// <returns>Array of source files and headers</returns>
		public static FileItem[] GetSourceFilesAndHeaders(DirectoryItem Directory, ILogger Logger)
		{
			bool bIgnoreFileFound;
			InputFileCollection InputFiles = new InputFileCollection();
			List<FileItem> Files = new List<FileItem>(FindInputFilesFromDirectory(Directory, InputFiles, out bIgnoreFileFound, Logger));
			if (bIgnoreFileFound)
			{
				return Array.Empty<FileItem>();
			}
			Files.AddRange(InputFiles.HeaderFiles);
			return Files.ToArray();
		}

		/// <summary>
		/// Checks a given directory path whether it exists and if it contains any Verse source files
		/// </summary>
		public static bool IsValidVerseDirectory(DirectoryReference MaybeVerseDirectory)
		{
			if (!DirectoryReference.Exists(MaybeVerseDirectory))
			{
				return false;
			}

			foreach (string FilePath in Directory.EnumerateFiles(MaybeVerseDirectory.FullName, "*.v*", SearchOption.AllDirectories))
			{
				if (FilePath.EndsWith(".verse", System.StringComparison.Ordinal) || FilePath.EndsWith(".vmodule", System.StringComparison.Ordinal))
				{
					return true;
				}
			}

			return false;
		}
		

		/// <summary>
		/// Checks a given directory path whether it exists and if it contains any Proto source files
		/// </summary>
		public static bool IsValidProtoDirectory(DirectoryReference MaybeProtoDirectory)
		{
			if (!DirectoryReference.Exists(MaybeProtoDirectory))
			{
				return false;
			}

			return Directory.EnumerateFiles(MaybeProtoDirectory.FullName, "*.proto", SearchOption.AllDirectories).Any();
		}
	}
}
