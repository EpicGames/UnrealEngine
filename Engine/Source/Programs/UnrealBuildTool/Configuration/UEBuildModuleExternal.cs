// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace UnrealBuildTool
{
	/// <summary>
	/// A module that is never compiled by us, and is only used to group include paths and libraries into a dependency unit.
	/// </summary>
	class UEBuildModuleExternal : UEBuildModule
	{
		public UEBuildModuleExternal(ModuleRules Rules, DirectoryReference IntermediateDirectory, DirectoryReference IntermediateDirectoryNoArch, ILogger Logger)
			: base(Rules, IntermediateDirectory, IntermediateDirectoryNoArch, Logger)
		{
		}
	}
}
