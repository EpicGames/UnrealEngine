// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using System.Linq;
using System.Reflection;
using System.Text;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using OpenTracing.Util;
using static UnrealBuildTool.DataDrivenPlatformInfo;

namespace UnrealBuildTool
{
	/// <summary>
	/// Enum for the different ways a platform can support multiple architectures within UnrealBuildTool
	/// </summary>
	public enum UnrealArchitectureMode
	{
		/// <summary>
		/// This platform only supports a single architecture (consoles, etc)
		/// </summary>
		SingleArchitecture,

		/// <summary>
		/// This platform needs separate targets per architecture (compiling for multiple will compile two entirely separate set of source/exectuable/intermediates)
		/// </summary>
		OneTargetPerArchitecture,

		/// <summary>
		/// This will create a single target, but compile each source file separately
		/// </summary>
		SingleTargetCompileSeparately,

		/// <summary>
		/// This will create a single target, but compile each source file and link the executable separately - but then packaged together into one final output
		/// </summary>
		SingleTargetLinkSeparately,
	}

	/// <summary>
	/// Full architecture configuation information for a platform. Can be found by platform with UnrealArchitectureConfig.ForPlatform(), or UEBuildPlatform.ArchitectureConfig
	/// </summary>
	public class UnrealArchitectureConfig
	{
		/// <summary>
		/// Get the architecture configuration object for a given platform
		/// </summary>
		/// <param name="Platform"></param>
		/// <returns></returns>
		public static UnrealArchitectureConfig ForPlatform(UnrealTargetPlatform Platform)
		{
			return UEBuildPlatform.GetBuildPlatform(Platform).ArchitectureConfig;
		}

		/// <summary>
		/// Get all known ArchitectureConfig objects
		/// </summary>
		/// <returns></returns>
		public static IEnumerable<UnrealArchitectureConfig> AllConfigs()
		{
			// return ArchConfigs for platforms that have a BuildPlatform (which is where the Configs are stored)
			return UnrealTargetPlatform.GetValidPlatforms().Where(x => UEBuildPlatform.TryGetBuildPlatform(x, out _)).Select(x => UEBuildPlatform.GetBuildPlatform(x).ArchitectureConfig);
		}

		/// <summary>
		/// The multi-architecture mode for this platform (potentially single-architecture)
		/// </summary>
		public UnrealArchitectureMode Mode { get; }

		/// <summary>
		/// The set of all architecture this platform supports. Any platform specified on the UBT commandline not in this list will be an error
		/// </summary>
		public UnrealArchitectures AllSupportedArchitectures { get; }

		/// <summary>
		/// This determines what architecture(s) to compile/package when no architeecture is specified on the commandline
		/// </summary>
		/// <param name="ProjectFile"></param>
		/// <param name="TargetName"></param>
		/// <returns></returns>
		public virtual UnrealArchitectures ActiveArchitectures(FileReference? ProjectFile, string? TargetName)
		{
			if (Mode != UnrealArchitectureMode.SingleArchitecture || AllSupportedArchitectures.bIsMultiArch)
			{
				throw new BuildException("Platforms that support multiple platforms are expected to override ActiveArchitectures in a platform-specifiec UnraelArchitectureConfig subclass");
			}
			return AllSupportedArchitectures;
		}

		/// <summary>
		/// Like ProjectSupportedArchitectures, except when building in distribution mode. Defaults to ActiveArchitectures
		/// </summary>
		/// <param name="ProjectFile"></param>
		/// <param name="TargetName"></param>
		/// <returns></returns>
		public virtual UnrealArchitectures DistributionArchitectures(FileReference? ProjectFile, string? TargetName)
		{
			return ActiveArchitectures(ProjectFile, TargetName);
		}

		/// <summary>
		/// Returns the set all architectures potentially supported by this project. Can be used by project file gnenerators to restrict IDE architecture options
		/// Defaults to AllSupportedArchitectures
		/// </summary>
		/// <param name="ProjectFile"></param>
		/// <param name="TargetName"></param>
		/// <returns></returns>
		public virtual UnrealArchitectures ProjectSupportedArchitectures(FileReference? ProjectFile, string? TargetName)
		{
			return AllSupportedArchitectures;
		}

		/// <summary>
		/// Returns if architecture name should be used when making intermediate directories, per-architecture filenames, etc
		/// It is virtual, so a platform can choose to skip architecture name for one platform, but not another 
		/// </summary>
		/// <returns></returns>
		public virtual bool RequiresArchitectureFilenames(UnrealArchitectures Architectures)
		{
			// @todo: this needs to also have directory vs file (we may need directory names but not filenames in the case of Single-target multi-arch)
			return Mode == UnrealArchitectureMode.OneTargetPerArchitecture;
		}

		/// <summary>
		/// Convert user specified architecture strings to what the platform wants
		/// </summary>
		/// <param name="Architecture"></param>
		/// <returns></returns>
		public virtual string ConvertToReadableArchitecture(UnrealArch Architecture)
		{
			return Architecture.ToString();
		}

		/// <summary>
		/// Get name for architecture-specific directories (can be shorter than architecture name itself)
		/// </summary>
		public virtual string GetFolderNameForArchitecture(UnrealArch Architecture)
		{
			// by default, use the architecture name
			return Architecture.ToString();
		}

		/// <summary>
		/// Get name for architecture-specific directories (can be shorter than architecture name itself)
		/// </summary>
		public virtual string GetFolderNameForArchitectures(UnrealArchitectures Architectures)
		{
			// by default, use the architecture names combined with +
			return Architectures.GetFolderNameForPlatform(this);
		}

		/// <summary>
		/// Returns the architecture of the currently running OS (only used for desktop platforms, so will throw an exception in the general case)
		/// </summary>
		/// <returns></returns>
		public virtual UnrealArch GetHostArchitecture()
		{
			if (AllSupportedArchitectures.bIsMultiArch)
			{
				throw new BuildException($"Asking for Host architecture from {GetType()} which supports multiple architectures, but did not override GetHostArchitecture()");
			}

			return AllSupportedArchitectures.SingleArchitecture;
		}

		/// <summary>
		/// Simple constructor for platforms with a single architecture
		/// </summary>
		/// <param name="SingleArchitecture"></param>
		public UnrealArchitectureConfig(UnrealArch SingleArchitecture)
		{
			Mode = UnrealArchitectureMode.SingleArchitecture;
			AllSupportedArchitectures = new UnrealArchitectures(SingleArchitecture);
		}

		/// <summary>
		/// Full constructor for platforms that support multiple architectures
		/// </summary>
		/// <param name="Mode"></param>
		/// <param name="SupportedArchitectures"></param>
		protected UnrealArchitectureConfig(UnrealArchitectureMode Mode, IEnumerable<UnrealArch> SupportedArchitectures)
		{
			this.Mode = Mode;
			AllSupportedArchitectures = new UnrealArchitectures(SupportedArchitectures.ToList());
		}
	}

	abstract class UEBuildPlatform
	{
		private static readonly Dictionary<UnrealTargetPlatform, UEBuildPlatform> BuildPlatformDictionary = new Dictionary<UnrealTargetPlatform, UEBuildPlatform>();

		// a mapping of a group to the platforms in the group (ie, Microsoft contains Win32 and Win64)
		static readonly Dictionary<UnrealPlatformGroup, List<UnrealTargetPlatform>> PlatformGroupDictionary = new Dictionary<UnrealPlatformGroup, List<UnrealTargetPlatform>>();

		/// <summary>
		/// The corresponding target platform enum
		/// </summary>
		public readonly UnrealTargetPlatform Platform;

		/// <summary>
		/// The configuration about the architecture(s) this platform supports
		/// </summary>
		public readonly UnrealArchitectureConfig ArchitectureConfig;

		/// <summary>
		/// Logger for this platform
		/// </summary>
		protected readonly ILogger Logger;

		/// <summary>
		/// All the platform folder names
		/// </summary>
		private static string[]? CachedPlatformFolderNames;

		/// <summary>
		/// Cached copy of the list of folders to include for this platform
		/// </summary>
		private IReadOnlySet<string>? CachedIncludedFolderNames;

		/// <summary>
		/// Cached copy of the list of folders to exclude for this platform
		/// </summary>
		private IReadOnlySet<string>? CachedExcludedFolderNames;

		/// <summary>
		/// Constructor.
		/// </summary>
		/// <param name="InPlatform">The enum value for this platform</param>
		/// <param name="SDK">The SDK management object for this platform</param>
		/// <param name="ArchitectureConfig">THe architecture configuraton for this platform. This is returned by UnrealArchitectureConfig.ForPlatform()</param>
		/// <param name="InLogger">Logger for output</param>
		public UEBuildPlatform(UnrealTargetPlatform InPlatform, UEBuildPlatformSDK SDK, UnrealArchitectureConfig ArchitectureConfig, ILogger InLogger)
		{
			Platform = InPlatform;
			Logger = InLogger;
			this.ArchitectureConfig = ArchitectureConfig;

			// check DDPI to see if the platform is enabled on this host platform
			string IniPlatformName = ConfigHierarchy.GetIniPlatformName(Platform);
			bool bIsEnabled = false;
			ConfigDataDrivenPlatformInfo? DDPI = DataDrivenPlatformInfo.GetDataDrivenInfoForPlatform(IniPlatformName);
			if (DDPI != null)
			{
				bIsEnabled = DDPI.bIsEnabled;
			}

			// set up the SDK if the platform is enabled
			UEBuildPlatformSDK.RegisterSDKForPlatform(SDK, Platform.ToString(), bIsEnabled);
			if (bIsEnabled)
			{
				SDK.ManageAndValidateSDK();
			}
		}

		private static readonly string[] UATProjectParams = { "-project=", "-scriptsforproject=" };
		private static void InitializePerPlatformSDKs(string[] Args, bool bArgumentsAreForUBT, ILogger Logger)
		{
			// Before we setup AutoSDK, we check to see if any projects need to override the Main version so that AutoSDK
			// will set up an alternate SDK
		
			Dictionary<string, string> PlatformToVersionMap = new();

			IEnumerable<FileReference?> ProjectFiles;

			if (bArgumentsAreForUBT == false)
			{
				ProjectFiles = Args
						// find arguments that start with one of the hard-coded parameters
						.Where(x => UATProjectParams.Any(y => x.StartsWith(y, StringComparison.OrdinalIgnoreCase)))
						// treat the part after the = as a path to a uproject, and ask the NativeProjects class to find a .uproject
						.Select(x => NativeProjects.FindProjectFile(x.Substring(x.IndexOf('=', StringComparison.Ordinal) + 1), Logger))
						// only use existant projects
						.Where(x => x != null && FileReference.Exists(x));
			}
			else
			{
				CommandLineArguments CommandLine = new(Args);
				BuildConfiguration BuildConfiguration = new();
				XmlConfig.ApplyTo(BuildConfiguration);
				CommandLine.ApplyTo(BuildConfiguration);

				// get the project files for all targets - we allow null uproject files which means to use the defaults
				// (same as a uproject that has no override SDK versions set)
				ProjectFiles = TargetDescriptor.ParseCommandLineForProjects(CommandLine, Logger);
			}

			UEBuildPlatformSDK.InitializePerProjectSDKVersions(ProjectFiles.OfType<FileReference>());

			// clear the cache used for auto-switching to the best manually-installed SDK
			UEBuildPlatformSDK.ClearManualSDKEnvVarCache();
		}

		/// <summary>
		/// Finds all the UEBuildPlatformFactory types in this assembly and uses them to register all the available platforms, and uses a UBT commandline to check for per-project SDKs
		/// </summary>
		/// <param name="bIncludeNonInstalledPlatforms">Whether to register platforms that are not installed</param>
		/// <param name="bHostPlatformOnly">Only register the host platform</param>
		/// <param name="ArgumentsForPerPlatform">Commandline args to look through for finding uprojects, to look up per-project SDK versions</param>
		/// <param name="UBTModeType">The UBT mode (usually Build, but </param>
		/// <param name="Logger">Logger for output</param>
		internal static void RegisterPlatforms(bool bIncludeNonInstalledPlatforms, bool bHostPlatformOnly, Type UBTModeType, string[] ArgumentsForPerPlatform, ILogger Logger)
		{
			bool bUseTargetTripleParams = false;
			// @todo : add a ToolModeOptions.UseTargetTripleCommandLine or something
			if (UBTModeType.Name == "BuildMode")
			{
				bUseTargetTripleParams = true;
			}
			RegisterPlatforms(bIncludeNonInstalledPlatforms, bHostPlatformOnly, ArgumentsForPerPlatform, bArgumentsAreForUBT: bUseTargetTripleParams, Logger);
		}

		/// <summary>
		/// Finds all the UEBuildPlatformFactory types in this assembly and uses them to register all the available platforms, and uses a UAT commandline to check for per-project SDKs
		/// </summary>
		/// <param name="bIncludeNonInstalledPlatforms">Whether to register platforms that are not installed</param>
		/// <param name="bHostPlatformOnly">Only register the host platform</param>
		/// <param name="ArgumentsForPerPlatform">Commandline args to look through for finding uprojects, to look up per-project SDK versions</param>
		/// <param name="Logger">Logger for output</param>
		internal static void RegisterPlatforms(bool bIncludeNonInstalledPlatforms, bool bHostPlatformOnly, string[] ArgumentsForPerPlatform, ILogger Logger)
		{
			RegisterPlatforms(bIncludeNonInstalledPlatforms, bHostPlatformOnly, ArgumentsForPerPlatform, bArgumentsAreForUBT: false, Logger);
		}

		private static void RegisterPlatforms(bool bIncludeNonInstalledPlatforms, bool bHostPlatformOnly, string[] ArgumentsForPerPlatform, bool bArgumentsAreForUBT, ILogger Logger)
		{
			// Initialize the installed platform info
			using (GlobalTracer.Instance.BuildSpan("Initializing InstalledPlatformInfo").StartActive())
			{
				InstalledPlatformInfo.Initialize();
			}

			using (GlobalTracer.Instance.BuildSpan("Initializing PerPlatformSDKs").StartActive())
			{
				InitializePerPlatformSDKs(ArgumentsForPerPlatform, bArgumentsAreForUBT, Logger);
			}

			// Find and register all tool chains and build platforms that are present
			Type[] AllTypes;
			using (GlobalTracer.Instance.BuildSpan("Querying types").StartActive())
			{
				AllTypes = Assembly.GetExecutingAssembly().GetTypes();
			}

			// register all build platforms first, since they implement SDK-switching logic that can set environment variables
			foreach (Type CheckType in AllTypes)
			{
				if (CheckType.IsClass && !CheckType.IsAbstract)
				{
					if (CheckType.IsSubclassOf(typeof(UEBuildPlatformFactory)))
					{
						Logger.LogDebug("    Registering build platform: {Platform}", CheckType.ToString());
						using (GlobalTracer.Instance.BuildSpan(CheckType.Name).StartActive())
						{
							UEBuildPlatformFactory TempInst = (UEBuildPlatformFactory)Activator.CreateInstance(CheckType)!;

							if (bHostPlatformOnly && TempInst.TargetPlatform != BuildHostPlatform.Current.Platform)
							{
								continue;
							}

							// We need all platforms to be registered when we run -validateplatform command to check SDK status of each
							if (bIncludeNonInstalledPlatforms || InstalledPlatformInfo.IsValidPlatform(TempInst.TargetPlatform))
							{
								TempInst.RegisterBuildPlatforms(Logger);
							}
						}
					}
				}
			}
		}

		/// <summary>
		/// Gets an array of all platform folder names
		/// </summary>
		/// <returns>Array of platform folders</returns>
		public static string[] GetPlatformFolderNames()
		{
			if (CachedPlatformFolderNames == null)
			{
				List<string> PlatformFolderNames = new List<string>();

				// Find all the platform folders to exclude from the list of precompiled modules
				PlatformFolderNames.AddRange(UnrealTargetPlatform.GetValidPlatformNames());

				// Also exclude all the platform groups that this platform is not a part of
				PlatformFolderNames.AddRange(UnrealPlatformGroup.GetValidGroupNames());

				// Save off the list as an array
				CachedPlatformFolderNames = PlatformFolderNames.ToArray();
			}
			return CachedPlatformFolderNames;
		}

		/// <summary>
		/// Finds a list of folder names to include when building for this platform
		/// </summary>
		public IReadOnlySet<string> GetIncludedFolderNames()
		{
			if (CachedIncludedFolderNames == null)
			{
				HashSet<string> Names = new HashSet<string>(DirectoryReference.Comparer);

				Names.Add(Platform.ToString());
				foreach (UnrealPlatformGroup Group in UEBuildPlatform.GetPlatformGroups(Platform))
				{
					Names.Add(Group.ToString());
				}

				CachedIncludedFolderNames = Names;
			}
			return CachedIncludedFolderNames;
		}

		/// <summary>
		/// Finds a list of folder names to exclude when building for this platform
		/// </summary>
		public IReadOnlySet<string> GetExcludedFolderNames()
		{
			CachedExcludedFolderNames ??= new HashSet<string>(GetPlatformFolderNames().Except(GetIncludedFolderNames()), DirectoryReference.Comparer);
			return CachedExcludedFolderNames;
		}

		/// <summary>
		/// Whether the required external SDKs are installed for this platform. Could be either a manual install or an AutoSDK.
		/// </summary>
		public SDKStatus HasRequiredSDKsInstalled()
		{
			UEBuildPlatformSDK? SDK = UEBuildPlatform.GetSDK(Platform);
			if (SDK == null || !SDK.bIsSdkAllowedOnHost)
			{
				return SDKStatus.Invalid;
			}
			return SDK.HasRequiredSDKsInstalled();
		}

		/// <summary>
		/// Whether this platform requires specific Visual Studio version.
		/// </summary>
		public virtual VCProjectFileFormat GetRequiredVisualStudioVersion()
		{
			return VCProjectFileFormat.Default;
		}

		/// <summary>
		/// The version required to support Visual Studio
		/// </summary>
		public virtual Version GetVersionRequiredForVisualStudio(VCProjectFileFormat Format)
		{
			return new Version();
		}

		/// <summary>
		/// Gets all the registered platforms
		/// </summary>
		/// <returns>Sequence of registered platforms</returns>
		public static IEnumerable<UnrealTargetPlatform> GetRegisteredPlatforms()
		{
			return BuildPlatformDictionary.Keys;
		}

		/// <summary>
		/// Searches a directory tree for build products to be cleaned.
		/// </summary>
		/// <param name="BaseDir">The directory to search</param>
		/// <param name="NamePrefixes">Target or application names that may appear at the start of the build product name (eg. "UnrealEditor", "ShooterGameEditor")</param>
		/// <param name="NameSuffixes">Suffixes which may appear at the end of the build product name</param>
		/// <param name="FilesToClean">List to receive a list of files to be cleaned</param>
		/// <param name="DirectoriesToClean">List to receive a list of directories to be cleaned</param>
		public void FindBuildProductsToClean(DirectoryReference BaseDir, string[] NamePrefixes, string[] NameSuffixes, List<FileReference> FilesToClean, List<DirectoryReference> DirectoriesToClean)
		{
			foreach (FileReference File in DirectoryReference.EnumerateFiles(BaseDir))
			{
				string FileName = File.GetFileName();
				if (IsDefaultBuildProduct(FileName, NamePrefixes, NameSuffixes) || IsBuildProduct(FileName, NamePrefixes, NameSuffixes))
				{
					FilesToClean.Add(File);
				}
			}
			foreach (DirectoryReference SubDir in DirectoryReference.EnumerateDirectories(BaseDir))
			{
				string SubDirName = SubDir.GetDirectoryName();
				if (IsBuildProduct(SubDirName, NamePrefixes, NameSuffixes))
				{
					DirectoriesToClean.Add(SubDir);
				}
				else
				{
					FindBuildProductsToClean(SubDir, NamePrefixes, NameSuffixes, FilesToClean, DirectoriesToClean);
				}
			}
		}

		/// <summary>
		/// Enumerates any additional directories needed to clean this target
		/// </summary>
		/// <param name="Target">The target to clean</param>
		/// <param name="FilesToDelete">Receives a list of files to be removed</param>
		/// <param name="DirectoriesToDelete">Receives a list of directories to be removed</param>
		public virtual void FindAdditionalBuildProductsToClean(ReadOnlyTargetRules Target, List<FileReference> FilesToDelete, List<DirectoryReference> DirectoriesToDelete)
		{
		}

		/// <summary>
		/// Determines if a filename is a default UBT build product
		/// </summary>
		/// <param name="FileName">The name to check</param>
		/// <param name="NamePrefixes">Target or application names that may appear at the start of the build product name (eg. "UnrealEditor", "ShooterGameEditor")</param>
		/// <param name="NameSuffixes">Suffixes which may appear at the end of the build product name</param>
		/// <returns>True if the substring matches the name of a build product, false otherwise</returns>
		public static bool IsDefaultBuildProduct(string FileName, string[] NamePrefixes, string[] NameSuffixes)
		{
			return UEBuildPlatform.IsBuildProductName(FileName, NamePrefixes, NameSuffixes, ".target")
				|| UEBuildPlatform.IsBuildProductName(FileName, NamePrefixes, NameSuffixes, ".modules")
				|| UEBuildPlatform.IsBuildProductName(FileName, NamePrefixes, NameSuffixes, ".version");
		}

		/// <summary>
		/// Determines if the given name is a build product for a target.
		/// </summary>
		/// <param name="FileName">The name to check</param>
		/// <param name="NamePrefixes">Target or application names that may appear at the start of the build product name (eg. "UnrealEditor", "ShooterGameEditor")</param>
		/// <param name="NameSuffixes">Suffixes which may appear at the end of the build product name</param>
		/// <returns>True if the string matches the name of a build product, false otherwise</returns>
		public abstract bool IsBuildProduct(string FileName, string[] NamePrefixes, string[] NameSuffixes);

		/// <summary>
		/// Determines if a string is in the canonical name of a UE build product, with a specific extension (eg. "UnrealEditor-Win64-Debug.exe" or "UnrealEditor-ModuleName-Win64-Debug.dll"). 
		/// </summary>
		/// <param name="FileName">The file name to check</param>
		/// <param name="NamePrefixes">Target or application names that may appear at the start of the build product name (eg. "UnrealEditor", "ShooterGameEditor")</param>
		/// <param name="NameSuffixes">Suffixes which may appear at the end of the build product name</param>
		/// <param name="Extension">The extension to check for</param>
		/// <returns>True if the string matches the name of a build product, false otherwise</returns>
		public static bool IsBuildProductName(string FileName, string[] NamePrefixes, string[] NameSuffixes, string Extension)
		{
			return IsBuildProductName(FileName, 0, FileName.Length, NamePrefixes, NameSuffixes, Extension);
		}

		/// <summary>
		/// Determines if a substring is in the canonical name of a UE build product, with a specific extension (eg. "UnrealEditor-Win64-Debug.exe" or "UnrealEditor-ModuleName-Win64-Debug.dll"). 
		/// </summary>
		/// <param name="FileName">The name to check</param>
		/// <param name="Index">Index of the first character to be checked</param>
		/// <param name="Count">Number of characters of the substring to check</param>
		/// <param name="NamePrefixes">Target or application names that may appear at the start of the build product name (eg. "UnrealEditor", "ShooterGameEditor")</param>
		/// <param name="NameSuffixes">Suffixes which may appear at the end of the build product name</param>
		/// <param name="Extension">The extension to check for</param>
		/// <returns>True if the substring matches the name of a build product, false otherwise</returns>
		public static bool IsBuildProductName(string FileName, int Index, int Count, string[] NamePrefixes, string[] NameSuffixes, string Extension)
		{
			// Check if the extension matches, and forward on to the next IsBuildProductName() overload without it if it does.
			if (Count > Extension.Length && String.Compare(FileName, Index + Count - Extension.Length, Extension, 0, Extension.Length, StringComparison.OrdinalIgnoreCase) == 0)
			{
				return IsBuildProductName(FileName, Index, Count - Extension.Length, NamePrefixes, NameSuffixes);
			}
			return false;
		}

		/// <summary>
		/// Determines if a substring is in the canonical name of a UE build product, excluding extension or other decoration (eg. "UnrealEditor-Win64-Debug" or "UnrealEditor-ModuleName-Win64-Debug"). 
		/// </summary>
		/// <param name="FileName">The name to check</param>
		/// <param name="Index">Index of the first character to be checked</param>
		/// <param name="Count">Number of characters of the substring to check</param>
		/// <param name="NamePrefixes">Target or application names that may appear at the start of the build product name (eg. "UnrealEditor", "ShooterGameEditor")</param>
		/// <param name="NameSuffixes">Suffixes which may appear at the end of the build product name</param>
		/// <returns>True if the substring matches the name of a build product, false otherwise</returns>
		public static bool IsBuildProductName(string FileName, int Index, int Count, string[] NamePrefixes, string[] NameSuffixes)
		{
			foreach (string NamePrefix in NamePrefixes)
			{
				if (Count >= NamePrefix.Length && String.Compare(FileName, Index, NamePrefix, 0, NamePrefix.Length, StringComparison.OrdinalIgnoreCase) == 0)
				{
					int MinIdx = Index + NamePrefix.Length;
					foreach (string NameSuffix in NameSuffixes)
					{
						int MaxIdx = Index + Count - NameSuffix.Length;
						if (MaxIdx >= MinIdx && String.Compare(FileName, MaxIdx, NameSuffix, 0, NameSuffix.Length, StringComparison.OrdinalIgnoreCase) == 0)
						{
							if (MinIdx < MaxIdx && FileName[MinIdx] == '-')
							{
								MinIdx++;
								while (MinIdx < MaxIdx && FileName[MinIdx] != '-' && FileName[MinIdx] != '.')
								{
									MinIdx++;
								}
							}
							if (MinIdx == MaxIdx)
							{
								return true;
							}
						}
					}
				}
			}
			return false;
		}

		public virtual void PostBuildSync(UEBuildTarget Target)
		{
		}

		/// <summary>
		/// Get the bundle directory for the shared link environment
		/// </summary>
		/// <param name="Rules">The target rules</param>
		/// <param name="OutputFiles">List of executable output files</param>
		/// <returns>Path to the bundle directory</returns>
		public virtual DirectoryReference? GetBundleDirectory(ReadOnlyTargetRules Rules, List<FileReference> OutputFiles)
		{
			return null;
		}

		/// <summary>
		/// Determines whether a given platform is available
		/// </summary>
		/// <param name="Platform">The platform to check for</param>
		/// <param name="bIgnoreSDKCheck">Ignore the sdks presence when checking for platform availablity (many platforms can be cooked as a target platform without requiring sdk)</param>
		/// <returns>True if it's available, false otherwise</returns>
		public static bool IsPlatformAvailable(UnrealTargetPlatform Platform, bool bIgnoreSDKCheck = false)
		{
			return BuildPlatformDictionary.ContainsKey(Platform) && (bIgnoreSDKCheck || BuildPlatformDictionary[Platform].HasRequiredSDKsInstalled() == SDKStatus.Valid);
		}

		/// <summary>
		/// Determines whether a given platform is available in the context of a particular Taget
		/// </summary>
		/// <param name="Platform">The platform to check for</param>
		/// <param name="Target">A Target object that may further restrict available platforms</param>
		/// <param name="bIgnoreSDKCheck">Ignore the sdks presence when checking for platform availablity (many platforms can be cooked as a target platform without requiring sdk)</param>
		/// <returns>True if it's available, false otherwise</returns>
		public static bool IsPlatformAvailableForTarget(UnrealTargetPlatform Platform, ReadOnlyTargetRules Target, bool bIgnoreSDKCheck = false)
		{
			return IsPlatformAvailable(Platform, bIgnoreSDKCheck) && Target.IsPlatformOptedIn(Platform);
		}

		/// <summary>
		/// Returns a detailed reason string explaining why a platform is not available to build.
		/// </summary>
		public static string GetPlatformUnavailableReason(UnrealTargetPlatform Platform)
		{
			if (!BuildPlatformDictionary.ContainsKey(Platform))
			{
				return $"Platform {Platform} is not registered. Check that the necessary platform support files exist (DataDrivenPlatformInfo.ini, SDK.json, etc).";
			}

			UEBuildPlatformSDK? Sdk = UEBuildPlatformSDK.GetSDKForPlatform(Platform.ToString());
			if (Sdk == null)
			{
				return $"Platform {Platform} is not a valid platform to build. No SDK is registered for this platform.";
			}

			StringBuilder Message = new();
			Message.AppendLine($"Platform {Platform} is not a valid platform to build. SDK validation failed:");

			SDKCollection SdkInfo = Sdk.GetAllSDKInfo();
			foreach (SDKDescriptor Desc in SdkInfo.Sdks)
			{
				if (Desc.Validity == SDKStatus.Valid)
				{
					continue;
				}

				string DisplayName = Desc.Name;
				string LocationHint = "";

				if (String.Equals(Desc.Name, "AutoSdk", StringComparison.OrdinalIgnoreCase))
				{
					DisplayName = "AutoSDK";
					if (UEBuildPlatformSDK.TryGetHostPlatformAutoSDKDir(out DirectoryReference? HostAutoSdkDir))
					{
						LocationHint = $" at {DirectoryReference.Combine(HostAutoSdkDir, Sdk.GetAutoSDKPlatformName())}";
					}
					else if (ReadOnlyBuildVersion.Current.IsLicenseeVersion
						|| !DirectoryReference.Exists(DirectoryReference.Combine(UnrealBuildBase.Unreal.EngineDirectory, "Restricted", "NotForLicensees")))
					{
						// Licensees don't use AutoSDK -- skip to avoid confusion about UE_SDKS_ROOT.
						continue;
					}
					else
					{
						LocationHint = " (UE_SDKS_ROOT is not set)";
					}
				}

				if (Desc.Current != null)
				{
					Message.AppendLine($"  {DisplayName}: found version {Desc.Current}{LocationHint}, required version {Desc.Min}.");
				}
				else
				{
					string Required = Desc.Min != null ? $" Required version {Desc.Min}." : "";
					Message.AppendLine($"  {DisplayName}: not found{LocationHint}.{Required}");
				}
			}

			return Message.ToString().TrimEnd();
		}

		/// <summary>
		/// Register the given platforms UEBuildPlatform instance
		/// </summary>
		/// <param name="InBuildPlatform"> The UEBuildPlatform instance to use for the InPlatform</param>
		/// <param name="Logger">Logger for output</param>
		public static void RegisterBuildPlatform(UEBuildPlatform InBuildPlatform, ILogger Logger)
		{
			Logger.LogDebug("        Registering build platform: {Platform} - buildable: {Buildable}", InBuildPlatform.Platform, InBuildPlatform.HasRequiredSDKsInstalled() == SDKStatus.Valid);

			if (BuildPlatformDictionary.ContainsKey(InBuildPlatform.Platform) == true)
			{
				Logger.LogWarning("RegisterBuildPlatform Warning: Registering build platform {Platform} for {ForPlatform} when it is already set to {CurPlatform}",
					InBuildPlatform.ToString(), InBuildPlatform.Platform.ToString(), BuildPlatformDictionary[InBuildPlatform.Platform].ToString());
				BuildPlatformDictionary[InBuildPlatform.Platform] = InBuildPlatform;
			}
			else
			{
				BuildPlatformDictionary.Add(InBuildPlatform.Platform, InBuildPlatform);
			}
		}

		/// <summary>
		/// Assign a platform as a member of the given group
		/// </summary>
		public static void RegisterPlatformWithGroup(UnrealTargetPlatform InPlatform, UnrealPlatformGroup InGroup)
		{
			// find or add the list of groups for this platform
			List<UnrealTargetPlatform>? Platforms;
			if (!PlatformGroupDictionary.TryGetValue(InGroup, out Platforms))
			{
				Platforms = new List<UnrealTargetPlatform>();
				PlatformGroupDictionary.Add(InGroup, Platforms);
			}
			Platforms.Add(InPlatform);
		}

		/// <summary>
		/// Retrieve the list of platforms in this group (if any)
		/// </summary>
		public static List<UnrealTargetPlatform> GetPlatformsInGroup(UnrealPlatformGroup InGroup)
		{
			List<UnrealTargetPlatform>? PlatformList;
			if (!PlatformGroupDictionary.TryGetValue(InGroup, out PlatformList))
			{
				PlatformList = new List<UnrealTargetPlatform>();
			}
			return PlatformList;
		}

		/// <summary>
		/// Enumerates all the platform groups for a given platform
		/// </summary>
		/// <param name="Platform">The platform to look for</param>
		/// <returns>List of platform groups that this platform is a member of</returns>
		public static IEnumerable<UnrealPlatformGroup> GetPlatformGroups(UnrealTargetPlatform Platform)
		{
			// Avoid allocations by using yield (one allocation for enumerable still created)
			foreach (KeyValuePair<UnrealPlatformGroup, List<UnrealTargetPlatform>> kvp in PlatformGroupDictionary)
			{
				if (kvp.Value.Contains(Platform))
				{
					yield return kvp.Key;
				}
			}
		}

		/// <summary>
		/// Retrieve the IUEBuildPlatform instance for the given TargetPlatform
		/// </summary>
		/// <param name="InPlatform">  The UnrealTargetPlatform being built</param>
		/// <returns>UEBuildPlatform  The instance of the build platform</returns>
		public static UEBuildPlatform GetBuildPlatform(UnrealTargetPlatform InPlatform)
		{
			UEBuildPlatform? Platform;
			if (!TryGetBuildPlatform(InPlatform, out Platform))
			{
				throw new BuildException("GetBuildPlatform: No BuildPlatform found for {0}", InPlatform.ToString());
			}
			return Platform;
		}

		/// <summary>
		/// Retrieve the IUEBuildPlatform instance for the given TargetPlatform
		/// </summary>
		/// <param name="InPlatform">  The UnrealTargetPlatform being built</param>
		/// <param name="Platform"></param>
		/// <returns>UEBuildPlatform  The instance of the build platform</returns>
		public static bool TryGetBuildPlatform(UnrealTargetPlatform InPlatform, [NotNullWhen(true)] out UEBuildPlatform? Platform)
		{
			return BuildPlatformDictionary.TryGetValue(InPlatform, out Platform);
		}

		/// <summary>
		/// Allow all registered build platforms to modify the newly created module
		/// passed in for the given platform.
		/// This is not required - but allows for hiding details of a particular platform.
		/// </summary>
		/// <param name="ModuleName">The name of the module</param>
		/// <param name="Rules">The module rules</param>
		/// <param name="Target">The target being build</param>
		public static void PlatformModifyHostModuleRules(string ModuleName, ModuleRules Rules, ReadOnlyTargetRules Target)
		{
			foreach (KeyValuePair<UnrealTargetPlatform, UEBuildPlatform> PlatformEntry in BuildPlatformDictionary)
			{
				PlatformEntry.Value.ModifyModuleRulesForOtherPlatform(ModuleName, Rules, Target);
			}
		}

		/// <summary>
		/// Returns the delimiter used to separate paths in the PATH environment variable for the platform we are executing on.
		/// </summary>
		[Obsolete("Deprecated in UE5.8. Replace with System.IO.Path.PathSeparator")]
		public static string GetPathVarDelimiter()
		{
			return Path.PathSeparator.ToString();
		}

		/// <summary>
		/// Gets the platform name that should be used.
		/// </summary>
		public virtual string GetPlatformName()
		{
			return Platform.ToString();
		}

		/// <summary>
		/// If this platform can be compiled with XGE
		/// </summary>
		public virtual bool CanUseXGE()
		{
			return true;
		}

		/// <summary>
		/// If this platform can be compiled with FASTBuild
		/// </summary>
		public virtual bool CanUseFASTBuild()
		{
			return false;
		}

		/// <summary>
		/// If this platform can be compiled with SN-DBS
		/// </summary>
		public virtual bool CanUseSNDBS()
		{
			return false;
		}

		/// <summary>
		/// Provides a set of SN-DBS tool templates for builds on this platform. The dictionary key is the name of the tool, such as "clang.exe", and the value is the template contents.
		/// </summary>
		/// <returns>Dictionary of tool templates or null if there are none.</returns>
		public virtual IReadOnlyDictionary<string, string>? GetSNDBSToolTemplates()
		{
			return null;
		}

		/// <summary>
		/// Set all the platform-specific defaults for a new target
		/// </summary>
		public virtual void ResetTarget(TargetRules Target)
		{
		}

		/// <summary>
		/// Validate a target's settings
		/// </summary>
		public virtual void ValidateTarget(TargetRules Target)
		{
			if (Target.bMergeModules)
			{
				ValidateMergedTarget(Target);
			}
		}

		/// <summary>
		/// Validate a target's settings for merged modules
		/// </summary>
		public virtual void ValidateMergedTarget(TargetRules Target)
		{
		}

		/// <summary>
		/// Validate a plugin's settings
		/// </summary>
		public virtual void ValidatePlugin(UEBuildPlugin Plugin, ReadOnlyTargetRules Target)
		{
		}

		/// <summary>
		/// Validate a UEBuildModule before it's processed
		/// <param name="Module">The UEBuildModule that needs to be validated</param>
		/// <param name="Target">Options for the target being built</param>
		/// <param name="Logger">The logger for output</param>
		/// </summary>
		public virtual void ValidateModule(UEBuildModule Module, ReadOnlyTargetRules Target, ILogger Logger)
		{
		}

		/// <summary>
		/// Validate a UEBuildModule's include paths before it's processed
		/// </summary>
		/// <param name="Module">The UEBuildModule that needs to be validated</param>
		/// <param name="Target">Options for the target being built</param>
		/// <param name="AllModules">Other modules to validate against, if needed</param>
		/// <param name="Logger">The logger for output</param>
		public virtual bool ValidateModuleIncludePaths(UEBuildModule Module, ReadOnlyTargetRules Target, IEnumerable<UEBuildModule> AllModules, ILogger Logger)
		{
			CppCompileWarnings ModuleWarningSettings = Module.Rules.CppCompileWarningSettings;
			if (ModuleWarningSettings.ModuleIncludePathWarningLevel <= WarningLevel.Off
				&& ModuleWarningSettings.ModuleIncludePrivateWarningLevel <= WarningLevel.Off
				&& ModuleWarningSettings.ModuleIncludeSubdirectoryWarningLevel <= WarningLevel.Off)
			{
				return false;
			}

#pragma warning disable CA2254 // Template should be a static expression - message is the static template here
			bool AnyErrors = false;
			void LoggerFunc(string? message, params object?[] args)
			{
				if (ModuleWarningSettings.ModuleIncludePathWarningLevel == WarningLevel.Warning)
				{
					Logger.LogWarning($"Warning: {message}", args);
				}
				else if (ModuleWarningSettings.ModuleIncludePathWarningLevel == WarningLevel.Error)
				{
					Logger.LogError($"Error: {message}", args);
					AnyErrors = true;
				}
			}

			void LoggerFuncPrivate(string? message, params object?[] args)
			{
				if (ModuleWarningSettings.ModuleIncludePrivateWarningLevel == WarningLevel.Warning)
				{
					Logger.LogWarning($"Warning: {message}", args);
				}
				else if (ModuleWarningSettings.ModuleIncludePrivateWarningLevel == WarningLevel.Error)
				{
					Logger.LogError($"Error: {message}", args);
					AnyErrors = true;
				}
			}

			void LoggerFuncSubDir(string? message, params object?[] args)
			{
				if (ModuleWarningSettings.ModuleIncludeSubdirectoryWarningLevel == WarningLevel.Warning)
				{
					Logger.LogWarning($"Warning: {message}", args);
				}
				else if (ModuleWarningSettings.ModuleIncludeSubdirectoryWarningLevel == WarningLevel.Error)
				{
					Logger.LogError($"Error: {message}", args);
					AnyErrors = true;
				}
			}
#pragma warning restore CA2254

			IOrderedEnumerable<DirectoryReference> PublicIncludePaths = Module.Rules.PublicIncludePaths.Select(x => DirectoryReference.FromString(x)!).OrderBy(x => x.FullName);
			IOrderedEnumerable<DirectoryReference> PrivateIncludePaths = Module.Rules.PrivateIncludePaths.Select(x => DirectoryReference.FromString(x)!).OrderBy(x => x.FullName);
			IOrderedEnumerable<DirectoryReference> InternalIncludePaths = Module.Rules.InternalIncludePaths.Select(x => DirectoryReference.FromString(x)!).OrderBy(x => x.FullName);
			IOrderedEnumerable<DirectoryReference> PublicSystemIncludePaths = Module.Rules.PublicSystemIncludePaths.Select(x => DirectoryReference.FromString(x)!).OrderBy(x => x.FullName);
			IOrderedEnumerable<UEBuildModule> OtherModules = AllModules.Where(x => x != Module).OrderBy(x => x.Name);

			if (Module is UEBuildModuleExternal)
			{
				foreach (DirectoryReference Path in PublicIncludePaths)
				{
					LoggerFunc("External module '{Name}' is adding '{Path}' to PublicIncludePaths. This path should be added to PublicSystemIncludePaths.", Module.Name, Path.MakeRelativeTo(Module.ModuleDirectory));
				}

				foreach (DirectoryReference Path in PublicSystemIncludePaths.Where(x => !x.IsUnderDirectory(Module.ModuleDirectory)))
				{
					UEBuildModule? OtherModule = OtherModules.FirstOrDefault(x => Path.IsUnderDirectory(x.ModuleDirectory));
					if (OtherModule is UEBuildModuleExternal)
					{
						LoggerFunc("External module '{Name}' is adding '{Path}' from external module '{OtherModule}' to PublicSystemIncludePaths. Did you intend to add a public reference?", Module.Name, Path, OtherModule.Name);
					}
					else if (OtherModule is UEBuildModuleCPP)
					{
						LoggerFunc("External module '{Name}' is adding '{Path}' from module '{OtherModule}' to PublicSystemIncludePaths. This is not allowed.", Module.Name, Path, OtherModule.Name);
					}
				}

				foreach (DirectoryReference Path in PrivateIncludePaths)
				{
					LoggerFunc("External module '{Name}' is adding '{Path}' to PrivateIncludePaths. This path is unused.", Module.Name, Path);
				}

				foreach (DirectoryReference Path in InternalIncludePaths)
				{
					LoggerFunc("External module '{Name}' is adding '{Path}' to InternalIncludePaths. This path is unused.", Module.Name, Path);
				}
			}
			else if (Module is UEBuildModuleCPP)
			{
				foreach (DirectoryReference Path in PublicIncludePaths)
				{
					if (Path.IsUnderDirectory(Module.ModuleDirectory))
					{
						// Warn on the root module directory being added, except for plugins as there are some plugins with a single module directly in the Source directory
						if (Path == Module.ModuleDirectory && (Module.Rules.Plugin == null || Module.ModuleDirectory.ParentDirectory != Module.Rules.Plugin.Directory))
						{
							LoggerFunc("Module '{Name}' is adding root directory to PublicIncludePaths. This is not allowed.", Module.Name);
						}
						else if (Path.IsUnderDirectory(DirectoryReference.Combine(Module.ModuleDirectory, "Private"))
							|| Path.IsUnderDirectory(DirectoryReference.Combine(Module.ModuleDirectory, "Internal")))
						{
							LoggerFuncPrivate("Module '{Name}' is adding subdirectory '{Path}' to PublicIncludePaths. This is not allowed.", Module.Name, Path.MakeRelativeTo(Module.ModuleDirectory));
						}
						else if (Path.IsUnderDirectory(DirectoryReference.Combine(Module.ModuleDirectory, "Public"))
							|| Path.IsUnderDirectory(DirectoryReference.Combine(Module.ModuleDirectory, "Classes")))
						{
							LoggerFuncSubDir("Module '{Name}' is adding subdirectory '{Path}' to PublicIncludePaths. This is not necessary.", Module.Name, Path.MakeRelativeTo(Module.ModuleDirectory));
						}
					}

					UEBuildModule? OtherModule = OtherModules.FirstOrDefault(x => Path.IsUnderDirectory(x.ModuleDirectory));
					if (OtherModule is UEBuildModuleExternal)
					{
						LoggerFunc("Module '{Name}' is adding '{Path}' from external module '{OtherModule}' to PublicIncludePaths. Did you intend to add a public reference?", Module.Name, Path, OtherModule.Name);
					}
					else if (OtherModule is UEBuildModuleCPP)
					{
						if (Path == OtherModule.ModuleDirectory)
						{
							LoggerFuncPrivate("Module '{Name}' is adding root directory from '{OtherModule}' to PublicIncludePaths. This is not allowed.", Module.Name, OtherModule.Name);
						}
						else if (Path.IsUnderDirectory(DirectoryReference.Combine(OtherModule.ModuleDirectory, "Private"))
							| Path.IsUnderDirectory(DirectoryReference.Combine(OtherModule.ModuleDirectory, "Internal")))
						{
							LoggerFuncPrivate("Module '{Name}' is adding '{Path}' from module '{OtherModule}' to PublicIncludePaths. This is not allowed.", Module.Name, Path, OtherModule.Name);
						}
						else if (Path.IsUnderDirectory(DirectoryReference.Combine(OtherModule.ModuleDirectory, "Public"))
							|| Path.IsUnderDirectory(DirectoryReference.Combine(OtherModule.ModuleDirectory, "Classes")))
						{
							LoggerFunc("Module '{Name}' is adding '{Path}' from module '{OtherModule}' to PublicIncludePaths. Did you intend to add a public reference?", Module.Name, Path, OtherModule.Name);
						}
					}
				}

				foreach (DirectoryReference Path in PrivateIncludePaths)
				{
					if (Path.IsUnderDirectory(Module.ModuleDirectory))
					{
						if (Path == Module.ModuleDirectory)
						{
							LoggerFunc("Module '{Name}' is adding root directory to PrivateIncludePaths. This is not recommended.", Module.Name);
						}
						else if (Path.IsUnderDirectory(DirectoryReference.Combine(Module.ModuleDirectory, "Private"))
							|| Path.IsUnderDirectory(DirectoryReference.Combine(Module.ModuleDirectory, "Internal"))
							|| Path.IsUnderDirectory(DirectoryReference.Combine(Module.ModuleDirectory, "Public"))
							|| Path.IsUnderDirectory(DirectoryReference.Combine(Module.ModuleDirectory, "Classes")))
						{
							LoggerFuncSubDir("Module '{Name}' is adding subdirectory '{Path}' to PrivateIncludePaths. This is not necessary.", Module.Name, Path.MakeRelativeTo(Module.ModuleDirectory));
						}
					}

					UEBuildModule? OtherModule = OtherModules.FirstOrDefault(x => Path.IsUnderDirectory(x.ModuleDirectory));
					if (OtherModule is UEBuildModuleExternal)
					{
						LoggerFunc("Module '{Name}' is adding '{Path}' from external module '{OtherModule}' to PrivateIncludePaths. Did you intend to add a private reference?", Module.Name, Path, OtherModule.Name);
					}
					else if (OtherModule is UEBuildModuleCPP)
					{
						if (Path == OtherModule.ModuleDirectory)
						{
							LoggerFuncPrivate("Module '{Name}' is adding root directory from '{OtherModule}' to PrivateIncludePaths. This is not allowed.", Module.Name, OtherModule.Name);
						}
						else if (Path.IsUnderDirectory(DirectoryReference.Combine(OtherModule.ModuleDirectory, "Private"))
							| Path.IsUnderDirectory(DirectoryReference.Combine(OtherModule.ModuleDirectory, "Internal")))
						{
							LoggerFuncPrivate("Module '{Name}' is adding '{Path}' from module '{OtherModule}' to PrivateIncludePaths. This is not allowed.", Module.Name, Path, OtherModule.Name);
						}
						else if (Path.IsUnderDirectory(DirectoryReference.Combine(OtherModule.ModuleDirectory, "Public"))
							|| Path.IsUnderDirectory(DirectoryReference.Combine(OtherModule.ModuleDirectory, "Classes")))
						{
							LoggerFunc("Module '{Name}' is adding '{Path}' from module '{OtherModule}' to PrivateIncludePaths. Did you intend to add a private reference?", Module.Name, Path, OtherModule.Name);
						}
					}
				}

				foreach (DirectoryReference Path in InternalIncludePaths)
				{
					if (Path.IsUnderDirectory(Module.ModuleDirectory))
					{
						if (Path == Module.ModuleDirectory)
						{
							LoggerFunc("Module '{Name}' is adding root directory to InternalIncludePaths. This is not allowed.", Module.Name);
						}
						else if (Path.IsUnderDirectory(DirectoryReference.Combine(Module.ModuleDirectory, "Private"))
						|| Path.IsUnderDirectory(DirectoryReference.Combine(Module.ModuleDirectory, "Internal")))
						{
							LoggerFunc("Module '{Name}' is adding subdirectory '{Path}' to InternalIncludePaths. This is not allowed.", Module.Name, Path.MakeRelativeTo(Module.ModuleDirectory));
						}
						else if (Path.IsUnderDirectory(DirectoryReference.Combine(Module.ModuleDirectory, "Public"))
							|| Path.IsUnderDirectory(DirectoryReference.Combine(Module.ModuleDirectory, "Classes")))
						{
							LoggerFuncSubDir("Module '{Name}' is adding subdirectory '{Path}' to InternalIncludePaths. This is not necessary.", Module.Name, Path.MakeRelativeTo(Module.ModuleDirectory));
						}
					}

					UEBuildModule? OtherModule = OtherModules.FirstOrDefault(x => Path.IsUnderDirectory(x.ModuleDirectory));
					if (OtherModule is UEBuildModuleExternal)
					{
						LoggerFunc("Module '{Name}' is adding '{Path}' from external module '{OtherModule}' to InternalIncludePaths. Did you intend to add a public reference?", Module.Name, Path, OtherModule.Name);
					}
					else if (OtherModule is UEBuildModuleCPP)
					{
						if (Path == OtherModule.ModuleDirectory)
						{
							LoggerFunc("Module '{Name}' is adding root directory from '{OtherModule}' to InternalIncludePaths. This is not allowed.", Module.Name, OtherModule.Name);
						}
						else if (Path.IsUnderDirectory(DirectoryReference.Combine(OtherModule.ModuleDirectory, "Private"))
							| Path.IsUnderDirectory(DirectoryReference.Combine(OtherModule.ModuleDirectory, "Internal")))
						{
							LoggerFunc("Module '{Name}' is adding '{Path}' from module '{OtherModule}' to InternalIncludePaths. This is not allowed.", Module.Name, Path, OtherModule.Name);
						}
						else if (Path.IsUnderDirectory(DirectoryReference.Combine(OtherModule.ModuleDirectory, "Public"))
							|| Path.IsUnderDirectory(DirectoryReference.Combine(OtherModule.ModuleDirectory, "Classes")))
						{
							LoggerFunc("Module '{Name}' is adding '{Path}' from module '{OtherModule}' to InternalIncludePaths. Did you intend to add a public reference?", Module.Name, Path, OtherModule.Name);
						}
					}
				}
			}

			return AnyErrors;
		}

		/// <summary>
		/// Return whether the given platform requires a monolithic build
		/// </summary>
		/// <param name="InPlatform">The platform of interest</param>
		/// <returns></returns>
		public static bool PlatformRequiresMonolithicBuilds(UnrealTargetPlatform InPlatform)
		{
			// Some platforms require monolithic builds...
			UEBuildPlatform? BuildPlatform;
			if (TryGetBuildPlatform(InPlatform, out BuildPlatform))
			{
				return BuildPlatform.ShouldCompileMonolithicBinary(InPlatform);
			}

			// We assume it does not
			return false;
		}

		/// <summary>
		/// Get the extension to use for the given binary type
		/// </summary>
		/// <param name="InBinaryType"> The binary type being built</param>
		/// <returns>string    The binary extension (i.e. 'exe' or 'dll')</returns>
		public virtual string GetBinaryExtension(UEBuildBinaryType InBinaryType)
		{
			throw new BuildException("GetBinaryExtensiton for {0} not handled in {1}", InBinaryType.ToString(), ToString());
		}

		/// <summary>
		/// Get the extensions to use for debug info for the given binary type
		/// </summary>
		/// <param name="InTarget">Options for the target being built</param>
		/// <param name="InBinaryType"> The binary type being built</param>
		/// <returns>string[]    The debug info extensions (i.e. 'pdb')</returns>
		public virtual string[] GetDebugInfoExtensions(ReadOnlyTargetRules InTarget, UEBuildBinaryType InBinaryType)
		{
			throw new BuildException("GetDebugInfoExtensions for {0} not handled in {1}", InBinaryType.ToString(), ToString());
		}

		/// <summary>
		/// Whether this platform should build a monolithic binary
		/// </summary>
		public virtual bool ShouldCompileMonolithicBinary(UnrealTargetPlatform InPlatform)
		{
			return false;
		}

		/// <summary>
		/// Modify the rules for a newly created module, where the target is a different host platform.
		/// This is not required - but allows for hiding details of a particular platform.
		/// </summary>
		/// <param name="ModuleName">The name of the module</param>
		/// <param name="Rules">The module rules</param>
		/// <param name="Target">The target being build</param>
		public virtual void ModifyModuleRulesForOtherPlatform(string ModuleName, ModuleRules Rules, ReadOnlyTargetRules Target)
		{
		}

		/// <summary>
		/// For platforms that need to output multiple files per binary (ie Android "fat" binaries)
		/// this will emit multiple paths. By default, it simply makes an array from the input
		/// </summary>
		public virtual List<FileReference> FinalizeBinaryPaths(FileReference BinaryName, FileReference? ProjectFile, ReadOnlyTargetRules Target)
		{
			List<FileReference> TempList = new List<FileReference>() { BinaryName };
			return TempList;
		}

		/// <summary>
		/// Return all valid configurations for this platform
		/// Typically, this is always Debug, Development, and Shipping - but Test is a likely future addition for some platforms
		/// </summary>
		public virtual List<UnrealTargetConfiguration> GetConfigurations(UnrealTargetPlatform InUnrealTargetPlatform, bool bIncludeDebug)
		{
			List<UnrealTargetConfiguration> Configurations = new List<UnrealTargetConfiguration>()
			{
				UnrealTargetConfiguration.Development,
			};

			if (bIncludeDebug)
			{
				Configurations.Insert(0, UnrealTargetConfiguration.Debug);
			}

			return Configurations;
		}

		protected static bool DoProjectSettingsMatchDefault(UnrealTargetPlatform Platform, DirectoryReference ProjectDirectoryName, string Section, string[]? BoolKeys, string[]? IntKeys, string[]? StringKeys)
		{
			ConfigHierarchy ProjIni = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, ProjectDirectoryName, Platform);
			ConfigHierarchy DefaultIni = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, (DirectoryReference?)null, Platform);

			// look at all bool values
			if (BoolKeys != null)
			{
				foreach (string Key in BoolKeys)
				{
					DefaultIni.GetBool(Section, Key, out bool Default);
					ProjIni.GetBool(Section, Key, out bool Project);
					if (Default != Project)
					{
						Log.TraceInformationOnce("{0} is not set to default. (Base: {1} vs. {2}: {3})", Key, Default, Path.GetFileName(ProjectDirectoryName.FullName), Project);
						return false;
					}
				}
			}

			// look at all int values
			if (IntKeys != null)
			{
				foreach (string Key in IntKeys)
				{
					DefaultIni.GetInt32(Section, Key, out int Default);
					ProjIni.GetInt32(Section, Key, out int Project);
					if (Default != Project)
					{
						Log.TraceInformationOnce("{0} is not set to default. (Base: {1} vs. {2}: {3})", Key, Default, Path.GetFileName(ProjectDirectoryName.FullName), Project);
						return false;
					}
				}
			}

			// look for all string values
			if (StringKeys != null)
			{
				foreach (string Key in StringKeys)
				{
					DefaultIni.GetString(Section, Key, out string? Default);
					ProjIni.GetString(Section, Key, out string? Project);
					if (Default != Project)
					{
						Log.TraceInformationOnce("{0} is not set to default. (Base: {1} vs. {2}: {3})", Key, Default, Path.GetFileName(ProjectDirectoryName.FullName), Project);
						return false;
					}
				}
			}

			// if we get here, we match all important settings
			return true;
		}

		/// <summary>
		/// Check for the default configuration
		/// return true if the project uses the default build config
		/// </summary>
		public virtual bool HasDefaultBuildConfig(UnrealTargetPlatform Platform, DirectoryReference ProjectDirectoryName)
		{
			string[] BoolKeys = new string[] {
				"bCompileApex", "bCompileICU",
				"bCompileRecast", "bCompileSpeedTree",
				"bCompileWithPluginSupport", "bCompilePhysXVehicle", "bCompileFreeType",
				"bCompileForSize", "bCompileCEF3", "bCompileCustomSQLitePlatform"
			};

			return DoProjectSettingsMatchDefault(Platform, ProjectDirectoryName, "/Script/BuildSettings.BuildSettings",
				BoolKeys, null, null);
		}

		/// <summary>
		/// Check for whether we require a build for platform reasons
		/// return true if the project requires a build
		/// </summary>
		public virtual bool RequiresBuild(UnrealTargetPlatform Platform, DirectoryReference ProjectDirectoryName)
		{
			return false;
		}

		/// <summary>
		/// Get a list of extra modules the platform requires.
		/// This is to allow undisclosed platforms to add modules they need without exposing information about the platform.
		/// </summary>
		/// <param name="Target">The target being build</param>
		/// <param name="ExtraModuleNames">List of extra modules the platform needs to add to the target</param>
		public virtual void AddExtraModules(ReadOnlyTargetRules Target, List<string> ExtraModuleNames)
		{
		}

		/// <summary>
		/// Modify the rules for a newly created module, in a target that's being built for this platform.
		/// This is not required - but allows for hiding details of a particular platform.
		/// </summary>
		/// <param name="ModuleName">The name of the module</param>
		/// <param name="Rules">The module rules</param>
		/// <param name="Target">The target being build</param>
		public virtual void ModifyModuleRulesForActivePlatform(string ModuleName, ModuleRules Rules, ReadOnlyTargetRules Target)
		{
		}

		/// <summary>
		/// Setup the target environment for building
		/// </summary>
		/// <param name="Target">Settings for the target being compiled</param>
		/// <param name="CompileEnvironment">The compile environment for this target</param>
		/// <param name="LinkEnvironment">The link environment for this target</param>
		public abstract void SetUpEnvironment(ReadOnlyTargetRules Target, CppCompileEnvironment CompileEnvironment, LinkEnvironment LinkEnvironment);

		/// <summary>
		/// Setup the configuration environment for building
		/// </summary>
		/// <param name="Target">The target being built</param>
		/// <param name="GlobalCompileEnvironment">The global compile environment</param>
		/// <param name="GlobalLinkEnvironment">The global link environment</param>
		public virtual void SetUpConfigurationEnvironment(ReadOnlyTargetRules Target, CppCompileEnvironment GlobalCompileEnvironment, LinkEnvironment GlobalLinkEnvironment)
		{
			if (GlobalCompileEnvironment.bUseDebugCRT)
			{
				GlobalCompileEnvironment.Definitions.Add("_DEBUG=1"); // the engine doesn't use this, but lots of 3rd party stuff does
			}
			else
			{
				GlobalCompileEnvironment.Definitions.Add("NDEBUG=1"); // the engine doesn't use this, but lots of 3rd party stuff does
			}

			switch (Target.Configuration)
			{
				default:
				case UnrealTargetConfiguration.Debug:
					GlobalCompileEnvironment.Definitions.Add("UE_BUILD_DEBUG=1");
					break;
				case UnrealTargetConfiguration.DebugGame:
				// Individual game modules can be switched to be compiled in debug as necessary. By default, everything is compiled in development.
				case UnrealTargetConfiguration.Development:
					GlobalCompileEnvironment.Definitions.Add("UE_BUILD_DEVELOPMENT=1");
					break;
				case UnrealTargetConfiguration.Shipping:
					GlobalCompileEnvironment.Definitions.Add("UE_BUILD_SHIPPING=1");
					break;
				case UnrealTargetConfiguration.Test:
					GlobalCompileEnvironment.Definitions.Add("UE_BUILD_TEST=1");
					break;
			}

			// Create debug info based on the heuristics specified by the user.
			GlobalCompileEnvironment.bCreateDebugInfo =
				Target.DebugInfo != DebugInfoMode.None && ShouldCreateDebugInfo(Target);
			GlobalLinkEnvironment.bCreateDebugInfo = GlobalCompileEnvironment.bCreateDebugInfo;
		}

		/// <summary>
		/// Allows the platform to return various build metadata that is not tracked by other means. If the returned string changes, the makefile will be invalidated.
		/// </summary>
		/// <param name="ProjectFile">The project file being built</param>
		/// <returns>String describing the current build metadata</returns>
		public string GetExternalBuildMetadata(FileReference? ProjectFile)
		{
			StringBuilder Result = new StringBuilder();
			GetExternalBuildMetadata(ProjectFile, Result);
			return Result.ToString();
		}

		/// <summary>
		/// Allows the platform to return various build metadata that is not tracked by other means. If the returned string changes, the makefile will be invalidated.
		/// </summary>
		/// <param name="ProjectFile">The project file being built</param>
		/// <param name="Metadata">String builder to contain build metadata</param>
		public virtual void GetExternalBuildMetadata(FileReference? ProjectFile, StringBuilder Metadata)
		{
		}

		/// <summary>
		/// Allows the platform to modify the binary link environment before the binary is built
		/// </summary>
		/// <param name="BinaryLinkEnvironment">The binary link environment being created</param>
		/// <param name="BinaryCompileEnvironment">The binary compile environment being used</param>
		/// <param name="Target">The target rules in use</param>
		/// <param name="ToolChain">The toolchain being used</param>
		/// <param name="Graph">Action graph that is used to build the binary</param>
		public virtual void ModifyBinaryLinkEnvironment(LinkEnvironment BinaryLinkEnvironment, CppCompileEnvironment BinaryCompileEnvironment, ReadOnlyTargetRules Target, UEToolChain ToolChain, IActionGraphBuilder Graph)
		{
		}

		/// <summary>
		/// Indicates whether this platform requires a .loadorder file to be generated for the build.
		/// .loadorder files contain a list of dynamic modules and the exact order in which they should be loaded
		/// to ensure that all dependencies are satisfied i.e. we don't attempt to load a module without loading
		/// all its dependencies first.
		/// As such, this file is only needed in modular builds on some platforms (depending on the way they implement dynamic modules such as DLLs).
		/// </summary>
		/// <param name="Target">The target rules in use</param>
		/// <returns>True if .loadorder file should be generated</returns>
		public virtual bool RequiresLoadOrderManifest(ReadOnlyTargetRules Target)
		{
			return false;
		}

		/// <summary>
		/// Checks if platform is part of a given platform group
		/// </summary>
		/// <param name="Platform">The platform to check</param>
		/// <param name="PlatformGroup">The platform group to check</param>
		/// <returns>True if platform is part of a platform group</returns>
		internal static bool IsPlatformInGroup(UnrealTargetPlatform Platform, UnrealPlatformGroup PlatformGroup)
		{
			List<UnrealTargetPlatform>? Platforms = UEBuildPlatform.GetPlatformsInGroup(PlatformGroup);
			if (Platforms != null)
			{
				return Platforms.Contains(Platform);
			}
			else
			{
				return false;
			}
		}

		/// <summary>
		/// Gets the SDK object that was passed in to the constructor
		/// </summary>
		/// <returns>The SDK object</returns>
		public UEBuildPlatformSDK? GetSDK()
		{
			return UEBuildPlatformSDK.GetSDKForPlatform(Platform.ToString());
		}
		/// <summary>
		/// Gets the SDK object that was passed in to the constructor to the UEBuildPlatform constructor for this platform
		/// </summary>
		/// <returns>The SDK object</returns>
		public static UEBuildPlatformSDK? GetSDK(UnrealTargetPlatform Platform)
		{
			return UEBuildPlatformSDK.GetSDKForPlatform(Platform.ToString());
		}

		/// <summary>
		/// Whether this platform should create debug information or not
		/// </summary>
		/// <param name="Target">The target being built</param>
		/// <returns>bool    true if debug info should be generated, false if not</returns>
		public abstract bool ShouldCreateDebugInfo(ReadOnlyTargetRules Target);

		/// <summary>
		/// Creates a toolchain instance for this platform. There should be a single toolchain instance per-target, as their may be
		/// state data and configuration cached between calls.
		/// </summary>
		/// <param name="Target">The target being built</param>
		/// <returns>New toolchain instance.</returns>
		public abstract UEToolChain CreateToolChain(ReadOnlyTargetRules Target);

		/// <summary>
		/// Deploys the given target
		/// </summary>
		/// <param name="Receipt">Receipt for the target being deployed</param>
		public abstract void Deploy(TargetReceipt Receipt);

		/// <summary>
		/// Struct used to add platform static shaders
		/// </summary>
		public struct StaticShader
		{
			/// <summary>
			/// Name of shader
			/// </summary>
			public string Name { set; get; }

			/// <summary>
			/// Index of shader
			/// </summary>
			public int Index { set; get; }
		}

		/// <summary>
		/// Called by platforms to add platform specific static shaders
		/// </summary>
		/// <param name="Target">Target rules</param>
		/// <param name="Rules">Module rules</param>
		/// <param name="StaticShaders">Static shaders supported by current platform</param>
		public void AddPlatformStaticShaders(ReadOnlyTargetRules Target, ModuleRules Rules, IEnumerable<StaticShader> StaticShaders)
		{
			if (Target.bNeedsExtraShaderFormats || Target.Platform == Platform)
			{
				if (Rules.Name == "BuildSettings") // Put list of platforms in BuildSettings since this array will vary depending on what platforms machine has installed
				{
					List<string> content = [.. Rules.FilesToGenerate["RHIStaticShaderPlatformNames.gen.cpp"]];
					content.InsertRange(content.Count - 2, StaticShaders.Select(shader =>
						$"	{{ TEXT(\"{shader.Name}\"), TEXT(\"{Platform}\"), {shader.Index} }},"));
					Rules.FilesToGenerate["RHIStaticShaderPlatformNames.gen.cpp"] = [.. content];
				}
				else if (Rules.Name == "RHI") // Add one file per platform with EShaderPlatform entries
				{
					IEnumerable<string> content = [
						"#pragma once", "#include \"HAL/Platform.h\"", "enum EShaderPlatform : uint16;",
						.. StaticShaders.Select(shader => $"inline constexpr EShaderPlatform SP_{shader.Name} = EShaderPlatform({shader.Index});")
					];
					Rules.FilesToGenerate.Add($"{Platform}RHIShaderPlatform.h", content);
				}
			}
		}
	}
}
