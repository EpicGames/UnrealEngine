// Copyright Epic Games, Inc. All Rights Reserved.

using Microsoft.Extensions.Logging;

namespace UnrealBuildTool
{
	/// <summary>
	/// Factory class for registering platforms at startup
	/// </summary>
	abstract class UEBuildPlatformFactory
	{
		/// <summary>
		/// Gets the target platform for an individual factory
		/// </summary>
		public abstract UnrealTargetPlatform TargetPlatform
		{
			get;
		}

		/// <summary>
		/// Register the platform with the UEBuildPlatform class
		/// </summary>
		public abstract void RegisterBuildPlatforms(ILogger Logger);
	}
}
