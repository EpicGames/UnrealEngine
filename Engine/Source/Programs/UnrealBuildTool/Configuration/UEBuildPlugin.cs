// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace UnrealBuildTool
{
	/// <summary>
	/// Stores information about a plugin that is being built for a target
	/// </summary>
	class UEBuildPlugin
	{
		/// <summary>
		/// Information about the plugin
		/// </summary>
		public PluginInfo Info;

		/// <summary>
		/// Modules that this plugin belongs to
		/// </summary>
		public List<UEBuildModuleCPP> Modules = new List<UEBuildModuleCPP>();

		/// <summary>
		/// Recursive
		/// </summary>
		public HashSet<UEBuildPlugin>? Dependencies;

		/// <summary>
		/// Whether the descriptor for this plugin is needed at runtime; because it has modules or content which is used, or because it references another module that does.
		/// </summary>
		public bool bDescriptorNeededAtRuntime;

		/// <summary>
		/// Whether this descriptor is referenced non-optionally by something else; a project file or other plugin. This is recursively applied to the plugin's references.
		/// </summary>
		public bool bDescriptorReferencedExplicitly;

		/// <summary>
		/// Chain of references to this plugin
		/// </summary>
		public string ReferenceChain;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="Info">The static plugin information</param>
		/// <param name="ReferenceChain">Chain of references to this plugin</param>
		public UEBuildPlugin(PluginInfo Info, string ReferenceChain)
		{
			this.Info = Info;
			this.ReferenceChain = ReferenceChain;
		}

		/// <summary>
		/// Accessor for the name of this plugin
		/// </summary>
		public string Name => Info.Name;

		/// <summary>
		/// Accessor for the file for this plugin
		/// </summary>
		public FileReference File => Info.File;

		/// <summary>
		/// Accessor for the child files for this plugin
		/// </summary>
		public List<FileReference> ChildFiles => Info.ChildFiles;

		/// <summary>
		/// Accessor for the type of the plugin
		/// </summary>
		public PluginType Type => Info.Type;

		/// <summary>
		/// Accessor for this plugin's root directory
		/// </summary>
		public DirectoryReference Directory => Info.Directory;

		/// <summary>
		/// Accessor for this plugin's descriptor
		/// </summary>
		public PluginDescriptor Descriptor => Info.Descriptor;

		/// <summary>
		/// Returns the name of this plugin for debugging
		/// </summary>
		/// <returns>Name of the plugin</returns>
		public override string ToString()
		{
			return Info.Name;
		}

		/// <summary>
		/// Public entry point to validate a module
		/// </summary>
		/// <param name="logger"></param>
		/// <param name="rules"></param>
		/// <returns>true if there are any fatal errors</returns>
		public bool ValidatePlugin(ILogger logger, ReadOnlyTargetRules rules)
		{
			bool anyErrors = false;
			if (Dependencies != null)
			{
				foreach (UEBuildPlugin dependencyPlugin in Dependencies)
				{
					if (dependencyPlugin.Descriptor.bIsSealed)
					{
						logger.LogError("Plugin '{PluginName}' cannot depend on plugin '{DependencyPluginName}' because it is sealed.", Name, dependencyPlugin.Name);
						anyErrors = true;
					}
					else if (Descriptor.DisallowedPlugins != null && Descriptor.DisallowedPlugins.Contains(dependencyPlugin.Name))
					{
						logger.LogError("Plugin '{PluginName}' cannot depend on plugin '{DependencyPluginName}' because it is disallowed.", Name, dependencyPlugin.Name);
						anyErrors = true;
					}
					else if (!String.IsNullOrEmpty(dependencyPlugin.Descriptor.DeprecatedEngineVersion))
					{
						logger.LogWarning("Plugin '{PluginName}' depends on plugin '{DependencyPluginName}' which was deprecated in {EngineVersion} and will soon be removed. Please update your dependencies.",
							Name, dependencyPlugin.Name, dependencyPlugin.Descriptor.DeprecatedEngineVersion);
					}
				}
			}

			if (!String.IsNullOrEmpty(Descriptor.DeprecatedEngineVersion))
			{
				logger.LogWarning("Project '{ProjectName}' depends on plugin '{PluginName}' which was deprecated in {EngineVersion} and will soon be removed. Please update your dependencies.",
					rules.Name, Name, Descriptor.DeprecatedEngineVersion);
			}
			
			// Check that any plugins with the NoCode specifier do not contain modules.
			if (Descriptor.bNoCode && Modules.Any())
			{
				logger.LogError("Plugin '{PluginName}' cannot contain any code or modules. See the plugin descriptor property `NoCode`", Name);
				anyErrors = true;
			}

			return anyErrors;
		}
	}
}
