// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.ComponentModel;
using System.Diagnostics.CodeAnalysis;
using System.Globalization;
using System.IO;
using System.Linq;
using System.Reflection;
using System.Runtime.InteropServices;
using System.Runtime.Serialization;
using System.Text;
using System.Text.Json;
using System.Text.Json.Serialization;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.UHT.Utils;
using Microsoft.Extensions.Logging;
using OpenTracing.Util;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// The platform we're building for
	/// </summary>
	[Serializable, TypeConverter(typeof(UnrealTargetPlatformTypeConverter))]
	[JsonConverter(typeof(UnrealTargetPlatformJsonConverter))]
	public partial struct UnrealTargetPlatform : ISerializable, IEquatable<UnrealTargetPlatform>
	{
		#region Private/boilerplate

		// internal concrete name of the group
		private int Id;

		// shared string instance registry - pass in a delegate to create a new one with a name that wasn't made yet
		private static UniqueStringRegistry? StringRegistry;

		// #jira UE-88908 if parts of a partial struct have each static member variables, their initialization order does not appear guaranteed
		// here this means initializing "StringRegistry" directly to "new UniqueStringRegistry()" may not be executed before FindOrAddByName() has been called as part of initializing a static member variable of another part of the partial struct
		private static UniqueStringRegistry GetUniqueStringRegistry()
		{
			StringRegistry ??= new UniqueStringRegistry();
			return StringRegistry;
		}

		private UnrealTargetPlatform(string Name)
		{
			Id = GetUniqueStringRegistry().FindOrAddByName(Name);
		}

		private UnrealTargetPlatform(int InId)
		{
			Id = InId;
		}

		/// <summary>
		/// 
		/// </summary>
		/// <param name="Info"></param>
		/// <param name="Context"></param>
		public void GetObjectData(SerializationInfo Info, StreamingContext Context)
		{
			Info.AddValue("Name", ToString());
		}

		/// <summary>
		/// 
		/// </summary>
		/// <param name="Info"></param>
		/// <param name="Context"></param>
		public UnrealTargetPlatform(SerializationInfo Info, StreamingContext Context)
		{
			Id = GetUniqueStringRegistry().FindOrAddByName((string)Info.GetValue("Name", typeof(string))!);
		}

		/// <summary>
		/// Return the single instance of the Group with this name
		/// </summary>
		/// <param name="Name"></param>
		/// <returns></returns>
		private static UnrealTargetPlatform FindOrAddByName(string Name)
		{
			return new UnrealTargetPlatform(GetUniqueStringRegistry().FindOrAddByName(Name));
		}

		/// <summary>
		/// Return the single instance of the Group with this name
		/// </summary>
		/// <param name="Alias"></param>
		/// /// <param name="Original"></param>
		/// <returns></returns>
#pragma warning disable IDE0051 // "unused private member" - used by platform-specific partial struct extensions
		private static UnrealTargetPlatform AddAliasByName(string Alias, UnrealTargetPlatform Original)
		{
			GetUniqueStringRegistry().FindOrAddAlias(Alias, Original.ToString());
			return Original;
		}
#pragma warning restore IDE0051

		/// <summary>
		///
		/// </summary>
		/// <param name="A"></param>
		/// <param name="B"></param>
		/// <returns></returns>
		public static bool operator ==(UnrealTargetPlatform A, UnrealTargetPlatform B)
		{
			return A.Id == B.Id;
		}
		/// <summary>
		/// 
		/// </summary>
		/// <param name="A"></param>
		/// <param name="B"></param>
		/// <returns></returns>
		public static bool operator !=(UnrealTargetPlatform A, UnrealTargetPlatform B)
		{
			return A.Id != B.Id;
		}
		/// <summary>
		/// 
		/// </summary>
		/// <param name="B"></param>
		/// <returns></returns>
		public override bool Equals(object? B)
		{
			if (B is null)
			{
				return false;
			}

			return Id == ((UnrealTargetPlatform)B).Id;
		}

		/// <inheritdoc/>
		public readonly bool Equals(UnrealTargetPlatform other) => this == other;

		/// <summary>
		/// 
		/// </summary>
		/// <returns></returns>
		public override int GetHashCode()
		{
			return Id;
		}

		#endregion

		/// <summary>
		/// Return the string representation
		/// </summary>
		/// <returns></returns>
		public override string ToString()
		{
			return GetUniqueStringRegistry().GetStringForId(Id);
		}

		/// <summary>
		/// 
		/// </summary>
		/// <param name="Name"></param>
		/// <param name="Platform"></param>
		/// <returns></returns>
		public static bool TryParse(string Name, out UnrealTargetPlatform Platform)
		{
			if (GetUniqueStringRegistry().HasString(Name))
			{
				Platform.Id = GetUniqueStringRegistry().FindOrAddByName(Name);
				return true;
			}

			if (GetUniqueStringRegistry().HasAlias(Name))
			{
				Platform.Id = GetUniqueStringRegistry().FindExistingAlias(Name);
				return true;
			}

			Platform.Id = -1;
			return false;
		}

		/// <summary>
		/// 
		/// </summary>
		/// <param name="Name"></param>
		/// <returns></returns>
		public static UnrealTargetPlatform Parse(string Name)
		{
			if (GetUniqueStringRegistry().HasString(Name))
			{
				return new UnrealTargetPlatform(Name);
			}

			if (GetUniqueStringRegistry().HasAlias(Name))
			{
				int Id = GetUniqueStringRegistry().FindExistingAlias(Name);
				return new UnrealTargetPlatform(Id);
			}

			throw new BuildException(String.Format("The platform name {0} is not a valid platform name. Valid names are ({1})", Name,
				String.Join(",", GetUniqueStringRegistry().GetStringNames())));
		}

		/// <summary>
		/// 
		/// </summary>
		/// <returns></returns>
		public static UnrealTargetPlatform[] GetValidPlatforms()
		{
			return Array.ConvertAll(GetUniqueStringRegistry().GetStringIds(), x => new UnrealTargetPlatform(x));
		}

		/// <summary>
		/// 
		/// </summary>
		/// <returns></returns>
		public static string[] GetValidPlatformNames()
		{
			return GetUniqueStringRegistry().GetStringNames();
		}

		/// <summary>
		/// 
		/// </summary>
		/// <param name="Name"></param>
		/// <returns></returns>
		public static bool IsValidName(string Name)
		{
			return GetUniqueStringRegistry().HasString(Name);
		}

		/// <summary>
		/// Helper that just calls UEBuildPlatform.IsPlatformInGroup
		/// </summary>
		/// <param name="Group"></param>
		/// <returns></returns>
		public bool IsInGroup(UnrealPlatformGroup Group)
		{
			return UEBuildPlatform.IsPlatformInGroup(this, Group);
		}

		/// <summary>
		/// Helper that just calls UEBuildPlatform.IsPlatformInGroup
		/// </summary>
		/// <param name="Group"></param>
		/// <returns></returns>
		public bool IsInGroup(string Group)
		{
			return UnrealPlatformGroup.TryParse(Group, out UnrealPlatformGroup ActualGroup) && UEBuildPlatform.IsPlatformInGroup(this, ActualGroup);
		}

		/// <summary>
		/// 64-bit Windows
		/// </summary>
		public static UnrealTargetPlatform Win64 { get; } = FindOrAddByName("Win64");

		/// <summary>
		/// Mac
		/// </summary>
		public static UnrealTargetPlatform Mac { get; } = FindOrAddByName("Mac");

		/// <summary>
		/// iOS
		/// </summary>
		public static UnrealTargetPlatform IOS { get; } = FindOrAddByName("IOS");

		/// <summary>
		/// Android
		/// </summary>
		public static UnrealTargetPlatform Android { get; } = FindOrAddByName("Android");

		/// <summary>
		/// Linux
		/// </summary>
		public static UnrealTargetPlatform Linux { get; } = FindOrAddByName("Linux");

		/// <summary>
		/// LinuxArm64
		/// </summary>
		public static UnrealTargetPlatform LinuxArm64 { get; } = FindOrAddByName("LinuxArm64");

		/// <summary>
		/// TVOS
		/// </summary>
		public static UnrealTargetPlatform TVOS { get; } = FindOrAddByName("TVOS");

		/// <summary>
		/// VisionOS
		/// </summary>
		public static UnrealTargetPlatform VisionOS { get; } = FindOrAddByName("VisionOS");
	}

	internal class UnrealTargetPlatformTypeConverter : TypeConverter
	{
		public override bool CanConvertFrom(ITypeDescriptorContext? context, Type sourceType)
		{
			if (sourceType == typeof(string))
			{
				return true;
			}

			return base.CanConvertFrom(context, sourceType);
		}

		public override bool CanConvertTo(ITypeDescriptorContext? context, Type? destinationType)
		{
			if (destinationType == typeof(string))
			{
				return true;
			}

			return base.CanConvertTo(context, destinationType);
		}

		public override object? ConvertFrom(ITypeDescriptorContext? context, CultureInfo? culture, object value)
		{
			if (value.GetType() == typeof(string))
			{
				return UnrealTargetPlatform.Parse((string)value);
			}
			return base.ConvertFrom(context, culture, value);
		}

		public override object? ConvertTo(ITypeDescriptorContext? context, CultureInfo? culture, object? value, Type destinationType)
		{
			if (destinationType == typeof(string) && value != null)
			{
				UnrealTargetPlatform Platform = (UnrealTargetPlatform)value;
				return Platform.ToString();
			}
			return base.ConvertTo(context, culture, value, destinationType);
		}
	}

	internal class UnrealTargetPlatformJsonConverter : JsonConverter<UnrealTargetPlatform>
	{
		public override UnrealTargetPlatform Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options)
		{
			if (reader.TokenType != JsonTokenType.String)
			{
				throw new JsonException("UnrealTargetPlatform values must be represented as JSON strings");
			}
			return UnrealTargetPlatform.Parse(reader.GetString()!);
		}

		public override UnrealTargetPlatform ReadAsPropertyName(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options)
		{
			if (reader.TokenType != JsonTokenType.PropertyName)
			{
				throw new JsonException("UnrealTargetPlatform keys must be represented as JSON strings");
			}
			return UnrealTargetPlatform.Parse(reader.GetString()!);
		}

		public override void Write(Utf8JsonWriter writer, UnrealTargetPlatform value, JsonSerializerOptions options)
		{
			writer.WriteStringValue(value.ToString());
		}

		public override void WriteAsPropertyName(Utf8JsonWriter writer, UnrealTargetPlatform value, JsonSerializerOptions options)
		{
			writer.WritePropertyName(value.ToString());
		}
	}

	/// <summary>
	/// Extension methods used for serializing UnrealTargetPlatform instances
	/// </summary>
	static class UnrealTargetPlatformExtensionMethods
	{
		/// <summary>
		/// Read an UnrealTargetPlatform instance from a binary archive
		/// </summary>
		/// <param name="Reader">Archive to read from</param>
		/// <returns>New UnrealTargetPlatform instance</returns>
		public static UnrealTargetPlatform ReadUnrealTargetPlatform(this BinaryArchiveReader Reader)
		{
			return UnrealTargetPlatform.Parse(Reader.ReadString()!);
		}

		/// <summary>
		/// Write an UnrealTargetPlatform instance to a binary archive
		/// </summary>
		/// <param name="Writer">The archive to write to</param>
		/// <param name="Platform">The platform to write</param>
		public static void WriteUnrealTargetPlatform(this BinaryArchiveWriter Writer, UnrealTargetPlatform Platform)
		{
			Writer.WriteString(Platform.ToString());
		}
	}

	/// <summary>
	/// Platform groups
	/// </summary>
	public partial struct UnrealPlatformGroup : IEquatable<UnrealPlatformGroup>
	{
		#region Private/boilerplate
		// internal concrete name of the group
		private int Id;

		// shared string instance registry - pass in a delegate to create a new one with a name that wasn't made yet
		private static UniqueStringRegistry? StringRegistry;

		// #jira UE-88908 (see above)
		private static UniqueStringRegistry GetUniqueStringRegistry()
		{
			StringRegistry ??= new UniqueStringRegistry();
			return StringRegistry;
		}

		private UnrealPlatformGroup(int InId)
		{
			Id = InId;
		}

		/// <summary>
		/// 
		/// </summary>
		/// <param name="Info"></param>
		/// <param name="Context"></param>
		public void GetObjectData(SerializationInfo Info, StreamingContext Context)
		{
			Info.AddValue("Name", ToString());
		}

		/// <summary>
		/// 
		/// </summary>
		/// <param name="Info"></param>
		/// <param name="Context"></param>
		public UnrealPlatformGroup(SerializationInfo Info, StreamingContext Context)
		{
			Id = GetUniqueStringRegistry().FindOrAddByName((string)Info.GetValue("Name", typeof(string))!);
		}

		/// <summary>
		/// Return the single instance of the Group with this name
		/// </summary>
		/// <param name="Name"></param>
		/// <returns></returns>
		private static UnrealPlatformGroup FindOrAddByName(string Name)
		{
			return new UnrealPlatformGroup(GetUniqueStringRegistry().FindOrAddByName(Name));
		}

		/// <summary>
		/// 
		/// </summary>
		/// <param name="A"></param>
		/// <param name="B"></param>
		/// <returns></returns>
		public static bool operator ==(UnrealPlatformGroup A, UnrealPlatformGroup B)
		{
			return A.Id == B.Id;
		}
		/// <summary>
		/// 
		/// </summary>
		/// <param name="A"></param>
		/// <param name="B"></param>
		/// <returns></returns>
		public static bool operator !=(UnrealPlatformGroup A, UnrealPlatformGroup B)
		{
			return A.Id != B.Id;
		}
		/// <summary>
		/// 
		/// </summary>
		/// <param name="B"></param>
		/// <returns></returns>
		public override bool Equals(object? B)
		{
			if (B is null)
			{
				return false;
			}

			return Id == ((UnrealPlatformGroup)B).Id;
		}

		/// <inheritdoc/>
		public readonly bool Equals(UnrealPlatformGroup other) => this == other;

		/// <summary>
		/// 
		/// </summary>
		/// <returns></returns>
		public override int GetHashCode()
		{
			return Id;
		}

		#endregion

		/// <summary>
		/// Return the string representation
		/// </summary>
		/// <returns></returns>
		public override string ToString()
		{
			return GetUniqueStringRegistry().GetStringForId(Id);
		}

		/// <summary>
		/// 
		/// </summary>
		/// <param name="Name"></param>
		/// <param name="Group"></param>
		/// <returns></returns>
		public static bool TryParse(string Name, out UnrealPlatformGroup Group)
		{
			if (GetUniqueStringRegistry().HasString(Name))
			{
				Group.Id = GetUniqueStringRegistry().FindOrAddByName(Name);
				return true;
			}

			Group.Id = -1;
			return false;
		}

		/// <summary>
		/// 
		/// </summary>
		/// <returns></returns>
		public static UnrealPlatformGroup[] GetValidGroups()
		{
			return Array.ConvertAll(GetUniqueStringRegistry().GetStringIds(), x => new UnrealPlatformGroup(x));
		}

		/// <summary>
		/// 
		/// </summary>
		/// <returns></returns>
		public static string[] GetValidGroupNames()
		{
			return GetUniqueStringRegistry().GetStringNames();
		}

		/// <summary>
		/// 
		/// </summary>
		/// <param name="Name"></param>
		/// <returns></returns>
		public static bool IsValidName(string Name)
		{
			return GetUniqueStringRegistry().HasString(Name);
		}

		/// <summary>
		/// this group is just to lump Win32 and Win64 into Windows directories, removing the special Windows logic in MakeListOfUnsupportedPlatforms
		/// </summary>
		public static UnrealPlatformGroup Windows { get; } = FindOrAddByName("Windows");

		/// <summary>
		/// Microsoft platforms
		/// </summary>
		public static UnrealPlatformGroup Microsoft { get; } = FindOrAddByName("Microsoft");

		/// <summary>
		/// Apple platforms
		/// </summary>
		public static UnrealPlatformGroup Apple { get; } = FindOrAddByName("Apple");

		/// <summary>
		/// making IOS a group allows TVOS to compile IOS code
		/// </summary>
		public static UnrealPlatformGroup IOS { get; } = FindOrAddByName("IOS");

		/// <summary>
		/// Unix platforms
		/// </summary>
		public static UnrealPlatformGroup Unix { get; } = FindOrAddByName("Unix");

		/// <summary>
		/// Linux platforms
		/// </summary>
		public static UnrealPlatformGroup Linux { get; } = FindOrAddByName("Linux");

		/// <summary>
		/// Android platforms
		/// </summary>
		public static UnrealPlatformGroup Android { get; } = FindOrAddByName("Android");

		/// <summary>
		/// Desktop group - used by UnrealPlatformClass.Desktop
		/// </summary>
		public static UnrealPlatformGroup Desktop { get; } = FindOrAddByName("Desktop");

		/// <summary>
		/// SDLPlatform is for platforms that use SDL for windows, cursors, etc
		/// </summary>
		public static UnrealPlatformGroup SDLPlatform { get; } = FindOrAddByName("SDLPlatform");

		/// <summary>
		/// 30Hz is for platforms that typically run at 30Hz (Android, Switch) vs. the baseline 60Hz 
		/// </summary>
		public static UnrealPlatformGroup ThirtyHz { get; } = FindOrAddByName("30Hz");

		/// <summary>
		/// POSIX-compliant platforms
		/// </summary>
		public static UnrealPlatformGroup PosixOS { get; } = FindOrAddByName("PosixOS");

	}

	/// <summary>
	/// The architecture we're building for
	/// </summary>
	[Serializable, TypeConverter(typeof(UnrealArchPlatformTypeConverter))]
	[JsonConverter(typeof(UnrealArchPlatformJsonConverter))]
	public partial struct UnrealArch : ISerializable, IEquatable<UnrealArch>
	{
		#region Private/boilerplate

		// internal concrete name of the group
		private int Id;

		// shared string instance registry - pass in a delegate to create a new one with a name that wasn't made yet
		private static UniqueStringRegistry? StringRegistry;

		// tracks if non-default architectures are an x64 arch
		private static readonly Dictionary<int, bool> IsX64Map = new();

		// #jira UE-88908 if parts of a partial struct have each static member variables, their initialization order does not appear guaranteed
		// here this means initializing "StringRegistry" directly to "new UniqueStringRegistry()" may not be executed before FindOrAddByName() has been called as part of initializing a static member variable of another part of the partial struct
		private static UniqueStringRegistry GetUniqueStringRegistry()
		{
			StringRegistry ??= new UniqueStringRegistry();
			return StringRegistry;
		}

		private UnrealArch(string Name)
		{
			Id = GetUniqueStringRegistry().FindOrAddByName(Name);
		}

		private UnrealArch(int InId)
		{
			Id = InId;
		}

		/// <summary>
		/// 
		/// </summary>
		/// <param name="Info"></param>
		/// <param name="Context"></param>
		public void GetObjectData(SerializationInfo Info, StreamingContext Context)
		{
			Info.AddValue("Name", ToString());
		}

		/// <summary>
		/// 
		/// </summary>
		/// <param name="Info"></param>
		/// <param name="Context"></param>
		public UnrealArch(SerializationInfo Info, StreamingContext Context)
		{
			Id = GetUniqueStringRegistry().FindOrAddByName((string)Info.GetValue("Name", typeof(string))!);
		}

		/// <summary>
		/// Return the single instance of the Group with this name
		/// </summary>
		/// <param name="Name"></param>
		/// <param name="bIsX64">True if X64, false if not, or null if it's Default and will ask the platform later</param>
		/// <returns></returns>
		private static UnrealArch FindOrAddByName(string Name, bool? bIsX64)
		{
			int Id = GetUniqueStringRegistry().FindOrAddByName(Name);
			if (bIsX64 != null)
			{
				IsX64Map[Id] = bIsX64.Value;
			}
			return new UnrealArch(Id);
		}

		/// <summary>
		/// 
		/// </summary>
		/// <param name="A"></param>
		/// <param name="B"></param>
		/// <returns></returns>
		public static bool operator ==(UnrealArch A, UnrealArch B)
		{
			return A.Id == B.Id;
		}
		/// <summary>
		/// 
		/// </summary>
		/// <param name="A"></param>
		/// <param name="B"></param>
		/// <returns></returns>
		public static bool operator !=(UnrealArch A, UnrealArch B)
		{
			return A.Id != B.Id;
		}
		/// <summary>
		/// 
		/// </summary>
		/// <param name="B"></param>
		/// <returns></returns>
		public override bool Equals(object? B)
		{
			if (B is null)
			{
				return false;
			}

			return Id == ((UnrealArch)B).Id;
		}

		/// <inheritdoc/>
		public readonly bool Equals(UnrealArch other) => this == other;

		/// <summary>
		/// 
		/// </summary>
		/// <returns></returns>
		public override int GetHashCode()
		{
			return Id;
		}

		#endregion

		/// <summary>
		/// Returns true if the architecure is X64 based
		/// </summary>
		public bool bIsX64 => IsX64Map[Id];

		//public static string operator +(string A, UnrealArch B)
		//{
		//	throw new BuildException("string + UnrealArch is not allowed");
		//}

		//static void Test(UnrealArch A)
		//{
		//	string Foo = "A" + A;
		//}

		/// <summary>
		/// Return the string representation
		/// </summary>
		/// <returns></returns>
		public override string ToString()
		{
			return GetUniqueStringRegistry().GetStringForId(Id);
		}

		private static string FixName(string Name)
		{
			// allow for some alternate names
			switch (Name.ToUpperInvariant())
			{
				case "A8":
				case "A":
				case "-ARM64":
					Name = "arm64";
					break;

				case "X86_64":
				case "X6":
				case "X":
				case "-X64":
				case "INTEL":
					Name = "x64";
					break;
			}

			return Name;
		}

		/// <summary>
		/// 
		/// </summary>
		/// <param name="Name"></param>
		/// <param name="Arch"></param>
		/// <returns></returns>
		public static bool TryParse(string Name, out UnrealArch Arch)
		{
			// handle the Host string
			if (Name.Equals("Host", StringComparison.OrdinalIgnoreCase))
			{
				Arch.Id = UnrealArch.Host.Value.Id;
				return true;
			}

			Name = FixName(Name);

			if (GetUniqueStringRegistry().HasString(Name))
			{
				Arch.Id = GetUniqueStringRegistry().FindOrAddByName(Name);
				return true;
			}

			if (GetUniqueStringRegistry().HasAlias(Name))
			{
				Arch.Id = GetUniqueStringRegistry().FindExistingAlias(Name);
				return true;
			}

			Arch.Id = -1;
			return false;
		}

		/// <summary>
		/// 
		/// </summary>
		/// <param name="Name"></param>
		/// <returns></returns>
		public static UnrealArch Parse(string Name)
		{
			// handle the Host string
			if (Name.Equals("Host", StringComparison.OrdinalIgnoreCase))
			{
				return UnrealArch.Host.Value;
			}

			Name = FixName(Name);

			if (GetUniqueStringRegistry().HasString(Name))
			{
				return new UnrealArch(Name);
			}

			if (GetUniqueStringRegistry().HasAlias(Name))
			{
				int Id = GetUniqueStringRegistry().FindExistingAlias(Name);
				return new UnrealArch(Id);
			}

			throw new BuildException(String.Format("The Architecture name {0} is not a valid Architecture name. Valid names are ({1})", Name,
				String.Join(",", GetUniqueStringRegistry().GetStringNames())));
		}

		/// <summary>
		/// 
		/// </summary>
		/// <returns></returns>
		public static UnrealArch[] GetValidPlatforms()
		{
			return Array.ConvertAll(GetUniqueStringRegistry().GetStringIds(), x => new UnrealArch(x));
		}

		/// <summary>
		/// 
		/// </summary>
		/// <returns></returns>
		public static string[] GetValidPlatformNames()
		{
			return GetUniqueStringRegistry().GetStringNames();
		}

		/// <summary>
		/// 
		/// </summary>
		/// <param name="Name"></param>
		/// <returns></returns>
		public static bool IsValidName(string Name)
		{
			return GetUniqueStringRegistry().HasString(Name);
		}

		/// <summary>
		/// 64-bit x86/intel
		/// </summary>
		public static UnrealArch X64 { get; } = FindOrAddByName("x64", bIsX64: true);

		/// <summary>
		/// 64-bit arm
		/// </summary>
		public static UnrealArch Arm64 { get; } = FindOrAddByName("arm64", bIsX64: false);

		/// <summary>
		/// Used in place when needing to handle deprecated architectures that a licensee may still have. Do not use in normal logic
		/// </summary>
		public static UnrealArch Deprecated { get; } = FindOrAddByName("deprecated", bIsX64: false);

		/// <summary>
		/// Maps to the currently running architecture on the host platform (lazy because this needs other classes to be intialized)
		/// </summary>
		public static Lazy<UnrealArch> Host { get; } = new(() => UnrealArchitectureConfig.ForPlatform(BuildHostPlatform.Current.Platform).GetHostArchitecture());
	}

	internal class UnrealArchPlatformTypeConverter : TypeConverter
	{
		public override bool CanConvertFrom(ITypeDescriptorContext? context, Type sourceType)
		{
			if (sourceType == typeof(string))
			{
				return true;
			}

			return base.CanConvertFrom(context, sourceType);
		}

		public override bool CanConvertTo(ITypeDescriptorContext? context, Type? destinationType)
		{
			if (destinationType == typeof(string))
			{
				return true;
			}

			return base.CanConvertTo(context, destinationType);
		}

		public override object? ConvertFrom(ITypeDescriptorContext? context, CultureInfo? culture, object value)
		{
			if (value.GetType() == typeof(string))
			{
				return UnrealArch.Parse((string)value);
			}
			return base.ConvertFrom(context, culture, value);
		}

		public override object? ConvertTo(ITypeDescriptorContext? context, CultureInfo? culture, object? value, Type destinationType)
		{
			if (destinationType == typeof(string) && value != null)
			{
				UnrealArch Platform = (UnrealArch)value;
				return Platform.ToString();
			}
			return base.ConvertTo(context, culture, value, destinationType);
		}
	}

	internal class UnrealArchPlatformJsonConverter : JsonConverter<UnrealArch>
	{
		public override UnrealArch Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options)
		{
			if (reader.TokenType != JsonTokenType.String)
			{
				throw new JsonException("UnrealArch values must be represented as JSON strings");
			}
			return UnrealArch.Parse(reader.GetString()!);
		}

		public override UnrealArch ReadAsPropertyName(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options)
		{
			if (reader.TokenType != JsonTokenType.PropertyName)
			{
				throw new JsonException("UnrealArch values must be represented as JSON strings");
			}
			return UnrealArch.Parse(reader.GetString()!);
		}

		public override void Write(Utf8JsonWriter writer, UnrealArch value, JsonSerializerOptions options)
		{
			writer.WriteStringValue(value.ToString());
		}

		public override void WriteAsPropertyName(Utf8JsonWriter writer, [DisallowNull] UnrealArch value, JsonSerializerOptions options)
		{
			writer.WritePropertyName(value.ToString());
		}
	}

	/// <summary>
	/// A collection of one or more architecetures
	/// </summary>
	[Serializable]
	public class UnrealArchitectures
	{
		/// <summary>
		/// Construct a UnrealArchitectures object from a list of Architectures
		/// </summary>
		/// <param name="Architectures"></param>
		public UnrealArchitectures(IEnumerable<UnrealArch> Architectures)
		{
			this.Architectures = new(Architectures.Distinct());
		}

		/// <summary>
		/// Construct a UnrealArchitectures object from a list of string Architectures
		/// </summary>
		/// <param name="Architectures"></param>
		/// <param name="ValidationPlatform"></param>
		public UnrealArchitectures(IEnumerable<string> Architectures, UnrealTargetPlatform? ValidationPlatform = null)
		{
			// validate before conversion (make sure all given architectures can be converted to an UnrealArch that is supported by the platform)
			if (ValidationPlatform != null)
			{
				UnrealArchitectureConfig ArchConfig = UnrealArchitectureConfig.ForPlatform(ValidationPlatform.Value);
				foreach (string ArchString in Architectures)
				{
					UnrealArch Arch;
					if (!UnrealArch.TryParse(ArchString, out Arch) || !ArchConfig.AllSupportedArchitectures.Contains(Arch))
					{
						string ValidArches = String.Join("\n  ", ArchConfig.AllSupportedArchitectures.Architectures);
						throw new BuildException($"Platform {ValidationPlatform} does not support architecture {ArchString}. Valid options are:\n  {ValidArches}");
					}
				}
			}

			// now we know we can convert to final form
			this.Architectures = new(Architectures.Select(x => UnrealArch.Parse(x)).Distinct());

			// standardize order so that passing in { X64, Arm64 } will always result in "arm64+x64" filenames, etc
			this.Architectures.SortBy(x => x.ToString());
		}

		/// <summary>
		/// Construct a UnrealArchitectures object from a single Architecture
		/// </summary>
		/// <param name="Architecture"></param>
		public UnrealArchitectures(UnrealArch Architecture)
			: this(new UnrealArch[] { Architecture })
		{
		}

		/// <summary>
		/// Construct a UnrealArchitectures object from a single string Architecture
		/// </summary>
		/// <param name="ArchitectureString"></param>
		/// <param name="ValidationPlatform"></param>
		public UnrealArchitectures(string ArchitectureString, UnrealTargetPlatform? ValidationPlatform)
			: this(ArchitectureString.Split('+', StringSplitOptions.TrimEntries | StringSplitOptions.RemoveEmptyEntries), ValidationPlatform)
		{
		}

		/// <summary>
		/// Construct a UnrealArchitectures object from another one
		/// </summary>
		/// <param name="Other"></param>
		public UnrealArchitectures(UnrealArchitectures Other)
			: this(Other.Architectures)
		{
		}

		[JsonConstructor]
		private UnrealArchitectures(List<UnrealArch> Architectures)
		{
			this.Architectures = [.. Architectures.Distinct()];
		}

		/// <summary>
		/// Creates an UnrealArchitectures from a string (created via ToString() or similar)
		/// </summary>
		/// <param name="ArchString"></param>
		/// <param name="ValidationPlatform"></param>
		/// <returns></returns>
		public static UnrealArchitectures? FromString(string? ArchString, UnrealTargetPlatform? ValidationPlatform)
		{
			if (String.IsNullOrEmpty(ArchString))
			{
				return null;
			}

			// if this validation platform was passed in, but was too early for the build platform to be registered, we can return
			// null here. we don't want to catch all exceptions and return null, because if ValidationPlatform is good, but the 
			// architecture is invalid, we want to keep the exception 
			if (ValidationPlatform != null && !UEBuildPlatform.TryGetBuildPlatform(ValidationPlatform.Value, out _))
			{
				return null;
			}

			return new UnrealArchitectures(ArchString, ValidationPlatform);
		}

		/// <summary>
		/// The list of architecture names in this set, sorted by architecture name
		/// </summary>
		public readonly List<UnrealArch> Architectures;

		/// <summary>
		/// Gets the Architecture in the normal case where there is a single Architecture in Architectures, or throw an Exception if not single
		/// </summary>
		[JsonIgnore]
		public UnrealArch SingleArchitecture
		{
			get
			{
				if (Architectures.Count > 1)
				{
					throw new Exception($"Asking CppCompileEnvironment for a single Architecture, but it has multiple Architectures ({String.Join(", ", Architectures)}). This indicates logic needs to be updated");
				}
				return Architectures[0];
			}
		}

		/// <summary>
		/// True if there is is more than one architecture specified
		/// </summary>
		[JsonIgnore]
		public bool bIsMultiArch => Architectures.Count > 1;

		/// <summary>
		/// Gets the Architectures list as a + delimited string
		/// </summary>
		public override string ToString()
		{
			return String.Join('+', Architectures);
		}

		internal string GetFolderNameForPlatform(UnrealArchitectureConfig Config)
		{
			return String.Join('+', Architectures.Select(x => Config.GetFolderNameForArchitecture(x)));
		}

		/// <summary>
		/// Convenience function to check if the given architecture is in this set
		/// </summary>
		/// <param name="Arch"></param>
		/// <returns></returns>
		public bool Contains(UnrealArch Arch)
		{
			return Architectures.Contains(Arch);
		}
	}

	/// <summary>
	/// The class of platform. See Utils.GetPlatformsInClass().
	/// </summary>
	public enum UnrealPlatformClass
	{
		/// <summary>
		/// All platforms
		/// </summary>
		All,

		/// <summary>
		/// All desktop platforms (Win32, Win64, Mac, Linux)
		/// </summary>
		Desktop,

		/// <summary>
		/// All platforms which support the editor (Win64, Mac, Linux)
		/// </summary>
		Editor,

		/// <summary>
		/// Platforms which support running servers (Win32, Win64, Mac, Linux)
		/// </summary>
		Server,
	}

	/// <summary>
	/// The type of configuration a target can be built for.  Roughly order by optimization level. 
	/// </summary>
	[JsonConverter(typeof(JsonStringEnumConverter))]
	public enum UnrealTargetConfiguration
	{
		/// <summary>
		/// Unknown
		/// </summary>
		Unknown,

		/// <summary>
		/// Debug configuration
		/// </summary>
		Debug,

		/// <summary>
		/// DebugGame configuration; equivalent to development, but with optimization disabled for game modules
		/// </summary>
		DebugGame,

		/// <summary>
		/// Development configuration
		/// </summary>
		Development,

		/// <summary>
		/// Test configuration
		/// </summary>
		Test,

		/// <summary>
		/// Shipping configuration
		/// </summary>
		Shipping,
	}

	/// <summary>
	/// Intermediate environment. Determines if the intermediates end up in a different folder than normal.
	/// </summary>
	public enum UnrealIntermediateEnvironment
	{
		/// <summary>
		/// Default environment
		/// </summary>
		Default,
		/// <summary>
		/// Generate clang database environment is non unity and needs it own environment
		/// </summary>
		GenerateClangDatabase,
		/// <summary>
		/// Include what you use
		/// </summary>
		IWYU,
		/// <summary>
		/// Nonunity
		/// </summary>
		NonUnity,
		/// <summary>
		/// Static code analysis
		/// </summary>
		Analyze,
		/// <summary>
		/// Query
		/// </summary>
		Query,
		/// <summary>
		/// Generate project files
		/// </summary>
		GenerateProjectFiles,

		/// <summary>
		/// Profiling compilation can modify a number of compilation settings.
		/// </summary>
		ProfilingCompile,
	}

	/// <summary>
	/// Extension methods for UnrealIntermediateEnvironment enum
	/// </summary>
	public static class UEBuildTargetExtensions
	{
		/// <summary>
		/// Returns if this environment should default to a unity build
		/// </summary>
		/// <param name="Environment">The environment to check</param>
		/// <returns>true if shold build unity by default</returns>
		public static bool IsUnity(this UnrealIntermediateEnvironment Environment)
		{
			switch (Environment)
			{
				case UnrealIntermediateEnvironment.IWYU:
				case UnrealIntermediateEnvironment.GenerateClangDatabase:
				case UnrealIntermediateEnvironment.NonUnity:
					return false;
			}
			return true;
		}
	}

	/// <summary>
	/// A container for a binary files (dll, exe) with its associated debug info.
	/// </summary>
	public class BuildManifest
	{
		/// <summary>
		/// 
		/// </summary>
		public readonly List<string> BuildProducts = new List<string>();

		/// <summary>
		/// 
		/// </summary>
		public readonly List<string> DeployTargetFiles = new List<string>();

		/// <summary>
		/// 
		/// </summary>
		public BuildManifest()
		{
		}

		/// <summary>
		/// 
		/// </summary>
		/// <param name="FileName"></param>
		public void AddBuildProduct(string FileName)
		{
			string FullFileName = Path.GetFullPath(FileName);
			if (!BuildProducts.Contains(FullFileName))
			{
				BuildProducts.Add(FullFileName);
			}
		}

		/// <summary>
		/// 
		/// </summary>
		/// <param name="FileName"></param>
		/// <param name="DebugInfoExtension"></param>
		public void AddBuildProduct(string FileName, string DebugInfoExtension)
		{
			AddBuildProduct(FileName);
			if (!String.IsNullOrEmpty(DebugInfoExtension))
			{
				AddBuildProduct(Path.ChangeExtension(FileName, DebugInfoExtension));
			}
		}
	}

	/// <summary>
	/// A target that can be built
	/// </summary>
	class UEBuildTarget
	{
		/// <summary>
		/// Creates a target object for the specified target name.
		/// </summary>
		/// <param name="Descriptor">Information about the target</param>
		/// <param name="BuildConfiguration">Build configuration</param>
		/// <param name="Logger">Logger for output</param>
		/// <returns></returns>
		public static UEBuildTarget Create(TargetDescriptor Descriptor, BuildConfiguration BuildConfiguration, ILogger Logger)
		{
			return Create(Descriptor, BuildConfiguration.bSkipRulesCompile, BuildConfiguration.bForceRulesCompile, BuildConfiguration.bUsePrecompiled, Descriptor.IntermediateEnvironment, Logger);
		}

		/// <summary>
		/// Creates a target object for the specified target name.
		/// </summary>
		/// <param name="Descriptor">Information about the target</param>
		/// <param name="Logger">Logger for output</param>
		/// <returns></returns>
		public static UEBuildTarget Create(TargetDescriptor Descriptor, ILogger Logger)
		{
			return Create(Descriptor, false, false, false, UnrealIntermediateEnvironment.Default, Logger);
		}

		/// <summary>
		/// Creates a target object for the specified target name.
		/// </summary>
		/// <param name="Descriptor">Information about the target</param>
		/// <param name="bSkipRulesCompile"></param>
		/// <param name="bForceRulesCompile"></param>
		/// <param name="bUsePrecompiled"></param>
		/// <param name="Logger">Logger for output</param>
		/// <returns></returns>
		public static UEBuildTarget Create(TargetDescriptor Descriptor, bool bSkipRulesCompile, bool bForceRulesCompile, bool bUsePrecompiled, ILogger Logger)
		{
			return Create(Descriptor, bSkipRulesCompile, bForceRulesCompile, bUsePrecompiled, UnrealIntermediateEnvironment.Default, Logger);
		}

		/// <summary>
		/// Creates a target object for the specified target name.
		/// </summary>
		/// <param name="Descriptor">Information about the target</param>
		/// <param name="bSkipRulesCompile">Whether to skip compiling any rules assemblies</param>
		/// <param name="bForceRulesCompile">Whether to always compile all rules assemblies</param>
		/// <param name="bUsePrecompiled">Whether to use a precompiled engine build</param>
		/// <param name="IntermediateEnvironment">Intermediate environment to use</param>
		/// <param name="Logger">Logger for output</param>
		/// <returns>The build target object for the specified build rules source file</returns>
		public static UEBuildTarget Create(TargetDescriptor Descriptor, bool bSkipRulesCompile, bool bForceRulesCompile, bool bUsePrecompiled, UnrealIntermediateEnvironment IntermediateEnvironment, ILogger Logger)
		{
			(TargetRules RulesObject, RulesAssembly RulesAssembly) = CreateRulesObject(Descriptor, bSkipRulesCompile, bForceRulesCompile, bUsePrecompiled, IntermediateEnvironment, Logger);

			// Generate a build target from this rules module
			UEBuildTarget Target;
			using (GlobalTracer.Instance.BuildSpan("UEBuildTarget constructor").StartActive())
			{
				Target = new UEBuildTarget(Descriptor, new ReadOnlyTargetRules(RulesObject), RulesAssembly);
			}
			using (GlobalTracer.Instance.BuildSpan("UEBuildTarget.PreBuildSetup()").StartActive())
			{
				Target.PreBuildSetup(Logger);
			}

			return Target;
		}

		internal static (TargetRules, RulesAssembly) CreateRulesObject(TargetDescriptor Descriptor, bool bSkipRulesCompile, bool bForceRulesCompile, bool bUsePrecompiled, UnrealIntermediateEnvironment IntermediateEnvironment, ILogger Logger)
		{
			// make sure we are allowed to build this platform
			if (!UEBuildPlatform.IsPlatformAvailable(Descriptor.Platform))
			{
				throw new BuildException("{0}", UEBuildPlatform.GetPlatformUnavailableReason(Descriptor.Platform));
			}

			RulesAssembly RulesAssembly;
			using (GlobalTracer.Instance.BuildSpan("RulesCompiler.CreateTargetRulesAssembly()").StartActive())
			{
				RulesAssembly = RulesCompiler.CreateTargetRulesAssembly(Descriptor.ProjectFile, Descriptor.Name, bSkipRulesCompile, bForceRulesCompile, bUsePrecompiled, Descriptor.ForeignPlugin, Descriptor.bBuildPluginAsLocal, Logger);
			}

			TargetRules RulesObject;
			using (GlobalTracer.Instance.BuildSpan("RulesAssembly.CreateTargetRules()").StartActive())
			{
				using ITimelineEvent _ = Timeline.ScopeEvent("CreateTargetRules");
				RulesObject = RulesAssembly.CreateTargetRules(Descriptor.Name, Descriptor.Platform, Descriptor.Configuration, Descriptor.Architectures, Descriptor.ProjectFile, Descriptor.AdditionalArguments, Logger, Descriptor.IsTestsTarget, IntermediateEnvironment: IntermediateEnvironment);
			}
			if ((ProjectFileGenerator.bGenerateProjectFiles == false) && !RulesObject.GetSupportedPlatforms().Contains(Descriptor.Platform))
			{
				throw new BuildException("{0} does not support the {1} platform.", Descriptor.Name, Descriptor.Platform.ToString());
			}

			// Make sure this configuration is supports
			if (!RulesObject.GetSupportedConfigurations().Contains(Descriptor.Configuration))
			{
				throw new BuildException("{0} does not support the {1} configuration", Descriptor.Name, Descriptor.Configuration);
			}

			// Make sure this target type is supported. Allow UHT in installed builds as a special case for now.
			if (!InstalledPlatformInfo.IsValid(RulesObject.Type, Descriptor.Platform, Descriptor.Configuration, EProjectType.Code, InstalledPlatformState.Downloaded) && Descriptor.Name != "UnrealHeaderTool")
			{
				if (InstalledPlatformInfo.IsValid(RulesObject.Type, Descriptor.Platform, Descriptor.Configuration, EProjectType.Code, InstalledPlatformState.Supported))
				{
					throw new BuildException("Download support for building {0} {1} targets from the launcher.", Descriptor.Platform, RulesObject.Type);
				}
				else if (!InstalledPlatformInfo.IsValid(RulesObject.Type, null, null, EProjectType.Code, InstalledPlatformState.Supported))
				{
					throw new BuildException("{0} targets are not currently supported from this engine distribution.", RulesObject.Type);
				}
				else if (!InstalledPlatformInfo.IsValid(null, RulesObject.Platform, null, EProjectType.Code, InstalledPlatformState.Supported))
				{
					throw new BuildException("The {0} platform is not currently supported from this engine distribution.", RulesObject.Platform);
				}
				else if (!InstalledPlatformInfo.IsValid(RulesObject.Type, RulesObject.Platform, null, EProjectType.Code, InstalledPlatformState.Supported))
				{
					throw new BuildException("{0} targets for the {1} platform are not currently supported from this engine distribution.", RulesObject.Type, RulesObject.Platform);
				}
				else
				{
					throw new BuildException("Targets cannot be built in the {0} configuration with this engine distribution.", RulesObject.Configuration);
				}
			}

			if (RulesObject.bEnableConstInitUObject && RulesObject.bWithLiveCoding)
			{
				throw new BuildException("bEnableConstInitUObject and bWithLiveCoding cannot currently be set simultaneously");
			}

			// If we're using the shared build environment, make sure all the settings are valid
			if (RulesObject.BuildEnvironment == TargetBuildEnvironment.Shared)
			{
				try
				{
					ValidateSharedEnvironment(RulesAssembly, Descriptor.Name, Descriptor.AdditionalArguments, RulesObject);
				}
				catch (Exception)
				{
					RulesObject.PrintBuildSettingsInfoWarnings();

					// Prepare and write a partial receipt for downstream systems to use in related messaging about shared environment invalidations.
					PrepareAndWritePartialReceipt(Descriptor, RulesObject, RulesAssembly, Logger);

					throw;
				}
			}

			// If we're precompiling, generate a list of all the files that we depend on
			if (RulesObject.bPrecompile)
			{
				DirectoryReference DependencyListDir;
				if (RulesObject.ProjectFile == null)
				{
					DependencyListDir = DirectoryReference.Combine(Unreal.EngineDirectory, "Intermediate", "DependencyLists", RulesObject.Name, RulesObject.Configuration.ToString(), RulesObject.Platform.ToString());
				}
				else
				{
					DependencyListDir = DirectoryReference.Combine(RulesObject.ProjectFile.Directory, "Intermediate", "DependencyLists", RulesObject.Name, RulesObject.Configuration.ToString(), RulesObject.Platform.ToString());
				}

				if (!UnrealArchitectureConfig.ForPlatform(Descriptor.Platform).RequiresArchitectureFilenames(Descriptor.Architectures))
				{
					DependencyListDir = new DirectoryReference(DependencyListDir.FullName + Descriptor.Architectures.ToString());
				}

				FileReference DependencyListFile;
				if (RulesObject.bBuildAllModules)
				{
					DependencyListFile = FileReference.Combine(DependencyListDir, "DependencyList-AllModules.txt");
				}
				else
				{
					DependencyListFile = FileReference.Combine(DependencyListDir, "DependencyList.txt");
				}

				RulesObject.DependencyListFileNames.Add(DependencyListFile);
			}

			// If we're compiling a plugin, and this target is monolithic, just create the object files
			if (Descriptor.ForeignPlugin != null && RulesObject.LinkType == TargetLinkType.Monolithic)
			{
				// Don't actually want an executable
				RulesObject.bDisableLinking = true;

				// Don't allow using shared PCHs; they won't be distributed with the plugin
				RulesObject.bUseSharedPCHs = false;
			}

			// Force shared PCHs off with delegate lifetime, as this is tricky to find out otherwise
			if (RulesObject.bCheckDelegateLifetime)
			{
				RulesObject.bUseSharedPCHs = false;
			}

			// Don't link if we're just preprocessing
			if (RulesObject.bPreprocessOnly)
			{
				RulesObject.bDisableLinking = true;
			}

			// Ignore build outputs when compiling specific files, as most actions are skipped.
			// Do not disable linking! Those actions will be skipped and compiling specific files doesn't invalidate the makefile
			// so it will be saved with no binary outputs. That would be bad!
			if (Descriptor.SpecificFilesToCompile.Any())
			{
				RulesObject.bIgnoreBuildOutputs = true;
			}
			return (RulesObject, RulesAssembly);
		}

		/// <summary>
		/// Validates that the build environment matches the shared build environment, by comparing the TargetRules instance to the vanilla target rules for the current target type.
		/// </summary>
		static void ValidateSharedEnvironment(RulesAssembly RulesAssembly, string ThisTargetName, CommandLineArguments Arguments, TargetRules ThisRules)
		{
			Dictionary<string, (string?, string?)> PropNamesThatRequiredUnique = new();
			string? BaseTargetName;
			if (ThisRules.RequiresUniqueEnvironment(RulesAssembly, Arguments, PropNamesThatRequiredUnique, out BaseTargetName))
			{
				throw new BuildException("{0} modifies the values of properties: [ {1} ]. This is not allowed, as {0} has build products in common with {2}.\nRemove the modified setting, change {0} to use a unique build environment by setting 'BuildEnvironment = TargetBuildEnvironment.Unique;' in the {3} constructor, or set bOverrideBuildEnvironment = true to force this setting on.",
					ThisTargetName, String.Join(", ", PropNamesThatRequiredUnique.Select(x => $"{x.Key}: {x.Value.Item1} != {x.Value.Item2}")), BaseTargetName, ThisRules.GetType().Name);
			}

			// Make sure that we don't explicitly enable or disable any plugins through the target rules. We can't do this with the shared build environment because it requires recompiling the "Projects" engine module.
			bool bUsesTargetReceiptToEnablePlugins = (ThisRules.Type == TargetType.Editor && ThisRules.LinkType != TargetLinkType.Monolithic);
			// programs can enable/disable plugins even when modular
			bool bIsProgramTarget = ThisRules.Type == TargetType.Program;

			if (!bUsesTargetReceiptToEnablePlugins && !bIsProgramTarget && (ThisRules.EnablePlugins.Count > 0 || ThisRules.DisablePlugins.Count > 0))
			{
				throw new BuildException(String.Format("Explicitly enabling and disabling plugins for a target is only supported when using a unique build environment (eg. for monolithic game targets). EnabledPlugins={0}, DisabledPlugins={1}",
					String.Join(", ", ThisRules.EnablePlugins),
					String.Join(", ", ThisRules.DisablePlugins)
				));
			}
		}

		/// <summary>
		/// Validate all plugins
		/// </summary>
		/// <param name="logger"></param>
		/// <returns>true if there are any fatal errors</returns>
		bool ValidatePlugins(ILogger logger)
		{
			if (BuildPlugins == null)
			{
				return false;
			}

			foreach (UEBuildPlugin plugin in BuildPlugins)
			{
				UEBuildPlatform.GetBuildPlatform(Platform).ValidatePlugin(plugin, Rules);
			}

			// Intentionally not ordered by name to maintain the reference include order
			return BuildPlugins.Select(plugin => ValidatePlugin(plugin, logger)).Any(x => x);
		}

		/// <summary>
		/// Validates a plugin
		/// </summary>
		/// <param name="plugin"></param>
		/// <param name="logger"></param>
		/// <returns>true if there are any fatal errors</returns>
		bool ValidatePlugin(UEBuildPlugin plugin, ILogger logger)
		{
			bool anyErrors = plugin.ValidatePlugin(logger, Rules);

			// Check that each plugin declares its dependencies explicitly
			foreach (UEBuildModule module in plugin.Modules)
			{
				HashSet<UEBuildModule> dependencyModules = module.GetDependencies(bWithIncludePathModules: true, bWithDynamicallyLoadedModules: true);
				// Only warn on modules that can be compiled
				foreach (UEBuildModuleCPP dependencyModule in dependencyModules.OfType<UEBuildModuleCPP>())
				{
					PluginInfo? dependencyPluginInfo = dependencyModule.Rules.Plugin;
					bool isOptional = plugin.Descriptor.Plugins?.Where(x => dependencyPluginInfo?.Name == x.Name).Any(x => x.bOptional) ?? false;
					if (dependencyPluginInfo != null
						&& dependencyPluginInfo != plugin.Info
						&& !Rules.InternalPluginDependencies.Contains(dependencyPluginInfo.Name)
						&& !(plugin.Dependencies!.Any(x => dependencyPluginInfo == x.Info) || isOptional))
					{
						logger.LogWarning("Warning: Plugin '{PluginName}' does not list plugin '{DependencyPluginName}' as a dependency, but module '{ModuleName}' depends on module '{DependencyModuleName}'.", plugin.Name, dependencyPluginInfo.Name, module.Name, dependencyModule.Name);
					}
				}
			}

			return anyErrors;
		}

		/// <summary>
		/// Validates all plugins
		/// </summary>
		/// <param name="logger"></param>
		/// <returns>true if there are any fatal errors</returns>
		bool ValidateModules(ILogger logger)
		{
			int errorCount = 0;
			CaptureLogger[] captureLoggers = new CaptureLogger[Modules.Count];
			Parallel.ForEach(Modules.Values.OrderBy(x => x.Name), (module, _, index) =>
			{
				captureLoggers[index] = new();
				UEBuildPlatform.GetBuildPlatform(Platform).ValidateModule(module, Rules, captureLoggers[index]);
				bool anyErrors = UEBuildPlatform.GetBuildPlatform(Platform).ValidateModuleIncludePaths(module, Rules, Modules.Values, captureLoggers[index]);
				anyErrors = ValidateModule(module, captureLoggers[index]) || anyErrors;
				if (anyErrors)
				{
					Interlocked.Increment(ref errorCount);
				}
			});

			foreach (CaptureLogger captureLogger in captureLoggers)
			{
				captureLogger.RenderTo(logger);
			}

			return errorCount > 0;
		}

		/// <summary>
		/// Validates a module
		/// </summary>
		/// <param name="module"></param>
		/// <param name="logger"></param>
		/// <returns>true if there are any fatal errors</returns>
		bool ValidateModule(UEBuildModule module, ILogger logger)
		{
			bool anyErrors = module.ValidateModule(logger);

			Lazy<HashSet<UEBuildModule>> referencedModules = new(() => module.GetDependencies(bWithIncludePathModules: true, bWithDynamicallyLoadedModules: true));

			// Check there aren't any engine binaries with dependencies on game modules. This can happen when game-specific plugins override engine plugins.
			if (module.Binary != null)
			{
				foreach (UEBuildModule referencedModule in referencedModules.Value)
				{
					if (!module.Rules.Context.Scope.Contains(referencedModule.Rules.Context.Scope) && !IsEngineToPluginReferenceAllowed(module.Name, referencedModule.Name))
					{
						logger.LogError(KnownLogEvents.Validation, "Module '{ModuleName}' ({ScopeName}) should not reference module '{DependencyModuleName}' ({DependencyScopeName}). Hierarchy is {Hierarchy}.", module.Name, module.Rules.Context.Scope.Name, referencedModule.Name, referencedModule.Rules.Context.Scope.Name, referencedModule.Rules.Context.Scope.FormatHierarchy());
						anyErrors = true;
					}
				}
			}

			// Check that each project module with a dependency on a plugin module, that the plugin is either enabled by default or it's enabled by the uproject file.
			ModuleDescriptor? moduleDescriptor = ProjectDescriptor?.Modules?.FirstOrDefault(x => x.Name == module.Name);
			if (moduleDescriptor != null)
			{
				string projectName = ProjectFile?.FullName ?? TargetRulesFile.GetFileName();

				// EffectiveAllow = TargetRulesAllow && ProjectAllow;
				bool bAllowEnginePluginsEnabledByDefault = Rules.bAllowEnginePluginsEnabledByDefault;
				if (ProjectDescriptor?.DisableEnginePluginsByDefault ?? false)
				{
					bAllowEnginePluginsEnabledByDefault = false;
				}

				// Only warn on modules that can be compiled
				foreach (UEBuildModuleCPP dependencyModule in referencedModules.Value.OfType<UEBuildModuleCPP>())
				{
					PluginInfo? dependencyPluginInfo = dependencyModule.Rules.Plugin;
					// Is the modules plugin enabled by default?
					if (dependencyPluginInfo != null && dependencyPluginInfo.IsEnabledByDefault(bAllowEnginePluginsEnabledByDefault) == false)
					{
						ModuleDescriptor? dependencyModuleInfo = dependencyPluginInfo.Descriptor.Modules?.FirstOrDefault(x => x.Name == dependencyModule.Name);
						// Try and find the project plugin reference in the plugins list.
						PluginReferenceDescriptor? projectPluginReference = ProjectDescriptor?.Plugins?.FirstOrDefault(x => x.Name == dependencyPluginInfo.Name);
						if (projectPluginReference != null)
						{
							if (!projectPluginReference.bEnabled)
							{
								logger.LogWarning(KnownLogEvents.Validation, "{ProjectName} plugin dependency '{DependencyPluginName}' is not enabled, but module '{ModuleName}' depends on '{DependencyModuleName}'.", projectName, dependencyPluginInfo.Name, module.Name, dependencyModule.Name);
							}
							else if (!projectPluginReference.IsEnabledForTarget(TargetType))
							{
								logger.LogWarning(KnownLogEvents.Validation, "{ProjectName} plugin dependency '{DependencyPluginName}' is not enabled for target '{TargetType}', but module '{ModuleName}' depends on '{DependencyModuleName}'.", projectName, dependencyPluginInfo.Name, TargetType.ToString(), module.Name, dependencyModule.Name);
							}
							else if (!projectPluginReference.IsEnabledForPlatform(Platform))
							{
								logger.LogWarning(KnownLogEvents.Validation, "{ProjectName} plugin dependency '{DependencyPluginName}' is not enabled for platform '{Platform}', but module '{ModuleName}' depends on '{DependencyModuleName}'.", projectName, dependencyPluginInfo.Name, Platform.ToString(), module.Name, dependencyModule.Name);
							}
							else if (!projectPluginReference.IsEnabledForTargetConfiguration(Configuration))
							{
								logger.LogWarning(KnownLogEvents.Validation, "{ProjectName} plugin dependency '{DependencyPluginName}' is not enabled for target configuration '{Configuration}', but module '{ModuleName}' depends on '{DependencyModuleName}'.", projectName, dependencyPluginInfo.Name, Configuration.ToString(), module.Name, dependencyModule.Name);
							}
						}
						else
						{
							// No project plugin reference exists. Check inclusion of a plugin was not made as part of a Target.cs file or Build.cs file
							if (!Rules.EnablePlugins.Contains(dependencyPluginInfo.Name) && !Rules.InternalPluginDependencies.Contains(dependencyPluginInfo.Name))
							{
								logger.LogWarning(KnownLogEvents.Validation, "{ProjectName} does not list plugin '{DependencyPluginName}' as a dependency, but module '{ModuleName}' depends on '{DependencyModuleName}'.", projectName, dependencyPluginInfo.Name, module.Name, dependencyModule.Name);
							}
						}
					}
				}
			}

			return anyErrors;
		}

		/// <summary>
		/// The target rules
		/// </summary>
		public ReadOnlyTargetRules Rules;

		/// <summary>
		/// The rules assembly to use when searching for modules
		/// </summary>
		public RulesAssembly RulesAssembly;

		/// <summary>
		/// The project file for this target
		/// </summary>
		public FileReference? ProjectFile;

		/// <summary>
		/// The project descriptor for this target
		/// </summary>
		public ProjectDescriptor? ProjectDescriptor;

		/// <summary>
		/// Type of target
		/// </summary>
		public TargetType TargetType;

		/// <summary>
		/// The name of the application the target is part of. For targets with bUseSharedBuildEnvironment = true, this is typically the name of the base application, eg. UnrealEditor for any game editor.
		/// </summary>
		public string AppName;

		/// <summary>
		/// The name of the target
		/// </summary>
		public string TargetName;

		/// <summary>
		/// Whether the target uses the shared build environment. If false, AppName==TargetName and all binaries should be written to the project directory.
		/// </summary>
		public bool bUseSharedBuildEnvironment;

		/// <summary>
		/// Platform as defined by the VCProject and passed via the command line. Not the same as internal config names.
		/// </summary>
		public UnrealTargetPlatform Platform;

		/// <summary>
		/// Target as defined by the VCProject and passed via the command line. Not necessarily the same as internal name.
		/// </summary>
		public UnrealTargetConfiguration Configuration;

		/// <summary>
		/// The architecture this target is being built for
		/// </summary>
		public UnrealArchitectures Architectures;

		/// <summary>
		/// Intermediate environment. Determines if the intermediates end up in a different folder than normal.
		/// </summary>
		public UnrealIntermediateEnvironment IntermediateEnvironment;

		/// <summary>
		/// Relative path for platform-specific intermediates (eg. Intermediate/Build/Win64/x64)
		/// </summary>
		public string PlatformIntermediateFolder;

		/// <summary>
		/// Relative path for platform-specific intermediates that explicitly do not need an architeecture (eg. Intermediate/Build/Win64)
		/// </summary>
		public string PlatformIntermediateFolderNoArch;

		/// <summary>
		/// Root directory for the active project. Typically contains the .uproject file, or the engine root.
		/// </summary>
		public DirectoryReference ProjectDirectory;

		/// <summary>
		/// Default directory for intermediate files. Typically underneath ProjectDirectory.
		/// </summary>
		public DirectoryReference ProjectIntermediateDirectory;

		/// <summary>
		/// Default directory for intermediate files. Typically underneath ProjectDirectory.
		/// </summary>
		public DirectoryReference ProjectIntermediateDirectoryNoArch;

		/// <summary>
		/// Directory for engine intermediates. For an agnostic editor/game executable, this will be under the engine directory. For monolithic executables this will be the same as the project intermediate directory.
		/// </summary>
		public DirectoryReference EngineIntermediateDirectory;

		/// <summary>
		/// All plugins which are built for this target
		/// </summary>
		public List<UEBuildPlugin>? BuildPlugins;

		/// <summary>
		/// All plugin dependencies for this target. This differs from the list of plugins that is built for Launcher, where we build everything, but link in only the enabled plugins.
		/// </summary>
		public List<UEBuildPlugin>? EnabledPlugins;

		/// <summary>
		/// Specifies the path to a specific plugin to compile.
		/// </summary>
		public FileReference? ForeignPlugin;

		/// <summary>
		/// When building a foreign plugin, whether to build plugins it depends on as well.
		/// </summary>
		public bool bBuildDependantPlugins = false;

		/// <summary>
		/// Collection of all UBT plugins (project files)
		/// </summary>
		public List<FileReference>? UbtPlugins;

		/// <summary>
		/// Collection of all enabled UBT plugins (project files)
		/// </summary>
		public List<FileReference>? EnabledUbtPlugins;

		/// <summary>
		/// Collection of all enabled UHT plugin assemblies
		/// </summary>
		public List<FileReference>? EnabledUhtPlugins;

		/// <summary>
		/// All application binaries; may include binaries not built by this target.
		/// </summary>
		public List<UEBuildBinary> Binaries = new List<UEBuildBinary>();

		/// <summary>
		/// Kept to determine the correct module parsing order when filtering modules.
		/// </summary>
		protected List<UEBuildBinary> NonFilteredModules = new List<UEBuildBinary>();

		/// <summary>
		/// true if target should be compiled in monolithic mode, false if not
		/// </summary>
		protected bool bCompileMonolithic = false;

		/// <summary>
		/// Used to keep track of all modules by name.
		/// </summary>
		private readonly Dictionary<string, UEBuildModule> Modules = new(StringComparer.InvariantCultureIgnoreCase);

		private readonly Dictionary<string, UEBuildModule> UsedShortNames = new(StringComparer.InvariantCultureIgnoreCase);

		/// <summary>
		/// Filename for the receipt for this target.
		/// </summary>
		public FileReference ReceiptFileName
		{
			get;
			private set;
		}

		/// <summary>
		/// The name of the .Target.cs file, if the target was created with one
		/// </summary>
		public readonly FileReference TargetRulesFile;

		/// <summary>
		/// Whether to deploy this target after compilation
		/// </summary>
		public bool bDeployAfterCompile;

		/// <summary>
		/// Whether this target should be compiled in monolithic mode
		/// </summary>
		/// <returns>true if it should, false if it shouldn't</returns>
		public bool ShouldCompileMonolithic()
		{
			return bCompileMonolithic;  // @todo ubtmake: We need to make sure this function and similar things aren't called in assembler mode
		}

		/// <summary>
		/// Whether this target was compiled with an overridden sdk (usually from a platform .ini specifying an alternate SDK version)
		/// </summary>
		/// <returns></returns>
		public static bool BuiltWithOverriddenSDKs()
		{
			return UEBuildPlatformSDK.bHasAnySDKOverride;
		}

		/// <summary>
		/// Constructor.
		/// </summary>
		/// <param name="InDescriptor">Target descriptor</param>
		/// <param name="InRules">The target rules, as created by RulesCompiler.</param>
		/// <param name="InRulesAssembly">The chain of rules assemblies that this target was created with</param>
		internal UEBuildTarget(TargetDescriptor InDescriptor, ReadOnlyTargetRules InRules, RulesAssembly InRulesAssembly)
		{
			ProjectFile = InDescriptor.ProjectFile;
			TargetName = InRules.Name;
			AppName = InRules.Name;
			Platform = InDescriptor.Platform;
			Configuration = InDescriptor.Configuration;
			Architectures = InDescriptor.Architectures;
			IntermediateEnvironment = Unreal.IsEngineInstalled() ? UnrealIntermediateEnvironment.Default : InDescriptor.IntermediateEnvironment != UnrealIntermediateEnvironment.Default ? InDescriptor.IntermediateEnvironment : InRules.IntermediateEnvironment;
			Rules = InRules;
			RulesAssembly = InRulesAssembly;
			TargetType = Rules.Type;
			ForeignPlugin = InDescriptor.ForeignPlugin;
			bBuildDependantPlugins = InDescriptor.bBuildDependantPlugins;
			bDeployAfterCompile = InRules.bDeployAfterCompile && !InRules.bDisableLinking && InDescriptor.OnlyModuleNames.Count == 0;

			// now that we have the platform, we can set the intermediate path to include the platform/architecture name
			PlatformIntermediateFolder = GetPlatformIntermediateFolder(Platform, Architectures, false);
			PlatformIntermediateFolderNoArch = GetPlatformIntermediateFolder(Platform, null, false);

			TargetRulesFile = InRules.File;

			bCompileMonolithic = (Rules.LinkType == TargetLinkType.Monolithic);

			// Set the build environment
			bUseSharedBuildEnvironment = (Rules.BuildEnvironment == TargetBuildEnvironment.Shared);
			if (bUseSharedBuildEnvironment)
			{
				if (Rules.Type == TargetType.Program)
				{
					AppName = TargetName;
				}
				else
				{
					AppName = GetAppNameForTargetType(Rules.Type);
				}
			}

			// track where intermediates and receipt files will be saved under (Programs with .uproject files still want to put the receipt
			// under Engine/Binaries, not Engine/Programs/Foo/Binaries
			DirectoryReference OutputRootDirectory = GetOutputDirectoryForExecutable(Unreal.EngineDirectory, Rules.File);

			// Figure out what the project directory is. If we have a uproject file, use that. Otherwise use the engine directory.
			if (ProjectFile != null)
			{
				ProjectDirectory = ProjectFile.Directory;
				// programs are often compiled with another project passed in for context, so if it's using a _different_ uproject, use that uproject
				// for the output location
				if (Rules.Type != TargetType.Program || !String.Equals(ProjectFile.GetFileNameWithoutAnyExtensions(), Rules.Name, StringComparison.OrdinalIgnoreCase))
				{
					OutputRootDirectory = ProjectDirectory;
				}
			}
			else
			{
				ProjectDirectory = Unreal.EngineDirectory;
			}

			// Build the project intermediate directory
			ProjectIntermediateDirectory = DirectoryReference.Combine(OutputRootDirectory, PlatformIntermediateFolder, GetTargetIntermediateFolderName(TargetName, IntermediateEnvironment), Configuration.ToString());
			ProjectIntermediateDirectoryNoArch = DirectoryReference.Combine(OutputRootDirectory, PlatformIntermediateFolderNoArch, GetTargetIntermediateFolderName(TargetName, IntermediateEnvironment), Configuration.ToString());

			// Build the engine intermediate directory. If we're building agnostic engine binaries, we can use the engine intermediates folder. Otherwise we need to use the project intermediates directory.
			if (!bUseSharedBuildEnvironment)
			{
				if (Configuration == UnrealTargetConfiguration.DebugGame)
				{
					EngineIntermediateDirectory = DirectoryReference.Combine(OutputRootDirectory, PlatformIntermediateFolder, GetTargetIntermediateFolderName(TargetName, IntermediateEnvironment), UnrealTargetConfiguration.Development.ToString());
				}
				else
				{
					EngineIntermediateDirectory = ProjectIntermediateDirectory;
				}
			}
			else if (Configuration == UnrealTargetConfiguration.DebugGame)
			{
				EngineIntermediateDirectory = DirectoryReference.Combine(Unreal.EngineDirectory, PlatformIntermediateFolder, GetTargetIntermediateFolderName(AppName, IntermediateEnvironment), UnrealTargetConfiguration.Development.ToString());
			}
			else
			{
				EngineIntermediateDirectory = DirectoryReference.Combine(Unreal.EngineDirectory, PlatformIntermediateFolder, GetTargetIntermediateFolderName(AppName, IntermediateEnvironment), Configuration.ToString());
			}

			// Get the receipt path for this target
			ReceiptFileName = TargetReceipt.GetDefaultPath(OutputRootDirectory, TargetName, Platform, Configuration, Architectures);

			// Read the project descriptor
			if (ProjectFile != null)
			{
				ProjectDescriptor = ProjectDescriptor.FromFile(ProjectFile);
			}
		}

		/// <summary>
		/// Gets the intermediate directory for a given platform
		/// </summary>
		/// <param name="Platform">Platform to get the folder for</param>
		/// <param name="Architectures">Architectures to get the folder for</param>
		/// <param name="External">Insert External after Intermediate - for out-of-tree plugins</param>
		/// <returns>The output directory for intermediates</returns>
		public static string GetPlatformIntermediateFolder(UnrealTargetPlatform Platform, UnrealArchitectures? Architectures, bool External)
		{
			// now that we have the platform, we can set the intermediate path to include the platform/architecture name
			string FolderPath = Path.Combine("Intermediate", External ? "External" : String.Empty, "Build", Platform.ToString());
			if (Architectures != null)
			{
				FolderPath = Path.Combine(FolderPath, UnrealArchitectureConfig.ForPlatform(Platform).GetFolderNameForArchitectures(Architectures));
			}
			return FolderPath;
		}

		/// <summary>
		/// Adjusts a target name to be a usable intermediate folder name.
		/// </summary>
		/// <param name="TargetName">Base target name</param>
		/// <param name="IntermediateEnvironment">Intermediate environment to use</param>
		/// <returns></returns>
		public static string GetTargetIntermediateFolderName(string TargetName, UnrealIntermediateEnvironment IntermediateEnvironment)
		{
			string TargetFolderName = TargetName;
			switch (IntermediateEnvironment)
			{
				case UnrealIntermediateEnvironment.IWYU:
					TargetFolderName += "IWYU";
					break;
				case UnrealIntermediateEnvironment.NonUnity:
					TargetFolderName += "NU";
					break;
				case UnrealIntermediateEnvironment.Analyze:
					TargetFolderName += "SA";
					break;
				case UnrealIntermediateEnvironment.Query:
					TargetFolderName += "QRY";
					break;
				case UnrealIntermediateEnvironment.GenerateClangDatabase:
					TargetFolderName += "GCD";
					break;
				case UnrealIntermediateEnvironment.GenerateProjectFiles:
					TargetFolderName += "GPF";
					break;
				case UnrealIntermediateEnvironment.ProfilingCompile:
					TargetFolderName += "Prof";
					break;
			}
			return TargetFolderName;
		}

		/// <summary>
		/// Gets the app name for a given target type
		/// </summary>
		/// <param name="Type">The target type</param>
		/// <returns>The app name for this target type</returns>
		public static string GetAppNameForTargetType(TargetType Type)
		{
			switch (Type)
			{
				case TargetType.Game:
					return "UnrealGame";
				case TargetType.Client:
					return "UnrealClient";
				case TargetType.Server:
					return "UnrealServer";
				case TargetType.Editor:
					return "UnrealEditor";
				default:
					throw new BuildException("Invalid target type ({0})", (int)Type);
			}
		}

		public static string GetAppName(TargetRules Rules)
		{
			if (Rules.BuildEnvironment == TargetBuildEnvironment.Shared && Rules.Type != TargetType.Program)
			{
				return GetAppNameForTargetType(Rules.Type);
			}
			return Rules.Name;
		}

		/// <summary>
		/// Writes a list of all the externally referenced files required to use the precompiled data for this target
		/// </summary>
		/// <param name="Location">Path to the dependency list</param>
		/// <param name="RuntimeDependencies">List of all the runtime dependencies for this target</param>
		/// <param name="RuntimeDependencyTargetFileToSourceFile">Map of runtime dependencies to their location in the source tree, before they are staged</param>
		/// <param name="Logger">Logger for output</param>
		void WriteDependencyList(FileReference Location, List<RuntimeDependency> RuntimeDependencies, Dictionary<FileReference, FileReference> RuntimeDependencyTargetFileToSourceFile, ILogger Logger)
		{
			HashSet<FileReference> Files = new HashSet<FileReference>();

			// Find all the runtime dependency files in their original location
			foreach (RuntimeDependency RuntimeDependency in RuntimeDependencies)
			{
				FileReference? SourceFile;
				if (!RuntimeDependencyTargetFileToSourceFile.TryGetValue(RuntimeDependency.Path, out SourceFile))
				{
					SourceFile = RuntimeDependency.Path;
				}
				if (RuntimeDependency.Type != StagedFileType.DebugNonUFS || FileReference.Exists(SourceFile))
				{
					Files.Add(SourceFile);
				}
			}

			// Figure out all the modules referenced by this target. This includes all the modules that are referenced, not just the ones compiled into binaries.
			HashSet<UEBuildModule> Modules = new HashSet<UEBuildModule>();
			foreach (UEBuildBinary Binary in Binaries)
			{
				foreach (UEBuildModule Module in Binary.Modules)
				{
					Modules.Add(Module);
					Modules.UnionWith(Module.GetDependencies(true, true));
				}
			}

			// Get the platform we're building for
			UEBuildPlatform BuildPlatform = UEBuildPlatform.GetBuildPlatform(Platform);
			foreach (UEBuildModule Module in Modules)
			{
				// Skip artificial modules
				if (Module.RulesFile == null)
				{
					continue;
				}

				// Create the module rules
				ModuleRules Rules = CreateModuleRulesAndSetDefaults(Module.Name, "external file list option", Logger);

				// Add Additional Bundle Resources for all modules
				foreach (ModuleRules.BundleResource Resource in Rules.AdditionalBundleResources)
				{
					if (Directory.Exists(Resource.ResourcePath))
					{
						Files.UnionWith(DirectoryReference.EnumerateFiles(new DirectoryReference(Resource.ResourcePath!), "*", SearchOption.AllDirectories));
					}
					else
					{
						Files.Add(new FileReference(Resource.ResourcePath!));
					}
				}

				// Add any zip files from Additional Frameworks
				foreach (ModuleRules.Framework Framework in Rules.PublicAdditionalFrameworks)
				{
					if (Framework.IsZipFile())
					{
						Files.Add(FileReference.Combine(Module.ModuleDirectory, Framework.Path));
					}
				}

				// Add the rules file itself
				Files.Add(Rules.File);

				// Add the subclass rules
				if (Rules.SubclassRules != null)
				{
					foreach (string SubclassRule in Rules.SubclassRules)
					{
						Files.Add(new FileReference(SubclassRule));
					}
				}

				// Get a list of all the library paths
				List<string> LibraryPaths = new List<string>();
				LibraryPaths.Add(Directory.GetCurrentDirectory());

				List<string> SystemLibraryPaths = new List<string>();
				SystemLibraryPaths.Add(Directory.GetCurrentDirectory());
				SystemLibraryPaths.AddRange(Rules.PublicSystemLibraryPaths.Where(x => !x.StartsWith("$(", StringComparison.Ordinal)).Select(x => Path.GetFullPath(x.Replace('/', Path.DirectorySeparatorChar))));

				// Get all the extensions to look for
				List<string> LibraryExtensions = new List<string>();
				LibraryExtensions.Add(BuildPlatform.GetBinaryExtension(UEBuildBinaryType.StaticLibrary));
				LibraryExtensions.Add(BuildPlatform.GetBinaryExtension(UEBuildBinaryType.DynamicLinkLibrary));

				// Add all the libraries
				foreach (string LibraryExtension in LibraryExtensions)
				{
					foreach (string LibraryName in Rules.PublicAdditionalLibraries)
					{
						foreach (string LibraryPath in LibraryPaths)
						{
							ResolveLibraryName(LibraryPath, LibraryName, LibraryExtension, Files);
						}
					}

					foreach (string LibraryName in Rules.PublicSystemLibraryPaths)
					{
						foreach (string LibraryPath in SystemLibraryPaths)
						{
							ResolveLibraryName(LibraryPath, LibraryName, LibraryExtension, Files);
						}
					}
				}

				// Find all the include paths
				List<string> AllIncludePaths = new List<string>();
				AllIncludePaths.AddRange(Rules.PublicIncludePaths);
				AllIncludePaths.AddRange(Rules.PublicSystemIncludePaths);

				// Add all the include paths
				foreach (string IncludePath in AllIncludePaths.Where(x => !x.StartsWith("$(", StringComparison.Ordinal)))
				{
					if (Directory.Exists(IncludePath))
					{
						foreach (string IncludeFileName in Directory.EnumerateFiles(IncludePath, "*", SearchOption.AllDirectories))
						{
							string Extension = Path.GetExtension(IncludeFileName).ToUpperInvariant();
							if (Extension == ".H" || Extension == ".INL" || Extension == ".HPP" || Extension == ".IPP" || String.IsNullOrEmpty(Extension))
							{
								Files.Add(new FileReference(IncludeFileName));
							}
						}
					}
				}
			}

			// Write the file
			Logger.LogInformation("Writing dependency list to {Location}", Location);
			DirectoryReference.CreateDirectory(Location.Directory);
			FileReference.WriteAllLines(Location, Files.Where(x => x.IsUnderDirectory(Unreal.RootDirectory)).Select(x => x.MakeRelativeTo(Unreal.RootDirectory).Replace(Path.DirectorySeparatorChar, '/')).OrderBy(x => x));
		}

		private static void ResolveLibraryName(string LibraryPath, string LibraryName, string LibraryExtension, HashSet<FileReference> Files)
		{
			string LibraryFileName = Path.Combine(LibraryPath, LibraryName);
			if (File.Exists(LibraryFileName))
			{
				Files.Add(new FileReference(LibraryFileName));
			}

			if (LibraryName.IndexOfAny(new char[] { Path.DirectorySeparatorChar, Path.AltDirectorySeparatorChar }) == -1)
			{
				string UnixLibraryFileName = Path.Combine(LibraryPath, "lib" + LibraryName + LibraryExtension);
				if (File.Exists(UnixLibraryFileName))
				{
					Files.Add(new FileReference(UnixLibraryFileName));
				}
			}
		}
		/// <summary>
		/// Generates a public manifest file for writing out
		/// </summary>
		public void GenerateManifest(FileReference ManifestPath, List<KeyValuePair<FileReference, BuildProductType>> BuildProducts, ILogger Logger)
		{
			BuildManifest Manifest = new BuildManifest();

			// Add the regular build products
			foreach (KeyValuePair<FileReference, BuildProductType> BuildProductPair in BuildProducts)
			{
				Manifest.BuildProducts.Add(BuildProductPair.Key.FullName);
			}

			// Add all the dependency lists
			foreach (FileReference DependencyListFileName in Rules.DependencyListFileNames)
			{
				Manifest.BuildProducts.Add(DependencyListFileName.FullName);
			}

			if (!Rules.bDisableLinking)
			{
				Manifest.AddBuildProduct(ReceiptFileName.FullName);

				if (bDeployAfterCompile)
				{
					Manifest.DeployTargetFiles.Add(ReceiptFileName.FullName);
				}
			}

			// Remove anything that's not part of the plugin
			if (ForeignPlugin != null)
			{
				DirectoryReference ForeignPluginDir = ForeignPlugin.Directory;
				Manifest.BuildProducts.RemoveAll(x => !new FileReference(x).IsUnderDirectory(ForeignPluginDir));
			}

			// Completely ignore build outputs if requested
			if (Rules.bIgnoreBuildOutputs)
			{
				Manifest.BuildProducts.Clear();
				Manifest.DeployTargetFiles.Clear();
			}

			Manifest.BuildProducts.Sort();
			Manifest.DeployTargetFiles.Sort();

			Logger.LogInformation("Writing manifest to {ManifestPath}", ManifestPath);
			Utils.WriteClass<BuildManifest>(Manifest, ManifestPath.FullName, "", Logger);
		}

		/// <summary>
		/// Combines multiple manifests into one. Target manifest can be included in the merge sources
		/// </summary>
		internal static void CombineManifests(FileReference TargetManifestPath, IEnumerable<FileReference> SourceManifestPaths, ILogger Logger)
		{
			HashSet<string> BuildProducts = [];
			HashSet<string> DeployTargetFiles = [];

			foreach (FileReference SourceManifestPath in SourceManifestPaths)
			{
				BuildManifest SourceManifest = Utils.ReadClass<BuildManifest>(SourceManifestPath.FullName, Logger);
				if (SourceManifest != null)
				{
					BuildProducts.AddRange(SourceManifest.BuildProducts);
					DeployTargetFiles.AddRange(SourceManifest.DeployTargetFiles);
				}
			}

			BuildManifest Manifest = new BuildManifest();
			Manifest.BuildProducts.AddRange(BuildProducts);
			Manifest.DeployTargetFiles.AddRange(DeployTargetFiles);

			Logger.LogInformation("Writing merged manifest to {ManifestPath}", TargetManifestPath);
			Utils.WriteClass<BuildManifest>(Manifest, TargetManifestPath.FullName, "", Logger);
		}

		/// <summary>
		/// Prepare all the module manifests for this target
		/// </summary>
		/// <returns>Dictionary mapping from filename to module manifest</returns>
		Dictionary<FileReference, ModuleManifest> PrepareModuleManifests()
		{
			Dictionary<FileReference, ModuleManifest> FileNameToModuleManifest = new Dictionary<FileReference, ModuleManifest>();
			if (!bCompileMonolithic)
			{
				// Create the receipts for each folder
				foreach (UEBuildBinary Binary in Binaries)
				{
					if (Binary.Type == UEBuildBinaryType.DynamicLinkLibrary)
					{
						DirectoryReference DirectoryName = Binary.OutputFilePath.Directory;
						bool bIsGameBinary = Binary.PrimaryModule.Rules.Context.bCanBuildDebugGame;
						FileReference ManifestFileName = FileReference.Combine(DirectoryName, ModuleManifest.GetStandardFileName(AppName, Platform, Configuration, Architectures, bIsGameBinary));

						ModuleManifest? Manifest;
						if (!FileNameToModuleManifest.TryGetValue(ManifestFileName, out Manifest))
						{
							Manifest = new ModuleManifest("");
							FileNameToModuleManifest.Add(ManifestFileName, Manifest);
						}

						foreach (UEBuildModuleCPP Module in Binary.Modules.OfType<UEBuildModuleCPP>())
						{
							Manifest.ModuleNameToFileName[Module.Name] = Binary.OutputFilePath.GetFileName();
						}

						// In order to correctly lookup shared merged library dependencies at runtime, use decorated filenames
						string GetFilenameFromLibraryName(string LibraryName)
						{
							string FilenameFragment = $"{AppName}{Rules.DecoratedSeparator}{LibraryName}";
							IEnumerable<UEBuildBinary> Candidates = Binaries.Where(x =>
								x.Type == UEBuildBinaryType.DynamicLinkLibrary && x.OutputFilePath.GetFileNameWithoutAnyExtensions().Contains(FilenameFragment, StringComparison.Ordinal));
							return Candidates.FirstOrDefault()?.OutputFilePath.GetFileName().ToString() ?? LibraryName;
						}

						foreach (string Root in Rules.MergePlugins)
						{
							string RootFilename = GetFilenameFromLibraryName(Root);
							if (!Manifest.LibraryDependencies.ContainsKey(RootFilename))
							{
								Manifest.LibraryDependencies.Add(RootFilename, new());
							}
						}

						foreach (KeyValuePair<string, IEnumerable<string>> SharedGroupRule in Rules.MergePluginsShared)
						{
							foreach (string Root in SharedGroupRule.Value)
							{
								string RootFilename = GetFilenameFromLibraryName(Root);
								if (Manifest.LibraryDependencies.ContainsKey(RootFilename))
								{
									Manifest.LibraryDependencies[RootFilename].Add(GetFilenameFromLibraryName(SharedGroupRule.Key));
								}
							}
						}
					}
				}
			}
			return FileNameToModuleManifest;
		}

		Dictionary<FileReference, LoadOrderManifest> PrepareLoadOrderManifests(ILogger Logger)
		{
			Dictionary<FileReference, LoadOrderManifest> FileNameToManifest = new Dictionary<FileReference, LoadOrderManifest>();

			UEBuildPlatform BuildPlatform = UEBuildPlatform.GetBuildPlatform(Platform);

			if (!bCompileMonolithic && BuildPlatform.RequiresLoadOrderManifest(Rules))
			{
				string ManifestFileName = LoadOrderManifest.GetStandardFileName(AppName, Platform, Configuration, Architectures);

				// Create a map module->binary for all primary modules.
				Dictionary<UEBuildModule, UEBuildBinary> PrimaryModules =
					Binaries
						.Where(x => x.Type == UEBuildBinaryType.DynamicLinkLibrary)
						.ToDictionary(x => x.PrimaryModule as UEBuildModule, x => x);

				List<Tuple<UEBuildModule, UEBuildModule>> Dependencies = new List<Tuple<UEBuildModule, UEBuildModule>>();

				// Gather information about all dependencies between primary modules (binaries).
				foreach (KeyValuePair<UEBuildModule, UEBuildBinary> ModuleInfo in PrimaryModules)
				{
					// Combine Private and Public dependencies and add a graph edge for each that is a primary module of some binary
					// i.e. we care only about dependencies between primary modules.
					// We remove Circular dependencies because we need to have a directed acyclic graph to be able to sort it.
					// The convention seems to be that if we have
					//     A -> B and B -> A (Circular: B)
					// B -> A is considered secondary i.e. less important and we choose to remove this one.
					(ModuleInfo.Key.PrivateDependencyModules ?? new List<UEBuildModule>())
						.Union(ModuleInfo.Key.PublicDependencyModules ?? new List<UEBuildModule>())
						.Where(x => PrimaryModules.ContainsKey(x) && !ModuleInfo.Key.Rules.CircularlyReferencedDependentModules.Contains(x.Name))
						.ToList()
						.ForEach(x => Dependencies.Add(new Tuple<UEBuildModule, UEBuildModule>(x, ModuleInfo.Key)));
				}

				// Sort the dependencies to generate a sequence of libraries to load respecting all the given dependencies.
				TopologicalSorter<UEBuildModule> Sorter = new TopologicalSorter<UEBuildModule>(Dependencies);
				Sorter.CycleHandling = TopologicalSorter<UEBuildModule>.CycleMode.BreakWithInfo;
				Sorter.Logger = Logger;
				Sorter.NodeToString = Module => Module.Name;

				if (!Sorter.Sort())
				{
					Logger.LogError("Failed to generate {ManifestFileName}: Couldn't sort dynamic modules in a way that would respect all dependencies (probably circular dependencies)", ManifestFileName);

					return FileNameToManifest;
				}

				// Create the load order manifest and fill the list of all binaries.
				LoadOrderManifest Manifest = new LoadOrderManifest();

				foreach (UEBuildModule Module in Sorter.GetResult())
				{
					UEBuildBinary? Binary;

					PrimaryModules.TryGetValue(Module, out Binary);

					Manifest.Libraries.Add(
						Path.GetRelativePath(GetExecutableDir().ToString(), Binary!.OutputFilePath.ToString()).Replace(Path.DirectorySeparatorChar, '/')
						);
				}

				FileReference ManifestPath = FileReference.Combine(GetExecutableDir(), ManifestFileName);

				FileNameToManifest.Add(ManifestPath, Manifest);
			}
			return FileNameToManifest;
		}

		/// <summary>
		/// Prepare all the receipts this target (all the .target and .modules files). See the VersionManifest class for an explanation of what these files are.
		/// </summary>
		/// <param name="ToolChain">The toolchain used to build the target</param>
		/// <param name="BuildProducts">Artifacts from the build</param>
		/// <param name="RuntimeDependencies">Output runtime dependencies</param>
		TargetReceipt PrepareReceipt(UEToolChain ToolChain, List<KeyValuePair<FileReference, BuildProductType>> BuildProducts, List<RuntimeDependency> RuntimeDependencies)
		{
			// Read the version file
			BuildVersion? Version;
			if (!BuildVersion.TryRead(BuildVersion.GetDefaultFileName(), out Version))
			{
				Version = new BuildVersion();
			}

			// Create a unique identifier for this build which can be used to identify modules which are compatible. It's fine to share this between runs with the same makefile.
			// By default we leave it blank when compiling a subset of modules (for hot reload, etc...), otherwise it won't match anything else. When writing to a directory
			// that already contains a manifest, we'll reuse the build id that's already in there (see below).
			if (String.IsNullOrEmpty(Version.BuildId))
			{
				if (Rules.bFormalBuild)
				{
					// If this is a formal build, we can just the compatible changelist as the unique id.
					Version.BuildId = String.Format("{0}", Version.EffectiveCompatibleChangelist);
				}
			}

			// If this is an installed engine build, clear the promoted flag on the output binaries. This will ensure we will rebuild them.
			if (Version.IsPromotedBuild && Unreal.IsEngineInstalled())
			{
				Version.IsPromotedBuild = false;
			}

			// Create the receipt
			TargetReceipt Receipt = new TargetReceipt(ProjectFile, TargetName, TargetType, Platform, Configuration, Version, Architectures, Rules.IsTestTarget, Rules.DefaultBuildSettings, Rules.BuildEnvironment);

			if (BuildProducts.Count == 0 && RuntimeDependencies.Count == 0)
			{
				return Receipt;
			}

			if (!Rules.bShouldCompileAsDLL)
			{
				// Set the launch executable if there is one
				foreach (KeyValuePair<FileReference, BuildProductType> Pair in BuildProducts)
				{
					if (Pair.Value == BuildProductType.Executable)
					{
						if (System.IO.Path.GetFileNameWithoutExtension(Pair.Key.FullName).EndsWith("-Cmd", System.StringComparison.Ordinal) && Receipt.LaunchCmd == null)
						{
							Receipt.LaunchCmd = Pair.Key;
						}
						else if (Receipt.Launch == null)
						{
							Receipt.Launch = Pair.Key;
						}

						if (Receipt.Launch != null && Receipt.LaunchCmd != null)
						{
							break;
						}
					}
				}
			}
			else
			{
				Receipt.AdditionalProperties.Add(new ReceiptProperty("CompileAsDll", "true"));
			}

			// Find all the build products and modules from this binary
			foreach (KeyValuePair<FileReference, BuildProductType> BuildProductPair in BuildProducts)
			{
				if (BuildProductPair.Value != BuildProductType.BuildResource)
				{
					Receipt.AddBuildProduct(BuildProductPair.Key, BuildProductPair.Value);
				}
			}

			// Add the project file
			if (ProjectFile != null)
			{
				Receipt.RuntimeDependencies.Add(ProjectFile, StagedFileType.UFS);
			}

			// Add the descriptors for all enabled plugins
			List<UEBuildPlugin>? RuntimeDependencyPluginsList = Rules.bRuntimeDependenciesComeFromBuildPlugins ? BuildPlugins : EnabledPlugins;
			foreach (UEBuildPlugin EnabledPlugin in RuntimeDependencyPluginsList!)
			{
				if (EnabledPlugin.bDescriptorNeededAtRuntime || EnabledPlugin.bDescriptorReferencedExplicitly)
				{
					Receipt.RuntimeDependencies.Add(EnabledPlugin.File, StagedFileType.UFS);

					// Only add child plugins that are named for the current Platform or Groups that it's part of
					if (EnabledPlugin.ChildFiles.Count > 0)
					{
						List<string> ValidFileNames = new List<string>();
						ValidFileNames.Add(EnabledPlugin.Name + "_" + Platform.ToString());

						foreach (UnrealPlatformGroup Group in UnrealPlatformGroup.GetValidGroups())
						{
							if (UEBuildPlatform.IsPlatformInGroup(Platform, Group))
							{
								ValidFileNames.Add(EnabledPlugin.Name + "_" + Group.ToString());
							}
						}

						foreach (FileReference ChildFile in EnabledPlugin.ChildFiles)
						{
							if (ValidFileNames.Contains(ChildFile.GetFileNameWithoutExtension(), StringComparer.InvariantCultureIgnoreCase))
							{
								Receipt.RuntimeDependencies.Add(ChildFile, StagedFileType.UFS);
							}
						}
					}
				}
			}

			// Add all the other runtime dependencies
			HashSet<FileReference> UniqueRuntimeDependencyFiles = new HashSet<FileReference>();
			foreach (RuntimeDependency RuntimeDependency in RuntimeDependencies)
			{
				if (UniqueRuntimeDependencyFiles.Add(RuntimeDependency.Path))
				{
					Receipt.RuntimeDependencies.Add(RuntimeDependency);
				}
			}

			// Add Rules-enabled and disabled plugins
			foreach (string EnabledPluginName in Rules.EnablePlugins)
			{
				Receipt.PluginNameToEnabledState[EnabledPluginName] = true;
			}
			foreach (string DisabledPluginName in Rules.DisablePlugins)
			{
				Receipt.PluginNameToEnabledState[DisabledPluginName] = false;
			}

			// Add all the build plugin names
			foreach (UEBuildPlugin Plugin in BuildPlugins!)
			{
				Receipt.BuildPlugins.Add(Plugin.Name);
			}

			if (!Rules.bAllowEnginePluginsEnabledByDefault)
			{
				Receipt.AdditionalProperties.Add(new ReceiptProperty("bAllowEnginePluginsEnabledByDefault", "false"));
			}

			// Find all the modules which are part of this target
			HashSet<UEBuildModule> UniqueLinkedModules = new HashSet<UEBuildModule>();
			foreach (UEBuildBinary Binary in Binaries)
			{
				foreach (UEBuildModule Module in Binary.Modules)
				{
					if (UniqueLinkedModules.Add(Module))
					{
						Receipt.AdditionalProperties.AddRange(Module.Rules.AdditionalPropertiesForReceipt.Inner);
					}
				}
			}

			// add the SDK used by the tool chain
			Receipt.AdditionalProperties.Add(new ReceiptProperty("SDK", ToolChain.GetSDKVersion()));

			if (!String.IsNullOrEmpty(Rules.CustomConfig))
			{
				// Pass through the custom config string
				Receipt.AdditionalProperties.Add(new ReceiptProperty("CustomConfig", Rules.CustomConfig));
			}

			ToolChain.ModifyTargetReceipt(Rules, Receipt);

			return Receipt;
		}

		/// <summary>
		/// Writes out a partial <see cref="TargetReceipt"/>. This contains a minimal set of properties.
		/// </summary>
		/// <param name="Descriptor">The target descriptor to use in target receipt construction.</param>
		/// <param name="RulesObject">The target rules to use in target receipt construction.</param>
		/// <param name="RulesAssembly">The rules assembly to use in target receipt construction.</param>
		/// <param name="Logger">Logger for error and info emission.</param>
		/// <remarks>This should be used to emit minimal target data in failure circumstances.</remarks>
		static TargetReceipt? PrepareAndWritePartialReceipt(TargetDescriptor Descriptor, TargetRules RulesObject, RulesAssembly RulesAssembly, ILogger Logger)
		{
			try
			{
				UEBuildTarget incompleteTarget = new UEBuildTarget(Descriptor, new ReadOnlyTargetRules(RulesObject), RulesAssembly);
				UEToolChain targetToolChain = incompleteTarget.CreateToolchain(incompleteTarget.Platform, Logger);
				TargetReceipt incompleteTargetReceipt = incompleteTarget.PrepareReceipt(targetToolChain, [], []);

				if (incompleteTargetReceipt != null)
				{
					FileReference receipt = incompleteTarget.ReceiptFileName;
					DirectoryReference.CreateDirectory(receipt.Directory);
					incompleteTargetReceipt.Write(receipt);

					Logger.LogInformation("Wrote partial receipt to {ReceiptPath}", receipt);
				}

				return incompleteTargetReceipt;
			}
			catch (Exception innerEx)
			{
				Logger.LogWarning("Failed to write partial receipt for target {Descriptor}: {Message}", Descriptor.Name, innerEx.Message);
				Logger.LogDebug("{InnerException}", innerEx);

				return null;
			}
		}

		/// <summary>
		/// Gathers dependency modules for given binaries list.
		/// </summary>
		/// <param name="Binaries">Binaries list.</param>
		/// <returns>Dependency modules set.</returns>
		static HashSet<UEBuildModuleCPP> GatherDependencyModules(List<UEBuildBinary> Binaries)
		{
			ConcurrentBag<UEBuildModuleCPP> Bag = new();
			Parallel2.ForEach(Binaries, Binary =>
			{
				IEnumerable<UEBuildModule> DependencyModules = Binary.GetAllDependencyModules(bIncludeDynamicallyLoaded: false, bForceCircular: false);
				foreach (UEBuildModuleCPP Module in DependencyModules.OfType<UEBuildModuleCPP>())
				{
					if (Module.Binary != null)
					{
						Bag.Add(Module);
					}
				}
			});
			return Bag.ToHashSet();
		}

		/// <summary>
		/// Creates a global compile environment suitable for generating project files.
		/// </summary>
		/// <returns>New compile environment</returns>
		public CppCompileEnvironment CreateCompileEnvironmentForProjectFiles(ILogger Logger, NullActionGraphBuilder? builder = null)
		{
			CppConfiguration CppConfiguration = GetCppConfiguration(Configuration);

			SourceFileMetadataCache MetadataCache = SourceFileMetadataCache.CreateHierarchy(ProjectFile, Logger);

			CppCompileEnvironment GlobalCompileEnvironment = new CppCompileEnvironment(Platform, CppConfiguration, Architectures, MetadataCache);
			LinkEnvironment GlobalLinkEnvironment = new LinkEnvironment(GlobalCompileEnvironment.Platform, GlobalCompileEnvironment.Configuration, GlobalCompileEnvironment.Architectures);

			UEToolChain TargetToolChain = CreateToolchain(Platform, Logger);
			SetupGlobalEnvironment(TargetToolChain, GlobalCompileEnvironment, GlobalLinkEnvironment);

			FindSharedPCHs(Binaries, GlobalCompileEnvironment, Logger);

			builder ??= new NullActionGraphBuilder(Logger);
			CreateSharedPCHInstances(Rules, TargetToolChain, Binaries, GlobalCompileEnvironment, builder, Logger);

			return GlobalCompileEnvironment;
		}

		/// <summary>
		/// Builds the target, appending list of output files and returns building result.
		/// </summary>
		public async Task<TargetMakefile> BuildAsync(BuildConfiguration BuildConfiguration, ISourceFileWorkingSet WorkingSet, TargetDescriptor TargetDescriptor, ILogger Logger, bool bInitOnly = false)
		{
			await Task.CompletedTask;
			(TargetMakefile Makefile, Task? UHTTimestampTask) = BuildMakefile(BuildConfiguration, WorkingSet, TargetDescriptor, Logger, bInitOnly);
			if (UHTTimestampTask is not null)
			{
				await UHTTimestampTask;
			}
			return Makefile;

		}

		public (TargetMakefile, Task?) BuildMakefile(BuildConfiguration BuildConfiguration, ISourceFileWorkingSet WorkingSet, TargetDescriptor TargetDescriptor, ILogger Logger, bool bInitOnly = false)
		{
			SourceFileMetadataCache cache = SourceFileMetadataCache.CreateHierarchy(ProjectFile, Logger);
			BuildContext context = BuildMakefile1(BuildConfiguration, TargetDescriptor, Logger, cache);

			// UHT mode uses this to create the makefile at least to the point where UHT would have valid manifest information
			if (bInitOnly)
			{
				return (context.Makefile, null);
			}

			CodeGenContext codeGenContext = new CodeGenContext()
			{
				Descriptor = TargetDescriptor,
				Makefile = context.Makefile,
				TargetName = TargetName,
				GenerateHeaderFuncs = context.GenerateHeaderFuncs,
			};

			RunCodeGen(BuildConfiguration, [codeGenContext], Logger);

			BuildMakefile2(WorkingSet, Logger, context, new(false, true));

			codeGenContext.uhtRun?.Wait();

			return (context.Makefile, codeGenContext.uhtTimestamp);
		}

		internal struct GenerateHeaderFunc
		{
			internal DirectoryReference Dir;
			internal Action<ILogger, DirectoryReference> Func;
		}

		internal class BuildContext
		{
			internal required SourceFileMetadataCache MetadataCache;
			internal required TargetDescriptor Descriptor;
			internal required TargetMakefile Makefile;
			internal required List<UEBuildBinary> OriginalBinaries;
			internal required HashSet<UEBuildModuleCPP> ModulesToGenerateHeadersFor;
			internal required CppCompileEnvironment GlobalCompileEnvironment;
			internal required UEToolChain TargetToolChain;
			internal required LinkEnvironment GlobalLinkEnvironment;
			internal required List<GenerateHeaderFunc> GenerateHeaderFuncs;
		}

		internal BuildContext BuildMakefile1(BuildConfiguration BuildConfiguration, TargetDescriptor TargetDescriptor, ILogger Logger, SourceFileMetadataCache MetadataCache)
		{
			CppConfiguration CppConfiguration = GetCppConfiguration(Configuration);
			CppCompileEnvironment GlobalCompileEnvironment = new CppCompileEnvironment(Platform, CppConfiguration, Architectures, MetadataCache);
			LinkEnvironment GlobalLinkEnvironment = new LinkEnvironment(GlobalCompileEnvironment.Platform, GlobalCompileEnvironment.Configuration, GlobalCompileEnvironment.Architectures);

			UEToolChain TargetToolChain = CreateToolchain(Platform, Logger);
			TargetToolChain.SetEnvironmentVariables();
			SetupGlobalEnvironment(TargetToolChain, GlobalCompileEnvironment, GlobalLinkEnvironment);

			// Save off the original list of binaries. We'll use this to figure out which PCHs to create later, to avoid switching PCHs when compiling single modules.
			List<UEBuildBinary> OriginalBinaries = Binaries;

			// For installed builds, filter out all the binaries that aren't in mods
			if (Unreal.IsProjectInstalled())
			{
				List<DirectoryReference> ModDirectories = EnabledPlugins!.Where(x => x.Type == PluginType.Mod).Select(x => x.Directory).ToList();

				List<UEBuildBinary> FilteredBinaries = new List<UEBuildBinary>();
				foreach (UEBuildBinary DLLBinary in Binaries)
				{
					if (ModDirectories.Any(x => DLLBinary.OutputFilePath.IsUnderDirectory(x)))
					{
						FilteredBinaries.Add(DLLBinary);
					}
				}
				Binaries = FilteredBinaries;

				if (Binaries.Count == 0)
				{
					throw new BuildException("No modules found to build. All requested binaries were already part of the installed data.");
				}
			}

			// Create the makefile
			string ExternalMetadata = UEBuildPlatform.GetBuildPlatform(Platform).GetExternalBuildMetadata(ProjectFile);
			TargetMakefile Makefile = new TargetMakefile(ExternalMetadata, Binaries[0].OutputFilePaths[0], ReceiptFileName,
				ProjectIntermediateDirectory, ProjectIntermediateDirectoryNoArch, TargetType,
				Rules.ConfigValueTracker, bDeployAfterCompile, UbtPlugins?.ToArray(), EnabledUbtPlugins?.ToArray(),
				EnabledUhtPlugins?.ToArray());
			Makefile.IsTestTarget = Rules.IsTestTarget;

			// Get diagnostic info to be printed before each build
			TargetToolChain.GetVersionInfo(Makefile.Diagnostics);
			Rules.GetBuildSettingsInfo(Makefile.Diagnostics);

			// Mark dependencies to other target rules files, as they can be referenced by this makefile
			Makefile.InternalDependencies.UnionWith(Rules.TargetFiles.Select(x => FileItem.GetItemByFileReference(x)));

			// Get any pre-build targets.
			Makefile.PreBuildTargets = Rules.PreBuildTargets.ToArray();

			// Setup the hot reload module list
			Makefile.HotReloadModuleNames = GetHotReloadModuleNames(TargetDescriptor);

			// If we're compiling monolithic, make sure the executable knows about all referenced modules
			if (ShouldCompileMonolithic())
			{
				UEBuildBinary ExecutableBinary = Binaries[0];

				// Add all the modules that the executable depends on. Plugins will be already included in this list.
				IEnumerable<UEBuildModule> AllReferencedModules = ExecutableBinary.GetAllDependencyModules(bIncludeDynamicallyLoaded: true, bForceCircular: true);
				foreach (UEBuildModule CurModule in AllReferencedModules)
				{
					if (CurModule.Binary == null || CurModule.Binary == ExecutableBinary || CurModule.Binary.Type == UEBuildBinaryType.StaticLibrary)
					{
						ExecutableBinary.AddModule(CurModule);
					}
				}
			}

			// On Mac and Linux we have actions that should be executed after all the binaries are created
			TargetToolChain.SetupBundleDependencies(Rules, Binaries, TargetName);

			// Gather modules we might want to generate headers for
			HashSet<UEBuildModuleCPP> ModulesToGenerateHeadersFor = GatherDependencyModules(OriginalBinaries.ToList());

			// Cache module dependencies for StableTopologicalSort
			UEBuildModule.CacheDependenciesForStableTopologicalSort(ModulesToGenerateHeadersFor);

			// Prepare cached data for UHT header generation
			using (GlobalTracer.Instance.BuildSpan("UHTExecution.SetupUObjectModules()").StartActive())
			{
				using ITimelineEvent _ = Timeline.ScopeEvent("UHTModules");

				UHTExecution.SetupUObjectModules(
					ModulesToGenerateHeadersFor,
					Rules.Platform,
					ProjectDescriptor,
					Makefile.UObjectModules,
					Makefile.UObjectModuleHeaders,
					Makefile.ExecutableFile.Directory,
					Rules.GeneratedCodeVersion,
					MetadataCache);
			}

			if (Rules.NativePointerMemberBehaviorOverride != null)
			{
				Makefile.UHTAdditionalArguments = new string[]
				{
					$"-ini:Engine:[UnrealHeaderTool]:EngineNativePointerMemberBehavior={Rules.NativePointerMemberBehaviorOverride}",
					$"-ini:Engine:[UnrealHeaderTool]:EnginePluginNativePointerMemberBehavior={Rules.NativePointerMemberBehaviorOverride}",
					$"-ini:Engine:[UnrealHeaderTool]:NonEngineNativePointerMemberBehavior={Rules.NativePointerMemberBehaviorOverride}",
				};
			}

			if (Rules.bEnableConstInitUObject)
			{
				Makefile.UHTTargetSettings.CompiledInObjectFormat = UhtCompiledInObjectFormat.ConstInit;
				// Generate max name/value pairs for UENUMs in the compiled-in data
				Makefile.UHTTargetSettings.GenerateEnumMaxValues = true;
			}

			bool bEnabledUHTInputCache = BuildConfiguration.bEnableUHTInputCache || Unreal.IsBuildMachine();
			Makefile.UHTTargetSettings.EnableInputCacheRead = bEnabledUHTInputCache && !BuildConfiguration.bForceHeaderGeneration;
			Makefile.UHTTargetSettings.EnableInputCacheWrite = bEnabledUHTInputCache;

#if __VPROJECT_AVAILABLE__
			{
				using ITimelineEvent _ = Timeline.ScopeEvent("VNIModules");
				VNIExecution.SetupVNIModules(ModulesToGenerateHeadersFor, out Makefile.VNIModules);

				// Copy Verse BPVM usage flag
				Makefile.bUseVerseBPVM = Rules.bUseVerseBPVM;
			}
#endif

			// Collect generate header funcs
			List<GenerateHeaderFunc> GenerateHeaderFuncs = [];
			foreach (UEBuildModuleCPP Module in Modules.Values.OfType<UEBuildModuleCPP>())
			{
				foreach ((string Subdirectory, Action<ILogger, DirectoryReference> Func) in Module.Rules.GenerateHeaderFuncs)
				{
					DirectoryReference GenDirectory = DirectoryReference.Combine(Module.GeneratedCodeDirectory!, Subdirectory);
					GenerateHeaderFuncs.Add(new() { Dir = GenDirectory, Func = Func });
				}
			}

			return new BuildContext()
			{
				MetadataCache = MetadataCache,
				Descriptor = TargetDescriptor,
				Makefile = Makefile,
				OriginalBinaries = OriginalBinaries,
				ModulesToGenerateHeadersFor = ModulesToGenerateHeadersFor,
				GlobalCompileEnvironment = GlobalCompileEnvironment,
				TargetToolChain = TargetToolChain,
				GlobalLinkEnvironment = GlobalLinkEnvironment,
				GenerateHeaderFuncs = GenerateHeaderFuncs,
			};
		}

		internal class CodeGenContext
		{
			internal required TargetDescriptor Descriptor;
			internal required TargetMakefile Makefile;
			internal required string TargetName;
			internal required List<GenerateHeaderFunc> GenerateHeaderFuncs;
			internal Task? uhtRun;
			internal Task? uhtTimestamp;
			internal Task? vniRun;
		}

		internal static void RunCodeGen(BuildConfiguration BuildConfiguration, List<CodeGenContext> contexts, ILogger Logger)
		{
			if (contexts.Count == 0)
			{
				return;
			}

			CodeGenContext? lastRunContext = null;
			Dictionary<string, UHTModuleInfo>? lastRunUObjectModules = null;

			string GetTargetName(TargetDescriptor desc) => $"{desc.Name}-{desc.Platform}-{desc.Configuration}";

			foreach (CodeGenContext context in contexts)
			{
				TargetMakefile makefile = context.Makefile;

				foreach (GenerateHeaderFunc entry in context.GenerateHeaderFuncs)
				{
					DirectoryReference.CreateDirectory(entry.Dir);
					entry.Func(Logger, entry.Dir);
				}

				if (lastRunContext != null)
				{
					bool rerunCodeGen = false;

					if (context.TargetName != lastRunContext.TargetName)
					{
						Logger.LogDebug("Different targetname!.. will rerun code gen!");
						rerunCodeGen = true;
					}

					if (!rerunCodeGen)
					{
						if (lastRunUObjectModules == null)
						{
							lastRunUObjectModules = lastRunContext.Makefile.UObjectModules.ToDictionary(k => k.ModuleName, k => k);
						}

						foreach (UHTModuleInfo module in makefile.UObjectModules)
						{
							if (lastRunUObjectModules.TryGetValue(module.ModuleName, out UHTModuleInfo? lastModule))
							{
								int CountHeaders(UHTModuleInfo info) => info.PublicUObjectClassesHeaders.Count + info.PublicUObjectHeaders.Count + info.InternalUObjectHeaders.Count + info.PrivateUObjectHeaders.Count;
								if (CountHeaders(module) != CountHeaders(lastModule))
								{
									Logger.LogDebug(
										"Target {CurrentTarget} module {Module} has a different number of headers than previous target {PreviousTarget}, will rerun code gen",
										GetTargetName(context.Descriptor),
										module.ModuleName,
										GetTargetName(lastRunContext.Descriptor));
									rerunCodeGen = true;
									break;
								}
							}
							else
							{
								Logger.LogDebug(
									"Target {CurrentTarget} has uht module {Module} that previous target {PreviousTarget} did not have, will rerun code gen",
									GetTargetName(context.Descriptor),
									module.ModuleName,
									GetTargetName(lastRunContext.Descriptor));
								rerunCodeGen = true;
								break;
							}
						}
					}

#if __VPROJECT_AVAILABLE__
					if (!rerunCodeGen)
					{
						HashSet<string>? lastRunVNIModules = new(lastRunContext.Makefile.VNIModules.Select(m => m.ModuleName));

						foreach (VNIModuleInfo module in makefile.VNIModules)
						{
							if (!lastRunVNIModules.Contains(module.ModuleName))
							{
								Logger.LogDebug(
									"Target {CurrentTarget} has vni modules that previous target {PreviousTarget} did not have, will rerun code gen",
									GetTargetName(context.Descriptor),
									GetTargetName(lastRunContext.Descriptor));
								rerunCodeGen = true;
								break;
							}
						}
					}
#endif
					if (!rerunCodeGen)
					{
						continue;
					}
				}

				// This should be very rare to rerun code gen.. can be reproed with having for example both these being built in one go:
				// -Target="UnrealGame Win64 Development"
				// -Target="UnrealGame Win64 DebugGame -allmodules"

				if (lastRunContext != null)
				{
					lastRunContext.uhtRun?.Wait();
					lastRunContext.uhtTimestamp?.Wait();
#if __VPROJECT_AVAILABLE__
					lastRunContext.vniRun?.Wait();
#endif
				}

				TargetDescriptor descriptor = context.Descriptor;

				if (makefile.UObjectModules.Count > 0)
				{
					// Because UHT execution runs alongside building the rest of the makefile, I suspect we need to pre-populate the cache
					// for the makefile generation to correctly pick up source files.
					UHTExecution.PrepopulateDirectoryCache(makefile.UObjectModules);
					(Task? uhtRun, Task? uhtTimestamp) = UHTExecution.ExecuteHeaderToolIfNecessary(BuildConfiguration, descriptor.ProjectFile, makefile, context.TargetName!, Logger);
					context.uhtRun = uhtRun;
					context.uhtTimestamp = uhtTimestamp;
				}

#if __VPROJECT_AVAILABLE__
				if (makefile.VNIModules.Count > 0)
				{
					VNIExecution.PrepopulateDirectoryCache(makefile);
					context.vniRun = Task.Run(() => VNIExecution.ExecuteVNITool(makefile, descriptor, Logger));
				}
#endif

				lastRunContext = context;
			}
		}

		internal static void WaitCodeGen(List<CodeGenContext> contexts, bool prefetchAfterReset)
		{
			foreach (CodeGenContext context in contexts)
			{
				if (context.uhtRun != null)
				{
					UHTExecution.WaitForExecution(context.uhtRun, context.Makefile);

					if (prefetchAfterReset)
					{
						Task task = FileMetadataPrefetch.QueueDirectoryTrees(context.Makefile.UObjectModules.Select(m => m.GeneratedCodeDirectory));
					}
				}
#if __VPROJECT_AVAILABLE__
				if (context.vniRun != null)
				{
					VNIExecution.WaitForExecution(context.vniRun, context.Makefile);
				}
#endif
			}
		}

		internal static List<Task> GetTimestampTasks(List<CodeGenContext> contexts)
		{
			List<Task> timestampTasks = [];
			foreach (CodeGenContext codeGenContext in contexts)
			{
				if (codeGenContext.uhtTimestamp != null)
				{
					timestampTasks.Add(codeGenContext.uhtTimestamp);
				}
			}
			return timestampTasks;
		}

		internal void BuildMakefile2(ISourceFileWorkingSet WorkingSet, ILogger Logger, BuildContext context, FileHashCache fileHashCache, IEnumerable<FileReference>? customManifestFileNames = null)
		{
			SourceFileMetadataCache MetadataCache = context.MetadataCache;
			TargetDescriptor TargetDescriptor = context.Descriptor;
			TargetMakefile Makefile = context.Makefile;
			List<UEBuildBinary> OriginalBinaries = context.OriginalBinaries;
			HashSet<UEBuildModuleCPP> ModulesToGenerateHeadersFor = context.ModulesToGenerateHeadersFor;
			CppCompileEnvironment GlobalCompileEnvironment = context.GlobalCompileEnvironment;
			UEToolChain TargetToolChain = context.TargetToolChain;
			LinkEnvironment GlobalLinkEnvironment = context.GlobalLinkEnvironment;

			TargetMakefileBuilder MakefileBuilder = new TargetMakefileBuilder(Makefile, fileHashCache, Logger);

			if (Rules.bUseSharedPCHs)
			{
				// Find all the shared PCHs.
				FindSharedPCHs(OriginalBinaries, GlobalCompileEnvironment, Logger);

				// Create all the shared PCH instances before processing the modules
				CreateSharedPCHInstances(Rules, TargetToolChain, OriginalBinaries, GlobalCompileEnvironment, MakefileBuilder, Logger);
			}

			// Compile the resource files common to all DLLs on Windows
			if (Platform.IsInGroup(UnrealPlatformGroup.Windows) && Rules.WindowsPlatform.bSharedResourceFiles)
			{
				FileReference DefaultResourceLocation = FileReference.Combine(Unreal.EngineDirectory, "Build", "Windows", "Resources", "Default.rc2");
				if (!Unreal.IsFileInstalled(DefaultResourceLocation))
				{
					CppCompileEnvironment DefaultResourceCompileEnvironment = new(GlobalCompileEnvironment);
					LinkEnvironment DefaultResourceLinkEnvironment = new(GlobalLinkEnvironment);
					DefaultResourceCompileEnvironment.bIsBuildingDLL = true;
					DefaultResourceLinkEnvironment.bIsBuildingDLL = true;

					FileItem DefaultResourceFile = FileItem.GetItemByFileReference(DefaultResourceLocation);

					CPPOutput DefaultResourceOutput = TargetToolChain.CompileRCFiles(DefaultResourceCompileEnvironment, [DefaultResourceFile], EngineIntermediateDirectory, MakefileBuilder);
					GlobalLinkEnvironment.DefaultResourceFiles.AddRange(DefaultResourceOutput.ObjectFiles);
					DefaultResourceOutput = TargetToolChain.ConvertRCFiles(DefaultResourceLinkEnvironment, DefaultResourceOutput.ObjectFiles, EngineIntermediateDirectory, MakefileBuilder);
					GlobalLinkEnvironment.DefaultConvertedResourceFiles.AddRange(DefaultResourceOutput.ObjectFiles);
				}
			}

			// Copy debugger visualizer files for each module to their intermediate directory 
			foreach (UEBuildModule Module in Modules.Values)
			{
				IEnumerable<FileItem> Items = Module.CopyDebuggerVisualizers(TargetToolChain, MakefileBuilder);
				MakefileBuilder.AddOutputFiles(Items);
			}

			// Cache module dependencies for StableTopologicalSort
			UEBuildModule.CacheDependenciesForStableTopologicalSort(Modules.Values);

			// Build the target's binaries.
			DirectoryReference ExeDir = GetExecutableDir();
			using (GlobalTracer.Instance.BuildSpan("UEBuildBinary.Build()").StartActive())
			{
				Parallel2.ForEach(Binaries, Binary =>
				{
					Binary.PrepareRuntimeDependencies(ExeDir);
					List<FileItem> BinaryOutputItems = Binary.Build(Rules, TargetToolChain, GlobalCompileEnvironment, GlobalLinkEnvironment, WorkingSet, ExeDir, MakefileBuilder, Logger);
					MakefileBuilder.AddOutputFiles(BinaryOutputItems);
				});
			}

			if (Rules.bStripExports)
			{
				CreateStripExportsActions(TargetToolChain, MakefileBuilder, GlobalCompileEnvironment);
			}

			// Cache inline gen cpp data
			using (GlobalTracer.Instance.BuildSpan("CacheInlineGenCppData").StartActive())
			{
				// We don't want to run this in parallel since the tasks most likely is choked on writing response files
				foreach (TargetMakefileSourceFileInfo[] Values in Makefile.DirectoryToSourceFiles.Values)
				{
					foreach (TargetMakefileSourceFileInfo SourceFileInfo in Values)
					{
						SourceFileInfo.InlineGenCppHash = TargetMakefileSourceFileInfo.CalculateInlineGenCppHash(MetadataCache.GetListOfInlinedGeneratedCppFiles(SourceFileInfo.SourceFileItem!));
					}
				}
			}

			// Prepare all the runtime dependencies, copying them from their source folders if necessary
			List<RuntimeDependency> RuntimeDependencies = new List<RuntimeDependency>();
			Dictionary<FileReference, FileReference> RuntimeDependencyTargetFileToSourceFile = new Dictionary<FileReference, FileReference>();
			foreach (UEBuildBinary Binary in Binaries)
			{
				Binary.CollectRuntimeDependencies(RuntimeDependencies, RuntimeDependencyTargetFileToSourceFile);
			}
			TargetToolChain.PrepareRuntimeDependencies(RuntimeDependencies, RuntimeDependencyTargetFileToSourceFile, ExeDir);

			foreach (KeyValuePair<FileReference, FileReference> Pair in RuntimeDependencyTargetFileToSourceFile)
			{
				if (!Unreal.IsFileInstalled(Pair.Key))
				{
					Makefile.OutputItems.Add(MakefileBuilder.CreateCopyAction(Pair.Value, Pair.Key));
				}
			}

			// If we're just precompiling a plugin, only include output items which are part of it
			if (ForeignPlugin != null)
			{
				HashSet<FileItem> RetainOutputItems = new HashSet<FileItem>();

				UEBuildPlugin? ForeignBuildPlugin = BuildPlugins!.Find(x => x.File == ForeignPlugin);

				foreach (UEBuildPlugin Plugin in BuildPlugins)
				{
					// Retain foreign plugin dependencies if it was specified.
					bool? bIsForeignPluginDependency = ForeignBuildPlugin?.Dependencies?.Contains(Plugin);
					if (Plugin.File == ForeignPlugin || (bBuildDependantPlugins && bIsForeignPluginDependency == true))
					{
						foreach (UEBuildModule Module in Plugin.Modules)
						{
							FileItem[]? ModuleOutputItems;
							if (Makefile.ModuleNameToOutputItems.TryGetValue(Module.Name, out ModuleOutputItems))
							{
								RetainOutputItems.UnionWith(ModuleOutputItems);
							}
						}
					}
				}
				Makefile.OutputItems.RemoveAll(x => !RetainOutputItems.Contains(x));
			}

			// Allow the toolchain to modify the final output items
			TargetToolChain.FinalizeOutput(Rules, MakefileBuilder, ExeDir);

			// Get all the regular build products
			List<KeyValuePair<FileReference, BuildProductType>> BuildProducts = new List<KeyValuePair<FileReference, BuildProductType>>();
			foreach (UEBuildBinary Binary in Binaries)
			{
				Dictionary<FileReference, BuildProductType> BinaryBuildProducts = new Dictionary<FileReference, BuildProductType>();
				Binary.GetBuildProducts(Rules, TargetToolChain, BinaryBuildProducts, GlobalLinkEnvironment.bCreateDebugInfo);
				BuildProducts.AddRange(BinaryBuildProducts);
			}

			{
				UEBuildPlatform BuildPlatform = UEBuildPlatform.GetBuildPlatform(Platform);
				string DynamicLinkLibraryExt = BuildPlatform.GetBinaryExtension(UEBuildBinaryType.DynamicLinkLibrary);
				string ExecutableExt = BuildPlatform.GetBinaryExtension(UEBuildBinaryType.Executable);
				HashSet<string> DebugInfoExtensions = new(BuildPlatform.GetDebugInfoExtensions(Rules, UEBuildBinaryType.DynamicLinkLibrary)
					.Concat(BuildPlatform.GetDebugInfoExtensions(Rules, UEBuildBinaryType.Executable)));
				HashSet<string> MapInfoExtensions = Platform.IsInGroup(UnrealPlatformGroup.Microsoft) ? new(new string[] { ".map", ".objpaths" }) : new();
				BuildProducts.AddRange(RuntimeDependencyTargetFileToSourceFile.Select(x =>
				{
					string Ext = x.Key.GetExtension();
					BuildProductType ProductType = BuildProductType.RequiredResource;
					if (DynamicLinkLibraryExt.Equals(Ext, StringComparison.OrdinalIgnoreCase))
					{
						ProductType = BuildProductType.DynamicLibrary;
					}
					else if (ExecutableExt.Equals(Ext, StringComparison.OrdinalIgnoreCase))
					{
						// Should not be marked as Executable as that enum value is used for the primary target
						ProductType = BuildProductType.RequiredResource;
					}
					else if (DebugInfoExtensions.Contains(Ext, StringComparer.OrdinalIgnoreCase))
					{
						ProductType = BuildProductType.SymbolFile;
					}
					else if (MapInfoExtensions.Contains(Ext, StringComparer.OrdinalIgnoreCase))
					{
						ProductType = BuildProductType.MapFile;
					}
					return new KeyValuePair<FileReference, BuildProductType>(x.Key, ProductType);
				}));
			}

			// Remove any installed build products that don't exist. They may be part of an optional install.
			if (Unreal.IsEngineInstalled())
			{
				BuildProducts.RemoveAll(x => Unreal.IsFileInstalled(x.Key) && !FileReference.Exists(x.Key));
			}

			// Make sure all the checked headers were valid
			List<UEBuildModuleCPP.InvalidIncludeDirective> InvalidIncludeDirectives = Modules.Values.OfType<UEBuildModuleCPP>().Where(x => x.InvalidIncludeDirectives != null).SelectMany(x => x.InvalidIncludeDirectives!).ToList();
			if (InvalidIncludeDirectives.Count > 0)
			{
				foreach (UEBuildModuleCPP.InvalidIncludeDirective InvalidIncludeDirective in InvalidIncludeDirectives)
				{
					Logger.LogWarning("{CppFile}(1): error: Expected {HeaderFile} to be first header included.", InvalidIncludeDirective.CppFile, InvalidIncludeDirective.HeaderFile.GetFileName());
				}
			}

			List<Task> writeMetadataTasks = [];

			// Finalize and generate metadata for this target
			if (!Rules.bDisableLinking)
			{
				// Also add any explicitly specified build products
				if (Rules.AdditionalBuildProducts.Count > 0)
				{
					Dictionary<string, string> Variables = GetTargetVariables(null);
					foreach (string AdditionalBuildProduct in Rules.AdditionalBuildProducts)
					{
						FileReference BuildProductFile = new FileReference(Utils.ExpandVariables(AdditionalBuildProduct, Variables));
						BuildProducts.Add(new KeyValuePair<FileReference, BuildProductType>(BuildProductFile, BuildProductType.RequiredResource));
					}
				}

				// Get the path to the version file unless this is a formal build (where it will be compiled in)
				FileReference? VersionFile = null;
				if (Rules.LinkType != TargetLinkType.Monolithic && Binaries[0].Type == UEBuildBinaryType.Executable)
				{
					UnrealTargetConfiguration VersionConfig = Configuration;
					if (VersionConfig == UnrealTargetConfiguration.DebugGame && !bCompileMonolithic && TargetType != TargetType.Program && bUseSharedBuildEnvironment)
					{
						VersionConfig = UnrealTargetConfiguration.Development;
					}
					VersionFile = BuildVersion.GetFileNameForTarget(ExeDir, bCompileMonolithic ? TargetName : AppName, Platform, VersionConfig, Architectures);
				}

				// Also add the version file as a build product
				if (VersionFile != null)
				{
					BuildProducts.Add(new KeyValuePair<FileReference, BuildProductType>(VersionFile, BuildProductType.RequiredResource));
				}

				// Prepare the module manifests, and add them to the list of build products
				Dictionary<FileReference, ModuleManifest> FileNameToModuleManifest = PrepareModuleManifests();
				BuildProducts.AddRange(FileNameToModuleManifest.Select(x => new KeyValuePair<FileReference, BuildProductType>(x.Key, BuildProductType.RequiredResource)));

				// Prepare load order manifests (if needed), and add them to the list of build products.
				Dictionary<FileReference, LoadOrderManifest> FileNameToLoadOrderManifest = PrepareLoadOrderManifests(Logger);
				BuildProducts.AddRange(FileNameToLoadOrderManifest.Select(x => new KeyValuePair<FileReference, BuildProductType>(x.Key, BuildProductType.RequiredResource)));

				// Prepare the receipt
				TargetReceipt Receipt = PrepareReceipt(TargetToolChain, BuildProducts, RuntimeDependencies);

				// Create an action which to generate the module receipts
				if (VersionFile == null)
				{
					WriteMetadataTargetInfo TargetInfo = new WriteMetadataTargetInfo(ProjectFile, null, null, ReceiptFileName, Receipt, FileNameToModuleManifest, FileNameToLoadOrderManifest, Rules.bMergeModules);
					FileReference TargetInfoFile = FileReference.Combine(ProjectIntermediateDirectory, "TargetMetadata.json");

					// Make a copy of this since OutputItems might be added byh post build actions
					List<FileItem> TargetPrereqItems = new (Makefile.OutputItems);

					writeMetadataTasks.Add(Task.Run(() =>
					{
						CreateWriteMetadataAction(MakefileBuilder, ReceiptFileName.GetFileName(), TargetInfoFile, TargetInfo, TargetPrereqItems);
					}));
				}
				else
				{
					Dictionary<FileReference, ModuleManifest> EngineFileToManifest = new Dictionary<FileReference, ModuleManifest>();
					Dictionary<FileReference, ModuleManifest> TargetFileToManifest = new Dictionary<FileReference, ModuleManifest>();

					foreach ((FileReference File, ModuleManifest Manifest) in FileNameToModuleManifest)
					{
						if (File.IsUnderDirectory(Unreal.EngineDirectory))
						{
							EngineFileToManifest.Add(File, Manifest);
						}
						else
						{
							TargetFileToManifest.Add(File, Manifest);
						}
					}

					List<FileItem> EnginePrereqItems = new List<FileItem>();
					List<FileItem> TargetPrereqItems = new List<FileItem>();

					foreach (FileItem OutputItem in Makefile.OutputItems)
					{
						if (OutputItem.Location.IsUnderDirectory(Unreal.EngineDirectory))
						{
							EnginePrereqItems.Add(OutputItem);
						}
						else
						{
							TargetPrereqItems.Add(OutputItem);
						}
					}

					writeMetadataTasks.Add(Task.Run(() =>
					{
					WriteMetadataTargetInfo EngineInfo = new WriteMetadataTargetInfo(null, VersionFile, Receipt.Version, null, null, EngineFileToManifest, null, Rules.bMergeModules);
						FileReference EngineInfoFile = FileReference.Combine(ProjectIntermediateDirectory, "EngineMetadata.json");
						string EngineInfoName = TargetName.Equals(VersionFile.GetFileNameWithoutExtension(), StringComparison.Ordinal) ? VersionFile.GetFileName() : $"{VersionFile.GetFileName()} ({TargetName})";
						CreateWriteMetadataAction(MakefileBuilder, EngineInfoName, EngineInfoFile, EngineInfo, EnginePrereqItems);
					}));
					writeMetadataTasks.Add(Task.Run(() =>
					{
					WriteMetadataTargetInfo TargetInfo = new WriteMetadataTargetInfo(ProjectFile, VersionFile, null, ReceiptFileName, Receipt, TargetFileToManifest, FileNameToLoadOrderManifest, Rules.bMergeModules);
						FileReference TargetInfoFile = FileReference.Combine(ProjectIntermediateDirectory, "TargetMetadata.json");
						CreateWriteMetadataAction(MakefileBuilder, ReceiptFileName.GetFileName(), TargetInfoFile, TargetInfo, TargetPrereqItems);
					}));
				}

				// Create actions to run the post build steps
				FileReference[] PostBuildScripts = CreatePostBuildScripts();
				foreach (FileReference PostBuildScript in PostBuildScripts)
				{
					FileReference OutputFile = new FileReference(PostBuildScript.FullName + ".ran");

					Action PostBuildStepAction = MakefileBuilder.CreateAction(ActionType.PostBuildStep);
					PostBuildStepAction.CommandPath = BuildHostPlatform.Current.Shell;
					if (BuildHostPlatform.Current.ShellType == ShellType.Cmd)
					{
						PostBuildStepAction.CommandArguments = String.Format("/C \"call \"{0}\" && type NUL >\"{1}\"\"", PostBuildScript, OutputFile);
					}
					else
					{
						PostBuildStepAction.CommandArguments = String.Format("\"{0}\" && touch \"{1}\"", PostBuildScript, OutputFile);
					}
					PostBuildStepAction.WorkingDirectory = Unreal.EngineSourceDirectory;
					PostBuildStepAction.StatusDescription = String.Format("Executing post build script ({0})", PostBuildScript.GetFileName());
					PostBuildStepAction.bCanExecuteRemotely = false;
					PostBuildStepAction.PrerequisiteItems.Add(FileItem.GetItemByFileReference(ReceiptFileName));
					PostBuildStepAction.ProducedItems.Add(FileItem.GetItemByFileReference(OutputFile));
					PostBuildStepAction.bCanExecuteInUBA = false;

					Makefile.OutputItems.AddRange(PostBuildStepAction.ProducedItems);
				}
			}

			// Build a list of all the files required to build
			foreach (FileReference DependencyListFileName in Rules.DependencyListFileNames)
			{
				WriteDependencyList(DependencyListFileName, RuntimeDependencies, RuntimeDependencyTargetFileToSourceFile, Logger);
			}

			// If we're only generating the manifest, return now
			foreach (FileReference ManifestFileName in customManifestFileNames ?? Rules.ManifestFileNames)
			{
				GenerateManifest(ManifestFileName, BuildProducts, Logger);
			}

			// Check there are no EULA or restricted folder violations
			if (!Rules.bDisableLinking)
			{
				// Check the distribution level of all binaries based on the dependencies they have
				if (ProjectFile == null && !Rules.bLegalToDistributeBinary)
				{
					List<DirectoryReference> RootDirectories = [Unreal.EngineDirectory];
					DirectoryReference? ProjectDir = DirectoryReference.FromFile(ProjectFile);
					if (ProjectDir != null)
					{
						RootDirectories.Add(ProjectDir);
						ProjectDescriptor?.AddAdditionalPaths(RootDirectories, ProjectDir);
					}

					Dictionary<UEBuildModule, Dictionary<RestrictedFolder, DirectoryReference>> ModuleRestrictedFolderCache = new Dictionary<UEBuildModule, Dictionary<RestrictedFolder, DirectoryReference>>();

					bool bResult = true;
					foreach (UEBuildBinary Binary in Binaries)
					{
						bResult &= Binary.CheckRestrictedFolders(RootDirectories, ModuleRestrictedFolderCache, Logger);
					}
					foreach (KeyValuePair<FileReference, FileReference> Pair in RuntimeDependencyTargetFileToSourceFile)
					{
						bResult &= CheckRestrictedFolders(Pair.Key, Pair.Value, Logger);
					}

					if (!bResult)
					{
						throw new BuildException("Unable to create binaries in less restricted locations than their input files.");
					}
				}
			}

			// Check for referencing modules prohibited by the EULA
			CheckForEULAViolation(Logger);

			// Add all the plugins to be tracked
			foreach (FileReference PluginFile in PluginsBase.EnumeratePlugins(ProjectFile))
			{
				FileItem PluginFileItem = FileItem.GetItemByFileReference(PluginFile);
				Makefile.PluginFiles.Add(PluginFileItem);
			}

			// Add all the input files to the predicate store
			Makefile.ExternalDependencies.Add(FileItem.GetItemByFileReference(TargetRulesFile));
			Makefile.ExternalDependencies.Add(FileItem.GetItemByFileReference(Rules.TargetSourceFile));
			foreach (UEBuildModule Module in Modules.Values)
			{
				Makefile.ExternalDependencies.Add(FileItem.GetItemByFileReference(Module.RulesFile));
				foreach (string ExternalDependency in Module.Rules.ExternalDependencies)
				{
					FileReference Location = FileReference.Combine(Module.RulesFile.Directory, ExternalDependency);
					Makefile.ExternalDependencies.Add(FileItem.GetItemByFileReference(Location));
				}
				if (Module.Rules.SubclassRules != null)
				{
					foreach (string SubclassRule in Module.Rules.SubclassRules)
					{
						FileItem SubclassRuleFileItem = FileItem.GetItemByFileReference(new FileReference(SubclassRule));
						Makefile.ExternalDependencies.Add(SubclassRuleFileItem);
					}
				}
			}
			Makefile.ExternalDependencies.UnionWith(Makefile.PluginFiles);

			// Track config files
			Makefile.ExternalDependencies.UnionWith(ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, Rules.ProjectFile?.Directory, Rules.Platform).FileReferences.Select(FileItem.GetItemByFileReference));
			Makefile.ExternalDependencies.UnionWith(ConfigCache.ReadHierarchy(ConfigHierarchyType.Game, Rules.ProjectFile?.Directory, Rules.Platform).FileReferences.Select(FileItem.GetItemByFileReference));
			Makefile.ExternalDependencies.UnionWith(ConfigCache.ReadHierarchy(ConfigHierarchyType.Encryption, Rules.ProjectFile?.Directory, Rules.Platform).FileReferences.Select(FileItem.GetItemByFileReference));
			Makefile.ExternalDependencies.UnionWith(ConfigCache.ReadHierarchy(ConfigHierarchyType.Crypto, Rules.ProjectFile?.Directory, Rules.Platform).FileReferences.Select(FileItem.GetItemByFileReference));
			if (Rules.Type == TargetType.Editor)
			{
				Makefile.ExternalDependencies.UnionWith(ConfigCache.ReadHierarchy(ConfigHierarchyType.Editor, Rules.ProjectFile?.Directory, Rules.Platform).FileReferences.Select(FileItem.GetItemByFileReference));
			}

			// Also track the version file
			Makefile.ExternalDependencies.Add(FileItem.GetItemByFileReference(BuildVersion.GetDefaultFileName()));

			// Add any toolchain dependencies
			TargetToolChain.GetExternalDependencies(Makefile.ExternalDependencies);

			// Add any relevent platform sdk info
			IEnumerable<UnrealTargetPlatform> SdkPlatforms = Rules.Type == TargetType.Editor || Rules.Type == TargetType.Program ? UnrealTargetPlatform.GetValidPlatforms() : [BuildHostPlatform.Current.Platform, Rules.Platform];
			foreach (UnrealTargetPlatform platform in SdkPlatforms.Distinct())
			{
				Makefile.PlatformSDKInfo.Add(platform.ToString(), UEBuildPlatformSDK.GetSDKForPlatform(platform.ToString())?.GetAllSDKInfo().ToString() ?? String.Empty);
			}

			// Write a header containing public definitions for this target
			if (Rules.ExportPublicHeader != null)
			{
				UEBuildBinary Binary = Binaries[0];
				FileReference Header = FileReference.Combine(Binary.OutputDir, Rules.ExportPublicHeader);
				WritePublicHeader(Binary, Header, GlobalCompileEnvironment, Logger);
			}

			Task.WaitAll(writeMetadataTasks.ToArray());

			// Wait for makefile tasks to finish (tasks writing files to disk etc)
			{
				using ITimelineEvent _ = Timeline.ScopeEvent("WaitOnWrites");
				MakefileBuilder.WaitOnWriteTasks();
			}

			// Clean any stale modules which exist in multiple output directories. This can lead to the wrong DLL being loaded on Windows.
			CleanStaleModules(Logger);
		}

		void CreateStripExportsActions(UEToolChain TargetToolChain, TargetMakefileBuilder MakefileBuilder, CppCompileEnvironment CompileEnvironment)
		{
			FileReference commandPath;
			if (OperatingSystem.IsWindows())
			{
#pragma warning disable CA1308 // Normalize strings to uppercase
				commandPath = FileReference.Combine(Unreal.EngineDirectory, "Binaries", "Win64", "UnrealBuildAccelerator", RuntimeInformation.ProcessArchitecture.ToString().ToLowerInvariant(), "UbaObjTool.exe");
#pragma warning restore CA1308 // Normalize strings to uppercase
			}
			else if (OperatingSystem.IsLinux())
			{
				if (RuntimeInformation.ProcessArchitecture == Architecture.X64)
				{
					commandPath = FileReference.Combine(Unreal.EngineDirectory, "Binaries", "Linux", "UnrealBuildAccelerator", "UbaObjTool");
				}
				else if (RuntimeInformation.ProcessArchitecture == Architecture.Arm64)
				{
					commandPath = FileReference.Combine(Unreal.EngineDirectory, "Binaries", "LinuxArm64", "UnrealBuildAccelerator", "UbaObjTool");
				}
				else
				{
					throw new PlatformNotSupportedException();
				}
			}
			else if (OperatingSystem.IsMacOS())
			{
				commandPath = FileReference.Combine(Unreal.EngineDirectory, "Binaries", "Mac", "UnrealBuildAccelerator", "UbaObjTool");
			}
			else
			{
				throw new PlatformNotSupportedException();
			}

			HashSet<FileItem> allObjects = new(Binaries.SelectMany(x => x.InputObjects));
			SortedSet<FileItem> allExis = new(allObjects.Select(x => FileItem.GetItemByFileReference(x.Location.ChangeExtension(".exi"))), FileItem.SortByPath);

			void CreateGenerateExportsAction(string Name, UEBuildBinaryType binaryType, IEnumerable< FileItem> toStrip, DirectoryReference IntermediateDirectory)
			{
				SortedSet<FileItem> toStripExis = new(toStrip.Select(x => FileItem.GetItemByFileReference(x.Location.ChangeExtension(".exi"))), FileItem.SortByPath);
				SortedSet<FileItem> otherExis = new(allExis, FileItem.SortByPath);
				otherExis.ExceptWith(toStripExis);

				string Normalize(FileReference file) => TargetToolChain.NormalizeCommandLinePath(file, CompileEnvironment.RootPaths);

				FileItem exportsRsp = FileItem.GetItemByFileReference(FileReference.Combine(IntermediateDirectory, $"{Name}.exports.rsp"));
				string Ext = TargetToolChain.GetExtraLinkFileExtension();
				FileReference outputFile = FileReference.Combine(IntermediateDirectory, $"{Name}.exports.{Ext}");

				IEnumerable<string> arguments =
				[
					.. toStripExis.Select(path => $"/S:{Normalize(path.Location)}"),
					.. otherExis.Select(path => $"/D:{Normalize(path.Location)}"),
					$"/O:{Normalize(outputFile)}",
					$"/T:{Platform}",
					$"/M:{Name}",
					..TargetToolChain.GetExtraLinkFileAdditionalSymbols(binaryType).Select(symbol=> $"/E:{symbol}"),
				];

				MakefileBuilder.CreateIntermediateTextFile(exportsRsp, arguments);

				Action exportsAction = MakefileBuilder.CreateAction(ActionType.Link);

				exportsAction.PrerequisiteItems.UnionWith(allExis);
				exportsAction.PrerequisiteItems.Add(exportsRsp);
				exportsAction.ProducedItems.Add(FileItem.GetItemByFileReference(outputFile));
				exportsAction.CommandPath = commandPath;
				exportsAction.CommandArguments = $"@\"{Normalize(exportsRsp.Location)}\"";
				exportsAction.CommandDescription = "GenerateExports";
				exportsAction.WorkingDirectory = Unreal.EngineSourceDirectory;
				exportsAction.StatusDescription = Name;
				exportsAction.bCanExecuteRemotely = false; // Can run remotely but is so fast so it would be a waste
				exportsAction.ArtifactMode = ArtifactMode.Enabled;
				exportsAction.RootPaths = CompileEnvironment.RootPaths;
				exportsAction.CacheBucket = ISPCToolChain.GetCacheBucket(Rules, CompileEnvironment);

				MakefileBuilder.AddOutputFiles(exportsAction.ProducedItems);
			}

			foreach (UEBuildBinary Binary in Binaries.Where(x => x.bStripUnusedExports))
			{
				CreateGenerateExportsAction(Binary.OutputFilePaths.First().GetFileNameWithoutExtension(), Binary.Type, Binary.InputObjects, Binary.IntermediateDirectory);
			}
		}

		void CreateWriteMetadataAction(TargetMakefileBuilder MakefileBuilder, string StatusDescription, FileReference InfoFile, WriteMetadataTargetInfo Info, IEnumerable<FileItem> PrerequisiteItems)
		{
			List<FileReference> ProducedItems = new List<FileReference>(Info.FileToManifest.Keys);
			ProducedItems.AddRange(Info.FileToLoadOrderManifest.Keys);

			if (Info.Version != null && Info.VersionFile != null)
			{
				ProducedItems.Add(Info.VersionFile);
			}
			if (Info.ReceiptFile != null)
			{
				ProducedItems.Add(Info.ReceiptFile);
			}

			if (!ProducedItems.Any(x => Unreal.IsFileInstalled(x)))
			{
				// Save the info file, but deliberately do not add a prerequisite from the action onto it unless we're updating the
				// target file. Since the outputs are not target specific, but the input file is always beneath the project intermediate
				// directory (even for the engine), it will cause -NoEngineChanges to block the build when updated. The behavior we want
				// is just to depend on the engine DLL timestamps. Instead, we add a makefile dependency on it, causing it to be
				// regenerated if missing, and only add it as a prereq if we're updating the .target file.
				FileItem InfoFileItem = FileItem.GetItemByFileReference(InfoFile);
				byte[] InfoBytes = JsonSerializer.SerializeToUtf8Bytes(Info, WriteMetadataMode.SerializerOptions);
				DirectoryReference.CreateDirectory(InfoFile.Directory);
				FileReference.WriteAllBytesIfDifferent(InfoFile, InfoBytes);

				InfoFileItem.ResetCachedInfo();
				MakefileBuilder.AddInternalDependency(InfoFileItem);

				// Get the argument list
				StringBuilder WriteMetadataArguments = new StringBuilder();
				WriteMetadataArguments.AppendFormat("-Input={0}", Utils.MakePathSafeToUseWithCommandLine(InfoFile));
				WriteMetadataArguments.AppendFormat(" -Version={0}", WriteMetadataMode.CurrentVersionNumber);
				if (Rules.bNoManifestChanges)
				{
					WriteMetadataArguments.Append(" -NoManifestChanges");
				}

				// Create the action
				Action WriteMetadataAction = MakefileBuilder.CreateRecursiveAction<WriteMetadataMode>(ActionType.WriteMetadata, WriteMetadataArguments.ToString());
				WriteMetadataAction.StatusDescription = StatusDescription;
				WriteMetadataAction.bUseActionHistory = false; // Different files for each target; do not want to invalidate based on this.
				WriteMetadataAction.ProducedItems.UnionWith(ProducedItems.Select(x => FileItem.GetItemByFileReference(x)));
				WriteMetadataAction.PrerequisiteItems.UnionWith(PrerequisiteItems);

				if (Info.Version == null && Info.VersionFile != null)
				{
					WriteMetadataAction.PrerequisiteItems.Add(FileItem.GetItemByFileReference(Info.VersionFile));
				}

				if (Info.ReceiptFile != null)
				{
					WriteMetadataAction.PrerequisiteItems.Add(InfoFileItem);
				}

				// Add the produced items as leaf nodes to be built
				MakefileBuilder.AddOutputFiles(WriteMetadataAction.ProducedItems);
			}
		}

		/// <summary>
		/// Gets the output directory for the main executable
		/// </summary>
		/// <returns>The executable directory</returns>
		public DirectoryReference GetExecutableDir()
		{
			DirectoryReference ExeDir = Binaries[0].OutputDir;
			if (Platform == UnrealTargetPlatform.Mac && ExeDir.FullName.EndsWith(".app/Contents/MacOS", System.StringComparison.Ordinal))
			{
				ExeDir = ExeDir.ParentDirectory!.ParentDirectory!.ParentDirectory!;
			}
			return ExeDir;
		}

		/// <summary>
		/// Check that copying a file from one location to another does not violate rules regarding restricted folders
		/// </summary>
		/// <param name="TargetFile">The destination location for the file</param>
		/// <param name="SourceFile">The source location of the file</param>
		/// <param name="Logger">Logger for output</param>
		/// <returns>True if the copy is permitted, false otherwise</returns>
		bool CheckRestrictedFolders(FileReference TargetFile, FileReference SourceFile, ILogger Logger)
		{
			List<RestrictedFolder> TargetRestrictedFolders = GetRestrictedFolders(TargetFile);
			List<RestrictedFolder> SourceRestrictedFolders = GetRestrictedFolders(SourceFile);
			foreach (RestrictedFolder SourceRestrictedFolder in SourceRestrictedFolders)
			{
				if (!TargetRestrictedFolders.Contains(SourceRestrictedFolder))
				{
					Logger.LogError("Runtime dependency '{SourceFile}' is copied to '{TargetFile}', which does not contain a '{SourceRestrictedFolder}' folder.", SourceFile, TargetFile, SourceRestrictedFolder);
					return false;
				}
			}
			return true;
		}

		/// <summary>
		/// Gets the restricted folders that the given file is in
		/// </summary>
		/// <param name="File">The file to test</param>
		/// <returns>List of restricted folders for the file</returns>
		List<RestrictedFolder> GetRestrictedFolders(FileReference File)
		{
			// Find the base directory for this binary
			DirectoryReference BaseDir;
			if (File.IsUnderDirectory(Unreal.RootDirectory))
			{
				BaseDir = Unreal.RootDirectory;
			}
			else if (ProjectDirectory != null && File.IsUnderDirectory(ProjectDirectory))
			{
				BaseDir = ProjectDirectory;
			}
			else
			{
				return new List<RestrictedFolder>();
			}

			// Find the restricted folders under the base directory
			return RestrictedFolders.FindPermittedRestrictedFolderReferences(BaseDir, File.Directory);
		}

		/// <summary>
		/// Creates a toolchain for the current target. May be overridden by the target rules.
		/// </summary>
		/// <returns>New toolchain instance</returns>
		public UEToolChain CreateToolchain(UnrealTargetPlatform Platform, ILogger Logger)
		{
			if (Rules.ToolChainName == null)
			{
				return UEBuildPlatform.GetBuildPlatform(Platform).CreateToolChain(Rules);
			}
			else
			{
				Type? ToolchainType = Assembly.GetExecutingAssembly().GetType(String.Format("UnrealBuildTool.{0}", Rules.ToolChainName), false, true);
				if (ToolchainType == null)
				{
					throw new BuildException("Unable to create toolchain '{0}'. Check that the name is correct.", Rules.ToolChainName);
				}
				return (UEToolChain)Activator.CreateInstance(ToolchainType, Rules, Logger)!;
			}
		}

		/// <summary>
		/// Cleans any stale modules that have changed output folder.
		/// 
		/// On Windows, the loader reads imported DLLs from the first location it finds them. If modules are moved from one place to another, we have to be sure to clean up the old versions
		/// so that they're not loaded accidentally causing unintuitive import errors.
		/// </summary>
		void CleanStaleModules(ILogger logger)
		{
			// Find all the output files
			HashSet<FileReference> outputFiles = [.. Binaries.SelectMany(x => x.OutputFilePaths)];

			// Build a map of base filenames to their full path
			Dictionary<string, FileReference> outputNameToLocation = new(StringComparer.InvariantCultureIgnoreCase);
			foreach (FileReference outputFile in outputFiles)
			{
				outputNameToLocation[outputFile.GetFileName()] = outputFile;
			}

			// Search all the output directories for files with a name matching one of our output files
			Parallel2.ForEach(outputFiles.Select(x => x.Directory).Distinct().Where(DirectoryReference.Exists), outputDirectory =>
			{
				foreach (FileReference existingFile in DirectoryReference.EnumerateFiles(outputDirectory))
				{
					if (outputNameToLocation.TryGetValue(existingFile.GetFileName(), out FileReference? OutputFile) && !outputFiles.Contains(existingFile))
					{
						logger.LogInformation("Deleting '{ExistingFile}' to avoid ambiguity with '{OutputFile}'", existingFile, OutputFile);
						try
						{
							FileReference.Delete(existingFile);
						}
						catch (Exception Ex)
						{
							logger.LogError("Unable to delete {File} ({Ex})", existingFile, Ex.Message.TrimEnd());
						}
					}
				}
			});
		}

		/// <summary>
		/// Check whether a reference from an engine module to a plugin module is allowed. 'Temporary' hack until these can be fixed up properly.
		/// </summary>
		/// <param name="EngineModuleName">Name of the engine module.</param>
		/// <param name="PluginModuleName">Name of the plugin module.</param>
		/// <returns>True if the reference is allowed.</returns>
		static bool IsEngineToPluginReferenceAllowed(string EngineModuleName, string PluginModuleName)
		{
			if (EngineModuleName == "AndroidDeviceDetection" && PluginModuleName == "TcpMessaging")
			{
				return true;
			}
			if (EngineModuleName == "Voice" && PluginModuleName == "AndroidPermission")
			{
				return true;
			}
			return false;
		}

		/// <summary>
		/// Export the definition of this target to a JSON file
		/// </summary>
		/// <param name="OutputFile">File to write to</param>
		/// <param name="Actions">List of actions to export</param>
		public void ExportJson(FileReference OutputFile, List<LinkedAction>? Actions = null)
		{
			DirectoryReference.CreateDirectory(OutputFile.Directory);
			using (JsonWriter Writer = new JsonWriter(OutputFile))
			{
				Writer.WriteObjectStart();

				Writer.WriteValue("Name", TargetName);
				Writer.WriteValue("Configuration", Configuration.ToString());
				Writer.WriteValue("Platform", Platform.ToString());
				if (ProjectFile != null)
				{
					Writer.WriteValue("ProjectFile", ProjectFile.FullName);
				}

				Writer.WriteArrayStart("Binaries");
				foreach (UEBuildBinary Binary in Binaries)
				{
					Writer.WriteObjectStart();
					Binary.ExportJson(Writer);
					Writer.WriteObjectEnd();
				}
				Writer.WriteArrayEnd();

				Writer.WriteObjectStart("Modules");
				foreach (UEBuildModule Module in Modules.Values)
				{
					Writer.WriteObjectStart(Module.Name);
					Module.ExportJson(Module.Binary?.OutputDir, GetExecutableDir(), Writer);
					Writer.WriteObjectEnd();
				}
				Writer.WriteObjectEnd();

				if (Actions != null)
				{
					Writer.WriteArrayStart("Actions");
					Dictionary<LinkedAction, int> LinkedActionToId = new();
					int Counter = 0;
					foreach (LinkedAction Action in Actions)
					{
						LinkedActionToId[Action] = Counter++;
					}
					foreach (LinkedAction Action in Actions)
					{
						if (Action.StatusDescription != "...")
						{
							Writer.WriteObjectStart();
							Action.ExportJson(LinkedActionToId, Writer);
							Writer.WriteObjectEnd();
						}
					}
					Writer.WriteArrayEnd();
				}

				Writer.WriteObjectEnd();
			}
		}

		/// <summary>
		/// Writes a header for the given binary that allows including headers from it in an external application
		/// </summary>
		/// <param name="Binary">Binary to write a header for</param>
		/// <param name="HeaderFile">Path to the header to output</param>
		/// <param name="GlobalCompileEnvironment">The global compile environment for this target</param>
		/// <param name="Logger"></param>
		static void WritePublicHeader(UEBuildBinary Binary, FileReference HeaderFile, CppCompileEnvironment GlobalCompileEnvironment, ILogger Logger)
		{
			DirectoryReference.CreateDirectory(HeaderFile.Directory);

			// Find all the public definitions from each module. We pass null as the source binary to AddModuleToCompileEnvironment, which forces all _API macros to 'export' mode
			List<string> Definitions = new List<string>(GlobalCompileEnvironment.Definitions);
			foreach (UEBuildModule Module in Binary.Modules)
			{
				Module.AddModuleToCompileEnvironment(null, null, new HashSet<DirectoryReference>(), new HashSet<DirectoryReference>(), new HashSet<DirectoryReference>(), Definitions, new List<UEBuildFramework>(), new HashSet<FileItem>(), new List<FileItem>(), false, false);
			}

			// Write the header
			using (StreamWriter Writer = new StreamWriter(HeaderFile.FullName))
			{
				Writer.WriteLine($"// Definitions for {HeaderFile.GetFileNameWithoutAnyExtensions()}");
				Writer.WriteLine();
				Writer.WriteLine("#pragma once");
				Writer.WriteLine();
				foreach (string Definition in Definitions)
				{
					int EqualsIdx = Definition.IndexOf('=', StringComparison.Ordinal);
					if (EqualsIdx == -1)
					{
						Writer.WriteLine(String.Format("#define {0} 1", Definition));
					}
					else
					{
						Writer.WriteLine(String.Format("#define {0} {1}", Definition.Substring(0, EqualsIdx), Definition.Substring(EqualsIdx + 1)));
					}
				}
			}
			Logger.LogInformation("Written public header to {HeaderFile}", HeaderFile);
		}

		/// <summary>
		/// Check for EULA violation dependency issues.
		/// </summary>
		private void CheckForEULAViolation(ILogger Logger)
		{
			if (TargetType != TargetType.Editor && TargetType != TargetType.Program && Configuration == UnrealTargetConfiguration.Shipping &&
				Rules.bCheckLicenseViolations)
			{
				bool bLicenseViolation = false;
				foreach (UEBuildBinary Binary in Binaries)
				{
					IEnumerable<UEBuildModule> AllDependencies = Binary.GetAllDependencyModules(true, false);
					IEnumerable<UEBuildModule> NonRedistModules = AllDependencies.Where((DependencyModule) =>
							!IsRedistributable(DependencyModule) && DependencyModule.Name != AppName
						);

					if (NonRedistModules.Any())
					{
						IEnumerable<UEBuildModule> NonRedistDeps = AllDependencies.Where((DependantModule) =>
							DependantModule.GetDirectDependencyModules().Intersect(NonRedistModules).Any()
						);
						string Message = String.Format("Non-editor build cannot depend on non-redistributable modules. {0} depends on '{1}'.", Binary.ToString(), String.Join("', '", NonRedistModules));
						if (NonRedistDeps.Any())
						{
							Message = String.Format("{0}\nDependant modules '{1}'", Message, String.Join("', '", NonRedistDeps));
						}
						if (Rules.bBreakBuildOnLicenseViolation)
						{
							Logger.LogError("ERROR: {Message}", Message);
						}
						else
						{
							Logger.LogWarning("WARNING: {Message}", Message);
						}
						bLicenseViolation = true;
					}
				}
				if (Rules.bBreakBuildOnLicenseViolation && bLicenseViolation)
				{
					throw new BuildException("Non-editor build cannot depend on non-redistributable modules.");
				}
			}
		}

		/// <summary>
		/// Tells if this module can be redistributed.
		/// </summary>
		public static bool IsRedistributable(UEBuildModule Module)
		{
			if (Module.Rules != null && Module.Rules.IsRedistributableOverride.HasValue)
			{
				return Module.Rules.IsRedistributableOverride.Value;
			}

			if (Module.RulesFile != null)
			{
				foreach (DirectoryReference ExtensionDir in Unreal.GetExtensionDirs(Unreal.EngineDirectory))
				{
					DirectoryReference SourceDeveloperDir = DirectoryReference.Combine(ExtensionDir, "Source/Developer");
					if (Module.RulesFile.IsUnderDirectory(SourceDeveloperDir))
					{
						return false;
					}

					DirectoryReference SourceEditorDir = DirectoryReference.Combine(ExtensionDir, "Source/Editor");
					if (Module.RulesFile.IsUnderDirectory(SourceEditorDir))
					{
						return false;
					}
				}
			}

			return true;
		}
		bool CanPluginBeMerged(UEBuildPlugin plugin)
		{
			if (Rules.MergePluginsExcluded.Contains(plugin.Name))
			{
				return false;
			}
			else if (Rules.MergeOnlyGameFeatures)
			{
				DirectoryReference gameFeaturesDir = DirectoryReference.Combine(ProjectDirectory, "Plugins", "GameFeatures");
				return plugin.File.IsUnderDirectory(gameFeaturesDir);
			}
			else
			{
				return true;
			}
		}

		/// <summary>
		/// Dynamically merge modules into libraries based on target settings
		/// </summary>
		/// <param name="Logger"></param>
		protected void SetupMergeModules(ILogger Logger)
		{
			void SetPerModuleInline(UEBuildBinary binary)
			{
				binary.PrimaryModule.bCreatePerModuleFile = false;

				Queue<(UEBuildModule Module, UEBuildModule Source)> queue = new();
				HashSet<UEBuildModule> visited = new();

				// Queue up primary module and all direct dependencies
				queue.Enqueue((binary.PrimaryModule, binary.PrimaryModule));
				foreach (UEBuildModule m in binary.Modules)
				{
					if (visited.Add(m))
					{
						queue.Enqueue((m, m));
					}
				}

				// Search for non-engine PerModuleInlineFile.. 
				while (queue.Count > 0)
				{
					(UEBuildModule Module, UEBuildModule Source) current = queue.Dequeue();
					if (current.Module.RulesFile.IsUnderDirectory(Unreal.EngineDirectory))
					{
						continue;
					}
					if (current.Module.Rules.PublicDefinitions.Exists(x => x.StartsWith("PER_MODULE_INLINE_FILE", StringComparison.Ordinal)))
					{
						((UEBuildModuleCPP)current.Source).bCreatePerModuleFile = true;
						return;
					}

					foreach (UEBuildModule? dep in current.Module.PublicDependencyModules!.Union(current.Module.PrivateDependencyModules!))
					{
						if (visited.Add(dep))
						{
							queue.Enqueue((dep, current.Source));
						}
					}
				}

				// No non-engine PerModuleInline files found, fall back to Core or primary module
				UEBuildModuleCPP module = binary.Modules.OfType<UEBuildModuleCPP>().FirstOrDefault(x => x.Name == "Core") ?? binary.PrimaryModule;
				module.bCreatePerModuleFile = true;
			}

			void MergeModulesToDLL(string name, IEnumerable<UEBuildModuleCPP> modules)
			{
				if (!modules.Any())
				{
					return;
				}

				IEnumerable<UEBuildModuleCPP> modulesList = modules;
				UEBuildBinary mergedBinary = CreateDynamicLibraryForModules(name, modulesList);
				foreach (UEBuildModuleCPP module in modules)
				{
					module.Binary = mergedBinary;
					module.bCreatePerModuleFile = false;
				}
				Binaries.Add(mergedBinary);
			}

			void MergeModulesToLaunch(IEnumerable<UEBuildModuleCPP> modules)
			{
				if (!modules.Any())
				{
					return;
				}
				foreach (UEBuildModuleCPP module in modules)
				{
					Binaries[0].AddModule(module);
					module.Binary = Binaries[0];
					module.bCreatePerModuleFile = false;
				}
			}

			void ConsumeModules(string name, IEnumerable<UEBuildModuleCPP> modules)
			{
				if (Rules.MergePluginsLaunch.Contains(name))
				{
					MergeModulesToLaunch(modules);
				}
				else
				{
					MergeModulesToDLL(name, modules);
				}
			}

			Dictionary<UEBuildPlugin, HashSet<UEBuildPlugin>> globalPluginUserMap = new();

			void GetPluginDependencies(UEBuildPlugin plugin, ref HashSet<UEBuildPlugin> dependencies, bool bShouldOnlyFollowCode = false, bool bHasUsersWithSourceCode = false)
			{
				if (plugin.Modules.Any())
				{
					bHasUsersWithSourceCode = true;
				}

				foreach (PluginReferenceDescriptor dependency in plugin.Info.Descriptor.Plugins ?? new())
				{
					if (!Rules.ShouldIgnorePluginDependency(plugin.Info, dependency))
					{
						IEnumerable<UEBuildPlugin> dependentPluginCandidates = (plugin.Dependencies ?? new()).Where(x => x.Name == dependency.Name);
						if (dependentPluginCandidates.Any())
						{
							UEBuildPlugin dependentPlugin = dependentPluginCandidates.First();

							bool bShouldFollowPlugin = (CanPluginBeMerged(dependentPlugin) && !bShouldOnlyFollowCode) || bHasUsersWithSourceCode;

							if (!dependencies.Contains(dependentPlugin) && bShouldFollowPlugin)
							{
								dependencies.Add(dependentPlugin);

								if (!globalPluginUserMap.ContainsKey(dependentPlugin))
								{
									globalPluginUserMap.Add(dependentPlugin, new());
								}
								globalPluginUserMap[dependentPlugin].Add(plugin);

								GetPluginDependencies(dependentPlugin, ref dependencies, bShouldOnlyFollowCode, bHasUsersWithSourceCode);
							}
						}
					}
				}
			}
		
			void GetModulesDependenciesRecursively(IEnumerable<UEBuildModuleCPP> Modules, ref HashSet<UEBuildModuleCPP> Needed)
			{
				IEnumerable<UEBuildModuleCPP> CPPModules = Modules.OfType<UEBuildModuleCPP>();

				foreach (UEBuildModuleCPP item in CPPModules)
				{
					bool bAddDependencies = false;
					if (!Needed.Contains(item))
					{
						Needed.Add(item);
						bAddDependencies = true;
					}

					if (bAddDependencies)
					{
						if (item.PrivateDependencyModules != null && item.PrivateDependencyModules.Count > 0)
						{
							GetModulesDependenciesRecursively(item.PrivateDependencyModules.OfType<UEBuildModuleCPP>(), ref Needed);
						}

						if (item.PublicDependencyModules != null && item.PublicDependencyModules.Count > 0)
						{
							GetModulesDependenciesRecursively(item.PublicDependencyModules.OfType<UEBuildModuleCPP>(), ref Needed);
						}
					}
				}
			}

			UEBuildBinary executableBinary = Binaries[0];
			executableBinary.bAllowExports = true; // Need to export symbols because other binaries might depend on this. (They will be stripped anyway if not used)

			UEBuildModuleCPP launchModule = executableBinary.PrimaryModule;
			HashSet<UEBuildModuleCPP> allModules = new(Binaries.SelectMany(x => x.Modules.OfType<UEBuildModuleCPP>().Where(x => x != launchModule)));

			if ((Rules.MergePlugins.Any() || Rules.MergePluginsShared.Any()) && BuildPlugins != null)
			{
				// Dependency rules:
				// - Regular (non-shared) merged libraries cannot contain plugins that are dependencies of other merged libraries.
				// - Regular (non-shared) merged libraries can depend on any number of shared merged libraries.
				// - Shared merged libraries cannot have dependencies other than the base executable to avoid circularity.
				// - Plugins that are part of no merged library will be pulled into the executable *with their dependencies*, so they should be minimized.
				// - Some plugins are organizational in nature and should be ignored for dependencies.

				// Config setup:
				// - Rules.MergePlugin represents merged libraries: everything in that plugin, direct plugin dependencies and dependent modules.
				// - Rules.MergePluginsShared represents shared merged libraries: same as above, except it's a list of plugins, and plugins in a merged library are excluded.
				// - Rules.MergePluginsIgnored: plugins ignored during the merging process.
				// - Rules.MergePluginsExcluded: plugins explicitly removed from all merged libraries.

				Dictionary<string, HashSet<UEBuildPlugin>> pluginGroups = new();
				Dictionary<string, HashSet<UEBuildPlugin>> pluginGroupDependencies = new();

				// We want the ability to add entire root GFPs to a shared library without a separate dedicated library.
				// If there's a sharing rule but no root library, we'll add a fake root here, and later on remap its contents to the shared one.
				List<string> rootPluginsToMerge = new();
				Dictionary<string, string> rootPluginRemappings = new();
				foreach (string name in Rules.MergePlugins)
				{
					rootPluginsToMerge.Add(name);
				}
				foreach (KeyValuePair<string, IEnumerable<string>> sharedGroupRule in Rules.MergePluginsShared)
				{
					foreach (string name in sharedGroupRule.Value)
					{
						if (BuildPlugins.Any(x => x.Name == name) && !Rules.MergePlugins.Contains(name))
						{
							rootPluginsToMerge.Add(name);
							rootPluginRemappings.Add(name, sharedGroupRule.Key);
						}
					}
				}

				// When processing root GFPs, we add the plugin and direct dependencies to a plugin group.
				// Groups will then be expanded with indirect dependencies.
				foreach (string name in rootPluginsToMerge)
				{
					UEBuildPlugin? plugin = BuildPlugins.FirstOrDefault(x => x.Name == name);
					if (plugin == null)
					{
						Logger.LogInformation("{Name}: setup for merging but not built", name);
					}
					else if (CanPluginBeMerged(plugin))
					{
						HashSet<UEBuildPlugin> self = new HashSet<UEBuildPlugin> { plugin };
						HashSet<UEBuildPlugin> dependencies = new();
						GetPluginDependencies(plugin, ref dependencies, false);

						pluginGroups.Add(name, self);
						pluginGroupDependencies.Add(name, dependencies);

						Logger.LogInformation("{Name}: setup for merging with {NumDependencies} total dependencies", name, dependencies.Count);
					}
				}

				// Set up shared dynamic libraries with an empty group
				foreach (KeyValuePair<string, IEnumerable<string>> sharedGroupRule in Rules.MergePluginsShared)
				{
					pluginGroups.Add(sharedGroupRule.Key, new());
				}

				// Expand dependencies from the roots
				HashSet<UEBuildPlugin> seenPlugins = new();
				foreach (KeyValuePair<string, HashSet<UEBuildPlugin>> group in pluginGroups)
				{
					void AddPluginToGroup(UEBuildPlugin plugin, string groupName)
					{
						Logger.LogDebug("{PluginName}: added to plugin group {GroupName}", plugin.Name, groupName);

						pluginGroups[groupName].Add(plugin);

						if (plugin.Dependencies != null)
						{
							ProcessPluginsRecursively(groupName, plugin.Dependencies);
						}
					}

					void ProcessPluginsRecursively(string groupName, HashSet<UEBuildPlugin> plugins)
					{
						foreach (UEBuildPlugin plugin in plugins)
						{
							if (!seenPlugins.Contains(plugin) && CanPluginBeMerged(plugin))
							{
								seenPlugins.Add(plugin);

								IEnumerable<string> groupsWithPlugin = pluginGroupDependencies.Where(x => x.Value.Contains(plugin)).Select(x => x.Key);

								// Plugin is unique to this group
								if (groupsWithPlugin.Count() == 1)
								{
									AddPluginToGroup(plugin, groupName);
								}

								// Plugin is shared
								else
								{
									IEnumerable<KeyValuePair<string, IEnumerable<string>>> sharedGroupCandidates =
										Rules.MergePluginsShared.Where(x => x.Value.Intersect(groupsWithPlugin).Count() == groupsWithPlugin.Count());

									if (sharedGroupCandidates.Any())
									{
										sharedGroupCandidates = sharedGroupCandidates.OrderByDescending(x => x.Value.Count());
										KeyValuePair<string, IEnumerable<string>> sharedGroupRule = sharedGroupCandidates.First();

										AddPluginToGroup(plugin, sharedGroupRule.Key);
									}
									else
									{
										Logger.LogDebug("{PluginName}: unmergeable", plugin.Name);
									}
								}
							}
						}
					}

					// There's only one entry here at this point
					HashSet<UEBuildPlugin>? dependencies = group.Value.FirstOrDefault()?.Dependencies;
					if (dependencies != null)
					{
						ProcessPluginsRecursively(group.Key, dependencies);
					}
				}

				// Process the remapping rules for fully shared root GFPs.
				foreach (KeyValuePair<string, string> remapping in rootPluginRemappings)
				{
					string source = remapping.Key;
					string destination = remapping.Value;

					pluginGroups[destination].UnionWith(pluginGroups[source]);
					pluginGroups.Remove(source);
				}

				// At this point, we have to exclude any code that falls under remaining unmergeable plugins, dependencies included.
				// We need that because there is no way the executable can depend on a merged library, and this remaining chunk would always be loaded as its own library.
				IEnumerable<UEBuildPlugin> ignoredPlugins = BuildPlugins.Where(x => Rules.MergePluginsIgnored.Contains(x.Name));
				HashSet<UEBuildPlugin> unmergablePlugins = BuildPlugins.Except(ignoredPlugins).Except(pluginGroups.SelectMany(x => x.Value)).ToHashSet();
				unmergablePlugins.UnionWith(BuildPlugins.Where(x => Rules.MergePluginsExcluded.Contains(x.Name)));

				// Update unmergeable plugins by accounting for their dependencies
				Dictionary<UEBuildPlugin, HashSet<string>> unmergeablePluginDependencies = new();
				foreach (UEBuildPlugin plugin in unmergablePlugins)
				{
					HashSet<UEBuildPlugin> dependencies = new();
					bool bShouldFollowEveryDependency = Rules.MergePluginsExcluded.Contains(plugin.Name);
					GetPluginDependencies(plugin, ref dependencies, !bShouldFollowEveryDependency);

					foreach (UEBuildPlugin dependency in dependencies)
					{
						if (!unmergeablePluginDependencies.ContainsKey(dependency))
						{
							HashSet<string> pluginNames = new();
							pluginNames.Add(plugin.Name);
							unmergeablePluginDependencies.Add(dependency, pluginNames);
						}
						else
						{
							unmergeablePluginDependencies[dependency].Add(plugin.Name);
						}
					}
				}

				// Report unmergeable dependencies
				unmergablePlugins = new(unmergablePlugins.Union(unmergeablePluginDependencies.Keys));
				foreach (KeyValuePair<UEBuildPlugin, HashSet<string>> dependency in unmergeablePluginDependencies)
				{
					if (CanPluginBeMerged(dependency.Key))
					{
						if (dependency.Key.Modules.Any())
						{
							Logger.LogDebug("{PluginName}: unmergeable (dependency of: {DependantPluginNames})", dependency.Key, String.Join(",", dependency.Value));
						}
					}
				}

				Logger.LogInformation("Found a total of {NumPlugins} plugins to merge", pluginGroups.SelectMany(x => x.Value).Count());

				// Remove unmergable plugin modules from plugin modules and create a set of all plugin modules
				HashSet<UEBuildModuleCPP> allPluginModules = new HashSet<UEBuildModuleCPP>();
				foreach (KeyValuePair<string, HashSet<UEBuildPlugin>> item in pluginGroups)
				{
					// Find all modules for the unique plugins to be merged, except those that were unmergable due to being a dependency of an unmerged plugin
					item.Value.ExceptWith(unmergablePlugins);
					IEnumerable<UEBuildModuleCPP> pluginModules = item.Value.SelectMany(x => x.Modules).Distinct();

					allPluginModules.UnionWith(pluginModules);
				}

				// create a set of leftover modules from all modules after removing plugin modules
				HashSet<UEBuildModuleCPP> allLeftoverModules = new HashSet<UEBuildModuleCPP>(allModules);
				allLeftoverModules.ExceptWith(allPluginModules);

				// create a set of all dependencies of leftover modules
				HashSet<UEBuildModuleCPP> allLeftoverDependencies = new HashSet<UEBuildModuleCPP>();
				GetModulesDependenciesRecursively(allLeftoverModules.OfType<UEBuildModuleCPP>(), ref allLeftoverDependencies);

				Dictionary<string, IEnumerable<ModuleRules>> moduleGroups = new();
				Dictionary<string, Dictionary<string, List<string>>> rootPluginUsageMap = new();
				if (Rules.bReportModuleMergingData)
				{ 
					// prepare the merged modules group data for validation by the game
					moduleGroups.Add("", allLeftoverDependencies.Select(x => x.Rules));
					foreach (UEBuildModuleCPP module in allLeftoverDependencies)
					{
						IEnumerable<UEBuildPlugin> pluginCandidates = BuildPlugins.Where(x => x.Modules.Contains(module));
						if (pluginCandidates.Any())
						{
							List<string> userNames = new();
							UEBuildPlugin plugin = pluginCandidates.First();

							void FindRootUsers(UEBuildPlugin plugin, ref Dictionary<string, List<string>> roots, List<string> path, HashSet<string> seen)
							{
								List<string> newPath = new(path);
								newPath.Add(plugin.Name);
								seen.Add(plugin.Name);

								if (globalPluginUserMap.ContainsKey(plugin))
								{
									foreach (UEBuildPlugin user in globalPluginUserMap[plugin])
									{
										if (!seen.Contains(user.Name))
										{
											FindRootUsers(user, ref roots, newPath, seen);
										}
									}
								}
								else
								{
									if (!roots.ContainsKey(plugin.Name))
									{
										roots.Add(plugin.Name, newPath);
									}
									else if (path.Count < roots[plugin.Name].Count)
									{
										roots[plugin.Name] = newPath;
									}
								}
							}

							Dictionary<string, List<string>> roots = new();
							FindRootUsers(plugin, ref roots, new(), new());
							rootPluginUsageMap.Add(module.Name, roots);
						}
					}
				}

				// Remove all leftover dependencies from plugin modules
				// then remove remaining plugin modules from all modules set
				// then merge the module into a binary and update plugin modules in all modules to that binary
				foreach (KeyValuePair<string, HashSet<UEBuildPlugin>> item in pluginGroups)
				{
					// Find all modules for the unique plugins to be merged, except those that were unmergable due to being a dependency of an unmerged plugin
					HashSet<UEBuildModuleCPP> pluginModules = item.Value.SelectMany(x => x.Modules).ToHashSet();

					pluginModules.ExceptWith(allLeftoverDependencies);
					allModules.ExceptWith(pluginModules);
					moduleGroups.Add(item.Key, pluginModules.Select(x => x.Rules));

					ConsumeModules(item.Key, pluginModules);
				}

				if (Rules.bReportModuleMergingData)
				{
					Rules.ReportMergedModules(moduleGroups, rootPluginUsageMap);
				}
			}

			List<UEBuildModuleCPP> engineModules = [];
			HashSet<UEBuildModuleCPP> excludedModules = [];

			// Exclude the Instrumentation module from merging so it stays as its own small DLL.
			// Its symbols are dllimport'd by PerModuleInline.inl in every other DLL, so it needs
			// a standalone import library. Combined with ModuleSymbolVisibility = VisibileForDll
			// in Instrumentation.Build.cs to ensure DLLEXPORT even with bMergeModules.
			allModules.RemoveWhere(m => m.Name == "Instrumentation");

			// Collect engine modules and find the ones that are explicitly moved to common
			foreach (UEBuildModuleCPP m in allModules)
			{
				if (!m.RulesFile.IsUnderDirectory(Unreal.EngineDirectory))
				{
					continue;
				}
				if (Rules.MergeAdditionalCommonModules.Contains(m.Name))
				{
					excludedModules.Add(m);
				}
				else
				{
					engineModules.Add(m);
				}
			}

			// Find all the engine modules that depends on the ones that was moved to common and move them too
			if (excludedModules.Count != 0)
			{
				while (engineModules.RemoveAll((m) =>
				{
					foreach (UEBuildModule m2 in m.PublicDependencyModules!.Union(m.PrivateDependencyModules!))
					{
						if (!excludedModules.Contains(m2))
						{
							continue;
						}
						excludedModules.Add(m);
						return true;
					}
					return false;
				}) != 0)
				{
				}
			}

			// Merge remaining engine modules
			ConsumeModules("Engine", engineModules);

			// Merge remaining modules to common set
			IEnumerable<UEBuildModuleCPP> commonModules = allModules.Except(engineModules);
			ConsumeModules("Common", commonModules);

			// Remove any binaries that no longer have modules due to merging
			Binaries.ForEach(x => x.Modules.RemoveAll(y => y.Binary != x));
			Binaries.RemoveAll(x => !x.Modules.Any());

			// Set module that should pull in PerModuleInline file
			Binaries.ForEach(SetPerModuleInline);
		}

		/// <summary>
		/// Setup target before build. This method finds dependencies, sets up global environment etc.
		/// </summary>
		public void PreBuildSetup(ILogger Logger)
		{
			// Describe what's being built.
			Logger.LogDebug("Building {AppName} - {TargetName} - {Platform} - {Configuration}", AppName, TargetName, Platform, Configuration);

			// Setup the target's binaries.
			SetupBinaries(Logger);

			// Setup the target's modules referenced in the .uproject
			SetupProjectModules(Logger);

			// Setup the target's plugins
			SetupPlugins(Logger);

			// Add the plugin binaries to the build
			foreach (UEBuildPlugin Plugin in BuildPlugins!)
			{
				foreach (UEBuildModuleCPP Module in Plugin.Modules)
				{
					AddModuleToBinary(Module);
				}
			}

			// Add all of the extra modules, including game modules, that need to be compiled along
			// with this app.  These modules are always statically linked in monolithic targets, but not necessarily linked to anything in modular targets,
			// and may still be required at runtime in order for the application to load and function properly!
			AddExtraModules(Logger);

			// Create all the modules referenced by the existing binaries
			UEBuildModule CreateModuleDelegate(string Name, string RefChain) => FindOrCreateModuleByName(Name, RefChain, Logger);
			StringBuilder ReferenceChain = new(TargetName);
			foreach (UEBuildBinary Binary in Binaries)
			{
				foreach (UEBuildModule Module in Binary.Modules)
				{
					Module.RecursivelyCreateModules(CreateModuleDelegate, ReferenceChain, Logger);
				}
			}

			// Cache dependencies since we can do this in parallel for all modules
			if (ShouldCompileMonolithic())
			{
				UEBuildModule.CacheDependenciesForStableTopologicalSort(Binaries.SelectMany(b => b.Modules.Take(1)));
			}
			else
			{
				UEBuildModule.CacheDependenciesForStableTopologicalSort(Modules.Values.OfType<UEBuildModuleCPP>());
			}

			// Bind every referenced C++ module to a binary
			for (int Idx = 0; Idx < Binaries.Count; Idx++)
				{
					IEnumerable<UEBuildModule> DependencyModules = Binaries[Idx].GetAllDependencyModules(true, true);
					foreach (UEBuildModule DependencyModule in DependencyModules)
					{
						if (DependencyModule.Binary == null && DependencyModule is UEBuildModuleCPP DependencyModuleCpp) // Do type check here, much faster than in enumerable
						{
							AddModuleToBinary(DependencyModuleCpp);
						}
					}
				}

			// Add all the modules to the target if necessary.
			if (Rules.bBuildAllModules)
			{
				AddAllValidModulesToTarget(Logger);
			}

			// Merge modules into fewer libraries if requested.
			if (Rules.bMergeModules)
			{
				SetupMergeModules(Logger);
			}

			// Add the external and non-C++ referenced modules to the binaries that reference them.
			// Note, we want to add them sorted since we want binary modules to be sorted
			foreach (UEBuildBinary binary in Binaries)
			{
				for (int i=0, e= binary.Modules.Count; i!=e; ++i)
				{
					if (binary.Modules[i] is UEBuildModuleCPP Module)
					{
						foreach (UEBuildModule ReferencedModule in Module.GetUnboundReferences())
						{
							binary.AddModule(ReferencedModule);
						}
					}
				}
			}

			if (Rules.bCheckDelegateLifetime)
			{
				// Add module name information for code in dynamic libraries.
				if (Rules.bMergeModules)
				{
					foreach (UEBuildModuleCPP Module in Modules.Values.OfType<UEBuildModuleCPP>())
					{
						if (Module.Binary != null && Module.Binary.Type == UEBuildBinaryType.DynamicLinkLibrary)
						{
							Module.Rules.PrivateDefinitions.Add($"UE_DYNAMIC_MODULE_NAME=\"{Module.Name}\"");
						}
					}
				}

				// When merged modules are disabled, simulate the behavior, except we'll do it for all mergeable plugins as we don't know how the merging would have been done.
				else
				{
					foreach (UEBuildPlugin Plugin in BuildPlugins!)
					{
						foreach (UEBuildModuleCPP Module in Plugin.Modules)
						{
							if (CanPluginBeMerged(Plugin))
							{
								Module.Rules.PrivateDefinitions.Add($"UE_DYNAMIC_MODULE_NAME=\"{Module.Name}\"");
							}
						}
					}
				}
			}

			bool anyValidationErrors = false;

			// Allow each platform to run any needed validation on the plugins.
			using (GlobalTracer.Instance.BuildSpan("UEBuildTarget.PreBuildSetup().ValidatePlugins").StartActive())
			{
				anyValidationErrors |= ValidatePlugins(Logger);
			}

			// Allow each platform to run any needed validation on the module.
			// ie: iOS/macOS do a static library version build check for unsupported clang version builds.
			using (GlobalTracer.Instance.BuildSpan("UEBuildTarget.PreBuildSetup().ValidateModules").StartActive())
			{
				anyValidationErrors |= ValidateModules(Logger);
			}

			if (anyValidationErrors)
			{
				throw new BuildException("Errors validating plugins or modules.");
			}

			if (!bCompileMonolithic)
			{
				if (Platform == UnrealTargetPlatform.Win64)
				{
					// On Windows create import libraries for all binaries ahead of time, since linking binaries often causes bottlenecks
					foreach (UEBuildBinary Binary in Binaries)
					{
						Binary.SetCreateImportLibrarySeparately(true);
					}
				}
				else
				{
					// On other platforms markup all the binaries containing modules with circular references
					foreach (UEBuildModule Module in Modules.Values.Where(x => x.Binary != null))
					{
						foreach (string CircularlyReferencedModuleName in Module.Rules.CircularlyReferencedDependentModules)
						{
							UEBuildModule? CircularlyReferencedModule;
							if (Modules.TryGetValue(CircularlyReferencedModuleName, out CircularlyReferencedModule) && CircularlyReferencedModule.Binary != null)
							{
								CircularlyReferencedModule.Binary.SetCreateImportLibrarySeparately(true);
							}
						}
					}
				}
			}
		}

		/// <summary>
		/// Creates scripts for executing the pre-build scripts
		/// </summary>
		public FileReference[] CreatePreBuildScripts()
		{
			// Find all the pre-build steps
			List<Tuple<string[], UEBuildPlugin?>> PreBuildCommandBatches = new List<Tuple<string[], UEBuildPlugin?>>();
			if (ProjectDescriptor != null && ProjectDescriptor.PreBuildSteps != null)
			{
				AddCustomBuildSteps(ProjectDescriptor.PreBuildSteps, null, PreBuildCommandBatches);
			}
			if (Rules.PreBuildSteps.Count > 0)
			{
				PreBuildCommandBatches.Add(new Tuple<string[], UEBuildPlugin?>(Rules.PreBuildSteps.ToArray(), null));
			}
			foreach (UEBuildPlugin BuildPlugin in BuildPlugins!.Where(x => x.Descriptor.PreBuildSteps != null))
			{
				AddCustomBuildSteps(BuildPlugin.Descriptor.PreBuildSteps!, BuildPlugin, PreBuildCommandBatches);
			}
			return WriteCustomBuildStepScripts(BuildHostPlatform.Current.Platform, ProjectIntermediateDirectory, "PreBuild", PreBuildCommandBatches);
		}

		/// <summary>
		/// Creates scripts for executing post-build steps
		/// </summary>
		/// <returns>Array of post-build scripts</returns>
		private FileReference[] CreatePostBuildScripts()
		{
			// Find all the post-build steps
			List<Tuple<string[], UEBuildPlugin?>> PostBuildCommandBatches = new List<Tuple<string[], UEBuildPlugin?>>();
			if (!Rules.bDisableLinking)
			{
				if (ProjectDescriptor != null && ProjectDescriptor.PostBuildSteps != null)
				{
					AddCustomBuildSteps(ProjectDescriptor.PostBuildSteps, null, PostBuildCommandBatches);
				}
				if (Rules.PostBuildSteps.Count > 0)
				{
					PostBuildCommandBatches.Add(new Tuple<string[], UEBuildPlugin?>(Rules.PostBuildSteps.ToArray(), null));
				}
				foreach (UEBuildPlugin BuildPlugin in BuildPlugins!.Where(x => x.Descriptor.PostBuildSteps != null))
				{
					AddCustomBuildSteps(BuildPlugin.Descriptor.PostBuildSteps!, BuildPlugin, PostBuildCommandBatches);
				}
			}
			return WriteCustomBuildStepScripts(BuildHostPlatform.Current.Platform, ProjectIntermediateDirectory, "PostBuild", PostBuildCommandBatches);
		}

		/// <summary>
		/// Adds custom build steps from the given JSON object to the list of command batches
		/// </summary>
		/// <param name="BuildSteps">The custom build steps</param>
		/// <param name="Plugin">The plugin to associate with these commands</param>
		/// <param name="CommandBatches">List to receive the command batches</param>
		private static void AddCustomBuildSteps(CustomBuildSteps BuildSteps, UEBuildPlugin? Plugin, List<Tuple<string[], UEBuildPlugin?>> CommandBatches)
		{
			string[]? Commands;
			if (BuildSteps.TryGetCommands(BuildHostPlatform.Current.Platform, out Commands))
			{
				CommandBatches.Add(Tuple.Create(Commands, Plugin));
			}
		}

		/// <summary>
		/// Gets a list of variables that can be expanded in paths referenced by this target
		/// </summary>
		/// <param name="Plugin">The current plugin</param>
		/// <returns>Map of variable names to values</returns>
		internal Dictionary<string, string> GetTargetVariables(UEBuildPlugin? Plugin)
		{
			Dictionary<string, string> Variables = new Dictionary<string, string>();
			Variables.Add("RootDir", Unreal.RootDirectory.FullName);
			Variables.Add("EngineDir", Unreal.EngineDirectory.FullName);
			Variables.Add("ProjectDir", ProjectDirectory.FullName);
			Variables.Add("TargetName", TargetName);
			Variables.Add("TargetPlatform", Platform.ToString());
			Variables.Add("TargetConfiguration", Configuration.ToString());
			Variables.Add("TargetType", TargetType.ToString());
			if (ProjectFile != null)
			{
				Variables.Add("ProjectFile", ProjectFile.FullName);
			}
			if (Plugin != null)
			{
				Variables.Add("PluginDir", Plugin.Directory.FullName);
			}
			Variables.Add("BinaryDir", Binaries[0].OutputDir.FullName);
			return Variables;
		}

		/// <summary>
		/// Write scripts containing the custom build steps for the given host platform
		/// </summary>
		/// <param name="HostPlatform">The current host platform</param>
		/// <param name="Directory">The output directory for the scripts</param>
		/// <param name="FilePrefix">Bare prefix for all the created script files</param>
		/// <param name="CommandBatches">List of custom build steps, and their matching PluginInfo (if appropriate)</param>
		/// <returns>List of created script files</returns>
		private FileReference[] WriteCustomBuildStepScripts(UnrealTargetPlatform HostPlatform, DirectoryReference Directory, string FilePrefix, List<Tuple<string[], UEBuildPlugin?>> CommandBatches)
		{
			List<FileReference> ScriptFiles = new List<FileReference>();
			foreach (Tuple<string[], UEBuildPlugin?> CommandBatch in CommandBatches)
			{
				// Find all the standard variables
				Dictionary<string, string> Variables = GetTargetVariables(CommandBatch.Item2);

				// Get the output path to the script
				string ScriptExtension = (HostPlatform == UnrealTargetPlatform.Win64) ? ".bat" : ".sh";
				FileReference ScriptFile = FileReference.Combine(Directory, String.Format("{0}-{1}{2}", FilePrefix, ScriptFiles.Count + 1, ScriptExtension));

				// Write it to disk
				List<string> Contents = new List<string>();
				if (HostPlatform == UnrealTargetPlatform.Win64)
				{
					Contents.Insert(0, "@echo off");
				}
				foreach (string Command in CommandBatch.Item1)
				{
					Contents.Add(Utils.ExpandVariables(Command, Variables));
				}
				if (!DirectoryReference.Exists(ScriptFile.Directory))
				{
					DirectoryReference.CreateDirectory(ScriptFile.Directory);
				}
				File.WriteAllLines(ScriptFile.FullName, Contents);

				// Add the output file to the list of generated scripts
				ScriptFiles.Add(ScriptFile);
			}
			return ScriptFiles.ToArray();
		}

		/// <summary>
		/// Finds a list of module names which can be hot-reloaded
		/// </summary>
		/// <returns>Set of module names</returns>
		private HashSet<string> GetHotReloadModuleNames(TargetDescriptor TargetDescriptor)
		{
			HashSet<string> HotReloadModuleNames = new HashSet<string>(StringComparer.OrdinalIgnoreCase);
			if (Rules.bAllowHotReload && !ShouldCompileMonolithic())
			{
				// Legacy hot reload cannot reload modules from the executable binary
				IEnumerable<UEBuildBinary> ValidBinaries = TargetDescriptor.HotReloadMode == HotReloadMode.FromEditor || TargetDescriptor.HotReloadMode == HotReloadMode.FromIDE
					? Binaries.Skip(1)
					: Binaries;
				foreach (UEBuildBinary Binary in ValidBinaries)
				{
					IEnumerable<UEBuildModule> GameModules = Binary.FindHotReloadModules();
					if (GameModules.Any())
					{
						if (!Unreal.IsProjectInstalled() || EnabledPlugins!.Where(x => x.Type == PluginType.Mod).Any(x => Binary.OutputFilePaths[0].IsUnderDirectory(x.Directory)))
						{
							HotReloadModuleNames.UnionWith(GameModules.OfType<UEBuildModuleCPP>().Where(x => !x.Rules.bUsePrecompiled).Select(x => x.Name));
						}
					}
				}
			}
			return HotReloadModuleNames;
		}

		/// <summary>
		/// Determines which modules can be used to create shared PCHs
		/// </summary>
		/// <param name="OriginalBinaries">The list of binaries</param>
		/// <param name="GlobalCompileEnvironment">The compile environment. The shared PCHs will be added to the SharedPCHs list in this.</param>
		/// <param name="Logger">Logger for output</param>
		public void FindSharedPCHs(List<UEBuildBinary> OriginalBinaries, CppCompileEnvironment GlobalCompileEnvironment, ILogger Logger)
		{
			// Find how many other shared PCH modules each module depends on, and use that to sort the shared PCHs by reverse order of size.
			HashSet<UEBuildModuleCPP> SharedPCHModules = new HashSet<UEBuildModuleCPP>();
			foreach (UEBuildBinary Binary in OriginalBinaries)
			{
				foreach (UEBuildModuleCPP Module in Binary.Modules.OfType<UEBuildModuleCPP>())
				{
					if (Module.Rules.SharedPCHHeaderFile != null)
					{
						SharedPCHModules.Add(Module);
					}
				}
			}

			// Find a priority for each shared PCH, determined as the number of other shared PCHs it includes.
			SortedDictionary<string, (int, PrecompiledHeaderTemplate)> SharedPCHModuleToPriority = new ();
			Parallel2.ForEach(SharedPCHModules, SharedPCHModule =>
			{
				using Pooled<HashSet<UEBuildModule>> rented = HashSetPool<UEBuildModule>.Rent();
				HashSet<UEBuildModule> Dependencies = rented.Value;
				SharedPCHModule.GetAllDependencyModulesForPCH(Dependencies, false);
				Dependencies.IntersectWith(SharedPCHModules);

				// TODO This is bad.. unfortunately does circular dependency logic not working when getting dependency modules
				// If we have Engine with circular dependency on UnrealEd but then is included via some other module, then we're screwed
				foreach (string CircularDependency in SharedPCHModule.Rules.CircularlyReferencedDependentModules)
				{
					if (Modules.TryGetValue(CircularDependency, out UEBuildModule? Module))
					{
						Dependencies.Remove(Module);
					}
				}
				int DependencyCount = Dependencies.Count;
				PrecompiledHeaderTemplate Template = SharedPCHModule.CreateSharedPCHTemplate(this, GlobalCompileEnvironment, Logger);
				lock (SharedPCHModuleToPriority)
				{
					SharedPCHModuleToPriority.Add(SharedPCHModule.Name, (DependencyCount, Template));
				}
			});

			// Create the shared PCH modules, in order
			List<PrecompiledHeaderTemplate> OrderedSharedPCHModules = GlobalCompileEnvironment.SharedPCHs;
			foreach (PrecompiledHeaderTemplate Template in SharedPCHModuleToPriority.OrderByDescending(x => x.Value.Item1).Select(v => v.Value.Item2))
			{
				OrderedSharedPCHModules.Add(Template);
			}

			// Print the ordered list of shared PCHs
			if (OrderedSharedPCHModules.Count > 0)
			{
				Logger.LogDebug("Found {SharedPCHModulesCount} shared PCH headers (listed in order of preference):", SharedPCHModules.Count);
				foreach (PrecompiledHeaderTemplate SharedPCHModule in OrderedSharedPCHModules)
				{
					Logger.LogDebug("	{ModuleName}", SharedPCHModule.Module.Name);
				}
			}
		}

		/// <summary>
		/// Creates all the shared PCH instances before processing the modules
		/// </summary>
		/// <param name="Target">The target being built</param>
		/// <param name="ToolChain">The toolchain to build with</param>
		/// <param name="OriginalBinaries">The list of binaries</param>
		/// <param name="GlobalCompileEnvironment">The compile environment. The shared PCHs will be added to the SharedPCHs list in this.</param>
		/// <param name="Logger">Logger for output</param>
		/// <param name="Graph">List of build actions</param>
		public static void CreateSharedPCHInstances(ReadOnlyTargetRules Target, UEToolChain ToolChain, List<UEBuildBinary> OriginalBinaries, CppCompileEnvironment GlobalCompileEnvironment, IActionGraphBuilder Graph, ILogger Logger)
		{
			int NumSharedPCHs = GlobalCompileEnvironment.SharedPCHs.Count;
			if (!Target.bUsePCHFiles || NumSharedPCHs == 0)
			{
				return;
			}

			ConcurrentBag<Tuple<UEBuildModuleCPP, CppCompileEnvironment>>[] ModuleInfos = new ConcurrentBag<Tuple<UEBuildModuleCPP, CppCompileEnvironment>>[NumSharedPCHs];
			for (int SharedPCHIndex = 0; SharedPCHIndex < NumSharedPCHs; SharedPCHIndex++)
			{
				ModuleInfos[SharedPCHIndex] = new ConcurrentBag<Tuple<UEBuildModuleCPP, CppCompileEnvironment>>();
			}

			// Sometimes it is one binary with lots of modules, sometimes it is lots of binaries with one module each
			Parallel2.ForEach(OriginalBinaries, Binary =>
			{
				CppCompileEnvironment BinaryCompileEnvironment = Binary.CreateBinaryCompileEnvironment(GlobalCompileEnvironment);

				Parallel2.ForEach(Binary.Modules.OfType<UEBuildModuleCPP>(), Module =>
				{
					CppCompileEnvironment ModuleCompileEnvironment = Module.CreateModuleCompileEnvironment(Target, BinaryCompileEnvironment, Logger);

					// Is it using a private PCH?
					if (Module.Rules.PrivatePCHHeaderFile != null && (Module.Rules.PCHUsage == ModuleRules.PCHUsageMode.NoSharedPCHs || Module.Rules.PCHUsage == ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs))
					{
						return;
					}

					// Is it using no PCH?
					if (Module.Rules.PCHUsage == ModuleRules.PCHUsageMode.NoPCHs)
					{
						return;
					}

					if (!ModuleCompileEnvironment.bIsBuildingLibrary && Module.Rules.PCHUsage != ModuleRules.PCHUsageMode.NoSharedPCHs)
					{
						// Find all the dependencies of this module
						using Pooled<HashSet<UEBuildModule>> rented = HashSetPool<UEBuildModule>.Rent();
						HashSet<UEBuildModule> ReferencedModules = rented.Value;
						Module.GetAllDependencyModules(new List<UEBuildModule>(), ReferencedModules, bIncludeDynamicallyLoaded: false, bForceCircular: false, bOnlyDirectDependencies: true);

						// Find the first shared PCH module we can use
						int SharedPCHIndex = ModuleCompileEnvironment.SharedPCHs.FindIndex(x => ReferencedModules.Contains(x.Module));
						if (SharedPCHIndex != -1 && GlobalCompileEnvironment.SharedPCHs[SharedPCHIndex].IsValidFor(ModuleCompileEnvironment))
						{
							ModuleInfos[SharedPCHIndex].Add(new Tuple<UEBuildModuleCPP, CppCompileEnvironment>(Module, ModuleCompileEnvironment));
						}
					}
				});
			});

			// Create the PCH instances
			for (int SharedPCHIndex = NumSharedPCHs - 1; SharedPCHIndex >= 0; SharedPCHIndex--)
			{
				foreach (Tuple<UEBuildModuleCPP, CppCompileEnvironment> ModuleInfo in ModuleInfos[SharedPCHIndex])
				{
					ModuleInfo.Item1.FindOrCreateSharedPCH(ToolChain, GlobalCompileEnvironment.SharedPCHs[SharedPCHIndex], ModuleInfo.Item2, Graph);
				}
			}

			// Throw warnings/errors if there are PCH performance issues
			if (Target.CppCompileWarningSettings.PCHPerformanceIssueWarningLevel != WarningLevel.Off)
			{
				bool bFoundIssues = false;
				LogLevel PCHLoggerLevel = Target.CppCompileWarningSettings.PCHPerformanceIssueWarningLevel == WarningLevel.Warning ? LogLevel.Warning : LogLevel.Error;
				string Prefix = Target.CppCompileWarningSettings.PCHPerformanceIssueWarningLevel == WarningLevel.Warning ? "Warning:" : "Error:";
				for (int SharedPCHIndex = NumSharedPCHs - 1; SharedPCHIndex >= 0; SharedPCHIndex--)
				{
					// Throw warnings or errors if there are module settings that could result in PCH performance issues
					PrecompiledHeaderTemplate SharedPCH = GlobalCompileEnvironment.SharedPCHs[SharedPCHIndex];
					if (SharedPCH.Instances.Any())
					{
						// UnsafeTypeCastWarningLevel
						foreach (PrecompiledHeaderInstance UnsafeTypeCastWarningInstance in SharedPCH.Instances.Where(instance => instance.Modules.First().Rules.CppCompileWarningSettings.UnsafeTypeCastWarningLevel == WarningLevel.Warning))
						{
							CppCompileEnvironment UpdatedCppCompileEnvironment = new CppCompileEnvironment(UnsafeTypeCastWarningInstance.CompileEnvironment);
							UpdatedCppCompileEnvironment.CppCompileWarnings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
							foreach (PrecompiledHeaderInstance UnsafeTypeCastErrorInstance in SharedPCH.Instances.Where(instance => instance.Modules.First().Rules.CppCompileWarningSettings.UnsafeTypeCastWarningLevel == WarningLevel.Error))
							{
								if (UEBuildModuleCPP.IsCompatibleForSharedPCH(UnsafeTypeCastErrorInstance.CompileEnvironment, UpdatedCppCompileEnvironment))
								{
									foreach (UEBuildModuleCPP Module in UnsafeTypeCastWarningInstance.Modules)
									{
										if (Module.RulesFile.ToFileInfo().IsReadOnly) // support for local module changes
										{
											Logger.Log(PCHLoggerLevel, "{Prefix} Module '{Module}': Please set 'UnsafeTypeCastWarningLevel' to 'WarningLevel.Error' instead of 'WarningLevel.Warning'. This creates PCH permutations.", Prefix, Module.Name);
											bFoundIssues = true;
										}
									}
								}
							}
						}

						// OptimizeCode
						foreach (PrecompiledHeaderInstance Instance in SharedPCH.Instances.Where(instance => instance.Modules.First().Rules.OptimizeCode == ModuleRules.CodeOptimization.Never))
						{
							foreach (UEBuildModuleCPP Module in Instance.Modules)
							{
								if (Module.RulesFile.ToFileInfo().IsReadOnly && !(Target.DisableOptimizeCodeForModules?.Contains(Module.Name) ?? false)) // support for local module changes
								{
									Logger.Log(PCHLoggerLevel, "{Prefix} Module '{Module}': Do not set 'OptimizeCode' to 'CodeOptimization.Never'. This creates PCH permutations.", Prefix, Module.Name);
									bFoundIssues = true;
								}
							}
						}
					}
				}

				if (bFoundIssues && PCHLoggerLevel == LogLevel.Error)
				{
					throw new BuildException("PCH performance issues were found.");
				}
			}
		}

		/// <summary>
		/// When building a target, this is called to add any additional modules that should be compiled along
		/// with the main target.  If you override this in a derived class, remember to call the base implementation!
		/// </summary>
		protected void AddExtraModules(ILogger Logger)
		{
			// Find all the extra module names
			List<string> ExtraModuleNames = new List<string>();
			ExtraModuleNames.AddRange(Rules.ExtraModuleNames);
			UEBuildPlatform.GetBuildPlatform(Platform).AddExtraModules(Rules, ExtraModuleNames);

			// Add extra modules that will either link into the main binary (monolithic), or be linked into separate DLL files (modular)
			foreach (string ModuleName in ExtraModuleNames)
			{
				UEBuildModuleCPP Module = FindOrCreateCppModuleByName(ModuleName, TargetRulesFile.GetFileName(), Logger);
				if (Module.Binary == null)
				{
					AddModuleToBinary(Module);
				}
			}
		}

		/// <summary>
		/// Adds all the precompiled modules into the target. Precompiled modules are compiled alongside the target, but not linked into it unless directly referenced.
		/// </summary>
		protected void AddAllValidModulesToTarget(ILogger Logger)
		{
			// Find all the modules that are part of the target
			using Pooled<HashSet<string>> rented = HashSetPool<string>.Rent();
			HashSet<string> ValidModuleNames = rented.Value;

			foreach (UEBuildModuleCPP Module in Modules.Values.OfType<UEBuildModuleCPP>())
			{
				if (Module.Binary != null)
				{
					ValidModuleNames.Add(Module.Name);
				}
			}

			// Find all the platform folders to exclude from the list of valid modules
			IReadOnlySet<string> ExcludeFolders = UEBuildPlatform.GetBuildPlatform(Platform).GetExcludedFolderNames();

			// Set of module names to build
			using Pooled<HashSet<string>> rented2 = HashSetPool<string>.Rent();
			HashSet<string> FilteredModuleNames = rented2.Value;

			// Only add engine modules for non-program targets. Programs only compile allowed modules through plugins.
			if (TargetType != TargetType.Program)
			{
				// Find all the known module names in this assembly
				List<string> ModuleNames = new List<string>();
				RulesAssembly.GetAllModuleNames(ModuleNames);

				// Find all the directories containing engine modules that may be compatible with this target
				List<DirectoryReference> Directories = new List<DirectoryReference>();
				if (TargetType == TargetType.Editor)
				{
					Directories.AddRange(Unreal.GetExtensionDirs(Unreal.EngineDirectory, "Source/Editor"));
				}
				Directories.AddRange(Unreal.GetExtensionDirs(Unreal.EngineDirectory, "Source/Runtime"));

				// Also allow anything in the developer directory in non-shipping configurations (though we deny by default
				// unless the PrecompileForTargets setting indicates that it's actually useful at runtime).
				if (Rules.bBuildDeveloperTools)
				{
					Directories.AddRange(Unreal.GetExtensionDirs(Unreal.EngineDirectory, "Source/Developer"));
				}

				// Find all the modules that are not part of the standard set
				foreach (string ModuleName in ModuleNames)
				{
					FileReference ModuleFileName = RulesAssembly.GetModuleFileName(ModuleName)!;
					if (!Directories.Any(BaseDir => ModuleFileName.IsUnderDirectory(BaseDir)))
					{
						Logger.LogDebug("Excluding module {Module}: Not under compatible directories", ModuleName);
						continue;
					}

					Type? RulesType = RulesAssembly.GetModuleRulesType(ModuleName);

					if (RulesType == null)
					{
						Logger.LogDebug("Excluding module {Module}: Unable to get module rules type from rules assemblies", ModuleName);
						continue;
					}
					else if (!ModuleRules.IsValidForTarget(RulesType, Rules, out string? InvalidReason))
					{
						Logger.LogDebug("Excluding module {Module}: Does not support {InvalidReason}", ModuleName, InvalidReason);
						continue;
					}

					// Skip platform extension modules. We only care about the base modules, not the platform overrides.
					// The platform overrides get applied at a later stage when we actually come to build the module.
					if (!UEBuildPlatform.GetPlatformFolderNames().Any(Name => RulesType.Name.EndsWith("_" + Name, StringComparison.Ordinal)))
					{
						if (ModuleFileName.ContainsAnyNames(ExcludeFolders, Unreal.EngineDirectory))
						{
							Logger.LogDebug("Excluding module {Module}: Excluded folder name", ModuleName);
							continue;
						}
						FilteredModuleNames.Add(ModuleName);
					}
				}
			}

			// Add all the plugin modules that need to be compiled
			IEnumerable<PluginInfo> Plugins = RulesAssembly.EnumeratePlugins().Select(x => x.ChoiceVersion).OfType<PluginInfo>();
			Dictionary<string, PluginInfo> ModulePluginSource = new();
			foreach (PluginInfo Plugin in Plugins)
			{
				// Ignore plugins which are specifically disabled by this target
				if (Rules.DisablePlugins.Contains(Plugin.Name))
				{
					continue;
				}

				// Ignore plugins without any modules
				if (Plugin.Descriptor.Modules == null)
				{
					continue;
				}

				// Disable any plugin which does not support the target platform. The editor should update such references in the .uproject file on load.
				if (!Rules.bIncludePluginsForTargetPlatforms && !Plugin.Descriptor.SupportsTargetPlatform(Platform))
				{
					continue;
				}

				// Disable any plugin that requires the build platform
				if (Plugin.Descriptor.bRequiresBuildPlatform && ShouldExcludePlugin(Plugin, ExcludeFolders))
				{
					continue;
				}

				// Disable any plugins that aren't compatible with this program
				if (TargetType == TargetType.Program && (Plugin.Descriptor.SupportedPrograms == null || !Plugin.Descriptor.SupportedPrograms.Contains(AppName)))
				{
					continue;
				}

				// Add all the modules
				foreach (ModuleDescriptor ModuleDescriptor in Plugin.Descriptor.Modules)
				{
					if (ModuleDescriptor.IsCompiledInConfiguration(Platform, Configuration, TargetName, TargetType, Rules.bBuildDeveloperTools, Rules.bBuildRequiresCookedData, Architectures))
					{
						FileReference? ModuleFileName = RulesAssembly.GetModuleFileName(ModuleDescriptor.Name);
						if (ModuleFileName == null)
						{
							throw new BuildException("Unable to find module '{0}' referenced by {1}", ModuleDescriptor.Name, Plugin.File);
						}
						Type? RulesType = RulesAssembly.GetModuleRulesType(ModuleDescriptor.Name);
						if (RulesType == null)
						{
							Logger.LogDebug("Excluding plugin {Plugin} module {Module}: Unable to get module rules type from rules assemblies", Plugin.Name, ModuleDescriptor.Name);
							continue;
						}
						else if (!ModuleRules.IsValidForTarget(RulesType, Rules, out string? InvalidReason))
						{
							Logger.LogDebug("Excluding plugin {Plugin} module {Module}: Does not support {InvalidReason}", Plugin.Name, ModuleDescriptor.Name, InvalidReason);
							continue;
						}
						if (ModuleFileName.ContainsAnyNames(ExcludeFolders, Plugin.Directory))
						{
							Logger.LogDebug("Excluding plugin {Plugin} module {Module}: Excluded folder name", Plugin.Name, ModuleDescriptor.Name);
							continue;
						}
						if (FilteredModuleNames.Add(ModuleDescriptor.Name))
						{
							ModulePluginSource.Add(ModuleDescriptor.Name, Plugin);
						}
					}
				}
			}

			// Create rules for each remaining module, and check that it's set to be compiled
			foreach (string FilteredModuleName in FilteredModuleNames)
			{
				// Try to create the rules object, but catch any exceptions if it fails. Some modules (eg. SQLite) may determine that they are unavailable in the constructor.
				ModuleRules? ModuleRules;
				try
				{
					string PrecompileReferenceChain = $"{TargetName} allmodules option";
					if (ModulePluginSource.TryGetValue(FilteredModuleName, out PluginInfo? value))
					{
						PrecompileReferenceChain = $"{PrecompileReferenceChain} -> {value.File.GetFileName()}";
					}
					ModuleRules = RulesAssembly.CreateModuleRules(FilteredModuleName, Rules, PrecompileReferenceChain, Logger);
				}
				catch (BuildException)
				{
					ModuleRules = null;
				}

				// Figure out if it can be precompiled
				if (ModuleRules != null)
				{
					if (!ModuleRules.IsValidForTarget(ModuleRules.File))
					{
						Logger.LogDebug("Excluding module {Module}: Not precompiled for target '{TargetType}'", ModuleRules.Name, Rules.Type);
						continue;
					}
					ValidModuleNames.Add(FilteredModuleName);
				}
			}

			// Used to capture modules that were referenced while creating all the precompiled modules

			// Now create all the precompiled modules, making sure they don't reference anything that's not in the precompiled set

			// Gather the set of all modules traversed while processing ValidModuleNames, to ensure we have any extra modules found during RecursivelyCreateModules()
			using Pooled<HashSet<UEBuildModule>> rented3 = HashSetPool<UEBuildModule>.Rent();
			HashSet<UEBuildModule> AllModules = rented3.Value;

			foreach (string ModuleName in ValidModuleNames)
			{
				StringBuilder PrecompileReferenceChain = new($"{TargetName} allmodules option");
				if (ModulePluginSource.TryGetValue(ModuleName, out PluginInfo? value))
				{
					PrecompileReferenceChain.Append(" -> ").Append(value.File.GetFileName());
				}
				UEBuildModule Module = FindOrCreateModuleByName(ModuleName, PrecompileReferenceChain.ToString(), Logger);
				AllModules.Add(Module);
				Module.RecursivelyCreateModules(
					(string ModuleName, string ReferenceChain) =>
					{
						UEBuildModule FoundModule = FindOrCreateModuleByName(ModuleName, ReferenceChain, Logger);
						AllModules.Add(FoundModule);
						return FoundModule;
					},
					PrecompileReferenceChain, Logger);
			}

			// Exclude additional modules that were added only for include-path-only purposes, and those that will have their Verse dependency satisfied
			HashSet<UEBuildModuleCPP> ValidModules = new HashSet<UEBuildModuleCPP>(
				AllModules.OfType<UEBuildModuleCPP>().Where(x => x.PrivateIncludePathModules != null));

			// Make sure precompiled modules don't reference any non-precompiled modules
			foreach (UEBuildModuleCPP ValidModule in ValidModules)
			{
				foreach (UEBuildModuleCPP ReferencedModule in ValidModule.GetDependencies(false, true).OfType<UEBuildModuleCPP>())
				{
					if (!ValidModules.Contains(ReferencedModule))
					{
						Logger.LogError("Module '{ValidModuleName}' is not usable without module '{ReferencedModuleName}', which is not valid for this target.", ValidModule.Name, ReferencedModule.Name);
					}
				}

				if (ValidModule.Binary == null)
				{
					AddModuleToBinary(ValidModule);
				}
			}
		}

		public void AddModuleToBinary(UEBuildModuleCPP Module)
		{
			if (ShouldCompileMonolithic())
			{
				// When linking monolithically, any unbound modules will be linked into the main executable
				Module.Binary = Binaries[0];
				Module.Binary.AddModule(Module);
			}
			else
			{
				// Otherwise create a new module for it
				Module.Binary = CreateDynamicLibraryForModule(Module);
				Binaries.Add(Module.Binary);
			}
		}

		/// <summary>
		/// Gets the output directory for a target
		/// </summary>
		/// <param name="BaseDir">The base directory for output files</param>
		/// <param name="TargetFile">Path to the target file</param>
		/// <returns>The output directory for this target</returns>
		public static DirectoryReference GetOutputDirectoryForExecutable(DirectoryReference BaseDir, FileReference TargetFile)
		{
			return Unreal.GetExtensionDirs(BaseDir).Where(x => TargetFile.IsUnderDirectory(x)).OrderByDescending(x => x.FullName.Length).FirstOrDefault() ?? Unreal.EngineDirectory;
		}

		/// <summary>
		/// Finds the base output directory for build products of the given module
		/// </summary>
		/// <param name="ModuleRules">The rules object created for this module</param>
		/// <returns>The base output directory for compiled object files for this module</returns>
		private DirectoryReference GetBaseOutputDirectory(ModuleRules ModuleRules)
		{
			// Get the root output directory and base name (target name/app name) for this binary
			DirectoryReference BaseOutputDirectory;

			// if this module cares about any SDK overrides, and at least one was overridden, then treat the module as a project module
			bool bHasImportantSDKOverride = false;
			if (UEBuildPlatformSDK.bHasAnySDKOverride)
			{
				foreach (UnrealTargetPlatform Platform in ModuleRules.SDKVersionRelevantPlatforms)
				{
					UEBuildPlatformSDK? SDK = UEBuildPlatformSDK.GetSDKForPlatform(Platform.ToString());
					if (SDK != null && SDK.bHasSDKOverride)
					{
						bHasImportantSDKOverride = true;
					}
				}
			}

			if (bUseSharedBuildEnvironment && !bHasImportantSDKOverride && (ModuleRules.Plugin == null || (ModuleRules.Plugin.Type != PluginType.External || ModuleRules.Plugin.bExplicitPluginTarget)))
			{
				BaseOutputDirectory = ModuleRules.Context.DefaultOutputBaseDir;
			}
			else
			{
				BaseOutputDirectory = ProjectDirectory;
			}
			return BaseOutputDirectory;
		}

		/// <summary>
		/// Finds the base output directory for a module
		/// </summary>
		/// <param name="ModuleRules">The rules object created for this module</param>
		/// <param name="Architectures">The architectures (or none) to insert into the directory structure</param>
		/// <returns>The output directory for compiled object files for this module</returns>
		private DirectoryReference GetModuleIntermediateDirectory(ModuleRules ModuleRules, UnrealArchitectures? Architectures)
		{
			// Get the root output directory and base name (target name/app name) for this binary
			DirectoryReference BaseOutputDirectory = GetBaseOutputDirectory(ModuleRules);
			if (BaseOutputDirectory == Unreal.EngineDirectory)
			{
				BaseOutputDirectory = GetOutputDirectoryForExecutable(BaseOutputDirectory, ModuleRules.File);
			}

			// Get the configuration that this module will be built in. Engine modules compiled in DebugGame will use Development.
			UnrealTargetConfiguration ModuleConfiguration = Configuration;
			if (Configuration == UnrealTargetConfiguration.DebugGame && !ModuleRules.Context.bCanBuildDebugGame && !ModuleRules.Name.Equals(Rules.LaunchModuleName, StringComparison.OrdinalIgnoreCase))
			{
				ModuleConfiguration = UnrealTargetConfiguration.Development;
			}

			bool bUseExternalFolder = ModuleRules.Plugin != null && (ModuleRules.Plugin.Type == PluginType.External && !ModuleRules.Plugin.bExplicitPluginTarget);

			// Get the output and intermediate directories for this module
			DirectoryReference IntermediateDirectory = DirectoryReference.Combine(BaseOutputDirectory, GetPlatformIntermediateFolder(Platform, Architectures, bUseExternalFolder), GetTargetIntermediateFolderName(AppName, IntermediateEnvironment), ModuleConfiguration.ToString());

			// Append a subdirectory if the module rules specifies one
			if (!String.IsNullOrEmpty(ModuleRules.BinariesSubFolder))
			{
				IntermediateDirectory = DirectoryReference.Combine(IntermediateDirectory, ModuleRules.BinariesSubFolder);
			}

			return DirectoryReference.Combine(IntermediateDirectory, ModuleRules.ShortName ?? ModuleRules.Name);
		}

		/// <summary>
		/// Adds a dynamic library for the given module. Does not check whether a binary already exists, or whether a binary should be created for this build configuration.
		/// </summary>
		/// <param name="Module">The module to create a binary for</param>
		/// <returns>The new binary. This has not been added to the target.</returns>
		private UEBuildBinary CreateDynamicLibraryForModule(UEBuildModuleCPP Module)
		{
			// Get the root output directory and base name (target name/app name) for this binary
			DirectoryReference BaseOutputDirectory = GetBaseOutputDirectory(Module.Rules);
			DirectoryReference OutputDirectory = DirectoryReference.Combine(BaseOutputDirectory, "Binaries", Platform.ToString());

			// Append a subdirectory if the module rules specifies one
			if (!String.IsNullOrEmpty(Module.Rules.BinariesSubFolder))
			{
				OutputDirectory = DirectoryReference.Combine(OutputDirectory, Module.Rules.BinariesSubFolder);
			}

			// Get the configuration that this module will be built in. Engine modules compiled in DebugGame will use Development.
			UnrealTargetConfiguration ModuleConfiguration = Configuration;
			if (Configuration == UnrealTargetConfiguration.DebugGame && !Module.Rules.Context.bCanBuildDebugGame)
			{
				ModuleConfiguration = UnrealTargetConfiguration.Development;
			}

			// Get the output filenames
			FileReference BaseBinaryPath = FileReference.Combine(OutputDirectory, MakeBinaryFileName($"{AppName}{Rules.DecoratedSeparator}{Module.Name}", Rules.DecoratedSeparator, Platform, ModuleConfiguration, Architectures, Rules.UndecoratedConfiguration, UEBuildBinaryType.DynamicLinkLibrary));
			List<FileReference> OutputFilePaths = UEBuildPlatform.GetBuildPlatform(Platform).FinalizeBinaryPaths(BaseBinaryPath, ProjectFile, Rules);

			// Create the binary
			return new UEBuildBinary(
				Type: UEBuildBinaryType.DynamicLinkLibrary,
				OutputFilePaths: OutputFilePaths,
				IntermediateDirectory: Module.IntermediateDirectory,
				bAllowExports: true,
				bBuildAdditionalConsoleApp: false,
				bBuildConsoleAppOnly: false,
				PrimaryModule: Module,
				bUsePrecompiled: Module.Rules.bUsePrecompiled
			);
		}

		/// <summary>
		/// Adds a dynamic library for the given set of modules. Does not check whether a binary already exists, or whether a binary should be created for this build configuration.
		/// </summary>
		/// <param name="MergedName">The name of the merged binary</param>
		/// <param name="Modules">The modules to create a binary for</param>
		/// <returns>The new binary. This has not been added to the target.</returns>
		private UEBuildBinary CreateDynamicLibraryForModules(string MergedName, IEnumerable<UEBuildModuleCPP> Modules)
		{
			DirectoryReference MergedIntermediateDirectory = DirectoryReference.Combine(ProjectIntermediateDirectory, MergedName);

			// Get the root output directory and base name (target name/app name) for this binary
			DirectoryReference OutputDirectory;
			if (ProjectFile != null && Rules.File.IsUnderDirectory(ProjectDirectory) || Modules.Any(x => x.Rules.bUsePrecompiled))
			{
				OutputDirectory = GetOutputDirectoryForExecutable(ProjectDirectory, Rules.File);
			}
			else
			{
				OutputDirectory = GetOutputDirectoryForExecutable(Unreal.EngineDirectory, Rules.File);
			}

			// Get the configuration that this module will be built in. Engine modules compiled in DebugGame will use Development.
			UnrealTargetConfiguration ModuleConfiguration = Configuration;
			if (Configuration == UnrealTargetConfiguration.DebugGame && !Modules.All(x => x.Rules.Context.bCanBuildDebugGame))
			{
				ModuleConfiguration = UnrealTargetConfiguration.Development;
			}

			List<FileReference> OutputPaths = MakeBinaryPaths(OutputDirectory, AppName + Rules.DecoratedSeparator + MergedName, Platform, Configuration, UEBuildBinaryType.DynamicLinkLibrary, Rules.Architectures, Rules.UndecoratedConfiguration, Rules.ExeBinariesSubFolder, ProjectFile, Rules);

			// Create the binary
			UEBuildBinary Binary = new UEBuildBinary(
				Type: UEBuildBinaryType.DynamicLinkLibrary,
				OutputFilePaths: OutputPaths,
				IntermediateDirectory: MergedIntermediateDirectory,
				bAllowExports: true,
				bBuildAdditionalConsoleApp: false,
				bBuildConsoleAppOnly: false,
				PrimaryModule: Modules.First(),
				bUsePrecompiled: false
			);

			Binary.Modules.AddRange(Modules.Skip(1));
			return Binary;
		}

		/// <summary>
		/// Makes a filename (without path) for a compiled binary (e.g. "Core-Win64-Debug.lib") */
		/// </summary>
		/// <param name="BinaryName">The name of this binary</param>
		/// <param name="Separator">The separator for a decorated filename</param>
		/// <param name="Platform">The platform being built for</param>
		/// <param name="Configuration">The configuration being built</param>
		/// <param name="Architectures">The target architectures being built</param>
		/// <param name="UndecoratedConfiguration">The target configuration which doesn't require a platform and configuration suffix. Development by default.</param>
		/// <param name="BinaryType">Type of binary</param>
		/// <returns>Name of the binary</returns>
		public static string MakeBinaryFileName(string BinaryName, string Separator, UnrealTargetPlatform Platform, UnrealTargetConfiguration Configuration, UnrealArchitectures Architectures, UnrealTargetConfiguration UndecoratedConfiguration, UEBuildBinaryType BinaryType)
		{
			StringBuilder Result = new StringBuilder();

			// If the binary is a library and is on a platform that expects the
			// "lib" library prefix, add the prefix to the library filename.
			// Platforms that expect this are Android, Apple (Mac & iOS), and Linux.
			bool bBinaryIsLibrary = BinaryType is UEBuildBinaryType.DynamicLinkLibrary or UEBuildBinaryType.StaticLibrary;
			bool bRequiresLibraryFilenamePrefix = UEBuildPlatform.IsPlatformInGroup(Platform, UnrealPlatformGroup.Android) ||
			                                      UEBuildPlatform.IsPlatformInGroup(Platform, UnrealPlatformGroup.Apple) ||
			                                      UEBuildPlatform.IsPlatformInGroup(Platform, UnrealPlatformGroup.Linux);
			if (bBinaryIsLibrary && bRequiresLibraryFilenamePrefix)
			{
				Result.Append("lib");
			}

			Result.Append(BinaryName);

			if (Configuration != UndecoratedConfiguration)
			{
				Result.Append(Separator);
				Result.Append(Platform);
				Result.Append(Separator);
				Result.Append(Configuration);
			}

			UEBuildPlatform BuildPlatform = UEBuildPlatform.GetBuildPlatform(Platform);
			if (BuildPlatform.ArchitectureConfig.RequiresArchitectureFilenames(Architectures))
			{
				Result.Append(Architectures.ToString());
			}

			Result.Append(BuildPlatform.GetBinaryExtension(BinaryType));

			return Result.ToString();
		}

		/// <summary>
		/// Determine the output path for a target's executable
		/// </summary>
		/// <param name="BaseDirectory">The base directory for the executable; typically either the engine directory or project directory.</param>
		/// <param name="BinaryName">Name of the binary</param>
		/// <param name="Platform">Target platform to build for</param>
		/// <param name="Configuration">Target configuration being built</param>
		/// <param name="Architectures">Architectures being built</param>
		/// <param name="BinaryType">The type of binary we're compiling</param>
		/// <param name="UndecoratedConfiguration">The configuration which doesn't have a "-{Platform}-{Configuration}" suffix added to the binary</param>
		/// <param name="ExeSubFolder">Subfolder for executables. May be null.</param>
		/// <param name="ProjectFile">The project file containing the target being built</param>
		/// <param name="Rules">Rules for the target being built</param>
		/// <returns>List of executable paths for this target</returns>
		public static List<FileReference> MakeBinaryPaths(DirectoryReference BaseDirectory, string BinaryName, UnrealTargetPlatform Platform, UnrealTargetConfiguration Configuration, UEBuildBinaryType BinaryType, UnrealArchitectures Architectures, UnrealTargetConfiguration UndecoratedConfiguration, string ExeSubFolder, FileReference? ProjectFile, ReadOnlyTargetRules Rules)
		{
			FileReference BinaryFile;
			if (Rules.OutputFile != null)
			{
				BinaryFile = FileReference.Combine(BaseDirectory, Rules.OutputFile);
			}
			else
			{
				// Get the configuration for the executable. If we're building DebugGame, and this executable only contains engine modules, use the same name as development.
				UnrealTargetConfiguration ExeConfiguration = Configuration;

				// Build the binary path
				DirectoryReference BinaryDirectory = DirectoryReference.Combine(BaseDirectory, "Binaries", Platform.ToString());
				if (!String.IsNullOrEmpty(ExeSubFolder))
				{
					BinaryDirectory = DirectoryReference.Combine(BinaryDirectory, ExeSubFolder);
				}
				BinaryFile = FileReference.Combine(BinaryDirectory, MakeBinaryFileName(BinaryName, Rules.DecoratedSeparator, Platform, ExeConfiguration, Architectures, UndecoratedConfiguration, BinaryType));
			}

			// Allow the platform to customize the output path (and output several executables at once if necessary)
			return UEBuildPlatform.GetBuildPlatform(Platform).FinalizeBinaryPaths(BinaryFile, ProjectFile, Rules);
		}

		/// <summary>
		/// Sets up the uproject modules for this target
		/// </summary>
		public void SetupProjectModules(ILogger Logger)
		{
			if (ProjectDescriptor?.Modules == null)
			{
				return;
			}

			foreach (ModuleDescriptor Descriptor in ProjectDescriptor.Modules.Where(x => x.IsCompiledInConfiguration(Platform, Configuration, TargetName, TargetType, Rules.bBuildDeveloperTools, Rules.bBuildRequiresCookedData, Architectures)))
			{
				// To maintain historical behavior any project modules that do not specifically list the program in the ProgramAllowList will not be added even if IsCompiledInConfiguration returns true
				if (Rules.Type == TargetType.Program && (Descriptor.ProgramAllowList == null || !Descriptor.ProgramAllowList.Contains(AppName)))
				{
					continue;
				}

				UEBuildModuleCPP Module = FindOrCreateCppModuleByName(Descriptor.Name, TargetRulesFile.GetFileName(), Logger);
				if (Module.Binary == null)
				{
					AddModuleToBinary(Module);
				}
			}
		}

		/// <summary>
		/// Sets up the plugins for this target
		/// </summary>
		public void SetupPlugins(ILogger Logger)
		{
			// Find all the valid plugins
			Dictionary<string, PluginSet> NameToInfos = RulesAssembly.EnumeratePlugins().ToDictionary(x => x.Name, x => x, StringComparer.InvariantCultureIgnoreCase);

			// Remove any plugins for platforms we don't have
			List<UnrealTargetPlatform> MissingPlatforms = new List<UnrealTargetPlatform>();
			foreach (UnrealTargetPlatform TargetPlatform in UnrealTargetPlatform.GetValidPlatforms())
			{
				if (!UEBuildPlatform.TryGetBuildPlatform(TargetPlatform, out _))
				{
					MissingPlatforms.Add(TargetPlatform);
				}
			}

			// Get an array of folders to filter out
			string[] ExcludeFolders = MissingPlatforms.Select(x => x.ToString()).ToArray();

			// Set of all the plugins that have been referenced
			HashSet<string> ReferencedNames = new HashSet<string>(StringComparer.InvariantCultureIgnoreCase);

			// Map of plugin names to instances of that plugin
			Dictionary<string, UEBuildPlugin> NameToInstance = new Dictionary<string, UEBuildPlugin>(StringComparer.InvariantCultureIgnoreCase);

			// Set up the foreign plugin
			if (ForeignPlugin != null)
			{
				PluginReferenceDescriptor PluginReference = new PluginReferenceDescriptor(ForeignPlugin.GetFileNameWithoutExtension(), null, true);
				AddPlugin(PluginReference, $"{TargetName} command line", ExcludeFolders, NameToInstance, NameToInfos, Logger);
			}

			// Configure plugins explicitly enabled via target settings
			foreach (string PluginName in Rules.EnablePlugins)
			{
				if (ReferencedNames.Add(PluginName))
				{
					PluginReferenceDescriptor PluginReference = new PluginReferenceDescriptor(PluginName, null, true);
					AddPlugin(PluginReference, $"{TargetName} EnablePlugins", ExcludeFolders, NameToInstance, NameToInfos, Logger);
				}
			}

			// Configure plugins explicitly disabled via target settings
			foreach (string PluginName in Rules.DisablePlugins)
			{
				if (ReferencedNames.Add(PluginName))
				{
					PluginReferenceDescriptor PluginReference = new PluginReferenceDescriptor(PluginName, null, false);
					AddPlugin(PluginReference, $"{TargetName} DisablePlugins", ExcludeFolders, NameToInstance, NameToInfos, Logger);
				}
			}

			// Configure optional plugins configured via target settings
			foreach (string PluginName in Rules.OptionalPlugins)
			{
				if (ReferencedNames.Add(PluginName))
				{
					PluginReferenceDescriptor PluginReference = new PluginReferenceDescriptor(PluginName, null, true);
					PluginReference.bOptional = true;
					AddPlugin(PluginReference, $"{TargetName} OptionalPlugins", ExcludeFolders, NameToInstance, NameToInfos, Logger);
				}
			}

			bool bAllowEnginePluginsEnabledByDefault = Rules.bAllowEnginePluginsEnabledByDefault;

			// Find a map of plugins which are explicitly referenced in the project file
			if (ProjectDescriptor != null)
			{
				// EffectiveAllow = TargetRulesAllow && ProjectAllow;
				if (ProjectDescriptor.DisableEnginePluginsByDefault)
				{
					bAllowEnginePluginsEnabledByDefault = false;
				}

				if (ProjectDescriptor.Plugins != null)
				{
					HashSet<string> EnablePlugins = new(Rules.EnablePlugins, StringComparer.InvariantCultureIgnoreCase);
					HashSet<string> DisablePlugins = new(Rules.DisablePlugins, StringComparer.InvariantCultureIgnoreCase);
					string ProjectReferenceChain = ProjectFile!.GetFileName();
					foreach (PluginReferenceDescriptor PluginReference in ProjectDescriptor.Plugins)
					{
						if (!EnablePlugins.Contains(PluginReference.Name, StringComparer.InvariantCultureIgnoreCase) && !DisablePlugins.Contains(PluginReference.Name, StringComparer.InvariantCultureIgnoreCase))
						{
							// Make sure we don't have multiple references to the same plugin
							if (!ReferencedNames.Add(PluginReference.Name))
							{
								Logger.LogWarning("Plugin '{PluginReferenceName}' is listed multiple times in project file '{ProjectFile}'.", PluginReference.Name, ProjectFile);
							}
							else
							{
								AddPlugin(PluginReference, ProjectReferenceChain, ExcludeFolders, NameToInstance, NameToInfos, Logger);
							}
						}
					}
				}
			}

			// Also synthesize references for plugins which are enabled by default
			if (Rules.bCompileAgainstEngine || Rules.bCompileWithPluginSupport)
			{
				foreach (PluginSet PluginVersions in NameToInfos.Values)
				{
					PluginInfo? Plugin = PluginVersions.ChoiceVersion;
					if (Plugin != null && Plugin.IsEnabledByDefault(bAllowEnginePluginsEnabledByDefault) && !ReferencedNames.Contains(Plugin.Name))
					{
						ReferencedNames.Add(Plugin.Name);

						PluginReferenceDescriptor PluginReference = new PluginReferenceDescriptor(Plugin.Name, null, true);
						PluginReference.SupportedTargetPlatforms = Plugin.Descriptor.GetSupportedTargetPlatformNames();
						PluginReference.bHasExplicitPlatforms = Plugin.Descriptor.bHasExplicitPlatforms;
						PluginReference.bOptional = true;

						if (PluginReference.bHasExplicitPlatforms)
						{
							PluginReference.PlatformAllowList = PluginReference.SupportedTargetPlatforms; //synthesize allow list if it must be explicit
						}

						AddPlugin(PluginReference, $"{TargetName} default plugins", ExcludeFolders, NameToInstance, NameToInfos, Logger);
					}
				}
			}

			// If this is a program, synthesize references for plugins which are enabled via the config file
			if (TargetType == TargetType.Program)
			{
				ConfigHierarchy EngineConfig = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, DirectoryReference.Combine(Unreal.EngineDirectory, "Programs", TargetName), Platform);

				List<string>? PluginNames;
				if (EngineConfig.GetArray("Plugins", "ProgramEnabledPlugins", out PluginNames))
				{
					foreach (string PluginName in PluginNames)
					{
						if (ReferencedNames.Add(PluginName))
						{
							PluginReferenceDescriptor PluginReference = new PluginReferenceDescriptor(PluginName, null, true);
							AddPlugin(PluginReference, $"{TargetName} DefaultEngine.ini", ExcludeFolders, NameToInstance, NameToInfos, Logger);
						}
					}
				}
			}

			// Create the list of enabled plugins
			EnabledPlugins = new List<UEBuildPlugin>(NameToInstance.Values);

			// Configure plugins explicitly built but not enabled via target settings
			foreach (string PluginName in Rules.BuildPlugins)
			{
				if (ReferencedNames.Add(PluginName))
				{
					PluginReferenceDescriptor PluginReference = new PluginReferenceDescriptor(PluginName, null, true);
					AddPlugin(PluginReference, $"{TargetName} BuildPlugins", ExcludeFolders, NameToInstance, NameToInfos, Logger);
				}
			}

			// Set the list of plugins that should be built
			BuildPlugins = new(NameToInstance.Values);
			BuildPlugins.SortBy(x => x.Name);

			// Setup the UHT plugins
			SetupUbtPlugins(Logger);
		}

		/// <summary>
		/// Modify a module rules based on the current target, be very careful not to violate the shared environment!
		/// </summary>
		/// <param name="moduleName">name of the module</param>
		/// <param name="moduleRules">the module's rules</param>
		private void ModifyModuleRulesForTarget(string moduleName, ModuleRules moduleRules)
		{
			if (moduleName == "Projects" && (Rules.Configuration != UnrealTargetConfiguration.Shipping))
			{
				// Monolithic and non-shared environment builds compile in the build plugins. Non-monolithic editor builds save them in the BuildPlugins receipt to avoid invalidating the shared build environment.
				// See Projects.Build.cs
				bool bUsingTargetReceipt = ((Rules.Type == TargetType.Editor) && (Rules.BuildEnvironment == TargetBuildEnvironment.Shared) && (Rules.LinkType != TargetLinkType.Monolithic));
				if (!bUsingTargetReceipt)
				{
					// Compile in the build plugins list for the runtime plugin manager to filter optional plugin references against. Only plugins that were actually built
					// should be enabled when another plugin has an optional dependency on the plugin.
					IEnumerable<string> BuildPluginStrings = BuildPlugins?.Select(x => $"TEXT(\"{x}\")") ?? Array.Empty<string>();
					moduleRules.PrivateDefinitions.Add($"UBT_TARGET_BUILD_PLUGINS={String.Join(", ", BuildPluginStrings)}");
				}
			}
		}

		/// <summary>
		/// Creates a plugin instance from a reference to it
		/// </summary>
		/// <param name="Reference">Reference to the plugin</param>
		/// <param name="ReferenceChain">Textual representation of the chain of references, for error reporting</param>
		/// <param name="ExcludeFolders">Array of folder names to be excluded</param>
		/// <param name="NameToInstance">Map from plugin name to instance of it</param>
		/// <param name="NameToInfos">Map from plugin name to information</param>
		/// <param name="Logger">Logger for diagnostic output</param>
		/// <returns>Instance of the plugin, or null if it should not be used</returns>
		private UEBuildPlugin? AddPlugin(PluginReferenceDescriptor Reference, string ReferenceChain, string[] ExcludeFolders, Dictionary<string, UEBuildPlugin> NameToInstance, Dictionary<string, PluginSet> NameToInfos, ILogger Logger)
		{
			// Ignore disabled references
			if (!Reference.bEnabled)
			{
				return null;
			}

			// Try to get an existing reference to this plugin
			UEBuildPlugin? Instance;
			if (NameToInstance.TryGetValue(Reference.Name, out Instance))
			{
				// If this is a non-optional reference, make sure that and every referenced dependency is staged
				if (!Reference.bOptional && !Instance.bDescriptorReferencedExplicitly)
				{
					Instance.bDescriptorReferencedExplicitly = true;
					if (Instance.Descriptor.Plugins != null)
					{
						foreach (PluginReferenceDescriptor NextReference in Instance.Descriptor.Plugins)
						{
							
							PluginInfo? Info = null;

							PluginSet? PluginVersions;
							if (NameToInfos.TryGetValue(Reference.Name, out PluginVersions))
							{
								if (Reference.RequestedVersion.HasValue)
								{
									Info = PluginVersions.KnownVersions.Find(x => x.Descriptor.Version == Reference.RequestedVersion.Value);
									if (Info != null)
									{
										PluginVersions.ChoiceVersion = Info;
									}
									else
									{
										Logger.LogWarning("Failed to find specific version (v{RequestedVersion}) of plugin '{ReferenceName}' (referenced via {ReferenceChain}). Other versions exist, but the explicit version requested could not be found.",
											Reference.RequestedVersion.Value, Reference.Name, ReferenceChain);
									}
								}
								else
								{
									Info = PluginVersions.ChoiceVersion;
								}
							}

							if (Info == null || !Rules.ShouldIgnorePluginDependency(Info, NextReference))
							{
								string NextReferenceChain = String.Format("{0} -> {1}", ReferenceChain, Instance.File.GetFileName());
								AddPlugin(NextReference, NextReferenceChain, ExcludeFolders, NameToInstance, NameToInfos, Logger);
							}
						}
					}
				}
			}
			else
			{
				// Check if the plugin is required for this platform
				if (!Reference.IsEnabledForPlatform(Platform))
				{
					Logger.LogTrace("Ignoring plugin '{ReferenceName}' (referenced via {ReferenceChain}), not enabled for platform {Platform}", Reference.Name, ReferenceChain, Platform);
					return null;
				}

				if (!Reference.IsEnabledForTargetConfiguration(Configuration))
				{
					Logger.LogTrace("Ignoring plugin '{ReferenceName}' (referenced via {ReferenceChain}), not enabled for target configuration {Configuration}", Reference.Name, ReferenceChain, Configuration);
					return null;
				}

				if (!Reference.IsEnabledForTarget(TargetType))
				{
					Logger.LogTrace("Ignoring plugin '{ReferenceName}' (referenced via {ReferenceChain}), not enabled for target {TargetType}", Reference.Name, ReferenceChain, TargetType);
					return null;
				}

				// Disable any plugin reference which does not support the target platform
				if (!Rules.bIncludePluginsForTargetPlatforms && !Reference.IsSupportedTargetPlatform(Platform))
				{
					Logger.LogTrace("Ignoring plugin '{ReferenceName}' (referenced via {ReferenceChain}) due to unsupported target platform.", Reference.Name, ReferenceChain);
					return null;
				}

				// Find the plugin being enabled
				PluginInfo? Info = null;

				PluginSet? PluginVersions;
				if (NameToInfos.TryGetValue(Reference.Name, out PluginVersions))
				{
					if (Reference.RequestedVersion.HasValue)
					{
						Info = PluginVersions.KnownVersions.Find(x => x.Descriptor.Version == Reference.RequestedVersion.Value);
						if (Info != null)
						{
							PluginVersions.ChoiceVersion = Info;
						}
						else
						{
							Logger.LogWarning("Failed to find specific version (v{RequestedVersion}) of plugin '{ReferenceName}' (referenced via {ReferenceChain}). Other versions exist, but the explicit version requested could not be found.",
								Reference.RequestedVersion.Value, Reference.Name, ReferenceChain);
						}
					}
					else
					{
						Info = PluginVersions.ChoiceVersion;
					}
				}

				if (Info == null)
				{
					if (Reference.bOptional)
					{
						return null;
					}
					else
					{
						throw new BuildException("Unable to find plugin '{0}' (referenced via {1}). Install it and try again, or remove it from the required plugin list.", Reference.Name, ReferenceChain);
					}
				}

				// Disable any plugin which does not support the target platform. The editor should update such references in the .uproject file on load.
				if (!Rules.bIncludePluginsForTargetPlatforms && !Info.Descriptor.SupportsTargetPlatform(Platform))
				{
					LogValue PluginLogValue = LogValue.SourceFile(Info.File, Info.File.GetFileName());
					throw new BuildLogEventException("{Plugin} is referenced via {ReferenceChain} with a mismatched 'SupportedTargetPlatforms' field. This will cause problems in packaged builds, because the .uplugin file will not be staged. Launch the editor to update references from your project file, or update references from other plugins manually.", PluginLogValue, ReferenceChain);
				}

				// Disable any plugin that requires the build platform
				if (Info.Descriptor.bRequiresBuildPlatform && ShouldExcludePlugin(Info, ExcludeFolders))
				{
					Logger.LogTrace("Ignoring plugin '{ReferenceName}' (referenced via {ReferenceChain}) due to missing build platform", Reference.Name, ReferenceChain);
					return null;
				}

				// Disable any plugins that aren't compatible with this program
				if (Rules.Type == TargetType.Program && (Info.Descriptor.SupportedPrograms == null || !Info.Descriptor.SupportedPrograms.Contains(AppName)))
				{
					Logger.LogTrace("Ignoring plugin '{ReferenceName}' (referenced via {ReferenceChain}) due to absence from supported programs list.", Reference.Name, ReferenceChain);
					return null;
				}

				// Detect when a plugin in DisablePlugins has been enabled by another plugin referencing it
				if (Rules.DisablePlugins.Contains(Reference.Name))
				{
					if (Rules.CppCompileWarningSettings.DisablePluginsConflictWarningLevel == WarningLevel.Error)
					{
						LogValue PluginLogValue = LogValue.SourceFile(Info.File, Info.File.GetFileName());
						throw new BuildLogEventException("Error: Plugin '{Plugin}' (referenced via {ReferenceChain}) is being enabled but found in DisablePlugins list. Suppress this message by setting DisablePluginsConflictWarningLevel = WarningLevel.Off in '{TargetRulesFile}'", PluginLogValue, ReferenceChain, TargetRulesFile.GetFileName());
					}
					else if (Rules.CppCompileWarningSettings.DisablePluginsConflictWarningLevel == WarningLevel.Warning)
					{
						Logger.LogWarning("Plugin '{ReferenceName}' (referenced via {ReferenceChain}) is being enabled but found in DisablePlugins list. Suppress this message by setting DisablePluginsConflictWarningLevel = WarningLevel.Off in '{TargetRulesFile}'", Reference.Name, ReferenceChain, TargetRulesFile.GetFileName());
					}
					else if (Rules.CppCompileWarningSettings.DisablePluginsConflictWarningLevel != WarningLevel.Off)
					{
						Logger.LogDebug("Plugin '{ReferenceName}' (referenced via {ReferenceChain}) is being enabled but found in DisablePlugins list. Suppress this message by setting DisablePluginsConflictWarningLevel = WarningLevel.Off in '{TargetRulesFile}'", Reference.Name, ReferenceChain, TargetRulesFile.GetFileName());
					}
				}

				// Create the new instance and add it to the cache
				Logger.LogTrace("Enabling plugin '{ReferenceName}' (referenced via {ReferenceChain})", Reference.Name, ReferenceChain);
				Instance = new UEBuildPlugin(Info, ReferenceChain);
				Instance.bDescriptorReferencedExplicitly = !Reference.bOptional;
				NameToInstance.Add(Info.Name, Instance);

				// Get the reference chain for this plugin
				string PluginReferenceChain = String.Format("{0} -> {1}", ReferenceChain, Info.File.GetFileName());

				// Create modules for this plugin
				//UEBuildBinaryType BinaryType = ShouldCompileMonolithic() ? UEBuildBinaryType.StaticLibrary : UEBuildBinaryType.DynamicLinkLibrary;
				if (Info.Descriptor.Modules != null)
				{
					foreach (ModuleDescriptor ModuleInfo in Info.Descriptor.Modules)
					{
						if (ModuleInfo.IsCompiledInConfiguration(Platform, Configuration, TargetName, TargetType, Rules.bBuildDeveloperTools, Rules.bBuildRequiresCookedData, Architectures))
						{
							UEBuildModuleCPP Module = FindOrCreateCppModuleByName(ModuleInfo.Name, PluginReferenceChain, Logger);
							if (!Instance.Modules.Contains(Module))
							{
								// This could be in a child plugin so scan thorugh those as well
								if (!Module.RulesFile.IsUnderDirectory(Info.Directory) && !Info.ChildFiles.Any(ChildFile => Module.RulesFile.IsUnderDirectory(ChildFile.Directory)))
								{
									throw new BuildException("Plugin '{0}' (referenced via {1}) does not contain the '{2}' module, but lists it in '{3}'.", Info.Name, ReferenceChain, ModuleInfo.Name, Info.File);
								}
								Instance.bDescriptorNeededAtRuntime = true;
								Instance.Modules.Add(Module);
							}
						}
					}
				}

				// Create the dependencies set
				HashSet<UEBuildPlugin> Dependencies;
				if (Info.Descriptor.Plugins != null)
				{
					Dependencies = new(Info.Descriptor.Plugins.Count + 1);
					foreach (PluginReferenceDescriptor NextReference in Info.Descriptor.Plugins)
					{
						// Ignore any plugin dependency which is programmatically filtered out by the target
						if (Rules.ShouldIgnorePluginDependency(Info, NextReference))
						{
							Logger.LogDebug("Ignoring plugin '{PluginName}' (referenced via {ReferencingPluginName}) due to it being filtered out by the target rules.", NextReference.Name, Info.Name);
							continue;
						}

						UEBuildPlugin? NextInstance = AddPlugin(NextReference, PluginReferenceChain, ExcludeFolders, NameToInstance, NameToInfos, Logger);
						if (NextInstance != null)
						{
							if (Dependencies.Add(NextInstance))
							{
								if (NextInstance.Dependencies == null)
								{
									throw new BuildException($"Found circular plugin dependency. {PluginReferenceChain} -> {NextReference.Name}");
								}
								Dependencies.UnionWith(NextInstance.Dependencies);
							}
						}
					}
				}
				else
				{
					Dependencies = [];
				}
					
				Instance.Dependencies = Dependencies;

				// Stage the descriptor if the plugin contains content or Verse code
				if (Info.Descriptor.bCanContainContent || Info.Descriptor.bCanContainVerse || Dependencies.Any(x => x.bDescriptorNeededAtRuntime))
				{
					Instance.bDescriptorNeededAtRuntime = true;
				}
			}
			return Instance;
		}

		/// <summary>
		/// Checks whether a plugin path contains a platform directory fragment
		/// </summary>
		private bool ShouldExcludePlugin(PluginInfo Plugin, IEnumerable<string> ExcludeFolders)
		{
			if (Plugin.LoadedFrom == PluginLoadedFrom.Engine)
			{
				return Plugin.File.ContainsAnyNames(ExcludeFolders, Unreal.EngineDirectory);
			}
			else if (ProjectFile != null)
			{
				return Plugin.File.ContainsAnyNames(ExcludeFolders, ProjectFile.Directory);
			}
			else
			{
				return false;
			}
		}

		/// <summary>
		/// Scan for UBT plugins.  This also detects if the current target has a C++ style UHT that don't have
		/// an associated UBT/UHT plugin.  When this happens, the old UHT must be used.
		/// </summary>
		private void SetupUbtPlugins(ILogger Logger)
		{
			if (BuildPlugins == null)
			{
				throw new BuildException("Build plugins has not be populated");
			}

			if (Rules.bGenerateProjectFiles)
			{
				return;
			}

			// Collect the plugins
			UbtPlugins = PluginsBase.EnumerateUbtPlugins(ProjectFile);

			// If we found possible plugins
			(FileReference ProjectFile, FileReference TargetAssembly)[]? BuiltPlugins = null;
			if (UbtPlugins != null)
			{

				// Filter the UBT plugins based on the enabled plugins
				EnabledUbtPlugins = new List<FileReference>();
				EnabledUhtPlugins = new List<FileReference>();
				foreach (UEBuildPlugin Plugin in BuildPlugins)
				{
					PluginInfo Info = Plugin.Info;
					EnabledUbtPlugins.AddRange(UbtPlugins.Where(P => P.IsUnderDirectory(Info.Directory)));
				}
				EnabledUbtPlugins.SortBy(P => P.FullName);

				// Build the plugins. Scope the rules preprocessor defines to *.ubtplugin.csproj via UnrealEngine.CSharp.targets
				// so they don't leak into shared deps and invalidate their build records (which UAT script compilation also uses).
				Dictionary<string, IEnumerable<string>> CustomGlobalProperties = new() { { "UbtPluginDefineConstants", RulesAssembly.PreprocessorDefines ?? [] } };
				bool bCompiled = PluginsBase.BuildUbtPlugins(ProjectFile, EnabledUbtPlugins, CustomGlobalProperties, Logger, out BuiltPlugins);
				if (!bCompiled)
				{
					throw new BuildException("Not all plugins compiled");
				}
			}

			// For all the plugins 
			foreach (UEBuildPlugin Plugin in BuildPlugins)
			{
				PluginInfo Info = Plugin.Info;

				// Check to see if this plugin has an associated UBT plugin
				if (BuiltPlugins != null)
				{
					foreach ((FileReference ProjectFile, FileReference TargetAssembly) BuiltPlugin in BuiltPlugins)
					{
						if (BuiltPlugin.ProjectFile.IsUnderDirectory(Info.Directory))
						{
							if (UhtTables.IsUhtPlugin(BuiltPlugin.TargetAssembly.FullName))
							{
								EnabledUhtPlugins!.Add(BuiltPlugin.TargetAssembly);
								break;
							}
						}
					}
				}

				EnabledUhtPlugins?.SortBy(P => P.FullName);
			}
		}

		/// <summary>
		/// Sets up the binaries for the target.
		/// </summary>
		protected void SetupBinaries(ILogger Logger)
		{
			// If we're using the new method for specifying binaries, fill in the binary configurations now
			if (Rules.LaunchModuleName == null)
			{
				throw new BuildException("LaunchModuleName must be set for all targets.");
			}

			// Create the launch module
			UEBuildModuleCPP LaunchModule = FindOrCreateCppModuleByName(Rules.LaunchModuleName, TargetRulesFile.GetFileName(), Logger);

			// Get the intermediate directory for the launch module directory. This can differ from the standard engine intermediate directory because it is always configuration-specific.
			DirectoryReference IntermediateDirectory = LaunchModule.RulesFile.IsUnderDirectory(Unreal.EngineDirectory) && !ShouldCompileMonolithic()
				? DirectoryReference.Combine(EngineIntermediateDirectory.ParentDirectory!, Configuration.ToString())
				: ProjectIntermediateDirectory;

			// Construct the output paths for this target's executable
			DirectoryReference OutputDirectory;
			// if we are building a program with a project file, and we are making a unique build environment, output the program to that project's binaries
			bool bIsUniqueBuildProgramWithExternalProject =
					Rules.Type == TargetType.Program && Rules.BuildEnvironment == TargetBuildEnvironment.Unique &&
					Rules.File.IsUnderDirectory(Unreal.EngineDirectory) &&
					ProjectFile != null && !ProjectFile.IsUnderDirectory(Unreal.EngineDirectory);

			if (bIsUniqueBuildProgramWithExternalProject)
			{
				// engine programs generally won't be under the directories searched in GetOutputDirectoryForExecutable, so it will return the EngineDir, even tho unique programs
				// built with a ProjectFile want to be under the Project dir, so just force it into the ProjectDir if one wasn't found otherwise
				Logger.LogInformation("Overriding output directory for unique build program {Name} from {Output} to {Project}, because it is being built with external project {ProjectFile}", Rules.Name, GetOutputDirectoryForExecutable(ProjectDirectory, Rules.File), ProjectDirectory, ProjectFile);
				OutputDirectory = ProjectDirectory;
			}
			else if (ProjectFile != null && (bCompileMonolithic || !bUseSharedBuildEnvironment) && (Rules.File.IsUnderDirectory(ProjectDirectory)))
			{
				OutputDirectory = GetOutputDirectoryForExecutable(ProjectDirectory, Rules.File);

			}
			else
			{
				OutputDirectory = GetOutputDirectoryForExecutable(Unreal.EngineDirectory, Rules.File);
			}

			bool bCompileAsDLL = Rules.bShouldCompileAsDLL && bCompileMonolithic;
			string BinaryName = bCompileMonolithic ? TargetName : AppName;
			if (Platform == UnrealTargetPlatform.Mac && Rules.MacPlatform.bForceTargetNameAsExeName)
			{
				BinaryName = TargetName;
			}
			List<FileReference> OutputPaths = MakeBinaryPaths(OutputDirectory, BinaryName, Platform, Configuration, bCompileAsDLL ? UEBuildBinaryType.DynamicLinkLibrary : UEBuildBinaryType.Executable, Rules.Architectures, Rules.UndecoratedConfiguration, Rules.ExeBinariesSubFolder, ProjectFile, Rules);

			// Create the binary
			UEBuildBinary Binary = new UEBuildBinary(
				Type: Rules.bShouldCompileAsDLL ? UEBuildBinaryType.DynamicLinkLibrary : UEBuildBinaryType.Executable,
				OutputFilePaths: OutputPaths,
				IntermediateDirectory: IntermediateDirectory,
				bAllowExports: Rules.bHasExports,
				bBuildAdditionalConsoleApp: Rules.bBuildAdditionalConsoleApp,
				bBuildConsoleAppOnly: Rules.bBuildConsoleAppOnly,
				PrimaryModule: LaunchModule,
				bUsePrecompiled: LaunchModule.Rules.bUsePrecompiled && OutputPaths[0].IsUnderDirectory(Unreal.EngineDirectory)
			);
			Binaries.Add(Binary);

			// Add the launch module to it
			LaunchModule.Binary = Binary;
			Binary.AddModule(LaunchModule);

		}

		/// <summary>
		/// Sets up the global compile and link environment for the target.
		/// </summary>
		public void SetupGlobalEnvironment(UEToolChain ToolChain, CppCompileEnvironment GlobalCompileEnvironment, LinkEnvironment GlobalLinkEnvironment)
		{
			UEBuildPlatform BuildPlatform = UEBuildPlatform.GetBuildPlatform(Platform);

			ToolChain.SetUpGlobalEnvironment(Rules, GlobalCompileEnvironment, GlobalLinkEnvironment);

			// @Hack: This to prevent UHT from listing CoreUObject.init.gen.cpp as its dependency.
			// We flag the compile environment when we build UHT so that we don't need to check
			// this for each file when generating their dependencies.
			GlobalCompileEnvironment.bHackHeaderGenerator = (AppName == "UnrealHeaderTool");

			bool bUseDebugCRT = GlobalCompileEnvironment.Configuration == CppConfiguration.Debug && Rules.bDebugBuildsActuallyUseDebugCRT;
			GlobalCompileEnvironment.bUseDebugCRT = bUseDebugCRT;
			GlobalCompileEnvironment.bEnableOSX109Support = Rules.bEnableOSX109Support;
			GlobalCompileEnvironment.Definitions.Add(String.Format("IS_PROGRAM={0}", TargetType == TargetType.Program ? "1" : "0"));
			GlobalCompileEnvironment.Definitions.AddRange(Rules.GlobalDefinitions);
			GlobalCompileEnvironment.bUseSharedBuildEnvironment = (Rules.BuildEnvironment == TargetBuildEnvironment.Shared);
			GlobalCompileEnvironment.bEnableExceptions = Rules.bForceEnableExceptions || (Rules.bCompileAgainstEditor && !Rules.bUseAutoRTFMCompiler);
			GlobalCompileEnvironment.bEnableObjCExceptions = Rules.bForceEnableObjCExceptions || Rules.bCompileAgainstEditor;
			GlobalCompileEnvironment.DefaultWarningLevel = Rules.DefaultWarningLevel;
			GlobalCompileEnvironment.CppCompileWarnings = Rules.CppCompileWarningSettings.CloneAsWriteable();
			GlobalCompileEnvironment.bRetainFramePointers = Rules.bRetainFramePointers;
			GlobalCompileEnvironment.bWarningsAsErrors = Rules.bWarningsAsErrors;
			GlobalCompileEnvironment.OptimizationLevel = Rules.OptimizationLevel;
			GlobalCompileEnvironment.bUseStaticCRT = Rules.bUseStaticCRT;
			GlobalCompileEnvironment.bOmitFramePointers = Rules.bOmitFramePointers;
			GlobalCompileEnvironment.bUsePDBFiles = Rules.bUsePDBFiles;
			GlobalCompileEnvironment.bSupportEditAndContinue = Rules.bSupportEditAndContinue;
			GlobalCompileEnvironment.bPreprocessOnly = Rules.bPreprocessOnly;
			GlobalCompileEnvironment.bWithAssembly = Rules.bWithAssembly;
			GlobalCompileEnvironment.bUseIncrementalLinking = Rules.bUseIncrementalLinking;
			GlobalCompileEnvironment.bAllowLTCG = Rules.bAllowLTCG;
			GlobalCompileEnvironment.bPGOOptimize = Rules.bPGOOptimize;
			GlobalCompileEnvironment.bPGOProfile = Rules.bPGOProfile;
			GlobalCompileEnvironment.bAllowRemotelyCompiledPCHs = Rules.bAllowRemotelyCompiledPCHs;
			GlobalCompileEnvironment.bUseHeaderUnitsForPch = Rules.bUseHeaderUnitsForPch;
			GlobalCompileEnvironment.bCheckSystemHeadersForModification = Rules.bCheckSystemHeadersForModification;
			GlobalCompileEnvironment.bPrintTimingInfo = Rules.bPrintToolChainTimingInfo;
			GlobalCompileEnvironment.bUseRTTI = Rules.bForceEnableRTTI;
			GlobalCompileEnvironment.bUsePIE = Rules.bEnablePIE;
			GlobalCompileEnvironment.bUseStackProtection = Rules.bEnableStackProtection;
			GlobalCompileEnvironment.bUseInlining = Rules.bUseInlining;
			GlobalCompileEnvironment.bCompileISPC = Rules.bCompileISPC;
			GlobalCompileEnvironment.bHideSymbolsByDefault = !Rules.bPublicSymbolsByDefault;
			GlobalCompileEnvironment.CppStandardEngine = Rules.CppStandardEngine;
			GlobalCompileEnvironment.CppStandard = Rules.CppStandard;
			GlobalCompileEnvironment.CStandard = Rules.CStandard;
			GlobalCompileEnvironment.MinCpuArchX64 = Rules.MinCpuArchX64;
			GlobalCompileEnvironment.bMinCpuArchAPX = Rules.bMinCpuArchAPX;
			GlobalCompileEnvironment.MinArm64CpuTarget = Rules.MinArm64CpuTarget;
			GlobalCompileEnvironment.AdditionalArguments = Rules.AdditionalCompilerArguments ?? String.Empty;
			GlobalCompileEnvironment.bDeterministic = Rules.bDeterministic;
			GlobalCompileEnvironment.CrashDiagnosticDirectory = Rules.CrashDiagnosticDirectory;
			GlobalCompileEnvironment.bCodeCoverage = Rules.bCodeCoverage;
			GlobalCompileEnvironment.bValidateFormatStrings = Rules.bValidateFormatStrings;

			GlobalLinkEnvironment.bUseDebugCRT = bUseDebugCRT;
			GlobalLinkEnvironment.bUseStaticCRT = Rules.bUseStaticCRT;
			GlobalLinkEnvironment.bIsBuildingConsoleApplication = Rules.bIsBuildingConsoleApplication;
			GlobalLinkEnvironment.bOmitFramePointers = Rules.bOmitFramePointers;
			GlobalLinkEnvironment.bSupportEditAndContinue = Rules.bSupportEditAndContinue;
			GlobalLinkEnvironment.bCreateMapFile = Rules.bCreateMapFile;
			GlobalLinkEnvironment.bHasExports = Rules.bHasExports;
			GlobalLinkEnvironment.bUsePDBFiles = Rules.bUsePDBFiles;
			GlobalLinkEnvironment.PackagePath = Rules.PackagePath;
			GlobalLinkEnvironment.CrashDiagnosticDirectory = Rules.CrashDiagnosticDirectory;
			GlobalLinkEnvironment.ThinLTOCacheDirectory = Rules.ThinLTOCacheDirectory;
			GlobalLinkEnvironment.ThinLTOCachePruningArguments = Rules.ThinLTOCachePruningArguments;
			GlobalLinkEnvironment.BundleDirectory = BuildPlatform.GetBundleDirectory(Rules, Binaries[0].OutputFilePaths);
			GlobalLinkEnvironment.BundleVersion = Rules.BundleVersion;
			GlobalLinkEnvironment.bAllowLTCG = Rules.bAllowLTCG;
			GlobalLinkEnvironment.bPGOOptimize = Rules.bPGOOptimize;
			GlobalLinkEnvironment.bPGOProfile = Rules.bPGOProfile;
			GlobalLinkEnvironment.bUseIncrementalLinking = Rules.bUseIncrementalLinking;
			GlobalLinkEnvironment.bDeterministic = Rules.bDeterministic;
			GlobalLinkEnvironment.bPrintTimingInfo = Rules.bPrintToolChainTimingInfo;
			GlobalLinkEnvironment.bUsePIE = Rules.bEnablePIE;
			GlobalLinkEnvironment.AdditionalArguments = Rules.AdditionalLinkerArguments ?? String.Empty;
			GlobalLinkEnvironment.bCodeCoverage = Rules.bCodeCoverage;

			if (Rules.bPGOOptimize && Rules.bPGOProfile)
			{
				throw new BuildException("bPGOProfile and bPGOOptimize are mutually exclusive.");
			}

			if (Rules.bPGOProfile)
			{
				GlobalCompileEnvironment.Definitions.Add("ENABLE_PGO_PROFILE=1");
			}
			else
			{
				GlobalCompileEnvironment.Definitions.Add("ENABLE_PGO_PROFILE=0");
			}

			if (Rules.bUseHeaderUnitsForPch)
			{
				GlobalCompileEnvironment.Definitions.Add("UE_DEBUG_SECTION=");
				GlobalCompileEnvironment.Definitions.Add("UE_HEADER_UNITS=");
			}

			// Toggle to enable vorbis for audio streaming where available
			GlobalCompileEnvironment.Definitions.Add("USE_VORBIS_FOR_STREAMING=1");

			// Toggle to enable XMA for audio streaming where available (XMA2 only supports up to stereo streams - surround streams will fall back to Vorbis etc)
			GlobalCompileEnvironment.Definitions.Add("USE_XMA2_FOR_STREAMING=1");

			// Add the 'Engine/Source' path as a global include path for all modules
			GlobalCompileEnvironment.UserIncludePaths.Add(Unreal.EngineSourceDirectory);

			//@todo.PLATFORM: Do any platform specific tool chain initialization here if required

			string IntermediateAppName = GetTargetIntermediateFolderName(AppName, IntermediateEnvironment);

			UnrealTargetConfiguration EngineTargetConfiguration = Configuration == UnrealTargetConfiguration.DebugGame ? UnrealTargetConfiguration.Development : Configuration;
			DirectoryReference LinkIntermediateDirectory = DirectoryReference.Combine(Unreal.EngineDirectory, PlatformIntermediateFolder, IntermediateAppName, EngineTargetConfiguration.ToString());

			// Installed Engine intermediates go to the project's intermediate folder. Installed Engine never writes to the engine intermediate folder. (Those files are immutable)
			// Also, when compiling in monolithic, all intermediates go to the project's folder.  This is because a project can change definitions that affects all engine translation
			// units too, so they can't be shared between different targets.  They are effectively project-specific engine intermediates.
			if (Unreal.IsEngineInstalled() || (ProjectFile != null && ShouldCompileMonolithic()))
			{
				if (ProjectFile != null)
				{
					LinkIntermediateDirectory = DirectoryReference.Combine(ProjectFile.Directory, PlatformIntermediateFolder, IntermediateAppName, Configuration.ToString());
				}
				else if (ForeignPlugin != null)
				{
					LinkIntermediateDirectory = DirectoryReference.Combine(ForeignPlugin.Directory, PlatformIntermediateFolder, IntermediateAppName, Configuration.ToString());
				}
			}

			// Put the non-executable output files (PDB, import library, etc) in the intermediate directory.
			GlobalLinkEnvironment.IntermediateDirectory = LinkIntermediateDirectory;
			GlobalLinkEnvironment.OutputDirectory = GlobalLinkEnvironment.IntermediateDirectory;

			// By default, shadow source files for this target in the root OutputDirectory
			GlobalLinkEnvironment.LocalShadowDirectory = GlobalLinkEnvironment.OutputDirectory;

			string? RelativeBaseDir = GetRelativeBaseDir(Binaries[0].OutputDir, Platform);
			if (RelativeBaseDir != null)
			{
				GlobalCompileEnvironment.Definitions.Add(String.Format("UE_RELATIVE_BASE_DIR=\"{0}/\"", RelativeBaseDir));
				if (Rules.bBuildAdditionalConsoleApp || Rules.bBuildConsoleAppOnly)
				{
					string? CmdletRelativeBaseDir = GetRelativeBaseDir(GetExecutableDir(), Platform);
					GlobalCompileEnvironment.Definitions.Add(String.Format("UE_CMDLET_RELATIVE_BASE_DIR=\"{0}/\"", CmdletRelativeBaseDir));
				}
			}

			bool bCompileDevTests = (Configuration != UnrealTargetConfiguration.Test && Configuration != UnrealTargetConfiguration.Shipping);
			bool bCompilePerfTests = bCompileDevTests;
			if (Rules.bForceCompileDevelopmentAutomationTests)
			{
				bCompileDevTests = true;
			}
			if (Rules.bForceCompilePerformanceAutomationTests)
			{
				bCompilePerfTests = true;
			}
			if (Rules.bForceDisableAutomationTests)
			{
				bCompileDevTests = bCompilePerfTests = false;
			}

			GlobalCompileEnvironment.Definitions.Add("WITH_DEV_AUTOMATION_TESTS=" + (bCompileDevTests ? "1" : "0"));
			GlobalCompileEnvironment.Definitions.Add("WITH_PERF_AUTOMATION_TESTS=" + (bCompilePerfTests ? "1" : "0"));

			// Target contains Low Level Tests
			GlobalCompileEnvironment.Definitions.Add(String.Format("WITH_LOW_LEVEL_TESTS={0}", Rules.WithLowLevelTests ? "1" : "0"));
			// Target derived from TestTargetRules containing Low Level Tests, i.e. an explicit target
			GlobalCompileEnvironment.Definitions.Add(String.Format("EXPLICIT_TESTS_TARGET={0}", Rules.ExplicitTestsTarget ? "1" : "0"));
			// Target contains Low Level Tests or Functional Tests
			GlobalCompileEnvironment.Definitions.Add(String.Format("WITH_TESTS={0}", Rules.WithLowLevelTests || bCompileDevTests || bCompilePerfTests ? "1" : "0"));

			GlobalCompileEnvironment.Definitions.Add("UNICODE");
			GlobalCompileEnvironment.Definitions.Add("_UNICODE");
			GlobalCompileEnvironment.Definitions.Add("__UNREAL__");

			GlobalCompileEnvironment.Definitions.Add(String.Format("IS_MONOLITHIC={0}", ShouldCompileMonolithic() ? "1" : "0"));
			GlobalCompileEnvironment.Definitions.Add(String.Format("UE_MERGED_MODULES={0}", Rules.bMergeModules ? "1" : "0"));

			GlobalCompileEnvironment.Definitions.Add(String.Format("WITH_ENGINE={0}", Rules.bCompileAgainstEngine ? "1" : "0"));
			GlobalCompileEnvironment.Definitions.Add(String.Format("WITH_UNREAL_DEVELOPER_TOOLS={0}", Rules.bBuildDeveloperTools ? "1" : "0"));
			GlobalCompileEnvironment.Definitions.Add(String.Format("WITH_UNREAL_TARGET_DEVELOPER_TOOLS={0}", Rules.bBuildTargetDeveloperTools ? "1" : "0"));

			// Set a macro to control whether to initialize ApplicationCore. Command line utilities should not generally need this.
			if (Rules.bCompileAgainstApplicationCore)
			{
				GlobalCompileEnvironment.Definitions.Add("WITH_APPLICATION_CORE=1");
			}
			else
			{
				GlobalCompileEnvironment.Definitions.Add("WITH_APPLICATION_CORE=0");
			}

			if (Rules.bCompileAgainstCoreUObject)
			{
				GlobalCompileEnvironment.Definitions.Add("WITH_COREUOBJECT=1");
			}
			else
			{
				GlobalCompileEnvironment.Definitions.Add("WITH_COREUOBJECT=0");
			}

			if (Rules.bEnableConstInitUObject)
			{
				GlobalCompileEnvironment.Definitions.Add("UE_WITH_CONSTINIT_UOBJECT=1");
				GlobalCompileEnvironment.Definitions.Add("WITH_HOT_RELOAD=0");
			}
			else
			{
				GlobalCompileEnvironment.Definitions.Add("UE_WITH_CONSTINIT_UOBJECT=0");
			}

			if (Rules.bEnableTrace)
			{
				GlobalCompileEnvironment.Definitions.Add("UE_TRACE_ENABLED=1");
				GlobalCompileEnvironment.Definitions.Add("UE_TRACE_ASSERT_FMT=checkf");
			}
			else
			{
				GlobalCompileEnvironment.Definitions.Add("UE_TRACE_ENABLED=0");
			}

			if (Rules.bForceEnableTrace)
			{
				GlobalCompileEnvironment.Definitions.Add("UE_TRACE_FORCE_ENABLED=1");
				GlobalCompileEnvironment.Definitions.Add("UE_TRACE_ASSERT_FMT=checkf");
			}
			else
			{
				GlobalCompileEnvironment.Definitions.Add("UE_TRACE_FORCE_ENABLED=0");
			}

			// TODO - Sooner or later these should be removed.  bUseVerse has already been removed
			GlobalCompileEnvironment.Definitions.Add("WITH_VERSE=1");
			GlobalCompileEnvironment.Definitions.Add("UE_USE_VERSE_PATHS=1");

			if (Rules.bUseVerseBPVM)
			{
				GlobalCompileEnvironment.Definitions.Add("WITH_VERSE_BPVM=1");
			}
			else
			{
				GlobalCompileEnvironment.Definitions.Add("WITH_VERSE_BPVM=0");
			}

			if (Rules.bCompileWithStatsWithoutEngine)
			{
				GlobalCompileEnvironment.Definitions.Add("USE_STATS_WITHOUT_ENGINE=1");
			}
			else
			{
				GlobalCompileEnvironment.Definitions.Add("USE_STATS_WITHOUT_ENGINE=0");
			}

			if (Rules.bCompileWithPluginSupport)
			{
				GlobalCompileEnvironment.Definitions.Add("WITH_PLUGIN_SUPPORT=1");
			}
			else
			{
				GlobalCompileEnvironment.Definitions.Add("WITH_PLUGIN_SUPPORT=0");
			}

			if (Rules.bCompileWithAccessibilitySupport && !Rules.bIsBuildingConsoleApplication)
			{
				GlobalCompileEnvironment.Definitions.Add("WITH_ACCESSIBILITY=1");
			}
			else
			{
				GlobalCompileEnvironment.Definitions.Add("WITH_ACCESSIBILITY=0");
			}

			if (Rules.bWithPerfCounters)
			{
				GlobalCompileEnvironment.Definitions.Add("WITH_PERFCOUNTERS=1");
			}
			else
			{
				GlobalCompileEnvironment.Definitions.Add("WITH_PERFCOUNTERS=0");
			}

			if (Rules.bWithFixedTimeStepSupport)
			{
				GlobalCompileEnvironment.Definitions.Add("WITH_FIXED_TIME_STEP_SUPPORT=1");
			}
			else
			{
				GlobalCompileEnvironment.Definitions.Add("WITH_FIXED_TIME_STEP_SUPPORT=0");
			}

			if (Rules.bUseLoggingInShipping)
			{
				GlobalCompileEnvironment.Definitions.Add("USE_LOGGING_IN_SHIPPING=1");
			}
			else
			{
				GlobalCompileEnvironment.Definitions.Add("USE_LOGGING_IN_SHIPPING=0");
			}

			if (Rules.bUseConsoleInShipping)
			{
				GlobalCompileEnvironment.Definitions.Add("ALLOW_CONSOLE_IN_SHIPPING=1");
			}
			else
			{
				GlobalCompileEnvironment.Definitions.Add("ALLOW_CONSOLE_IN_SHIPPING=0");
			}

			if (Rules.bAllowProfileGPUInTest)
			{
				GlobalCompileEnvironment.Definitions.Add("ALLOW_PROFILEGPU_IN_TEST=1");
			}
			else
			{
				GlobalCompileEnvironment.Definitions.Add("ALLOW_PROFILEGPU_IN_TEST=0");
			}

			if (Rules.bAllowProfileGPUInShipping)
			{
				GlobalCompileEnvironment.Definitions.Add("ALLOW_PROFILEGPU_IN_SHIPPING=1");
			}
			else
			{
				GlobalCompileEnvironment.Definitions.Add("ALLOW_PROFILEGPU_IN_SHIPPING=0");
			}

			if (Rules.bLoggingToMemoryEnabled)
			{
				GlobalCompileEnvironment.Definitions.Add("WITH_LOGGING_TO_MEMORY=1");
			}
			else
			{
				GlobalCompileEnvironment.Definitions.Add("WITH_LOGGING_TO_MEMORY=0");
			}

			if (Rules.bUseCacheFreedOSAllocs)
			{
				GlobalCompileEnvironment.Definitions.Add("USE_CACHE_FREED_OS_ALLOCS=1");
			}
			else
			{
				GlobalCompileEnvironment.Definitions.Add("USE_CACHE_FREED_OS_ALLOCS=0");
			}

			if (Rules.bUseChecksInShipping)
			{
				GlobalCompileEnvironment.Definitions.Add("USE_CHECKS_IN_SHIPPING=1");
			}
			else
			{
				GlobalCompileEnvironment.Definitions.Add("USE_CHECKS_IN_SHIPPING=0");
			}

			if (Rules.bTCHARIsUTF8)
			{
				GlobalCompileEnvironment.Definitions.Add("USE_UTF8_TCHARS=1");
			}
			else
			{
				GlobalCompileEnvironment.Definitions.Add("USE_UTF8_TCHARS=0");
			}

			if (Rules.bUseEstimatedUtcNow)
			{
				GlobalCompileEnvironment.Definitions.Add("USE_ESTIMATED_UTCNOW=1");
			}
			else
			{
				GlobalCompileEnvironment.Definitions.Add("USE_ESTIMATED_UTCNOW=0");
			}

			if (Rules.bUseExecCommandsInShipping)
			{
				GlobalCompileEnvironment.Definitions.Add("UE_ALLOW_EXEC_COMMANDS_IN_SHIPPING=1");
			}
			else
			{
				GlobalCompileEnvironment.Definitions.Add("UE_ALLOW_EXEC_COMMANDS_IN_SHIPPING=0");
			}

			if ((Rules.bCompileAgainstEditor && (Rules.Type == TargetType.Editor || Rules.Type == TargetType.Program)))
			{
				GlobalCompileEnvironment.Definitions.Add("WITH_EDITOR=1");
				GlobalCompileEnvironment.Definitions.Add("WITH_IOSTORE_IN_EDITOR=1");
			}
			else if (!GlobalCompileEnvironment.Definitions.Contains("WITH_EDITOR=0"))
			{
				GlobalCompileEnvironment.Definitions.Add("WITH_EDITOR=0");
			}

			if (Rules.bBuildWithEditorOnlyData == false)
			{
				GlobalCompileEnvironment.Definitions.Add("WITH_EDITORONLY_DATA=0");
			}

			// Check if client-only code should be compiled out.
			GlobalCompileEnvironment.Definitions.Add(String.Format("WITH_CLIENT_CODE={0}", Rules.bWithClientCode ? 1 : 0));

			// Check if server-only code should be compiled out.
			GlobalCompileEnvironment.Definitions.Add(String.Format("WITH_SERVER_CODE={0}", Rules.bWithServerCode ? 1 : 0));

			GlobalCompileEnvironment.Definitions.Add(String.Format("UE_FNAME_OUTLINE_NUMBER={0}", Rules.bFNameOutlineNumber ? 1 : 0));

			// Set the defines for Push Model
			if (Rules.bWithPushModel)
			{
				GlobalCompileEnvironment.Definitions.Add("WITH_PUSH_MODEL=1");
			}
			else
			{
				GlobalCompileEnvironment.Definitions.Add("WITH_PUSH_MODEL=0");
			}

			// Set the define for whether we're compiling with CEF3
			if (Rules.bCompileCEF3 && (Platform == UnrealTargetPlatform.Win64 || Platform == UnrealTargetPlatform.Mac || Platform == UnrealTargetPlatform.Linux))
			{
				GlobalCompileEnvironment.Definitions.Add("WITH_CEF3=1");
			}
			else
			{
				GlobalCompileEnvironment.Definitions.Add("WITH_CEF3=0");
			}

			if(Rules.bMultiViewPerSlateWindowSupport && Rules.bCompileAgainstEditor && UEBuildPlatform.IsPlatformInGroup(Platform, UnrealPlatformGroup.Apple))
			{
				GlobalCompileEnvironment.Definitions.Add("WITH_MULTI_VIEW_SLATE_WINDOW_SUPPORT=1");
			}
			else
			{
				GlobalCompileEnvironment.Definitions.Add("WITH_MULTI_VIEW_SLATE_WINDOW_SUPPORT=0");
			}

			// Set the define for enabling live coding
			if (Rules.bWithLiveCoding)
			{
				GlobalCompileEnvironment.Definitions.Add("WITH_LIVE_CODING=1");
			}
			else
			{
				GlobalCompileEnvironment.Definitions.Add("WITH_LIVE_CODING=0");
			}

			// Whether C++20 modules are enabled
			if (Rules.bEnableCppModules)
			{
				GlobalCompileEnvironment.Definitions.Add("WITH_CPP_MODULES=1");
			}
			else
			{
				GlobalCompileEnvironment.Definitions.Add("WITH_CPP_MODULES=0");
			}

			if (Rules.bEnableProcessPriorityControl)
			{
				GlobalCompileEnvironment.Definitions.Add("WITH_PROCESS_PRIORITY_CONTROL=1");
			}
			else
			{
				GlobalCompileEnvironment.Definitions.Add("WITH_PROCESS_PRIORITY_CONTROL=0");
			}

			if (Rules.bDefineForceInlineHintToInline)
			{
				GlobalCompileEnvironment.Definitions.Add("UE_DEFINE_FORCEINLINE_HINT_TO_INLINE=1");
			}
			else
			{
				GlobalCompileEnvironment.Definitions.Add("UE_DEFINE_FORCEINLINE_HINT_TO_INLINE=0");
			}

			if (Rules.bUseSparseSetStrictMode)
			{
				GlobalCompileEnvironment.Definitions.Add("UE_USE_SPARSE_SET_STRICT_MODE=1");
			}
			else
			{
				GlobalCompileEnvironment.Definitions.Add("UE_USE_SPARSE_SET_STRICT_MODE=0");
			}

			if (Rules.bUseCompactSetAsDefault)
			{
				GlobalCompileEnvironment.Definitions.Add("UE_USE_COMPACT_SET_AS_DEFAULT=1");
			}
			else
			{
				GlobalCompileEnvironment.Definitions.Add("UE_USE_COMPACT_SET_AS_DEFAULT=0");
			}

			// Define the custom config if specified, this should not be set in an editor build
			if (!bUseSharedBuildEnvironment && !String.IsNullOrEmpty(Rules.CustomConfig))
			{
				GlobalCompileEnvironment.Definitions.Add(String.Format("CUSTOM_CONFIG=\"{0}\"", Rules.CustomConfig));
			}

			// Compile in the names of the module manifests
			GlobalCompileEnvironment.Definitions.Add(String.Format("UBT_MODULE_MANIFEST=\"{0}\"", ModuleManifest.GetStandardFileName(AppName, Platform, Configuration, Architectures, false)));
			GlobalCompileEnvironment.Definitions.Add(String.Format("UBT_MODULE_MANIFEST_DEBUGGAME=\"{0}\"", ModuleManifest.GetStandardFileName(AppName, Platform, UnrealTargetConfiguration.DebugGame, Architectures, true)));

			// Compile in the names of the loadorder manifests.
			if (!bCompileMonolithic && BuildPlatform.RequiresLoadOrderManifest(Rules))
			{
				GlobalCompileEnvironment.Definitions.Add(String.Format("UBT_LOADORDER_MANIFEST=\"{0}\"", LoadOrderManifest.GetStandardFileName(AppName, Platform, Configuration, Architectures)));
				GlobalCompileEnvironment.Definitions.Add(String.Format("UBT_LOADORDER_MANIFEST_DEBUGGAME=\"{0}\"", LoadOrderManifest.GetStandardFileName(AppName, Platform, UnrealTargetConfiguration.DebugGame, Architectures)));
			}

			// tell the compiled code the name of the UBT platform (this affects folder on disk, etc that the game may need to know)
			GlobalCompileEnvironment.Definitions.Add(String.Format("UBT_COMPILED_PLATFORM=" + Platform.ToString()));
			GlobalCompileEnvironment.Definitions.Add(String.Format("UBT_COMPILED_TARGET=" + TargetType.ToString()));
#pragma warning disable CA1308 // Normalize strings to uppercase - preserve casing of compiler define
			GlobalCompileEnvironment.Definitions.Add(String.Format("UBT_COMPILED_ARCHITECTURE=" + (Architectures.bIsMultiArch ? "MULTI" : Architectures.SingleArchitecture.ToString().ToLowerInvariant()) ));
#pragma warning restore CA1308

			// Set the global app name
			GlobalCompileEnvironment.Definitions.Add(String.Format("UE_APP_NAME=\"{0}\"", AppName));

			// Set the global
			if (Rules.bWarningsAsErrors)
			{
				GlobalCompileEnvironment.Definitions.Add("UE_WARNINGS_AS_ERRORS=1");
			}
			else
			{
				GlobalCompileEnvironment.Definitions.Add("UE_WARNINGS_AS_ERRORS=0");
			}

			// Add global definitions for project-specific binaries. HACK: Also defining for monolithic builds in binary releases. Might be better to set this via command line instead?
			if (!bUseSharedBuildEnvironment || bCompileMonolithic)
			{
				UEBuildBinary ExecutableBinary = Binaries[0];

				bool IsCurrentPlatform;
				if (!OperatingSystem.IsWindows())
				{
					IsCurrentPlatform = Platform == UnrealTargetPlatform.Mac || (UEBuildPlatform.IsPlatformInGroup(Platform, UnrealPlatformGroup.Unix) && Platform == BuildHostPlatform.Current.Platform);
				}
				else
				{
					IsCurrentPlatform = Platform.IsInGroup(UnrealPlatformGroup.Windows);
				}

				if (IsCurrentPlatform)
				{
					// The hardcoded engine directory needs to be a relative path to match the normal EngineDir format. Not doing so breaks the network file system (TTP#315861).
					string OutputFilePath = ExecutableBinary.OutputFilePath.FullName;
					if (Platform == UnrealTargetPlatform.Mac && OutputFilePath.Contains(".app/Contents/MacOS", StringComparison.Ordinal))
					{
						OutputFilePath = OutputFilePath.Substring(0, OutputFilePath.LastIndexOf(".app/Contents/MacOS", StringComparison.Ordinal) + 4);
					}

					DirectoryReference OutputDir = new DirectoryReference(OutputFilePath).ParentDirectory!;
					DirectoryReference RootDir = Unreal.EngineDirectory.ParentDirectory!;
					string RootPath = RootDir.MakeRelativeTo(OutputDir);

					// now get the cleaned engine path (Root/Engine), and make sure to end with /
					string EnginePath = Utils.CleanDirectorySeparators(Path.Combine(RootPath, "Engine/"), '/');
					GlobalCompileEnvironment.Definitions.Add(String.Format("UE_ENGINE_DIRECTORY=\"{0}\"", EnginePath));
				}
			}

			// Setup defines for statically selected allocator type
			string[] Mapping = {
				"FORCE_ANSI_ALLOCATOR",
				"USE_MALLOC_BINNED2",
				"USE_MALLOC_BINNED3",
			};

			for (int i = 0; i < (int)StaticAllocatorType.None; ++i)
			{
				if (GlobalCompileEnvironment.Definitions.Contains(Mapping[i] + "=0") || GlobalCompileEnvironment.Definitions.Contains(Mapping[i] + "=1"))
				{
					throw new BuildException("Target `{0}` contains a definition for `{1}` allocator define. It conflicts with the new allocator type declatation via UBT. Please use `StaticAllocator = StaticAllocatorType.{2};` in your *.Target.cs instead", TargetName, Mapping[i], ((StaticAllocatorType)i).ToString());
				}

				string EnabledValue = Rules.StaticAllocator == (StaticAllocatorType)i ? "=1" : "=0";
				GlobalCompileEnvironment.Definitions.Add(Mapping[i] + EnabledValue);
			}

			// Initialize the compile and link environments for the platform, configuration, and project.
			BuildPlatform.SetUpEnvironment(Rules, GlobalCompileEnvironment, GlobalLinkEnvironment);
			BuildPlatform.SetUpConfigurationEnvironment(Rules, GlobalCompileEnvironment, GlobalLinkEnvironment);
		}

		public static CppConfiguration GetCppConfiguration(UnrealTargetConfiguration Configuration)
		{
			switch (Configuration)
			{
				case UnrealTargetConfiguration.Debug:
					return CppConfiguration.Debug;
				case UnrealTargetConfiguration.DebugGame:
				case UnrealTargetConfiguration.Development:
					return CppConfiguration.Development;
				case UnrealTargetConfiguration.Shipping:
					return CppConfiguration.Shipping;
				case UnrealTargetConfiguration.Test:
					return CppConfiguration.Shipping;
				default:
					throw new BuildException("Unhandled target configuration");
			}
		}

		/// <summary>
		/// Create a rules object for the given module, and set any default values for this target
		/// </summary>
		private ModuleRules CreateModuleRulesAndSetDefaults(string ModuleName, string ReferenceChain, ILogger Logger)
		{
			// Create the rules from the assembly
			ModuleRules RulesObject = RulesAssembly.CreateModuleRules(ModuleName, Rules, ReferenceChain, Logger);

			// Set whether the module requires an IMPLEMENT_MODULE macro
			if (!RulesObject.bRequiresImplementModule.HasValue)
			{
				RulesObject.bRequiresImplementModule = (RulesObject.Type == ModuleRules.ModuleType.CPlusPlus && RulesObject.Name != Rules.LaunchModuleName);
			}

			// Reads additional dependencies array for project module from project file and fills PrivateDependencyModuleNames.
			if (ProjectDescriptor != null && ProjectDescriptor.Modules != null)
			{
				ModuleDescriptor? Module = ProjectDescriptor.Modules.FirstOrDefault(x => x.Name.Equals(ModuleName, StringComparison.OrdinalIgnoreCase));
				if (Module != null && Module.AdditionalDependencies != null)
				{
					RulesObject.PrivateDependencyModuleNames.AddRange(Module.AdditionalDependencies);
				}
			}

			// If compiling with the AutoRTFM compiler, implicitly depend on the AutoRTFM runtime.
			// Exclude Instrumentation to avoid circular dependency (Instrumentation -> AutoRTFM -> Instrumentation).
			if (Rules.bUseAutoRTFMCompiler && ModuleName != "AutoRTFM" && ModuleName != "Instrumentation")
			{
				RulesObject.PrivateDependencyModuleNames.Add("AutoRTFM");
			}

			// Make sure include paths don't end in trailing slashes. This can result in enclosing quotes being escaped when passed to command line tools.
			RemoveTrailingSlashes(RulesObject.PublicIncludePaths);
			RemoveTrailingSlashes(RulesObject.PublicSystemIncludePaths);
			RemoveTrailingSlashes(RulesObject.InternalIncludePaths);
			RemoveTrailingSlashes(RulesObject.PrivateIncludePaths);
			RemoveTrailingSlashes(RulesObject.PublicSystemLibraryPaths);

			RulesObject.PrivateDefinitions.Add($"UE_MODULE_NAME=\"{ModuleName}\"");
			RulesObject.PrivateDefinitions.Add($"UE_PLUGIN_NAME=\"{(RulesObject.Plugin?.Name ?? String.Empty)}\"");

			// Module validation can be found in UEBuildModuleCPP.ValidateModule()

			return RulesObject;
		}

		/// <summary>
		/// Utility function to remove trailing slashes from a list of paths
		/// </summary>
		/// <param name="Paths">List of paths to process</param>
		private static void RemoveTrailingSlashes(List<string> Paths)
		{
			for (int Idx = 0; Idx < Paths.Count; Idx++)
			{
				Paths[Idx] = Paths[Idx].TrimEnd('\\');
			}
		}

		/// <summary>
		/// Determines the relative base directory path from the specified Binary directory to the Engine base directory if possible.
		/// </summary>
		/// <param name="BinaryDirectory">Directory that contains the platform subdirectoies for each UBT built binary</param>
		/// <param name="TargetPlatform">Platform that the binary is for</param>
		/// <returns>A string specifying the relative path from the binary directory to the engine base dir, or null if this isn't contained within the engine hierarhcy</returns>
		public static string? GetRelativeBaseDir(DirectoryReference BinaryDirectory, UnrealTargetPlatform TargetPlatform)
		{
			if (BinaryDirectory.IsUnderDirectory(Unreal.EngineDirectory))
			{
				DirectoryReference BaseDir = DirectoryReference.Combine(Unreal.EngineDirectory, "Binaries", TargetPlatform.ToString());
				if (BaseDir != BinaryDirectory)
				{
					return BaseDir.MakeRelativeTo(BinaryDirectory).Replace(Path.DirectorySeparatorChar, '/');
				}
			}

			return null;
		}

		/// <summary>
		/// Finds a module given its name.  Throws an exception if the module couldn't be found.
		/// </summary>
		/// <param name="ModuleName">Name of the module</param>
		/// <param name="ReferenceChain">Chain of references causing this module to be instantiated, for display in error messages</param>
		/// <param name="Logger">Logger for output</param>
		public UEBuildModule CreateModuleByName(string ModuleName, string ReferenceChain, ILogger Logger)
		{
			// @todo projectfiles: Cross-platform modules can appear here during project generation, but they may have already
			//   been filtered out by the project generator.  This causes the projects to not be added to directories properly.
			ModuleRules RulesObject = CreateModuleRulesAndSetDefaults(ModuleName, ReferenceChain, Logger);

			// Clear the bUsePrecompiled flag if we're compiling a foreign plugin; since it's treated like an engine module, it will default to true in an installed build.
			if (RulesObject.Plugin != null && RulesObject.Plugin.File == ForeignPlugin)
			{
				RulesObject.bPrecompile = true;
				RulesObject.bUsePrecompiled = false;
			}

			// Get the base directory for paths referenced by the module. If the module's under the UProject source directory use that, otherwise leave it relative to the Engine source directory.
			if (ProjectFile != null)
			{
				DirectoryReference ProjectSourceDirectoryName = DirectoryReference.Combine(ProjectFile.Directory, "Source");
				if (RulesObject.File.IsUnderDirectory(ProjectSourceDirectoryName))
				{
					RulesObject.PublicIncludePaths = CombinePathList(ProjectSourceDirectoryName, RulesObject.PublicIncludePaths);
					RulesObject.InternalIncludePaths = CombinePathList(ProjectSourceDirectoryName, RulesObject.InternalIncludePaths);
					RulesObject.PrivateIncludePaths = CombinePathList(ProjectSourceDirectoryName, RulesObject.PrivateIncludePaths);
					RulesObject.PublicSystemLibraryPaths = CombinePathList(ProjectSourceDirectoryName, RulesObject.PublicSystemLibraryPaths);
				}
			}

			// Get the generated code directory. Plugins always write to their own intermediate directory so they can be copied between projects, shared engine
			// intermediates go in the engine intermediate folder, and anything else goes in the project folder.
			DirectoryReference? GeneratedCodeDirectory = null;
			if (RulesObject.Type != ModuleRules.ModuleType.External)
			{
				// Get the base directory
				if (bUseSharedBuildEnvironment)
				{
					GeneratedCodeDirectory = RulesObject.Context.DefaultOutputBaseDir;
				}
				else
				{
					GeneratedCodeDirectory = ProjectDirectory;
				}

				// Get the subfolder containing generated code - we don't need architecture information since these are shared between all arches for a platform, as well as shared between all intermediate environment variants
				GeneratedCodeDirectory = DirectoryReference.Combine(GeneratedCodeDirectory, PlatformIntermediateFolderNoArch, AppName, "Inc");

				// Append the binaries subfolder, if present. We rely on this to ensure that build products can be filtered correctly.
				if (RulesObject.BinariesSubFolder != null)
				{
					GeneratedCodeDirectory = DirectoryReference.Combine(GeneratedCodeDirectory, RulesObject.BinariesSubFolder);
				}

				// Finally, append the module name (using the ShortName if it has been set)
				GeneratedCodeDirectory = DirectoryReference.Combine(GeneratedCodeDirectory, RulesObject.ShortName ?? ModuleName);
			}

			// For legacy modules, add a bunch of default include paths.
			if (RulesObject.Type == ModuleRules.ModuleType.CPlusPlus && RulesObject.bAddDefaultIncludePaths && (RulesObject.Plugin != null || (ProjectFile != null && RulesObject.File.IsUnderDirectory(ProjectFile.Directory))))
			{
				// Add the module source directory
				DirectoryReference BaseSourceDirectory;
				if (RulesObject.Plugin != null)
				{
					BaseSourceDirectory = DirectoryReference.Combine(RulesObject.Plugin.Directory, "Source");
				}
				else
				{
					BaseSourceDirectory = DirectoryReference.Combine(ProjectFile!.Directory, "Source");
				}

				// If it's a game module (plugin or otherwise), add the root source directory to the include paths.
				if (RulesObject.File.IsUnderDirectory(BaseSourceDirectory) || (RulesObject.Plugin != null && RulesObject.Plugin.LoadedFrom == PluginLoadedFrom.Project))
				{
					if (DirectoryItem.GetItemByDirectoryReference(BaseSourceDirectory).Exists)
					{
						RulesObject.PublicIncludePaths.Add(NormalizeIncludePath(BaseSourceDirectory));
					}
				}

				// Resolve private include paths against the project source root
				for (int Idx = 0; Idx < RulesObject.PrivateIncludePaths.Count; Idx++)
				{
					string PrivateIncludePath = RulesObject.PrivateIncludePaths[Idx];
					if (!Path.IsPathRooted(PrivateIncludePath))
					{
						PrivateIncludePath = DirectoryReference.Combine(BaseSourceDirectory, PrivateIncludePath).FullName;
					}
					RulesObject.PrivateIncludePaths[Idx] = PrivateIncludePath;
				}
			}

			// Allow the current target to modify the module rules
			ModifyModuleRulesForTarget(ModuleName, RulesObject);

			// Allow the current platform to modify the module rules
			UEBuildPlatform.GetBuildPlatform(Platform).ModifyModuleRulesForActivePlatform(ModuleName, RulesObject, Rules);

			// Allow all build platforms to 'adjust' the module setting.
			// This will allow undisclosed platforms to make changes without
			// exposing information about the platform in publicly accessible
			// locations.
			UEBuildPlatform.PlatformModifyHostModuleRules(ModuleName, RulesObject, Rules);

			// If there are files to generate for module, then we have to create Gen directory and add it as public include
			if (RulesObject.FilesToGenerate.Count != 0)
			{
				DirectoryReference GenDir = DirectoryReference.Combine(GetModuleIntermediateDirectory(RulesObject, Architectures), "Gen");
				DirectoryReference.CreateDirectory(GenDir);
				RulesObject.PublicIncludePaths.Add(NormalizeIncludePath(GenDir));
			}

			// Now, go ahead and create the module builder instance
			UEBuildModule Module = InstantiateModule(RulesObject, GeneratedCodeDirectory, Logger);

			// Check if this module conflicts with the ShortName of any other existing modules
			if (!UsedShortNames.TryAdd(Module.Rules.ShortName ?? Module.Name, Module))
			{
				UEBuildModule? ConflictingModule = UsedShortNames[Module.Rules.ShortName ?? Module.Name];
				Logger.LogError("Conflicting module short names found, unable to create module {Name}", Module.Name);
				Logger.LogInformation(" * Name={Name} ShortName={ShortName} Path={Path}", ConflictingModule.Name, ConflictingModule.Rules.ShortName ?? ConflictingModule.Name, ConflictingModule.RulesFile);
				Logger.LogInformation(" * Name={Name} ShortName={ShortName} Path={Path}", Module.Name, Module.Rules.ShortName ?? Module.Name, Module.RulesFile);

				throw new BuildException("Unable to create module {0} (ShortName={1}) as it conflicts with existing module {2} (ShortName={3})",
					Module.Name, Module.Rules.ShortName ?? Module.Name,
					ConflictingModule.Name, ConflictingModule.Rules.ShortName ?? ConflictingModule.Name);
			}
			return Module;
		}

		/// <summary>
		/// Finds a module given its name.  Throws an exception if the module couldn't be found.
		/// </summary>
		/// <param name="ModuleName">Name of the module</param>
		/// <param name="ReferenceChain">Chain of references causing this module to be instantiated, for display in error messages</param>
		/// <param name="Logger">Logger for output</param>
		public UEBuildModule FindOrCreateModuleByName(string ModuleName, string ReferenceChain, ILogger Logger)
		{
			UEBuildModule? Module;
			if (!Modules.TryGetValue(ModuleName, out Module))
			{
				Module = CreateModuleByName(ModuleName, ReferenceChain, Logger);
				Modules.Add(Module.Name, Module);
			}

			// Warn if the module reference has incorrect text case
			if (!Module.Name.Equals(ModuleName, StringComparison.Ordinal) && Module.Name.Equals(ModuleName, StringComparison.OrdinalIgnoreCase))
			{
				Logger.LogWarning("Module '{ModuleName}' (referenced via {ReferenceChain}) has incorrect text case. Did you mean '{PossibleModuleName}'?", ModuleName, ReferenceChain, Module.Name);
			}
			return Module;
		}

		/// <summary>
		/// Constructs a new C++ module
		/// </summary>
		/// <param name="ModuleName">Name of the module</param>
		/// <param name="ReferenceChain">Chain of references causing this module to be instantiated, for display in error messages</param>
		/// <param name="Logger">Logger for output</param>
		/// <returns>New C++ module</returns>
		public UEBuildModuleCPP FindOrCreateCppModuleByName(string ModuleName, string ReferenceChain, ILogger Logger)
		{
			UEBuildModuleCPP? CppModule = FindOrCreateModuleByName(ModuleName, ReferenceChain, Logger) as UEBuildModuleCPP;
			if (CppModule == null)
			{
				throw new BuildException("'{0}' is not a C++ module (referenced via {1})", ModuleName, ReferenceChain);
			}
			return CppModule;
		}

		protected UEBuildModule InstantiateModule(
			ModuleRules RulesObject,
			DirectoryReference? GeneratedCodeDirectory,
			ILogger Logger)
		{
			switch (RulesObject.Type)
			{
				case ModuleRules.ModuleType.CPlusPlus:
					return new UEBuildModuleCPP(
							Rules: RulesObject,
							IntermediateDirectory: GetModuleIntermediateDirectory(RulesObject, Architectures),
							IntermediateDirectoryNoArch: GetModuleIntermediateDirectory(RulesObject, null),
							GeneratedCodeDirectory: GeneratedCodeDirectory,
							Logger
						);

				case ModuleRules.ModuleType.External:
					return new UEBuildModuleExternal(
							Rules: RulesObject,
							IntermediateDirectory: GetModuleIntermediateDirectory(RulesObject, Architectures),
							IntermediateDirectoryNoArch: GetModuleIntermediateDirectory(RulesObject, null),
							Logger
						);

				default:
					throw new BuildException("Unrecognized module type specified by 'Rules' object {0}", RulesObject.ToString());
			}
		}

		/// <summary>
		/// Normalize an include path to be relative to the engine source directory
		/// </summary>
		public static string NormalizeIncludePath(DirectoryReference Directory)
		{
			return Utils.CleanDirectorySeparators(Directory.MakeRelativeTo(Unreal.EngineSourceDirectory), '/');
		}

		/// <summary>
		/// Finds a module given its name.  Throws an exception if the module couldn't be found.
		/// </summary>
		public UEBuildModule GetModuleByName(string Name)
		{
			UEBuildModule? Result;
			if (Modules.TryGetValue(Name, out Result))
			{
				return Result;
			}
			else
			{
				throw new BuildException("Couldn't find referenced module '{0}'.", Name);
			}
		}

		/// <summary>
		/// Combines a list of paths with a base path.
		/// </summary>
		/// <param name="BasePath">Base path to combine with. May be null or empty.</param>
		/// <param name="PathList">List of input paths to combine with. May be null.</param>
		/// <returns>List of paths relative The build module object for the specified build rules source file</returns>
		private static List<string> CombinePathList(DirectoryReference BasePath, List<string> PathList)
		{
			List<string> NewPathList = new List<string>();
			foreach (string Path in PathList)
			{
				if (Path.StartsWith("$(", StringComparison.Ordinal))
				{
					NewPathList.Add(Path);
				}
				else
				{
					NewPathList.Add(System.IO.Path.Combine(BasePath.FullName, Path));
				}
			}
			return NewPathList;
		}
	}
}
