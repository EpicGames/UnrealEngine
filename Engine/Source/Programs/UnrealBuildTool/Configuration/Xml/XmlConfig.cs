// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using System.Linq;
using System.Reflection;
using System.Text;
using System.Xml;
using System.Xml.Linq;
using System.Xml.Schema;
using System.Xml.Serialization;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// Functions for manipulating the XML config cache
	/// </summary>
	static class XmlConfig
	{
		/// <summary>
		/// An input config file
		/// </summary>
		public class InputFile
		{
			/// <summary>
			/// Location of the file
			/// </summary>
			public FileReference Location { get; init; }

			/// <summary>
			/// Which folder to display the config file under in the generated project files
			/// </summary>
			public string FolderName { get; init; }

			/// <summary>
			/// Constructor
			/// </summary>
			/// <param name="location"></param>
			/// <param name="folderName"></param>
			public InputFile(FileReference location, string folderName)
			{
				Location = location;
				FolderName = folderName;
			}
		}

		/// <summary>
		/// The cache file that is being used
		/// </summary>
		public static FileReference? CacheFile;

		/// <summary>
		/// Parsed config values
		/// </summary>
		static XmlConfigData? s_values;

		/// <summary>
		/// Cached serializer for the XML schema
		/// </summary>
		static XmlSerializer? s_cachedSchemaSerializer;

		/// <summary>
		/// Initialize the config system with the given types
		/// </summary>
		/// <param name="overrideCacheFile">Force use of the cached XML config without checking if it's valid (useful for remote builds)</param>
		/// <param name="projectRootDirectory">Read XML configuration with a Project directory</param>
		/// <param name="logger">Logger for output</param>
		public static void ReadConfigFiles(FileReference? overrideCacheFile, DirectoryReference? projectRootDirectory, ILogger logger)
		{
			string XmlConfigMutexName = GlobalSingleInstanceMutex.GetUniqueMutexForPath("UnrealBuildTool_Mutex_XmlConfig", Unreal.EngineDirectory);
			using IDisposable XmlConfigMutex = SingleInstanceMutex.Acquire(XmlConfigMutexName);

			// Find all the configurable types
			List<Type> configTypes = FindConfigurableTypes();

			if (projectRootDirectory == null)
			{
				logger.LogDebug("No project directory provided for project scope BuildConfiguration.xml resolution.");
			}

			// Update the cache if necessary
			if (overrideCacheFile != null)
			{
				// Set the cache file to the overriden value
				CacheFile = overrideCacheFile;

				// Never rebuild the cache; just try to load it.
				if (!XmlConfigData.TryRead(CacheFile, configTypes, out s_values))
				{
					throw new BuildException("Unable to load XML config cache ({0})", CacheFile);
				}
			}
			else
			{
				if (projectRootDirectory == null)
				{
					// Get the default cache file
					CacheFile = FileReference.Combine(Unreal.EngineDirectory, "Intermediate", "Build", "XmlConfigCache.bin");
					if (Unreal.IsEngineInstalled())
					{
						DirectoryReference? userSettingsDir = Unreal.UserSettingDirectory;
						if (userSettingsDir != null)
						{
							CacheFile = FileReference.Combine(userSettingsDir, "UnrealEngine", $"XmlConfigCache-{Unreal.RootDirectory.FullName.Replace(":", "", StringComparison.OrdinalIgnoreCase).Replace(Path.DirectorySeparatorChar, '+')}.bin");
						}
					}
				}
				else
				{
					CacheFile = FileReference.Combine(projectRootDirectory, "Intermediate", "Build", "XmlConfigCache.bin");
					s_values = null;
				}

				// Find all the input files
				List<FileReference> temporaryInputFileLocations = InputFiles.Select(x => x.Location).ToList();

				if (projectRootDirectory != null)
				{
					FileReference projectRootConfigLocation = FileReference.Combine(projectRootDirectory, "Saved", "UnrealBuildTool", "BuildConfiguration.xml");
					if (!FileReference.Exists(projectRootConfigLocation))
					{
						logger.LogDebug("No project scope config file at {ProjectRootConfigLocation}. Creating default.", projectRootConfigLocation.FullName);
						CreateDefaultConfigFile(projectRootConfigLocation);
					}

					temporaryInputFileLocations.Add(projectRootConfigLocation);
				}

				// Write any configuration environment variables to a file so they can be loaded
				FileReference environmentXml = FileReference.Combine(Unreal.WritableEngineDirectory, "Intermediate", "Build", "UnrealBuildTool.Env.BuildConfiguration.xml");
				WriteEnvironmentXml(environmentXml, logger);
				temporaryInputFileLocations.Add(environmentXml);

				FileReference[] inputFileLocations = temporaryInputFileLocations.ToArray();

				// Get the path to the schema
				FileReference schemaFile = GetSchemaLocation(projectRootDirectory);

				// Try to read the existing cache from disk
				XmlConfigData? cachedValues;
				if (IsCacheUpToDate(CacheFile, inputFileLocations) && FileReference.Exists(schemaFile))
				{
					if (XmlConfigData.TryRead(CacheFile, configTypes, out cachedValues) && Enumerable.SequenceEqual(inputFileLocations, cachedValues.InputFiles))
					{
						s_values = cachedValues;
					}
				}

				// If that failed, regenerate it
				if (s_values == null)
				{
					// Find all the configurable fields from the given types
					Dictionary<string, Dictionary<string, XmlConfigData.TargetMember>> categoryToFields = new Dictionary<string, Dictionary<string, XmlConfigData.TargetMember>>();
					FindConfigurableFields(configTypes, categoryToFields);

					// Create a schema for the config files
					XmlSchema schema = CreateSchema(categoryToFields);
					if (!Unreal.IsEngineInstalled())
					{
						WriteSchema(schema, schemaFile);
					}

					// Read all the XML files and validate them against the schema
					Dictionary<Type, Dictionary<XmlConfigData.TargetMember, XmlConfigData.ValueInfo>> typeToValues =
						new Dictionary<Type, Dictionary<XmlConfigData.TargetMember, XmlConfigData.ValueInfo>>();
					foreach (FileReference inputFile in inputFileLocations)
					{
						logger.LogDebug("Reading configuration file from: {Location}", inputFile.FullName);
						if (!TryReadFile(inputFile, categoryToFields, typeToValues, schema, logger))
						{
							throw new BuildException("Failed to properly read XML file : {0}", inputFile.FullName);
						}
					}

					// Make sure the cache directory exists
					DirectoryReference.CreateDirectory(CacheFile.Directory);

					// Create the new cache
					s_values = new XmlConfigData(inputFileLocations, typeToValues.ToDictionary(
							x => x.Key,
							x => x.Value.Select(x => x.Value).ToArray()));
					s_values.Write(CacheFile);
				}
				else
				{
					foreach (FileReference inputFile in s_values.InputFiles)
					{
						logger.LogDebug("Reading configuration file from cache ({CacheLocaltion}): {Location}", CacheFile.FullName, inputFile.FullName);
					}
				}
			}

			// Apply all the static field values
			foreach (KeyValuePair<Type, XmlConfigData.ValueInfo[]> typeValuesPair in s_values.TypeToValues)
			{
				foreach (XmlConfigData.ValueInfo memberValue in typeValuesPair.Value)
				{
					if (memberValue.Target.IsStatic)
					{
						object value = InstanceValue(memberValue.Value, memberValue.Target.Type);
						memberValue.Target.SetValue(null, value);
					}
				}
			}
		}

		/// <summary>
		/// Find all the configurable types in the current assembly
		/// </summary>
		/// <returns>List of configurable types</returns>
		static List<Type> FindConfigurableTypes()
		{
			List<Type> configTypes = new List<Type>();
			try
			{
				foreach (Type configType in Assembly.GetExecutingAssembly().GetTypes())
				{
					if (HasXmlConfigFileAttribute(configType))
					{
						configTypes.Add(configType);
					}
				}
			}
			catch (ReflectionTypeLoadException ex)
			{
				Console.WriteLine("TypeLoadException: {0}", String.Join("\n", ex.LoaderExceptions.Select(x => x?.Message)));
				throw;
			}
			return configTypes;
		}

		/// <summary>
		/// Determines whether the given type has a field with an XmlConfigFile attribute
		/// </summary>
		/// <param name="type">The type to check</param>
		/// <returns>True if the type has a field with the XmlConfigFile attribute</returns>
		static bool HasXmlConfigFileAttribute(Type type)
		{
			foreach (FieldInfo field in type.GetFields(BindingFlags.Instance | BindingFlags.Static | BindingFlags.Public | BindingFlags.NonPublic))
			{
				foreach (CustomAttributeData customAttribute in field.CustomAttributes)
				{
					if (customAttribute.AttributeType == typeof(XmlConfigFileAttribute))
					{
						return true;
					}
				}
			}
			foreach (PropertyInfo property in type.GetProperties(BindingFlags.Instance | BindingFlags.Static | BindingFlags.Public | BindingFlags.NonPublic))
			{
				foreach (CustomAttributeData customAttribute in property.CustomAttributes)
				{
					if (customAttribute.AttributeType == typeof(XmlConfigFileAttribute))
					{
						return true;
					}
				}
			}
			return false;
		}

		/// <summary>
		/// Find the location of the XML config schema
		/// </summary>
		/// <param name="projectRootDirecory">Optional project root directory</param>
		/// <returns>The location of the schema file</returns>
		public static FileReference GetSchemaLocation(DirectoryReference? projectRootDirecory = null)
		{
			if (projectRootDirecory != null)
			{
				FileReference projectSchema = FileReference.Combine(projectRootDirecory, "Saved", "UnrealBuildTool", "BuildConfiguration.Schema.xsd");
				if (FileReference.Exists(projectSchema))
				{
					return projectSchema;
				}
			}
			return FileReference.Combine(Unreal.EngineDirectory, "Saved", "UnrealBuildTool", "BuildConfiguration.Schema.xsd");
		}

		static InputFile[]? s_cachedInputFiles;

		/// <summary>
		/// Initialize the list of input files
		/// </summary>
		public static InputFile[] InputFiles
		{
			get
			{
				if (s_cachedInputFiles != null)
				{
					return s_cachedInputFiles;
				}

				ILogger logger = Log.Logger;

				// Find all the input file locations
				List<InputFile> inputFilesFound = new List<InputFile>(5);

				// InputFile info and if a default file should be created if missing
				List<KeyValuePair<InputFile, bool>> configs = new();

				// Skip all the config files under the Engine folder if it's an installed build
				if (!Unreal.IsEngineInstalled())
				{
					// Check for the engine config file under /Engine/Programs/NotForLicensees/UnrealBuildTool
					configs.Add(new(new InputFile(FileReference.Combine(Unreal.EngineDirectory, "Restricted", "NotForLicensees", "Programs", "UnrealBuildTool", "BuildConfiguration.xml"), "Engine (NotForLicensees)"), false));

					// Check for the engine user config file under /Engine/Saved/UnrealBuildTool
					configs.Add(new(new InputFile(FileReference.Combine(Unreal.EngineDirectory, "Saved", "UnrealBuildTool", "BuildConfiguration.xml"), "Engine (Saved)"), true));
				}

				// Check for the global config file under ProgramData/Unreal Engine/UnrealBuildTool
				DirectoryReference? commonProgramsFolder = DirectoryReference.FromString(Environment.GetFolderPath(Environment.SpecialFolder.CommonApplicationData));
				if (commonProgramsFolder != null)
				{
					configs.Add(new(new InputFile(FileReference.Combine(commonProgramsFolder, "Unreal Engine", "UnrealBuildTool", "BuildConfiguration.xml"), "Global (ProgramData)"), false));
				}

				// Check for the global config file under AppData/Unreal Engine/UnrealBuildTool (Roaming)
				DirectoryReference? appDataFolder = DirectoryReference.FromString(Environment.GetFolderPath(Environment.SpecialFolder.ApplicationData));
				if (appDataFolder != null)
				{
					configs.Add(new(new InputFile(FileReference.Combine(appDataFolder, "Unreal Engine", "UnrealBuildTool", "BuildConfiguration.xml"), "Global (AppData)"), true));
				}

				// Check for the global config file under LocalAppData/Unreal Engine/UnrealBuildTool
				DirectoryReference? localAppDataFolder = DirectoryReference.FromString(Environment.GetFolderPath(Environment.SpecialFolder.LocalApplicationData));
				if (localAppDataFolder != null)
				{
					configs.Add(new(new InputFile(FileReference.Combine(localAppDataFolder, "Unreal Engine", "UnrealBuildTool", "BuildConfiguration.xml"), "Global (LocalAppData)"), false));
				}

				// Check for the global config file under My Documents/Unreal Engine/UnrealBuildTool
				DirectoryReference? personalFolder = DirectoryReference.FromString(Environment.GetFolderPath(Environment.SpecialFolder.Personal));
				if (personalFolder != null)
				{
					configs.Add(new(new InputFile(FileReference.Combine(personalFolder, "Unreal Engine", "UnrealBuildTool", "BuildConfiguration.xml"), "Global (Documents)"), false));
				}

				foreach (KeyValuePair<InputFile, bool> config in configs)
				{
					if (config.Value && !FileReference.Exists(config.Key.Location))
					{
						logger.LogDebug("Creating default config file at {ConfigLocation}", config.Key.Location);
						CreateDefaultConfigFile(config.Key.Location);
					}
					if (FileReference.Exists(config.Key.Location))
					{
						inputFilesFound.Add(config.Key);
					}
					else
					{
						logger.LogDebug("No config file at {ConfigLocation}", config.Key.Location);
					}
				}

				s_cachedInputFiles = inputFilesFound.ToArray();

				logger.LogDebug("Configuration will be read from:");
				foreach (InputFile inputFile in InputFiles)
				{
					logger.LogDebug("  {File}", inputFile.Location.FullName);
				}

				return s_cachedInputFiles;
			}
		}

		/// <summary>
		/// Create a default config file at the given location
		/// </summary>
		/// <param name="location">Location to read from</param>
		static void CreateDefaultConfigFile(FileReference location)
		{
			DirectoryReference.CreateDirectory(location.Directory);
			using (StreamWriter writer = new StreamWriter(location.FullName))
			{
				writer.WriteLine("<?xml version=\"1.0\" encoding=\"utf-8\" ?>");
				writer.WriteLine("<Configuration xmlns=\"{0}\">", XmlConfigFile.SchemaNamespaceURI);
				writer.WriteLine("</Configuration>");
			}
		}

		/// <summary>
		/// Applies config values to the given object
		/// </summary>
		/// <param name="targetObject">The object instance to be configured</param>
		public static void ApplyTo(object targetObject)
		{
			ILogger logger = Log.Logger;
			for (Type? targetType = targetObject.GetType(); targetType != null; targetType = targetType.BaseType)
			{
				XmlConfigData.ValueInfo[]? fieldValues;
				if (s_values!.TypeToValues.TryGetValue(targetType, out fieldValues))
				{
					foreach (XmlConfigData.ValueInfo fieldValue in fieldValues)
					{
						if (!fieldValue.Target.IsStatic)
						{
							XmlConfigData.TargetMember targetToWrite = fieldValue.Target;

							// Check if setting has been deprecated
							if (fieldValue.XmlConfigAttribute.Deprecated)
							{
								string currentSettingName = fieldValue.XmlConfigAttribute.Name ?? fieldValue.Target.MemberInfo.Name;

								logger.LogWarning("Deprecated setting found in \"{SourceFile}\":", fieldValue.SourceFile);
								logger.LogWarning("The setting \"{Setting}\" is deprecated. Support for this setting will be removed in a future version of Unreal Engine.", currentSettingName);

								if (fieldValue.XmlConfigAttribute.NewAttributeName != null)
								{
									// NewAttributeName is the name of a member in code. However, the log messages below are written from the XML's perspective,
									// so we need to check if the new target member is not exposed under a custom name in the config.
									string newSettingName = GetMemberConfigAttributeName(targetType, fieldValue.XmlConfigAttribute.NewAttributeName);

									logger.LogWarning("Use \"{NewAttributeName}\" in place of \"{OldAttributeName}\"", newSettingName, currentSettingName);
									logger.LogInformation("The value provided for deprecated setting \"{OldName}\" will be applied to \"{NewName}\"", currentSettingName, newSettingName);

									targetToWrite = GetTargetMember(targetType, fieldValue.XmlConfigAttribute.NewAttributeName) ?? targetToWrite;
								}
							}

							object valueInstance = InstanceValue(fieldValue.Value, fieldValue.Target.Type);
							targetToWrite.SetValue(targetObject, valueInstance);
						}
					}
				}
			}
		}

		private static string GetMemberConfigAttributeName(Type targetType, string memberName)
		{
			MemberInfo? memberInfo = targetType.GetRuntimeFields().FirstOrDefault(x => x.Name == memberName) as MemberInfo
				?? targetType.GetRuntimeProperties().FirstOrDefault(x => x.Name == memberName) as MemberInfo;

			XmlConfigFileAttribute? attribute = memberInfo?.GetCustomAttributes<XmlConfigFileAttribute>().FirstOrDefault();

			return attribute?.Name ?? memberName;
		}

		private static XmlConfigData.TargetMember? GetTargetMember(Type targetType, string memberName)
		{
			// First, try to find the new field to which the setting should be actually applied.
			FieldInfo? fieldInfo = targetType.GetRuntimeFields()
				.FirstOrDefault(x => x.Name == memberName);

			if (fieldInfo != null)
			{
				return new XmlConfigData.TargetField(fieldInfo);
			}

			// If not found, try to find the new property to which the setting should be actually applied.
			PropertyInfo? propertyInfo = targetType.GetRuntimeProperties()
				.FirstOrDefault(x => x.Name == memberName);

			if (propertyInfo != null)
			{
				return new XmlConfigData.TargetProperty(propertyInfo);
			}

			return null;
		}

		/// <summary>
		/// Instances a value for assignment to a target object
		/// </summary>
		/// <param name="value">The value to instance</param>
		/// <param name="valueType">The type of value</param>
		/// <returns>New instance of the given value, if necessary</returns>
		static object InstanceValue(object value, Type valueType)
		{
			if (valueType == typeof(string[]))
			{
				return ((string[])value).Clone();
			}
			else
			{
				return value;
			}
		}

		/// <summary>
		/// Gets a config value for a single value, without writing it to an instance of that class
		/// </summary>
		/// <param name="targetType">Type to find config values for</param>
		/// <param name="name">Name of the field to receive</param>
		/// <param name="value">On success, receives the value of the field</param>
		/// <returns>True if the value was read, false otherwise</returns>
		public static bool TryGetValue(Type targetType, string name, [NotNullWhen(true)] out object? value)
		{
			// Find all the config values for this type
			XmlConfigData.ValueInfo[]? fieldValues;
			if (!s_values!.TypeToValues.TryGetValue(targetType, out fieldValues))
			{
				value = null;
				return false;
			}

			// Find the value with the matching name
			foreach (XmlConfigData.ValueInfo fieldValue in fieldValues)
			{
				if (fieldValue.Target.MemberInfo.Name == name)
				{
					value = fieldValue.Value;
					return true;
				}
			}

			// Not found
			value = null;
			return false;
		}

		/// <summary>
		/// Find all the configurable fields in the given types by searching for XmlConfigFile attributes.
		/// </summary>
		/// <param name="configTypes">Array of types to search</param>
		/// <param name="categoryToFields">Dictionaries populated with category -> name -> field mappings on return</param>
		static void FindConfigurableFields(IEnumerable<Type> configTypes, Dictionary<string, Dictionary<string, XmlConfigData.TargetMember>> categoryToFields)
		{
			foreach (Type configType in configTypes)
			{
				foreach (FieldInfo fieldInfo in configType.GetFields(BindingFlags.Instance | BindingFlags.Static | BindingFlags.GetField | BindingFlags.Public | BindingFlags.NonPublic))
				{
					ProcessConfigurableMember<FieldInfo>(configType, fieldInfo, categoryToFields, fieldInfo => new XmlConfigData.TargetField(fieldInfo));
				}
				foreach (PropertyInfo propertyInfo in configType.GetProperties(BindingFlags.Instance | BindingFlags.Static | BindingFlags.GetField | BindingFlags.Public | BindingFlags.NonPublic))
				{
					ProcessConfigurableMember<PropertyInfo>(configType, propertyInfo, categoryToFields, propertyInfo => new XmlConfigData.TargetProperty(propertyInfo));
				}
			}
		}

		private static void ProcessConfigurableMember<MEMBER>(Type type, MEMBER memberInfo, Dictionary<string, Dictionary<string, XmlConfigData.TargetMember>> categoryToFields, Func<MEMBER, XmlConfigData.TargetMember> createTarget)
			where MEMBER : System.Reflection.MemberInfo
		{
			IEnumerable<XmlConfigFileAttribute> attributes = memberInfo.GetCustomAttributes<XmlConfigFileAttribute>();
			foreach (XmlConfigFileAttribute attribute in attributes)
			{
				string categoryName = attribute.Category ?? type.Name;

				Dictionary<string, XmlConfigData.TargetMember>? nameToTarget;
				if (!categoryToFields.TryGetValue(categoryName, out nameToTarget))
				{
					nameToTarget = new Dictionary<string, XmlConfigData.TargetMember>();
					categoryToFields.Add(categoryName, nameToTarget);
				}

				nameToTarget[attribute.Name ?? memberInfo.Name] = createTarget(memberInfo);
			}
		}

		/// <summary>
		/// Creates a schema from attributes in the given types
		/// </summary>
		/// <param name="categoryToFields">Lookup for all field settings</param>
		/// <returns>New schema instance</returns>
		static XmlSchema CreateSchema(Dictionary<string, Dictionary<string, XmlConfigData.TargetMember>> categoryToFields)
		{
			// Create elements for all the categories
			XmlSchemaAll rootAll = new XmlSchemaAll();
			foreach (KeyValuePair<string, Dictionary<string, XmlConfigData.TargetMember>> categoryPair in categoryToFields)
			{
				string categoryName = categoryPair.Key;

				XmlSchemaAll categoryAll = new XmlSchemaAll();
				foreach (KeyValuePair<string, XmlConfigData.TargetMember> fieldPair in categoryPair.Value)
				{
					XmlSchemaElement element = CreateSchemaFieldElement(fieldPair.Key, fieldPair.Value.Type);
					categoryAll.Items.Add(element);
				}

				XmlSchemaComplexType categoryType = new XmlSchemaComplexType();
				categoryType.Particle = categoryAll;

				XmlSchemaElement categoryElement = new XmlSchemaElement();
				categoryElement.Name = categoryName;
				categoryElement.SchemaType = categoryType;
				categoryElement.MinOccurs = 0;
				categoryElement.MaxOccurs = 1;

				rootAll.Items.Add(categoryElement);
			}

			// Create the root element and schema object
			XmlSchemaComplexType rootType = new XmlSchemaComplexType();
			rootType.Particle = rootAll;

			XmlSchemaElement rootElement = new XmlSchemaElement();
			rootElement.Name = XmlConfigFile.RootElementName;
			rootElement.SchemaType = rootType;

			XmlSchema schema = new XmlSchema();
			schema.TargetNamespace = XmlConfigFile.SchemaNamespaceURI;
			schema.ElementFormDefault = XmlSchemaForm.Qualified;
			schema.Items.Add(rootElement);

			// Finally compile it
			XmlSchemaSet schemaSet = new XmlSchemaSet();
			schemaSet.Add(schema);
			schemaSet.Compile();
			return schemaSet.Schemas().OfType<XmlSchema>().First();
		}

		/// <summary>
		/// Creates an XML schema element for reading a value of the given type
		/// </summary>
		/// <param name="name">Name of the field</param>
		/// <param name="type">Type of the field</param>
		/// <returns>New schema element representing the field</returns>
		static XmlSchemaElement CreateSchemaFieldElement(string name, Type type)
		{
			XmlSchemaElement element = new XmlSchemaElement();
			element.Name = name;
			element.MinOccurs = 0;
			element.MaxOccurs = 1;

			if (TryGetNullableStructType(type, out Type? innerType))
			{
				type = innerType;
			}

			if (type == typeof(string))
			{
				element.SchemaTypeName = XmlSchemaType.GetBuiltInSimpleType(XmlTypeCode.String).QualifiedName;
			}
			else if (type == typeof(bool))
			{
				element.SchemaTypeName = XmlSchemaType.GetBuiltInSimpleType(XmlTypeCode.Boolean).QualifiedName;
			}
			else if (type == typeof(int))
			{
				element.SchemaTypeName = XmlSchemaType.GetBuiltInSimpleType(XmlTypeCode.Int).QualifiedName;
			}
			else if (type == typeof(float))
			{
				element.SchemaTypeName = XmlSchemaType.GetBuiltInSimpleType(XmlTypeCode.Float).QualifiedName;
			}
			else if (type == typeof(double))
			{
				element.SchemaTypeName = XmlSchemaType.GetBuiltInSimpleType(XmlTypeCode.Double).QualifiedName;
			}
			else if (type == typeof(FileReference))
			{
				element.SchemaTypeName = XmlSchemaType.GetBuiltInSimpleType(XmlTypeCode.String).QualifiedName;
			}
			else if (type.IsEnum)
			{
				XmlSchemaSimpleTypeRestriction restriction = new XmlSchemaSimpleTypeRestriction();
				restriction.BaseTypeName = XmlSchemaType.GetBuiltInSimpleType(XmlTypeCode.String).QualifiedName;

				foreach (string enumName in Enum.GetNames(type))
				{
					XmlSchemaEnumerationFacet facet = new XmlSchemaEnumerationFacet();
					facet.Value = enumName;
					restriction.Facets.Add(facet);
				}

				XmlSchemaSimpleType enumType = new XmlSchemaSimpleType();
				enumType.Content = restriction;
				element.SchemaType = enumType;
			}
			else if (MaybeGetInnerCollectionType(type) is Type genericType)
			{
				XmlSchemaElement itemElement = CreateSchemaFieldElement("Item", genericType);
				itemElement.MaxOccursString = "unbounded";

				XmlSchemaSequence sequence = new XmlSchemaSequence();
				sequence.Items.Add(itemElement);

				XmlSchemaComplexType arrayType = new XmlSchemaComplexType();
				arrayType.Particle = sequence;
				element.SchemaType = arrayType;
			}
			else
			{
				throw new Exception("Unsupported field type for XmlConfigFile attribute");
			}
			return element;
		}

		private static Type? MaybeGetInnerCollectionType(Type type)
		{
			if (type.IsArray)
			{
				return type.GetElementType();
			}
			if (type.GetInterfaces().FirstOrDefault(x => x.IsGenericType && x.GetGenericTypeDefinition() == typeof(ICollection<>)) is Type collectionType)
			{
				return collectionType.GetGenericArguments()[0];
			}
			return null;
		}

		/// <summary>
		/// Writes a schema to the given location. Avoids writing it if the file is identical.
		/// </summary>
		/// <param name="schema">The schema to be written</param>
		/// <param name="location">Location to write to</param>
		static void WriteSchema(XmlSchema schema, FileReference location)
		{
			XmlWriterSettings settings = new XmlWriterSettings();
			settings.Indent = true;
			settings.IndentChars = "\t";
			settings.NewLineChars = Environment.NewLine;
			settings.OmitXmlDeclaration = true;

			s_cachedSchemaSerializer ??= XmlSerializer.FromTypes(new Type[] { typeof(XmlSchema) })[0]!;

			StringBuilder output = new StringBuilder();
			output.AppendLine("<?xml version=\"1.0\" encoding=\"utf-8\"?>");
			using (XmlWriter writer = XmlWriter.Create(output, settings))
			{
				XmlSerializerNamespaces namespaces = new XmlSerializerNamespaces();
				namespaces.Add("", "http://www.w3.org/2001/XMLSchema");
				s_cachedSchemaSerializer.Serialize(writer, schema, namespaces);
			}

			string outputText = output.ToString();
			if (!FileReference.Exists(location) || File.ReadAllText(location.FullName) != outputText)
			{
				DirectoryReference.CreateDirectory(location.Directory);
				File.WriteAllText(location.FullName, outputText);
			}
		}

		/// <summary>
		/// Tests whether a type is a nullable struct, and extracts the inner type if it is
		/// </summary>
		static bool TryGetNullableStructType(Type type, [NotNullWhen(true)] out Type? innerType)
		{
			if (type.IsGenericType && type.GetGenericTypeDefinition() == typeof(Nullable<>))
			{
				innerType = type.GetGenericArguments()[0];
				return true;
			}
			else
			{
				innerType = null;
				return false;
			}
		}

		/// <summary>
		/// Reads an XML config file and merges it to the given cache
		/// </summary>
		/// <param name="location">Location to read from</param>
		/// <param name="categoryToFields">Lookup for configurable fields by category</param>
		/// <param name="typeToValues">Map of types to fields and their associated values</param>
		/// <param name="schema">Schema to validate against</param>
		/// <param name="logger">Logger for output</param>
		/// <returns>True if the file was read successfully</returns>
		static bool TryReadFile(FileReference location, Dictionary<string, Dictionary<string, XmlConfigData.TargetMember>> categoryToFields,
			Dictionary<Type, Dictionary<XmlConfigData.TargetMember, XmlConfigData.ValueInfo>> typeToValues, XmlSchema schema, ILogger logger)
		{
			// Read the XML file, and validate it against the schema
			XmlConfigFile? configFile;
			if (!XmlConfigFile.TryRead(location, schema, logger, out configFile))
			{
				return false;
			}

			// Parse the document
			foreach (XmlElement categoryElement in configFile.DocumentElement!.ChildNodes.OfType<XmlElement>())
			{
				Dictionary<string, XmlConfigData.TargetMember>? nameToField;
				if (categoryToFields.TryGetValue(categoryElement.Name, out nameToField))
				{
					foreach (XmlElement keyElement in categoryElement.ChildNodes.OfType<XmlElement>())
					{
						if (nameToField.TryGetValue(keyElement.Name, out XmlConfigData.TargetMember? field))
						{
							object value;
							if (MaybeGetInnerCollectionType(field.Type) is Type innerType)
							{
								if (field.Type.IsArray)
								{
									string[] stringValues = [.. keyElement.ChildNodes.OfType<XmlElement>().Where(x => x.Name == "Item").Select(x => x.InnerText)];
									Array valueArray = Array.CreateInstance(innerType, stringValues.Length);
									for (int i = 0; i < stringValues.Length; ++i)
									{
										string stringValue = stringValues[i];
										valueArray.SetValue(ParseValue(innerType, stringValue), i);
									}
									value = valueArray;
								}
								else
								{
									throw new NotImplementedException("XmlConfig support for non-array types is not implemented.");
								}
							}
							else if (TryGetNullableStructType(field.Type, out Type? structType))
							{
								value = ParseValue(structType, keyElement.InnerText);
							}
							else
							{
								value = ParseValue(field.Type, keyElement.InnerText);
							}

							// Add it to the set of values for the type containing this field
							Dictionary<XmlConfigData.TargetMember, XmlConfigData.ValueInfo>? fieldToValue;
							if (!typeToValues.TryGetValue(field.MemberInfo.DeclaringType!, out fieldToValue))
							{
								fieldToValue = new Dictionary<XmlConfigData.TargetMember, XmlConfigData.ValueInfo>();
								typeToValues.Add(field.MemberInfo.DeclaringType!, fieldToValue);
							}

							// Parse the corresponding value
							XmlConfigData.ValueInfo fieldValue = new XmlConfigData.ValueInfo(field, value, location,
								field.MemberInfo.GetCustomAttribute<XmlConfigFileAttribute>()!);

							fieldToValue[field] = fieldValue;
						}
					}
				}
			}
			return true;
		}

		/// <summary>
		/// Writes environment variables as a BuildConfiguration xml
		/// </summary>
		/// <param name="location">Path to write to</param>
		/// <param name="logger">Logger for output</param>
		/// <returns>True if the environment was written successfully</returns>
		static bool WriteEnvironmentXml(FileReference location, ILogger logger)
		{
			// Read the environment variables
			Dictionary<string, Dictionary<string, string>> contents = [];
			foreach (DictionaryEntry EnvironmentVariable in Environment.GetEnvironmentVariables())
			{
				string key = EnvironmentVariable.Key?.ToString() ?? String.Empty;
				string value = EnvironmentVariable.Value?.ToString() ?? String.Empty;
				if (!key.StartsWith("UnrealBuildTool_", StringComparison.Ordinal) || !key.Contains("__", StringComparison.Ordinal))
				{
					continue;
				}
				string[] split = key.Substring("UnrealBuildTool_".Length).Split("__");
				string category = split.First();
				string field = split.Last();
				if (!contents.TryGetValue(category, out Dictionary<string, string>? fields))
				{
					fields = new();
					contents.Add(category, fields);
				}

				fields.Add(field, value);
			}
			StringBuilder output = new StringBuilder();
			output.AppendLine("<?xml version=\"1.0\" encoding=\"UTF-8\"?>");
			XmlWriterSettings xmlSettings = new XmlWriterSettings();
			xmlSettings.Encoding = new UTF8Encoding(false);
			xmlSettings.Indent = true;
			xmlSettings.OmitXmlDeclaration = true;
			using (XmlWriter writer = XmlWriter.Create(output, xmlSettings))
			{
				XNamespace ns = XNamespace.Get("https://www.unrealengine.com/BuildConfiguration");
				XElement root = new XElement(ns + "Configuration");
				foreach (KeyValuePair<string, Dictionary<string, string>> CategoryItem in contents)
				{
					XElement categoryElement = new XElement(ns + CategoryItem.Key);
					foreach (KeyValuePair<string, string> FieldItem in CategoryItem.Value)
					{
						XElement fieldElement = new XElement(ns + FieldItem.Key);
						if (FieldItem.Value.Contains(';', StringComparison.Ordinal))
						{
							foreach (string item in FieldItem.Value.Split(';', StringSplitOptions.RemoveEmptyEntries | StringSplitOptions.TrimEntries))
							{
								fieldElement.Add(new XElement(ns + "Item", item));
							}
						}
						else
						{
							fieldElement.Add(FieldItem.Value);
						}
						categoryElement.Add(fieldElement);
					}
					root.Add(categoryElement);
				}
				new XDocument(root).Save(writer);
			}
			Utils.WriteFileIfChanged(location, output.ToString(), logger);
			return true;
		}

		/// <summary>
		/// Parse the value for a field from its text based representation in an XML file
		/// </summary>
		/// <param name="fieldType">The type of field being read</param>
		/// <param name="text">Text to parse</param>
		/// <returns>The object that was parsed</returns>
		static object ParseValue(Type fieldType, string text)
		{
			// ignore whitespace in all fields except for Strings which we leave unprocessed
			string trimmedText = text.Trim();
			if (fieldType == typeof(string))
			{
				return text;
			}
			else if (fieldType == typeof(bool) || fieldType == typeof(bool?))
			{
				if (trimmedText == "1" || trimmedText.Equals("true", StringComparison.OrdinalIgnoreCase))
				{
					return true;
				}
				else if (trimmedText == "0" || trimmedText.Equals("false", StringComparison.OrdinalIgnoreCase))
				{
					return false;
				}
				else
				{
					throw new Exception(String.Format("Unable to convert '{0}' to boolean. 'true/false/0/1' are the supported formats.", text));
				}
			}
			else if (fieldType == typeof(int))
			{
				return Int32.Parse(trimmedText);
			}
			else if (fieldType == typeof(float))
			{
				return Single.Parse(trimmedText, System.Globalization.CultureInfo.InvariantCulture);
			}
			else if (fieldType == typeof(double))
			{
				return Double.Parse(trimmedText, System.Globalization.CultureInfo.InvariantCulture);
			}
			else if (fieldType.IsEnum)
			{
				return Enum.Parse(fieldType, trimmedText);
			}
			else if (fieldType == typeof(FileReference))
			{
				return new FileReference(text);
			}
			else
			{
				throw new Exception(String.Format("Unsupported config type '{0}'", fieldType.Name));
			}
		}

		/// <summary>
		/// Checks that the given cache file exists and is newer than the given input files, and attempts to read it. Verifies that the resulting cache was created
		/// from the same input files in the same order.
		/// </summary>
		/// <param name="cacheFile">Path to the config cache file</param>
		/// <param name="inputFiles">The expected set of input files in the cache</param>
		/// <returns>True if the cache was valid and could be read, false otherwise.</returns>
		static bool IsCacheUpToDate(FileReference cacheFile, FileReference[] inputFiles)
		{
			// Always rebuild if the cache doesn't exist
			if (!FileReference.Exists(cacheFile))
			{
				return false;
			}

			// Get the timestamp for the cache
			DateTime cacheWriteTime = File.GetLastWriteTimeUtc(cacheFile.FullName);

			// Always rebuild if this executable is newer
			if (File.GetLastWriteTimeUtc(Assembly.GetExecutingAssembly().Location) > cacheWriteTime)
			{
				return false;
			}

			// Check if any of the input files are newer than the cache
			foreach (FileReference inputFile in inputFiles)
			{
				if (File.GetLastWriteTimeUtc(inputFile.FullName) > cacheWriteTime)
				{
					return false;
				}
			}

			// Otherwise, it's up to date
			return true;
		}

		/// <summary>
		/// Generates documentation files for the available settings, by merging the XML documentation from the compiler.
		/// </summary>
		/// <param name="outputFile">The documentation file to write</param>
		/// <param name="logger">Logger for output</param>
		public static void WriteDocumentation(FileReference outputFile, ILogger logger)
		{
			// Find all the configurable types
			List<Type> configTypes = FindConfigurableTypes();

			// Find all the configurable fields from the given types
			Dictionary<string, Dictionary<string, XmlConfigData.TargetMember>> categoryToFields = new Dictionary<string, Dictionary<string, XmlConfigData.TargetMember>>();
			FindConfigurableFields(configTypes, categoryToFields);
			categoryToFields = categoryToFields.Where(x => x.Value.Count > 0).ToDictionary(x => x.Key, x => x.Value);

			// Get the path to the XML documentation
			FileReference inputDocumentationFile = new FileReference(Assembly.GetExecutingAssembly().Location).ChangeExtension(".xml");
			if (!FileReference.Exists(inputDocumentationFile))
			{
				throw new BuildException("Generated assembly documentation not found at {0}.", inputDocumentationFile);
			}

			// Read the documentation
			XmlDocument inputDocumentation = new XmlDocument();
			inputDocumentation.Load(inputDocumentationFile.FullName);

			// Make sure we can write to the output file
			if (FileReference.Exists(outputFile))
			{
				FileReference.MakeWriteable(outputFile);
			}
			else
			{
				DirectoryReference.CreateDirectory(outputFile.Directory);
			}

			// Generate the documentation file
			if (outputFile.HasExtension(".udn"))
			{
				WriteDocumentationUDN(outputFile, inputDocumentation, categoryToFields, logger);
			}
			else if (outputFile.HasExtension(".html"))
			{
				WriteDocumentationHTML(outputFile, inputDocumentation, categoryToFields, logger);
			}
			else
			{
				throw new BuildException("Unable to detect format from extension of output file ({0})", outputFile);
			}

			// Success!
			logger.LogInformation("Written documentation to {OutputFile}.", outputFile);
		}

		/// <summary>
		/// Writes out documentation in UDN format
		/// </summary>
		/// <param name="outputFile">The output file</param>
		/// <param name="inputDocumentation">The XML documentation for this assembly</param>
		/// <param name="categoryToFields">Map of string to types to fields</param>
		/// <param name="logger">Logger for output</param>
		private static void WriteDocumentationUDN(FileReference outputFile, XmlDocument inputDocumentation, Dictionary<string, Dictionary<string, XmlConfigData.TargetMember>> categoryToFields, ILogger logger)
		{
			using (StreamWriter writer = new StreamWriter(outputFile.FullName))
			{
				writer.WriteLine("Availability: NoPublish");
				writer.WriteLine("Title: Build Configuration Properties Page");
				writer.WriteLine("Crumbs:");
				writer.WriteLine("Description: This is a procedurally generated markdown page.");
				writer.WriteLine("Version: {0}.{1}", ReadOnlyBuildVersion.Current.MajorVersion, ReadOnlyBuildVersion.Current.MinorVersion);
				writer.WriteLine("");

				foreach (KeyValuePair<string, Dictionary<string, XmlConfigData.TargetMember>> categoryPair in categoryToFields)
				{
					string categoryName = categoryPair.Key;
					writer.WriteLine("### {0}", categoryName);
					writer.WriteLine();

					Dictionary<string, XmlConfigData.TargetMember> fields = categoryPair.Value;
					foreach (KeyValuePair<string, XmlConfigData.TargetMember> fieldPair in fields)
					{
						// Get the XML comment for this field
						List<string>? lines;
						if (!RulesDocumentation.TryGetXmlComment(inputDocumentation, fieldPair.Value.MemberInfo, logger, out lines) || lines.Count == 0)
						{
							continue;
						}

						// Write the result to the .udn file
						writer.WriteLine("$ {0} : {1}", fieldPair.Key, lines[0]);
						for (int idx = 1; idx < lines.Count; idx++)
						{
							if (lines[idx].StartsWith("*", StringComparison.OrdinalIgnoreCase) || lines[idx].StartsWith("-", StringComparison.OrdinalIgnoreCase))
							{
								writer.WriteLine("        * {0}", lines[idx].Substring(1).TrimStart());
							}
							else
							{
								writer.WriteLine("    * {0}", lines[idx]);
							}
						}
						writer.WriteLine();
					}
				}
			}
		}

		/// <summary>
		/// Writes out documentation in HTML format
		/// </summary>
		/// <param name="outputFile">The output file</param>
		/// <param name="inputDocumentation">The XML documentation for this assembly</param>
		/// <param name="categoryToFields">Map of string to types to fields</param>
		/// <param name="logger">Logger for output</param>
		private static void WriteDocumentationHTML(FileReference outputFile, XmlDocument inputDocumentation, Dictionary<string, Dictionary<string, XmlConfigData.TargetMember>> categoryToFields, ILogger logger)
		{
			using (StreamWriter writer = new StreamWriter(outputFile.FullName))
			{
				writer.WriteLine("<html>");
				writer.WriteLine("  <body>");
				writer.WriteLine("  <h2>BuildConfiguration Properties</h2>");
				foreach (KeyValuePair<string, Dictionary<string, XmlConfigData.TargetMember>> categoryPair in categoryToFields)
				{
					string categoryName = categoryPair.Key;
					writer.WriteLine("    <h3>{0}</h3>", categoryName);
					writer.WriteLine("    <dl>");

					Dictionary<string, XmlConfigData.TargetMember> fields = categoryPair.Value;
					foreach (KeyValuePair<string, XmlConfigData.TargetMember> fieldPair in fields)
					{
						// Get the XML comment for this field
						List<string>? lines;
						if (!RulesDocumentation.TryGetXmlComment(inputDocumentation, fieldPair.Value.MemberInfo, logger, out lines) || lines.Count == 0)
						{
							continue;
						}

						// Write the result to the .udn file
						writer.WriteLine("      <dt>{0}</dt>", fieldPair.Key);

						if (lines.Count == 1)
						{
							writer.WriteLine("      <dd>{0}</dd>", lines[0]);
						}
						else
						{
							writer.WriteLine("      <dd>");
							for (int idx = 0; idx < lines.Count; idx++)
							{
								if (lines[idx].StartsWith("*", StringComparison.OrdinalIgnoreCase) || lines[idx].StartsWith("-", StringComparison.OrdinalIgnoreCase))
								{
									writer.WriteLine("        <ul>");
									for (; idx < lines.Count && (lines[idx].StartsWith("*", StringComparison.OrdinalIgnoreCase) || lines[idx].StartsWith("-", StringComparison.OrdinalIgnoreCase)); idx++)
									{
										writer.WriteLine("          <li>{0}</li>", lines[idx].Substring(1).TrimStart());
									}
									writer.WriteLine("        </ul>");
								}
								else
								{
									writer.WriteLine("        {0}", lines[idx]);
								}
							}
							writer.WriteLine("      </dd>");
						}
					}

					writer.WriteLine("    </dl>");
				}
				writer.WriteLine("  </body>");
				writer.WriteLine("</html>");
			}
		}
	}
}

