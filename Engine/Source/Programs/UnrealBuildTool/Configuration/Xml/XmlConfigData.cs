// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using System.Linq;
using System.Reflection;
using EpicGames.Core;

namespace UnrealBuildTool
{
	/// <summary>
	/// Stores parsed values from XML config files which can be applied to a configurable type. Can be serialized to disk in binary form as a cache.
	/// </summary>
	class XmlConfigData
	{
		/// <summary>
		/// The current cache serialization version
		/// </summary>
		const int SerializationVersion = 2;

		/// <summary>
		/// List of input files. Stored to allow checking cache validity.
		/// </summary>
		public FileReference[] InputFiles { get; set; }

		/// <summary>
		/// Abstract description of a target data member.
		/// </summary>
		public abstract class TargetMember
		{
			/// <summary>
			/// Returns Reflection.MemberInfo describing the target class member.
			/// </summary>
			public abstract MemberInfo MemberInfo { get; }

			/// <summary>
			/// Returns Reflection.Type of the target class member.
			/// </summary>
			public abstract Type Type { get; }

			/// <summary>
			/// Indicates whether the target class member is static or not.
			/// </summary>
			public abstract bool IsStatic { get; }

			/// <summary>
			/// Returns the value setter of the target class member.
			/// </summary>
			public abstract Action<object?, object?> SetValue { get; }

			/// <summary>
			/// Returns the value getter of the target class member.
			/// </summary>
			public abstract Func<object?, object?> GetValue { get; }
		}

		/// <summary>
		/// Description of a field member.
		/// </summary>
		public class TargetField : TargetMember
		{
			public override MemberInfo MemberInfo => _fieldInfo;
			public override Type Type => _fieldInfo.FieldType;
			public override bool IsStatic => _fieldInfo.IsStatic;
			public override Action<object?, object?> SetValue => _fieldInfo.SetValue;
			public override Func<object?, object?> GetValue => _fieldInfo.GetValue;

			private readonly FieldInfo _fieldInfo;

			public TargetField(FieldInfo fieldInfo)
			{
				_fieldInfo = fieldInfo;
			}
		}

		/// <summary>
		/// Description of a property member.
		/// </summary>
		public class TargetProperty : TargetMember
		{
			public override MemberInfo MemberInfo => _propertyInfo;
			public override Type Type => _propertyInfo.PropertyType;
			public override bool IsStatic => _propertyInfo.GetGetMethod()!.IsStatic;
			public override Action<object?, object?> SetValue => _propertyInfo.SetValue;
			public override Func<object?, object?> GetValue => _propertyInfo.GetValue;

			private readonly PropertyInfo _propertyInfo;

			public TargetProperty(PropertyInfo propertyInfo)
			{
				_propertyInfo = propertyInfo;
			}
		}

		public class ValueInfo
		{
			public TargetMember Target { get; init; }
			public object Value { get; init; }
			public FileReference SourceFile { get; init; }
			public XmlConfigFileAttribute XmlConfigAttribute { get; init; }

			public ValueInfo(FieldInfo fieldInfo, object value, FileReference sourceFile, XmlConfigFileAttribute xmlConfigAttribute)
				: this(new TargetField(fieldInfo), value, sourceFile, xmlConfigAttribute)
			{
			}

			public ValueInfo(PropertyInfo propertyInfo, object value, FileReference sourceFile, XmlConfigFileAttribute xmlConfigAttribute)
				: this(new TargetProperty(propertyInfo), value, sourceFile, xmlConfigAttribute)
			{
			}

			public ValueInfo(TargetMember target, object value, FileReference sourceFile, XmlConfigFileAttribute xmlConfigAttribute)
			{
				Target = target;
				Value = value;
				SourceFile = sourceFile;
				XmlConfigAttribute = xmlConfigAttribute;
			}
		}

		/// <summary>
		/// Stores a mapping from type -> member -> value, with all the config values for configurable fields.
		/// </summary>
		public Dictionary<Type, ValueInfo[]> TypeToValues { get; init; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="inputFiles"></param>
		/// <param name="typeToValues"></param>
		public XmlConfigData(FileReference[] inputFiles, Dictionary<Type, ValueInfo[]> typeToValues)
		{
			InputFiles = inputFiles;
			TypeToValues = typeToValues;
		}

		/// <summary>
		/// Attempts to read a previous block of config values from disk
		/// </summary>
		/// <param name="location">The file to read from</param>
		/// <param name="types">Array of valid types. Used to resolve serialized type names to concrete types.</param>
		/// <param name="data">On success, receives the parsed data</param>
		/// <returns>True if the data was read and is valid</returns>
		public static bool TryRead(FileReference location, IEnumerable<Type> types, [NotNullWhen(true)] out XmlConfigData? data)
		{
			// Check the file exists first
			if (!FileReference.Exists(location))
			{
				data = null;
				return false;
			}

			// Read the cache from disk
			using (BinaryReader reader = new BinaryReader(File.Open(location.FullName, FileMode.Open, FileAccess.Read, FileShare.Read)))
			{
				// Check the serialization version matches
				if (reader.ReadInt32() != SerializationVersion)
				{
					data = null;
					return false;
				}

				// Read the input files
				FileReference[] inputFiles = reader.ReadArray(() => reader.ReadFileReference())!;

				// Read the types
				int numTypes = reader.ReadInt32();
				Dictionary<Type, ValueInfo[]> typeToValues = new Dictionary<Type, ValueInfo[]>(numTypes);
				for (int typeIdx = 0; typeIdx < numTypes; typeIdx++)
				{
					// Read the type name
					string typeName = reader.ReadString();

					// Try to find it in the list of configurable types
					Type? type = types.FirstOrDefault(x => x.Name == typeName);
					if (type == null)
					{
						data = null;
						return false;
					}

					// Read all the values
					ValueInfo[] values = new ValueInfo[reader.ReadInt32()];
					for (int valueIdx = 0; valueIdx < values.Length; valueIdx++)
					{
						string memberName = reader.ReadString();

						TargetMember? targetMember = GetTargetMemberWithAttribute<XmlConfigFileAttribute>(type, memberName);

						if (targetMember != null)
						{
							// If TargetMember is not null, we know it has our attribute.
							XmlConfigFileAttribute xmlConfigAttribute = targetMember!.MemberInfo.GetCustomAttribute<XmlConfigFileAttribute>()!;

							// Try to parse the value and add it to the output array
							object value = reader.ReadObject(targetMember.Type)!;

							// Read the path of the config file that provided this setting
							FileReference sourceFile = reader.ReadFileReference();

							values[valueIdx] = new ValueInfo(targetMember, value, sourceFile, xmlConfigAttribute);
						}
						else
						{
							data = null;
							return false;
						}
					}

					// Add it to the type map
					typeToValues.Add(type, values);
				}

				// Return the parsed data
				data = new XmlConfigData(inputFiles.ToArray(), typeToValues);
				return true;
			}
		}

		/// <summary>
		/// Find a data member (field or property) with the given name and attribute and returns TargetMember wrapper created for it.
		/// </summary>
		/// <typeparam name="T">Attribute a member has to have to be considered.</typeparam>
		/// <param name="type">Type which members are to be searched</param>
		/// <param name="memberName">Name of a member (field or property) to find.</param>
		/// <returns>TargetMember wrapper or null if no member has been found.</returns>
		private static TargetMember? GetTargetMemberWithAttribute<T>(Type type, string memberName)
			where T : Attribute
		{
			FieldInfo? field = type.GetField(memberName, BindingFlags.Public | BindingFlags.NonPublic | BindingFlags.Instance | BindingFlags.Static);
			T? xmlConfigAttribute = field?.GetCustomAttribute<T>();
			if (field != null && xmlConfigAttribute != null)
			{
				return new TargetField(field);
			}

			PropertyInfo? property = type.GetProperty(memberName, BindingFlags.Public | BindingFlags.NonPublic | BindingFlags.Instance | BindingFlags.Static);
			xmlConfigAttribute = property?.GetCustomAttribute<T>();

			if (property != null && xmlConfigAttribute != null)
			{
				return new TargetProperty(property);
			}

			return null;
		}

		/// <summary>
		/// Writes the coalesced config hierarchy to disk
		/// </summary>
		/// <param name="location">File to write to</param>
		public void Write(FileReference location)
		{
			DirectoryReference.CreateDirectory(location.Directory);
			using (BinaryWriter writer = new BinaryWriter(File.Open(location.FullName, FileMode.Create, FileAccess.Write, FileShare.Read)))
			{
				writer.Write(SerializationVersion);

				// Save all the input files. The cache will not be valid if these change.
				writer.Write(InputFiles, item => writer.Write(item));

				// Write all the categories
				writer.Write(TypeToValues.Count);
				foreach (KeyValuePair<Type, ValueInfo[]> typePair in TypeToValues)
				{
					writer.Write(typePair.Key.Name);
					writer.Write(typePair.Value.Length);
					foreach (ValueInfo memberPair in typePair.Value)
					{
						writer.Write(memberPair.Target.MemberInfo.Name);
						writer.Write(memberPair.Target.Type, memberPair.Value);
						writer.Write(memberPair.SourceFile);
					}
				}
			}
		}
	}
}
