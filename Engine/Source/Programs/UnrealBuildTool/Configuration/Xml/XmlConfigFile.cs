// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics.CodeAnalysis;
using System.Xml;
using System.Xml.Schema;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace UnrealBuildTool
{
	/// <summary>
	/// Implementation of XmlDocument which preserves line numbers for its elements
	/// </summary>
	class XmlConfigFile : XmlDocument
	{
		/// <summary>
		/// Root element for the XML document
		/// </summary>
		public const string RootElementName = "Configuration";

		/// <summary>
		/// Namespace for the XML schema
		/// </summary>
		public const string SchemaNamespaceURI = "https://www.unrealengine.com/BuildConfiguration";

		/// <summary>
		/// The file being read
		/// </summary>
		readonly FileReference _file;

		/// <summary>
		/// Interface to the LineInfo on the active XmlReader
		/// </summary>
		IXmlLineInfo _lineInfo = null!;

		/// <summary>
		/// Set to true if the reader encounters an error
		/// </summary>
		bool _bHasErrors;

		/// <summary>
		/// Private constructor. Use XmlConfigFile.TryRead to read an XML config file.
		/// </summary>
		private XmlConfigFile(FileReference inFile)
		{
			_file = inFile;
		}

		/// <summary>
		/// Overrides XmlDocument.CreateElement() to construct ScriptElements rather than XmlElements
		/// </summary>
		public override XmlElement CreateElement(string? prefix, string localName, string? namespaceUri) => new XmlConfigFileElement(_file, _lineInfo.LineNumber, prefix!, localName, namespaceUri, this);

		/// <summary>
		/// Loads a script document from the given file
		/// </summary>
		/// <param name="file">The file to load</param>
		/// <param name="schema">The schema to validate against</param>
		/// <param name="logger">Logger for output</param>
		/// <param name="outConfigFile">If successful, the document that was read</param>
		/// <returns>True if the document could be read, false otherwise</returns>
		public static bool TryRead(FileReference file, XmlSchema schema, ILogger logger, [NotNullWhen(true)] out XmlConfigFile? outConfigFile)
		{
			XmlConfigFile configFile = new XmlConfigFile(file);

			XmlReaderSettings settings = new XmlReaderSettings();
			settings.Schemas.Add(schema);
			settings.ValidationType = ValidationType.Schema;
			settings.ValidationEventHandler += configFile.ValidationEvent;

			using (XmlReader reader = XmlReader.Create(file.FullName, settings))
			{
				// Read the document
				configFile._lineInfo = (IXmlLineInfo)reader;
				try
				{
					configFile.Load(reader);
				}
				catch (XmlException ex)
				{
					if (!configFile._bHasErrors)
					{
						Log.TraceErrorTask(file, ex.LineNumber, "{0}", ex.Message);
						configFile._bHasErrors = true;
					}
				}

				// If we hit any errors while parsing
				if (configFile._bHasErrors)
				{
					outConfigFile = null;
					return false;
				}

				// Check that the root element is valid. If not, we didn't actually validate against the schema.
				if (configFile.DocumentElement?.Name != RootElementName)
				{
					logger.LogError("Script does not have a root element called '{RootElementName}'", RootElementName);
					outConfigFile = null;
					return false;
				}
				if (configFile.DocumentElement.NamespaceURI != SchemaNamespaceURI)
				{
					logger.LogError("Script root element is not in the '{NamespaceUri}' namespace (add the xmlns=\"{NamespaceUri2}\" attribute)", SchemaNamespaceURI, SchemaNamespaceURI);
					outConfigFile = null;
					return false;
				}
			}

			outConfigFile = configFile;
			return true;
		}

		/// <summary>
		/// Callback for validation errors in the document
		/// </summary>
		/// <param name="sender">Standard argument for ValidationEventHandler</param>
		/// <param name="args">Standard argument for ValidationEventHandler</param>
		void ValidationEvent(object? sender, ValidationEventArgs args) => Log.TraceWarningTask(_file, args.Exception.LineNumber, "{0}", args.Message);
	}

	/// <summary>
	/// Implementation of XmlElement which preserves line numbers
	/// </summary>
	class XmlConfigFileElement : XmlElement
	{
		/// <summary>
		/// The file containing this element
		/// </summary>
		public FileReference File { get; init; }

		/// <summary>
		/// The line number containing this element
		/// </summary>
		public int LineNumber { get; init; }

		/// <summary>
		/// Constructor
		/// </summary>
		public XmlConfigFileElement(FileReference inFile, int inLineNumber, string prefix, string localName, string? namespaceUri, XmlConfigFile configFile)
			: base(prefix, localName, namespaceUri, configFile)
		{
			File = inFile;
			LineNumber = inLineNumber;
		}
	}
}
