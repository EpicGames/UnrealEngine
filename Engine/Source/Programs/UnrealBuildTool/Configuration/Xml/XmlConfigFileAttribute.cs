// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace UnrealBuildTool
{
	/// <summary>
	/// Marks a field as being serializable from a config file
	/// </summary>
	[AttributeUsage(AttributeTargets.Field | AttributeTargets.Property, AllowMultiple = true)]
	sealed class XmlConfigFileAttribute : Attribute
	{
		/// <summary>
		/// The category for this config value. Optional; defaults to the declaring type name.
		/// </summary>
		public string? Category { get; set; } = null;

		/// <summary>
		/// Name of the key to read. Optional; defaults to the field name.
		/// </summary>
		public string? Name { get; set; } = null;

		/// <summary>
		/// Use this field to indicate that the XML attribute has been deprecated, and that a warning should
		/// be shown to the user if it is used.
		/// A deprecated field should also be marked with the [Obsolete] attribute.
		/// </summary>
		public bool Deprecated { get; set; } = false;

		/// <summary>
		/// If the attribute has been deprecated because it has been renamed, this field can be used to apply the
		/// value used for this field to another.
		/// </summary>
		public string? NewAttributeName { get; set; } = null;
	}
}
