// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace UnrealBuildTool
{
	/// <summary>
	/// Results from a run action
	/// </summary>
	/// <param name="LogLines">Console log lines</param>
	/// <param name="ExitCode">Process return code.  Zero is assumed to be success and all other values an error.</param>
	/// <param name="ExecutionTime">Wall time</param>
	/// <param name="ProcessorTime">CPU time</param>
	/// <param name="PeakMemoryUsed">Peak Memory Used</param>
	/// <param name="Action">Action</param>
	/// <param name="AdditionalDescription">Additional description of action</param>
	record ExecuteResults(List<string> LogLines, int ExitCode, TimeSpan ExecutionTime, TimeSpan ProcessorTime, ulong PeakMemoryUsed, LinkedAction Action, string? AdditionalDescription = null)
	{
		public string GetDescription()
		{
			string commandDescription = Action.CommandDescription ?? Action.CommandPath.GetFileNameWithoutExtension();
			string status = Action.bShouldOutputStatusDescription || (LogLines.Count == 0) ? Action.StatusDescription : LogLines[0];
			string description = $"{commandDescription} {status}".Trim();

			if (!String.IsNullOrEmpty(AdditionalDescription))
			{
				description = $"{description} {AdditionalDescription}";
			}

			return description;
		}
	}

	class ActionLogger : IDisposable
	{
		/// <summary>
		/// Add target names for each action executed
		/// </summary>
		public bool PrintActionTargetNames = false;

		/// <summary>
		/// Whether to log command lines for actions being executed
		/// </summary>
		public bool LogActionCommandLines = false;

		/// <summary>
		/// Whether to show compilation times for each executed action
		/// </summary>
		public bool ShowPerActionCompilationTimes = false;

		/// <summary>
		/// Collapse non-error output lines
		/// </summary>
		public bool CompactOutput = false;

		/// <summary>
		/// Whether to show compilation times along with worst offenders or not.
		/// </summary>
		public bool ShowCompilationTimes = false;

		/// <summary>
		/// Output logging
		/// </summary>
		public readonly ILogger Logger;

		/// <summary>
		/// Progress writer
		/// </summary>
		public readonly ProgressWriter ProgressWriter;

		/// <summary>
		/// Process group
		/// </summary>		
		public readonly ManagedProcessGroup ProcessGroup = new();

		/// <summary>
		/// The last action group printed in multi-target builds
		/// </summary>
		private string? LastGroupPrefix = null;

		/// <summary>
		/// Whether to show CPU utilization after the work is complete.
		/// </summary>
		public bool ShowCPUUtilization = false;

		/// <summary>
		/// Action to invoke when writing tool output
		/// </summary>
		private readonly Action<string> _writeToolOutput;

		/// <summary>
		/// Flush the tool output after logging has completed
		/// </summary>
		protected readonly System.Action _flushToolOutput;

		/// <summary>
		/// Used only by the logger to track the [x,total] output
		/// </summary>
		private int _loggedCompletedActions = 0;

		/// <summary>
		/// Collection of all actions remaining to be logged
		/// </summary>
		protected readonly List<int> _actionsToLog = new();

		/// <summary>
		/// Task waiting to process logging
		/// </summary>
		protected Task? _actionsToLogTask = null;

		/// <summary>
		/// Timer used to collect CPU utilization
		/// </summary>
		protected Timer? _cpuUtilizationTimer;

		/// <summary>
		/// Per-second logging of cpu utilization
		/// </summary>
		protected readonly List<float> _cpuUtilization = new();

		/// <summary>
		/// Optional execution results (ordered like actions)
		/// </summary>
		protected readonly ExecuteResults?[] Results;

		private bool _disposedValue;

		public ActionLogger(int actionCount, string progressWriterText, Action<string> writeToolOutput, System.Action flushToolOutput, ILogger logger)
		{
			Results = new ExecuteResults[actionCount];

			Logger = logger;
			ProgressWriter = new(progressWriterText, false, logger);

			_writeToolOutput = writeToolOutput;
			_flushToolOutput = flushToolOutput;
		}

		public void AddActionToLog(ExecuteResults executeTaskResult)
		{
			int index = executeTaskResult.Action.SortIndex;
			Results[index] = executeTaskResult;
			lock (_actionsToLog)
			{
				_actionsToLog.Add(index);
				_actionsToLogTask ??= Task.Run(LogActions2);
			}
		}

		private static int s_previousLineLength = -1;

		/// <summary>
		/// Log an action that has completed
		/// </summary>
		/// <param name="executeTaskResult">Results of the action</param>
		/// <param name="totalActions">Number of total actions</param>
		private void LogAction(ExecuteResults executeTaskResult, int totalActions)
		{
			List<string>? logLines = executeTaskResult.LogLines;
			int exitCode = executeTaskResult.ExitCode;
			TimeSpan executionTime = executeTaskResult.ExecutionTime;
			TimeSpan processorTime = executeTaskResult.ProcessorTime;
			ulong peakMemory = executeTaskResult.PeakMemoryUsed;

			// Write it to the log
			string description = executeTaskResult.GetDescription();
			LinkedAction action = executeTaskResult.Action;

			lock (ProgressWriter)
			{
				int completedActions = Interlocked.Increment(ref _loggedCompletedActions);
				ProgressWriter.Write(completedActions, totalActions);

				// Canceled
				if (exitCode == Int32.MaxValue)
				{
					//Logger.LogInformation("[{CompletedActions}/{TotalActions}] {Description} canceled", completedActions, totalActions, description);
					return;
				}

				string targetDetails = "";
				TargetDescriptor? target = action.Target;
				if (PrintActionTargetNames && target != null)
				{
					targetDetails = $"[{target.Name} {target.Platform} {target.Configuration}]";
				}

				if (LogActionCommandLines)
				{
					Logger.LogDebug("[{CompletedActions}/{TotalActions}]{TargetDetails} Command: {CommandPath} {CommandArguments}", completedActions, totalActions, targetDetails, action.CommandPath, action.CommandArguments);
				}

				string compilationTimes = "";

				if (ShowPerActionCompilationTimes)
				{
					if (processorTime.Ticks > 0 && peakMemory > 0)
					{
						compilationTimes = $" (Wall: {executionTime.TotalSeconds:0.00}s CPU: {processorTime.TotalSeconds:0.00}s Mem: {StringUtils.FormatBytesString(peakMemory)})";
					}
					else if (processorTime.Ticks > 0)
					{
						compilationTimes = $" (Wall: {executionTime.TotalSeconds:0.00}s CPU: {processorTime.TotalSeconds:0.00}s)";
					}
					else if (peakMemory > 0)
					{
						compilationTimes = $" (Wall: {executionTime.TotalSeconds:0.00}s Mem: {StringUtils.FormatBytesString(peakMemory)})";
					}
					else
					{
						compilationTimes = $" (Wall: {executionTime.TotalSeconds:0.00}s)";
					}
				}

				string message = ($"[{completedActions}/{totalActions}]{targetDetails}{compilationTimes} {description}");

				if (CompactOutput)
				{
					if (s_previousLineLength > 0)
					{
						// move the cursor to the far left position, one line back
						Console.CursorLeft = 0;
						Console.CursorTop -= 1;
						// clear the line
						Console.Write("".PadRight(s_previousLineLength));
						// move the cursor back to the left, so output is written to the desired location
						Console.CursorLeft = 0;
					}
				}
				else
				{
					// If the action group has changed for a multi target build, write it to the log
					if (action.GroupNames.Count > 0)
					{
						string ActionGroup = $"** For {String.Join(" + ", action.GroupNames)} **";
						if (!ActionGroup.Equals(LastGroupPrefix, StringComparison.Ordinal))
						{
							LastGroupPrefix = ActionGroup;
							_writeToolOutput(ActionGroup);
						}
					}
				}

				s_previousLineLength = message.Length;

				_writeToolOutput(message);
				if (logLines != null && action.bShouldOutputLog)
				{
					foreach (string Line in logLines.Skip(action.bShouldOutputStatusDescription ? 0 : 1))
					{
						// suppress library creation messages when writing compact output
						if (CompactOutput && Line.StartsWith("   Creating library ", StringComparison.OrdinalIgnoreCase) && Line.EndsWith(".exp", StringComparison.OrdinalIgnoreCase))
						{
							continue;
						}

						_writeToolOutput(Line);

						// Prevent overwriting of logged lines
						s_previousLineLength = -1;
					}
				}

				if (exitCode != 0)
				{
					string exitCodeStr = String.Empty;
					if ((uint)exitCode == 0xC0000005)
					{
						exitCodeStr = "(Access violation)";
					}
					else if ((uint)exitCode == 0xC0000409)
					{
						exitCodeStr = "(Stack buffer overflow)";
					}

					// If we have an error code but no output, chances are the tool crashed.  Generate more detailed information to let the
					// user know something went wrong.
					if (logLines == null || logLines.Count <= (action.bShouldOutputStatusDescription ? 0 : 1))
					{
						Logger.LogError("{TargetDetails} {Description}: Exited with error code {ExitCode} {ExitCodeStr}. The build will fail.", targetDetails, description, exitCode, exitCodeStr);
						Logger.LogInformation("{TargetDetails} {Description}: WorkingDirectory {WorkingDirectory}", targetDetails, description, action.WorkingDirectory);
						Logger.LogInformation("{TargetDetails} {Description}: {CommandPath} {CommandArguments}", targetDetails, description, action.CommandPath, action.CommandArguments);
					}
					// Always print error details to to the log file
					else
					{
						Logger.LogDebug("{TargetDetails} {Description}: Exited with error code {ExitCode} {ExitCodeStr}. The build will fail.", targetDetails, description, exitCode, exitCodeStr);
						Logger.LogDebug("{TargetDetails} {Description}: WorkingDirectory {WorkingDirectory}", targetDetails, description, action.WorkingDirectory);
						Logger.LogDebug("{TargetDetails} {Description}: {CommandPath} {CommandArguments}", targetDetails, description, action.CommandPath, action.CommandArguments);
					}

					// prevent overwriting of error text
					s_previousLineLength = -1;
				}
			}
		}

		/// <summary>
		/// Generate the final summary display
		/// </summary>
		public void TraceSummary(bool success)
		{

			// Wait for logging to complete
			Task? loggingTask = null;
			lock (_actionsToLog)
			{
				loggingTask = _actionsToLogTask;
			}
			loggingTask?.Wait();
			_flushToolOutput();

			LogLevel LogLevel = success ? LogLevel.Information : LogLevel.Debug;
			Logger.Log(LogLevel, "");
			if (ShowCPUUtilization)
			{
				lock (_cpuUtilization)
				{
					if (_cpuUtilization.Count > 0)
					{
						Logger.Log(LogLevel, "Average CPU Utilization : {CPUPercentage:0.00}%", _cpuUtilization.Average());
					}
				}
			}

			if (!ShowCompilationTimes)
			{
				return;
			}

			if (ProcessGroup.TotalProcessorTime.Ticks + ProcessGroup.PeakProcessMemoryUsed + ProcessGroup.PeakJobMemoryUsed > 0)
			{
				if (ProcessGroup.TotalProcessorTime.Ticks > 0)
				{
					Logger.Log(LogLevel, "Total Processor Time    : {TotalSeconds:0.00}s", ProcessGroup.TotalProcessorTime.TotalSeconds);
				}
				if (ProcessGroup.PeakProcessMemoryUsed > 0)
				{
					Logger.Log(LogLevel, "Peak Process Memory Used: {PeakProcessMemory}", StringUtils.FormatBytesString(ProcessGroup.PeakProcessMemoryUsed));
				}
				if (ProcessGroup.PeakJobMemoryUsed > 0)
				{
					Logger.Log(LogLevel, "Peak Total Memory Used  : {PeakTotalMemory}", StringUtils.FormatBytesString(ProcessGroup.PeakJobMemoryUsed));
				}
				Logger.Log(LogLevel, "");
			}

			IEnumerable<int> CompletedActions = Enumerable.Range(0, Results.Length)
				.Where(x => Results[x] != null && Results[x]!.ExecutionTime > TimeSpan.Zero && Results[x]!.ExitCode != Int32.MaxValue)
				.OrderByDescending(x => Results[x]!.ExecutionTime)
				.Take(20);

			if (CompletedActions.Any())
			{
				Logger.Log(LogLevel, "Compilation Time Top {CompletedTaskCount}", CompletedActions.Count());
				Logger.Log(LogLevel, "");
				foreach (int Index in CompletedActions)
				{
					ExecuteResults Result = Results[Index]!;
					IExternalAction Action = Result.Action.Inner;

					string Description = $"{(Action.CommandDescription ?? Action.CommandPath.GetFileNameWithoutExtension())} {Action.StatusDescription}".Trim();
					if (Result.ProcessorTime.Ticks > 0 && Result.PeakMemoryUsed > 0)
					{
						Logger.Log(LogLevel, "{Description} [ Wall Time {ExecutionTime:0.00}s / CPU Time {ProcessorTime:0.00}s / Mem {Memory} ]", Description, Result.ExecutionTime.TotalSeconds, Result.ProcessorTime.TotalSeconds, StringUtils.FormatBytesString(Result.PeakMemoryUsed));
					}
					else if (Result.ProcessorTime.Ticks > 0)
					{
						Logger.Log(LogLevel, "{Description} [ Wall Time {ExecutionTime:0.00}s / CPU Time {ProcessorTime:0.00}s ]", Description, Result.ExecutionTime.TotalSeconds, Result.ProcessorTime.TotalSeconds);
					}
					else if (Result.PeakMemoryUsed > 0)
					{
						Logger.Log(LogLevel, "{Description} [ Time {ExecutionTime:0.00}s / Mem {Memory} ]", Description, Result.ExecutionTime.TotalSeconds, StringUtils.FormatBytesString(Result.PeakMemoryUsed));
					}
					else
					{
						Logger.Log(LogLevel, "{Description} [ Time {ExecutionTime:0.00}s ]", Description, Result.ExecutionTime.TotalSeconds);
					}
				}
				Logger.Log(LogLevel, "");
			}
		}

		public void Dispose()
		{
			Dispose(disposing: true);
			GC.SuppressFinalize(this);
		}

		protected virtual void Dispose(bool disposing)
		{
			if (!_disposedValue)
			{
				if (disposing)
				{
					ProgressWriter.Dispose();
					ProcessGroup.Dispose();
				}

				_disposedValue = true;
			}
		}

		private void LogActions2()
		{
			for (; ; )
			{
				int[]? actionsToLog = null;
				lock (_actionsToLog)
				{
					if (_actionsToLog.Count == 0)
					{
						_actionsToLogTask = null;
					}
					else
					{
						actionsToLog = _actionsToLog.ToArray();
						_actionsToLog.Clear();
					}
				}

				if (actionsToLog == null)
				{
					return;
				}

				foreach (int index in actionsToLog)
				{
					ExecuteResults result = Results[index]!;
					LogAction(result, Results.Length);
				}
			}
		}
	}
}