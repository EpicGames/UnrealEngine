// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool.Executors;

internal interface IExecutorFactory
{
	ActionExecutor GetExecutor(BuildConfiguration buildConfiguration, int actionCount, IEnumerable<TargetDescriptor> targetDescriptors, ILogger Logger);
	UBAExecutor GetUBAExecutor(BuildConfiguration buildConfiguration, IEnumerable<TargetDescriptor> targetDescriptors, ILogger Logger);
	ActionExecutor? PrecreateExecutor(BuildConfiguration buildConfiguration, IEnumerable<TargetDescriptor> targetDescriptors, ILogger Logger);
}

internal class ExecutorFactory : IExecutorFactory
{
	public static ExecutorFactory Instance { get; } = new ExecutorFactory();

	/// <summary>
	/// Selects an ActionExecutor
	/// </summary>
	public ActionExecutor GetExecutor(BuildConfiguration buildConfiguration, int actionCount, IEnumerable<TargetDescriptor> targetDescriptors, ILogger logger)
	{
		UnrealBuildAcceleratorConfig config = UBAExecutor.GetAppliedConfig(targetDescriptors);
		int minActionsForRemote = UBAExecutor.GetDefaultNumParallelProcesses(config, buildConfiguration.MaxParallelActions, buildConfiguration.bAllCores, logger);
		foreach (string name in buildConfiguration.RemoteExecutorPriority)
		{
			ActionExecutor? executor = GetRemoteExecutorByName(name, buildConfiguration, actionCount, minActionsForRemote, targetDescriptors, logger);
			if (executor != null)
			{
				return executor;
			}
		}

		return GetUBAExecutor(buildConfiguration, targetDescriptors, logger);
	}

	public UBAExecutor GetUBAExecutor(BuildConfiguration buildConfiguration, IEnumerable<TargetDescriptor> targetDescriptors, ILogger logger)
	{
		if (!UBAExecutor.IsAvailable())
		{
			throw new BuildLogEventException("UBA is not available - please ensure the UBA binaries exist for your host platform");
		}

		UnrealBuildAcceleratorConfig Config = UBAExecutor.GetAppliedConfig(targetDescriptors);
		// We always use the UBA executor, but we disable detouring to mirror legacy behaviour if the config disables it.
		if (!buildConfiguration.bAllowUBAExecutor)
		{
			Config.bAllowDetour = false;
		}

		return new UBAExecutor(
			Config,
			buildConfiguration.MaxParallelActions,
			buildConfiguration.bAllCores,
			buildConfiguration.bCompactOutput,
			logger,
			targetDescriptors);
	}

	public ActionExecutor? PrecreateExecutor(BuildConfiguration buildConfiguration, IEnumerable<TargetDescriptor> targetDescriptors, ILogger logger)
	{
		foreach (string name in buildConfiguration.RemoteExecutorPriority)
		{
			switch (name)
			{
				case XGE.ExecutorName:
					if (buildConfiguration.bAllowXGE && XGE.IsAvailable(logger))
					{
						return null;
					}
					break;
				case SNDBS.ExecutorName:
					if (buildConfiguration.bAllowSNDBS && SNDBS.IsAvailable(logger))
					{
						return null;
					}
					break;
				case FASTBuild.ExecutorName:
					if (buildConfiguration.bAllowFASTBuild && FASTBuild.IsAvailable(logger))
					{
						return null;
					}
					break;
				case UBAExecutor.ExecutorName:
					if (buildConfiguration.bAllowUBAExecutor)
					{
						return GetUBAExecutor(buildConfiguration, targetDescriptors, logger);
					}
					break;
			}
		}

		return GetUBAExecutor(buildConfiguration, targetDescriptors, logger);
	}

	private ActionExecutor? GetRemoteExecutorByName(
		string name,
		BuildConfiguration buildConfiguration,
		int actionCount,
		int minActionsForRemote,
		IEnumerable<TargetDescriptor> targetDescriptors,
		ILogger logger)
	{
		switch (name)
		{
			case XGE.ExecutorName:
				{
					if (actionCount >= minActionsForRemote && buildConfiguration.bAllowXGE && XGE.IsAvailable(logger) && actionCount >= XGE.MinActions)
					{
						return new XGE(logger);
					}
					return null;
				}
			case SNDBS.ExecutorName:
				{
					if (actionCount >= minActionsForRemote && buildConfiguration.bAllowSNDBS && SNDBS.IsAvailable(logger))
					{
						return new SNDBS(targetDescriptors, logger);
					}
					return null;
				}
			case FASTBuild.ExecutorName:
				{
					if (actionCount >= minActionsForRemote && buildConfiguration.bAllowFASTBuild && FASTBuild.IsAvailable(logger))
					{
						return new FASTBuild(buildConfiguration, logger, targetDescriptors);
					}
					return null;
				}
			case UBAExecutor.ExecutorName:
				{
					// Intentionally not checking MinActionsForRemote
					if (buildConfiguration.bAllowUBAExecutor)
					{
						return GetUBAExecutor(buildConfiguration, targetDescriptors, logger);
					}
					return null;
				}
			default:
				{
					logger.LogWarning("Unknown remote executor {Name}", name);
					return null;
				}
		}
	}
}
