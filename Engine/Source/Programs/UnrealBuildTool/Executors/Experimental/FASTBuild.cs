// Copyright Epic Games, Inc. All Rights Reserved.

// This is an experimental integration and requires custom FASTBuild binaries available in Engine/Extras/ThirdPartyNotUE/FASTBuild
// Currently only Windows, Mac, iOS and tvOS targets are supported.

///////////////////////////////////////////////////////////////////////////
// Copyright 2018 Yassine Riahi and Liam Flookes. Provided under a MIT License, see license file on github.
// Used to generate a fastbuild .bff file from UnrealBuildTool to allow caching and distributed builds.
///////////////////////////////////////////////////////////////////////////
// Modified by Nick Edwards @ Sumo Digital to implement support for building on
// MacOS for MacOS, iOS and tvOS targets. Includes RiceKab's alterations for
// providing 4.21 support (https://gist.github.com/RiceKab/60d7dd434afaab295d1c21d2fe1981b0)
///////////////////////////////////////////////////////////////////////////

using System;
using System.Collections;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Runtime.Versioning;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{

	///////////////////////////////////////////////////////////////////////

	internal static class VCEnvironmentFastbuildExtensions
	{
		/// <summary>
		/// This replaces the VCToolPath64 readonly property that was available in 4.19 . Note that GetVCToolPath64
		/// is still used internally, but the property for it is no longer exposed.
		/// </summary>
		/// <param name="VCEnv"></param>
		/// <returns></returns>
		public static DirectoryReference GetToolPath(this VCEnvironment VCEnv)
		{
			return VCEnv.CompilerPath.Directory;
		}

		/// <summary>
		/// This replaces the InstallDir readonly property that was available in 4.19.
		///
		///
		/// </summary>
		/// <param name="VCEnv"></param>
		/// <returns></returns>
		public static DirectoryReference GetVCInstallDirectory(this VCEnvironment VCEnv)
		{
			// TODO: Check registry values before moving up ParentDirectories (as in 4.19)
			return VCEnv.ToolChainDir.ParentDirectory!.ParentDirectory!.ParentDirectory!;
		}
	}

	///////////////////////////////////////////////////////////////////////

	internal enum FASTBuildCacheMode
	{
		ReadWrite, // This machine will both read and write to the cache
		ReadOnly,  // This machine will only read from the cache, use for developer machines when you have centralized build machines
		WriteOnly, // This machine will only write from the cache, use for build machines when you have centralized build machines
	}

	///////////////////////////////////////////////////////////////////////

	sealed class FASTBuild : ActionExecutor
	{
		/// <summary>
		/// Executor to use for local actions
		/// </summary>
		readonly UBAExecutor LocalExecutor;

		public static readonly string DefaultExecutableBasePath = Path.Combine(Unreal.EngineDirectory.FullName, "Extras", "ThirdPartyNotUE", "FASTBuild");

		//////////////////////////////////////////
		// Tweakables

		/////////////////
		// Executable

		/// <summary>
		/// Used to specify the location of fbuild.exe if the distributed binary isn't being used
		/// </summary>
		[XmlConfigFile]
		public static string? FBuildExecutablePath = null;

		/////////////////
		// Distribution

		/// <summary>
		/// Controls network build distribution
		/// </summary>
		[XmlConfigFile]
		public static bool bEnableDistribution = true;

		/// <summary>
		/// Used to specify the location of the brokerage. If null, FASTBuild will fall back to checking FASTBUILD_BROKERAGE_PATH
		/// </summary>
		[XmlConfigFile]
		public static string? FBuildBrokeragePath = null;

		/// <summary>
		/// Used to specify the FASTBuild coordinator IP or network name. If null, FASTBuild will fall back to checking FASTBUILD_COORDINATOR
		/// </summary>
		[XmlConfigFile]
		public static string? FBuildCoordinator = null;

		/////////////////
		// Caching

		/// <summary>
		/// Controls whether to use caching at all. CachePath and FASTCacheMode are only relevant if this is enabled.
		/// </summary>
		[XmlConfigFile]
		public static bool bEnableCaching = true;

		/// <summary>
		/// Cache access mode - only relevant if bEnableCaching is true;
		/// </summary>
		[XmlConfigFile]
		public static FASTBuildCacheMode CacheMode = FASTBuildCacheMode.ReadOnly;

		/// <summary>
		/// Used to specify the location of the cache. If null, FASTBuild will fall back to checking FASTBUILD_CACHE_PATH
		/// </summary>
		[XmlConfigFile]
		public static string? FBuildCachePath = null;

		/////////////////
		// Misc Options

		/// <summary>
		/// Whether to force remote
		/// </summary>
		[XmlConfigFile]
		public static bool bForceRemote = false;

		/// <summary>
		/// Whether to stop on error
		/// </summary>
		[XmlConfigFile]
		public static bool bStopOnError = false;

		/// <summary>
		/// Which MSVC CRT Redist version to use
		/// </summary>
		[XmlConfigFile]
		public static string MsvcCRTRedistVersion = "";

		/// <summary>
		/// Which MSVC Compiler version to use
		/// </summary>
		[XmlConfigFile]
		public static string CompilerVersion = "";
		//////////////////////////////////////////

		/// <summary>
		/// Constructor
		/// </summary>
		public FASTBuild(BuildConfiguration BuildConfiguration, ILogger Logger, IEnumerable<TargetDescriptor> TargetDescriptors)
			: base(Logger)
		{
			XmlConfig.ApplyTo(this);

			UnrealBuildAcceleratorConfig Config = UBAExecutor.GetAppliedConfig(TargetDescriptors);

			// Disable remote, as this is intended as a local executor.
			Config.bDisableRemote = true;

			// We always use the UBA executor, but we disable detouring to mirror legacy behaviour if the config disables it.
			if (!BuildConfiguration.bAllowUBAExecutor)
			{
				Config.bAllowDetour = false;
			}
			LocalExecutor = new UBAExecutor(
				Config,
				BuildConfiguration.MaxParallelActions,
				BuildConfiguration.bAllCores,
				BuildConfiguration.bCompactOutput,
				Logger,
				TargetDescriptors);
		}

		internal const string ExecutorName = "FASTBuild";

		public override string Name => ExecutorName;

		public static string GetExecutableName()
		{
			return Path.GetFileName(GetExecutablePath())!;
		}

		public static string? GetExecutablePath()
		{
			if (String.IsNullOrEmpty(FBuildExecutablePath))
			{
				string? EnvPath = Environment.GetEnvironmentVariable("FASTBUILD_EXECUTABLE_PATH");
				if (!String.IsNullOrEmpty(EnvPath))
				{
					FBuildExecutablePath = EnvPath;
				}
			}

			return FBuildExecutablePath;
		}

		public static string? GetCachePath()
		{
			if (String.IsNullOrEmpty(FBuildCachePath))
			{
				string? EnvPath = Environment.GetEnvironmentVariable("FASTBUILD_CACHE_PATH");
				if (!String.IsNullOrEmpty(EnvPath))
				{
					FBuildCachePath = EnvPath;
				}
			}

			return FBuildCachePath;
		}

		public static string? GetBrokeragePath()
		{
			if (String.IsNullOrEmpty(FBuildBrokeragePath))
			{
				string? EnvPath = Environment.GetEnvironmentVariable("FASTBUILD_BROKERAGE_PATH");
				if (!String.IsNullOrEmpty(EnvPath))
				{
					FBuildBrokeragePath = EnvPath;
				}
			}

			return FBuildBrokeragePath;
		}

		public static string? GetCoordinator()
		{
			if (String.IsNullOrEmpty(FBuildCoordinator))
			{
				string? EnvPath = Environment.GetEnvironmentVariable("FASTBUILD_COORDINATOR");
				if (!String.IsNullOrEmpty(EnvPath))
				{
					FBuildCoordinator = EnvPath;
				}
			}

			return FBuildCoordinator;
		}

		public static bool IsAvailable(ILogger Logger)
		{
			string? ExecutablePath = GetExecutablePath();
			if (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Mac)
			{
				if (String.IsNullOrEmpty(ExecutablePath))
				{
					FBuildExecutablePath = Path.Combine(DefaultExecutableBasePath, BuildHostPlatform.Current.Platform.ToString(), "FBuild");
				}
			}
			else if (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Win64)
			{
				if (String.IsNullOrEmpty(ExecutablePath))
				{
					FBuildExecutablePath = Path.Combine(DefaultExecutableBasePath, BuildHostPlatform.Current.Platform.ToString(), "FBuild.exe");
				}
			}
			else
			{
				// Linux is not supported yet. Win32 likely never.
				return false;
			}

			// UBT is faster than FASTBuild for local only builds, so only allow FASTBuild if the environment is fully set up to use FASTBuild.
			// That's when the FASTBuild coordinator or brokerage folder is available.
			// On Mac the latter needs the brokerage folder to be mounted, on Windows the brokerage env variable has to be set or the path specified in UBT's config
			string? Coordinator = GetCoordinator();
			if (String.IsNullOrEmpty(Coordinator))
			{
				string? BrokeragePath = GetBrokeragePath();
				if (String.IsNullOrEmpty(BrokeragePath) || !Directory.Exists(BrokeragePath))
				{
					return false;
				}
			}

			if (!String.IsNullOrEmpty(FBuildExecutablePath))
			{
				if (File.Exists(FBuildExecutablePath))
				{
					return true;
				}

				Logger.LogWarning("FBuildExecutablePath '{FBuildExecutablePath}' doesn't exist! Attempting to find executable in PATH.", FBuildExecutablePath);
			}

			// Get the name of the FASTBuild executable.
			string FBuildExecutableName = GetExecutableName();

			// Search the path for it
			string? PathVariable = Environment.GetEnvironmentVariable("PATH");
			if (PathVariable != null)
			{
				foreach (string SearchPath in PathVariable.Split(Path.PathSeparator))
				{
					try
					{
						string PotentialPath = Path.Combine(SearchPath, FBuildExecutableName);
						if (File.Exists(PotentialPath))
						{
							FBuildExecutablePath = PotentialPath;
							return true;
						}
					}
					catch (ArgumentException)
					{
						// PATH variable may contain illegal characters; just ignore them.
					}
				}
			}

			Logger.LogError("FASTBuild disabled. Unable to find any executable to use.");
			return false;
		}

		private TelemetryExecutorEvent? telemetryEvent;

		/// <inheritdoc/>
		public override TelemetryExecutorEvent? GetTelemetryEvent() => telemetryEvent ?? LocalExecutor.GetTelemetryEvent();

		//////////////////////////////////////////
		// Action Helpers

		private readonly Dictionary<LinkedAction, long>  _actionToId = new();
		private long _actionIdIndex = 0;

		private long GetActionID(LinkedAction Action)
		{
			if (!_actionToId.ContainsKey(Action))
			{
				_actionToId.Add(Action, ++_actionIdIndex);
			}
			return _actionToId[Action];
		}

		private string ActionToActionString(LinkedAction Action)
		{
			return ActionToActionString(GetActionID(Action));
		}

		private static string ActionToActionString(long UniqueId)
		{
			return $"Action_{UniqueId}";
		}

		private static string ActionToDependencyString(long UniqueId, string StatusDescription, string? CommandDescription = null, ActionType? ActionType = null)
		{
			string? ExtraInfoString = null;
			if ((CommandDescription != null) && String.IsNullOrEmpty(CommandDescription))
			{
				ExtraInfoString = CommandDescription;
			}
			else if (ActionType != null)
			{
				ExtraInfoString = ActionType.Value.ToString();
			}

			if ((ExtraInfoString != null) && !String.IsNullOrEmpty(ExtraInfoString))
			{
				ExtraInfoString = $" ({ExtraInfoString})";
			}

			return $"\t\t'{ActionToActionString(UniqueId)}', ;{StatusDescription}{ExtraInfoString}";
		}

		private string ActionToDependencyString(LinkedAction Action)
		{
			return ActionToDependencyString(GetActionID(Action), Action.StatusDescription, Action.CommandDescription, Action.ActionType);
		}

		private readonly HashSet<string> ForceLocalCompileModules = new HashSet<string>()
		{
			"Module.ProxyLODMeshReduction"
		};

		private readonly HashSet<string> ForceOverwriteCompilerOptionModules = new HashSet<string>()
		{
			"Module.USDStageImporter",
			"Module.USDUtilities",
			"Module.UnrealUSDWrapper",
			"Module.USDStage",
			"Module.USDSchemas",
			"Module.GeometryCacheUSD",
			"Module.USDStageEditorViewModels",
			"Module.USDTests",
			"Module.USDStageEditor",
			"Module.USDExporter"
		};

		private enum FBBuildType
		{
			Windows,
			Apple
		}

		private FBBuildType BuildType = FBBuildType.Windows;

		private static readonly Tuple<string, Func<LinkedAction, string>, FBBuildType>[] BuildTypeSearchParams = new Tuple<string, Func<LinkedAction, string>, FBBuildType>[]
		{
			Tuple.Create<string, Func<LinkedAction, string>, FBBuildType>
			(
				"Xcode",
				Action => Action.CommandArguments,
				FBBuildType.Apple
			),
			Tuple.Create<string, Func<LinkedAction, string>, FBBuildType>
			(
				"apple",
#pragma warning disable CA1308 // Normalize strings to uppercase - external command may be case-sensitive
				Action => Action.CommandArguments.ToLowerInvariant(),
#pragma warning restore CA1308
				FBBuildType.Apple
			),
			Tuple.Create<string, Func<LinkedAction, string>, FBBuildType>
			(
				"/bin/sh",
#pragma warning disable CA1308 // Normalize strings to uppercase - external command may be case-sensitive
				Action => Action.CommandPath.FullName.ToLowerInvariant(),
#pragma warning restore CA1308
				FBBuildType.Apple
			),
			Tuple.Create<string, Func<LinkedAction, string>, FBBuildType>
			(
				"Windows",		// Not a great test
				Action => Action.CommandPath.FullName,
				FBBuildType.Windows
			),
			Tuple.Create<string, Func<LinkedAction, string>, FBBuildType>
			(
				"Microsoft",	// Not a great test
				Action => Action.CommandPath.FullName,
				FBBuildType.Windows
			),
			Tuple.Create<string, Func<LinkedAction, string>, FBBuildType>
			(
				"Win64",
				Action => Action.CommandPath.FullName,
				FBBuildType.Windows
			),
		};

		private bool DetectBuildType(IEnumerable<LinkedAction> Actions, ILogger Logger)
		{
			foreach (LinkedAction Action in Actions)
			{
				foreach (Tuple<string, Func<LinkedAction, string>, FBBuildType> BuildTypeSearchParam in BuildTypeSearchParams)
				{
					if (BuildTypeSearchParam.Item3.Equals(FBBuildType.Apple) &&
						(BuildTypeSearchParam.Item2(Action).Contains("Win64", StringComparison.OrdinalIgnoreCase) ||
						BuildTypeSearchParam.Item2(Action).Contains("X64", StringComparison.OrdinalIgnoreCase)))
					{
						continue;
					}
					if (BuildTypeSearchParam.Item2(Action).Contains(BuildTypeSearchParam.Item1, StringComparison.Ordinal))
					{
						BuildType = BuildTypeSearchParam.Item3;
						Logger.LogInformation("Detected build type as {Type} from '{From}' using search term '{Term}'", BuildTypeSearchParam.Item3.ToString(), BuildTypeSearchParam.Item2(Action), BuildTypeSearchParam.Item1);
						return true;
					}
				}
			}

			Logger.LogError("Couldn't detect build type from actions! Unsupported platform?");
			foreach (LinkedAction Action in Actions)
			{
				PrintActionDetails(Action, Logger);
			}
			return false;
		}

		private bool IsMSVC() { return BuildType == FBBuildType.Windows; }
		private bool IsApple() { return BuildType == FBBuildType.Apple; }

		private string GetCompilerName()
		{
			switch (BuildType)
			{
				default:
				case FBBuildType.Windows: return "UECompiler";
				case FBBuildType.Apple: return "UEAppleCompiler";
			}
		}

		/// <inheritdoc/>
		[SupportedOSPlatform("windows")]
		public override async Task<bool> ExecuteActionsAsync(IEnumerable<LinkedAction> Actions, ILogger Logger)
		{
			if (!Actions.Any())
			{
				return true;
			}

			IEnumerable<LinkedAction> CompileActions = Actions.Where(Action => Action.ActionType == ActionType.Compile && Action.bCanExecuteRemotely && Action.bCanExecuteRemotelyWithSNDBS);
			if (CompileActions.Any() && DetectBuildType(CompileActions, Logger))
			{
				DateTime startTimeUTC = DateTime.UtcNow;
				string FASTBuildFilePath = Path.Combine(Unreal.EngineDirectory.FullName, "Intermediate", "Build", "fbuild.bff");
				if (!CreateBffFile(Actions, FASTBuildFilePath, Logger))
				{
					return false;
				}

				bool result = ExecuteBffFile(FASTBuildFilePath, Logger);
				telemetryEvent = new TelemetryExecutorEvent(Name, startTimeUTC, result, Actions.Count(), -1, -1, 0, 0, DateTime.UtcNow, 0, 0);
				return result;
			}

			return await LocalExecutor.ExecuteActionsAsync(Actions, Logger);
		}

		private void AddText(string StringToWrite)
		{
			byte[] Info = new System.Text.UTF8Encoding(true).GetBytes(StringToWrite);
			bffOutputMemoryStream!.Write(Info, 0, Info.Length);
		}

		private void AddPreBuildDependenciesText(IEnumerable<LinkedAction>? PreBuildDependencies)
		{
			if (PreBuildDependencies == null || !PreBuildDependencies.Any())
			{
				return;
			}

			AddText($"\t.PreBuildDependencies = {{\n");
			AddText($"{String.Join("\n", PreBuildDependencies.Select(ActionToDependencyString))}\n");
			AddText($"\t}} \n");
		}

		private static string SubstituteEnvironmentVariables(string commandLineString)
		{
			return commandLineString
				.Replace("$(DXSDK_DIR)", "$DXSDK_DIR$", StringComparison.Ordinal)
				.Replace("$(CommonProgramFiles)", "$CommonProgramFiles$", StringComparison.Ordinal);
		}

		private static Dictionary<string, string> ParseCommandLineOptions(string LocalToolName, string CompilerCommandLine, string[] SpecialOptions, ILogger Logger, bool SaveResponseFile = false)
		{
			Dictionary<string, string> ParsedCompilerOptions = new Dictionary<string, string>();

			// Make sure we substituted the known environment variables with corresponding BFF friendly imported vars
			CompilerCommandLine = SubstituteEnvironmentVariables(CompilerCommandLine);

			// Some tricky defines /DTROUBLE=\"\\\" abc  123\\\"\" aren't handled properly by either Unreal or FASTBuild, but we do our best.
			char[] SpaceChar = { ' ' };
			string[] RawTokens = CompilerCommandLine.Trim().Split(' ');
			List<string> ProcessedTokens = new List<string>();
			bool QuotesOpened = false;
			string PartialToken = "";
			string ResponseFilePath = "";
			List<string> AllTokens = new List<string>();

			int ResponseFileTokenIndex = Array.FindIndex(RawTokens, RawToken => RawToken.StartsWith("@\"", StringComparison.Ordinal));
			if (ResponseFileTokenIndex == -1)
			{
				ResponseFileTokenIndex = Array.FindIndex(RawTokens, RawToken => RawToken.StartsWith("@", StringComparison.Ordinal));
			}
			if (ResponseFileTokenIndex > -1) //Response files are in 4.13 by default. Changing VCToolChain to not do this is probably better.
			{
				string responseCommandline = RawTokens[ResponseFileTokenIndex];

				for (int i = 0; i < ResponseFileTokenIndex; ++i)
				{
					AllTokens.Add(RawTokens[i]);
				}

				// If we had spaces inside the response file path, we need to reconstruct the path.
				for (int i = ResponseFileTokenIndex + 1; i < RawTokens.Length; ++i)
				{
					if (RawTokens[i - 1].Contains(".response", StringComparison.Ordinal) || RawTokens[i - 1].Contains(".rsp", StringComparison.Ordinal))
					{
						break;
					}

					responseCommandline += " " + RawTokens[i];
				}

				ResponseFilePath = responseCommandline.TrimStart('"', '@').TrimEnd('"');
				try
				{
					if (!File.Exists(ResponseFilePath))
					{
						throw new Exception($"ResponseFilePath '{ResponseFilePath}' does not exist!");
					}

					string ResponseFileText = File.ReadAllText(ResponseFilePath);

					// Make sure we substituted the known environment variables with corresponding BFF friendly imported vars
					ResponseFileText = SubstituteEnvironmentVariables(ResponseFileText);

					string[] Separators = { "\n", " ", "\r" };
					if (File.Exists(ResponseFilePath))
					{
						RawTokens = ResponseFileText.Split(Separators, StringSplitOptions.RemoveEmptyEntries); //Certainly not ideal
					}
				}
				catch (Exception e)
				{
					if (!String.IsNullOrEmpty(e.Message))
					{
						Logger.LogInformation("{Message}", e.Message);
					}

					Logger.LogError("Looks like a response file in: {CompilerCommandLine}, but we could not load it! {Ex}", CompilerCommandLine, e.Message);
					ResponseFilePath = "";
				}
			}

			for (int i = 0; i < RawTokens.Length; ++i)
			{
				AllTokens.Add(RawTokens[i]);
			}

			// Raw tokens being split with spaces may have split up some two argument options and
			// paths with multiple spaces in them also need some love
			for (int i = 0; i < AllTokens.Count; ++i)
			{
				string Token = AllTokens[i];
				if (String.IsNullOrEmpty(Token))
				{
					if (ProcessedTokens.Count > 0 && QuotesOpened)
					{
						string CurrentToken = ProcessedTokens.Last();
						CurrentToken += " ";
					}

					continue;
				}

				int numQuotes = 0;
				// Look for unescaped " symbols, we want to stick those strings into one token.
				for (int j = 0; j < Token.Length; ++j)
				{
					if (Token[j] == '\\') //Ignore escaped quotes
					{
						++j;
					}
					else if (Token[j] == '"')
					{
						numQuotes++;
					}
				}

				// Handle nested response files
				if (Token.StartsWith('@'))
				{
					foreach (KeyValuePair<string, string> Pair in ParseCommandLineOptions(LocalToolName, Token, SpecialOptions, Logger))
					{
						ParsedCompilerOptions.Add(Pair.Key, Pair.Value);
					}
					continue;
				}

				// Defines can have escaped quotes and other strings inside them
				// so we consume tokens until we've closed any open unescaped parentheses.
				if ((Token.StartsWith("/D", StringComparison.Ordinal) || Token.StartsWith("-D", StringComparison.Ordinal)) && !QuotesOpened)
				{
					if (numQuotes == 0 || numQuotes == 2)
					{
						ProcessedTokens.Add(Token);
					}
					else
					{
						PartialToken = Token;
						++i;
						bool AddedToken = false;
						for (; i < AllTokens.Count; ++i)
						{
							string NextToken = AllTokens[i];
							if (String.IsNullOrEmpty(NextToken))
							{
								PartialToken += " ";
							}
							else if (!NextToken.EndsWith("\\\"", System.StringComparison.Ordinal) && NextToken.EndsWith("\"", System.StringComparison.Ordinal)) //Looking for a token that ends with a non-escaped "
							{
								ProcessedTokens.Add(PartialToken + " " + NextToken);
								AddedToken = true;
								break;
							}
							else
							{
								PartialToken += " " + NextToken;
							}
						}
						if (!AddedToken)
						{
							Logger.LogWarning("Warning! Looks like an unterminated string in tokens. Adding PartialToken and hoping for the best. Command line: {CompilerCommandLine}", CompilerCommandLine);
							ProcessedTokens.Add(PartialToken);
						}
					}
					continue;
				}

				if (!QuotesOpened)
				{
					if (numQuotes % 2 != 0) //Odd number of quotes in this token
					{
						PartialToken = Token + " ";
						QuotesOpened = true;
					}
					else
					{
						ProcessedTokens.Add(Token);
					}
				}
				else
				{
					if (numQuotes % 2 != 0) //Odd number of quotes in this token
					{
						ProcessedTokens.Add(PartialToken + Token);
						QuotesOpened = false;
					}
					else
					{
						PartialToken += Token + " ";
					}
				}
			}

			//Processed tokens should now have 'whole' tokens, so now we look for any specified special options
			foreach (string specialOption in SpecialOptions)
			{
				for (int i = 0; i < ProcessedTokens.Count; ++i)
				{
					if (ProcessedTokens[i] == specialOption && i + 1 < ProcessedTokens.Count)
					{
						ParsedCompilerOptions[specialOption] = ProcessedTokens[i + 1];
						ProcessedTokens.RemoveRange(i, 2);
						break;
					}
					else if (ProcessedTokens[i].StartsWith(specialOption, StringComparison.Ordinal))
					{
						ParsedCompilerOptions[specialOption] = ProcessedTokens[i].Replace(specialOption, null, StringComparison.Ordinal);
						ProcessedTokens.RemoveAt(i);
						break;
					}
				}
			}

			//The search for the input file... we take the first non-argument we can find
			for (int i = 0; i < ProcessedTokens.Count; ++i)
			{
				string Token = ProcessedTokens[i];
				if (Token.Length == 0)
				{
					continue;
				}

				// Skip the following tokens:
				if ((Token == "/I") ||
					(Token == "/external:I") ||
					(Token == "/l") ||
					(Token == "/D") ||
					(Token == "-D") ||
					(Token == "-x") ||
					(Token == "-F") ||
					(Token == "-arch") ||
					(Token == "-isysroot") ||
					(Token == "-include") ||
					(Token == "-current_version") ||
					(Token == "-compatibility_version") ||
					(Token == "-rpath") ||
					(Token == "-weak_library") ||
					(Token == "-weak_framework") ||
					(Token == "-framework") ||
					(Token == "/sourceDependencies") ||
					(Token == "/sourceDependencies:directives") ||
					(Token.Contains("/experimental", StringComparison.Ordinal)))
				{
					++i;
				}
				else if (Token == "/we4668")
				{
					// Replace this to make Windows builds compile happily
					ProcessedTokens[i] = "/wd4668";
				}
				else if (Token.Contains("clang++", StringComparison.Ordinal))
				{
					ProcessedTokens.RemoveAt(i);
					i--;
				}
				else if (Token == "--")
				{
					ProcessedTokens.RemoveAt(i);
					ParsedCompilerOptions["CLFilterChildCommand"] = ProcessedTokens[i];
					ProcessedTokens.RemoveAt(i);
					i--;
				}
				else if (!Token.StartsWith("/", StringComparison.Ordinal) && !Token.StartsWith("-", StringComparison.Ordinal) && !Token.Contains(".framework", StringComparison.Ordinal))
				{
					ParsedCompilerOptions["InputFile"] = Token;
					ProcessedTokens.RemoveAt(i);
					i--;
				}
			}

			if (ParsedCompilerOptions.ContainsKey("OtherOptions"))
			{
				ProcessedTokens.Insert(0, ParsedCompilerOptions["OtherOptions"]);
			}

			ParsedCompilerOptions["OtherOptions"] = String.Join(" ", ProcessedTokens) + " ";

			if (SaveResponseFile && !String.IsNullOrEmpty(ResponseFilePath))
			{
				ParsedCompilerOptions["@"] = ResponseFilePath;
			}

			return ParsedCompilerOptions;
		}

		private static string GetOptionValue(Dictionary<string, string> OptionsDictionary, string Key, LinkedAction Action, ILogger Logger, bool ProblemIfNotFound = false)
		{
			if (OptionsDictionary.TryGetValue(Key, out string? Value))
			{
				return Value.Trim(new char[] { '\"' });
			}

			if (ProblemIfNotFound)
			{
				Logger.LogWarning("We failed to find {Key}, which may be a problem.", Key);
				Logger.LogWarning("Action.CommandArguments: {CommandArguments}", Action.CommandArguments);
			}

			return String.Empty;
		}

		[SupportedOSPlatform("windows")]
		public static string GetRegistryValue(string keyName, string valueName, object defaultValue)
		{
			object? returnValue = Microsoft.Win32.Registry.GetValue("HKEY_LOCAL_MACHINE\\SOFTWARE\\" + keyName, valueName, defaultValue);
			if (returnValue != null)
			{
				return returnValue.ToString()!;
			}

			returnValue = Microsoft.Win32.Registry.GetValue("HKEY_CURRENT_USER\\SOFTWARE\\" + keyName, valueName, defaultValue);
			if (returnValue != null)
			{
				return returnValue.ToString()!;
			}

			returnValue = Microsoft.Win32.Registry.GetValue("HKEY_LOCAL_MACHINE\\SOFTWARE\\Wow6432Node\\" + keyName, valueName, defaultValue);
			if (returnValue != null)
			{
				return returnValue.ToString()!;
			}

			returnValue = Microsoft.Win32.Registry.GetValue("HKEY_CURRENT_USER\\SOFTWARE\\Wow6432Node\\" + keyName, valueName, defaultValue);
			if (returnValue != null)
			{
				return returnValue.ToString()!;
			}

			return defaultValue.ToString()!;
		}

		[SupportedOSPlatform("windows")]
		private void WriteEnvironmentSetup(ILogger Logger)
		{
			VCEnvironment? VCEnv = null;

			try
			{
				// This may fail if the caller emptied PATH; we try to ignore the problem since
				// it probably means we are building for another platform.
				if (BuildType == FBBuildType.Windows)
				{
					VCEnv = VCEnvironment.Create(
						Compiler: WindowsPlatform.GetDefaultCompiler(null, UnrealArch.X64, Logger, true),
						ToolChain: WindowsCompiler.Default,
						Platform: UnrealTargetPlatform.Win64,
						Architecture: UnrealArch.X64,
						CompilerVersion: String.IsNullOrEmpty(CompilerVersion) ? null : CompilerVersion,
						ToolchainVersion: null,
						WindowsSdkVersion: null,
						SuppliedSdkDirectoryForVersion: null,
						bUseCPPWinRT: false,
						bAllowClangLinker: false,
						bAllowRadLinker: false,
						Logger);
				}
			}
			catch (Exception)
			{
				Logger.LogWarning("Failed to get Visual Studio environment.");
			}

			// Copy environment into a case-insensitive dictionary for easier key lookups
			Dictionary<string, string> envVars = new Dictionary<string, string>(StringComparer.OrdinalIgnoreCase);
			foreach (Nullable<DictionaryEntry> entry in Environment.GetEnvironmentVariables())
			{
				if (entry.HasValue)
				{
					envVars[(string)entry.Value.Key] = (string)entry.Value.Value!;
				}
			}

			if (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Win64)
			{
				if (envVars.ContainsKey("CommonProgramFiles"))
				{
					AddText("#import CommonProgramFiles\n");
				}

				if (envVars.ContainsKey("DXSDK_DIR"))
				{
					AddText("#import DXSDK_DIR\n");
				}

				if (envVars.ContainsKey("DurangoXDK"))
				{
					AddText("#import DurangoXDK\n");
				}
			}

			if (VCEnv != null)
			{
				string platformVersionNumber = "VSVersionUnknown";
				AddText($".WindowsSDKBasePath = '{VCEnv.WindowsSdkDir}'\n");
				AddText($"Compiler('UEResourceCompiler') \n{{\n");

				switch (VCEnv.Compiler)
				{
					case WindowsCompiler.VisualStudio2022:
					case WindowsCompiler.VisualStudio2026:
						// For now we are working with the 140 version, might need to change to 141 or 150 depending on the version of the Toolchain you chose
						// to install
						platformVersionNumber = "140";
						AddText($"\t.Executable = '$WindowsSDKBasePath$/bin/{VCEnv.WindowsSdkVersion}/x64/rc.exe'\n");
						break;

					default:
						string exceptionString = "Error: Unsupported Visual Studio Version.";
						Logger.LogError("{Ex}", exceptionString);
						throw new BuildException(exceptionString);
				}

				AddText($"\t.CompilerFamily  = 'custom'\n");
				AddText($"}}\n\n");

				AddText("Compiler('UECompiler') \n{\n");

				bool UsingCLFilter = VCEnv.ToolChainVersion < VersionNumber.Parse("14.27");

				DirectoryReference CLFilterDirectory = DirectoryReference.Combine(Unreal.EngineDirectory, "Build", "Windows", "cl-filter");

				AddText($"\t.Root = '{VCEnv.GetToolPath()}'\n");

				if (UsingCLFilter)
				{
					AddText($"\t.CLFilterRoot = '{CLFilterDirectory.FullName}'\n");
					AddText($"\t.Executable = '$CLFilterRoot$\\cl-filter.exe'\n");
				}
				else
				{
					AddText($"\t.Executable = '$Root$\\{VCEnv.CompilerPath.GetFileName()}'\n");
				}
				AddText($"\t.ExtraFiles =\n\t{{\n");
				if (UsingCLFilter)
				{
					AddText($"\t\t'$Root$/cl.exe'\n");
				}
				AddText($"\t\t'$Root$/c1.dll'\n");
				AddText($"\t\t'$Root$/c1xx.dll'\n");
				AddText($"\t\t'$Root$/c2.dll'\n");

				FileReference? cluiDllPath = null;
				string cluiSubDirName = "1033";
				if (File.Exists(VCEnv.GetToolPath() + "{cluiSubDirName}/clui.dll")) //Check English first...
				{
					if (UsingCLFilter)
					{
						AddText("\t\t'$CLFilterRoot$/{cluiSubDirName}/clui.dll'\n");
					}
					else
					{
						AddText("\t\t'$Root$/{cluiSubDirName}/clui.dll'\n");
					}
					cluiDllPath = new FileReference(VCEnv.GetToolPath() + "{cluiSubDirName}/clui.dll");
				}
				else
				{
					IEnumerable<string> numericDirectories = Directory.GetDirectories(VCEnv.GetToolPath().ToString()).Where(d => Path.GetFileName(d).All(Char.IsDigit));
					IEnumerable<string> cluiDirectories = numericDirectories.Where(d => Directory.GetFiles(d, "clui.dll").Any());
					if (cluiDirectories.Any())
					{
						cluiSubDirName = Path.GetFileName(cluiDirectories.First());
						if (UsingCLFilter)
						{
							AddText(String.Format("\t\t'$CLFilterRoot$/{0}/clui.dll'\n", cluiSubDirName));
						}
						else
						{
							AddText(String.Format("\t\t'$Root$/{0}/clui.dll'\n", cluiSubDirName));
						}
						cluiDllPath = new FileReference(cluiDirectories.First() + "/clui.dll");
					}
				}

				// FASTBuild only preserves the directory structure of compiler files for files in the same directory or sub-directories of the primary executable
				// Since our primary executable is cl-filter.exe and we need clui.dll in a sub-directory on the worker, we need to copy it to cl-filter's subdir
				if (UsingCLFilter && cluiDllPath != null)
				{
					Directory.CreateDirectory(Path.Combine(CLFilterDirectory.FullName, cluiSubDirName));
					File.Copy(cluiDllPath.FullName, Path.Combine(CLFilterDirectory.FullName, cluiSubDirName, "clui.dll"), true);
				}

				AddText("\t\t'$Root$/mspdbsrv.exe'\n");
				AddText("\t\t'$Root$/mspdbcore.dll'\n");

				AddText($"\t\t'$Root$/mspft{platformVersionNumber}.dll'\n");
				AddText($"\t\t'$Root$/msobj{platformVersionNumber}.dll'\n");
				AddText($"\t\t'$Root$/mspdb{platformVersionNumber}.dll'\n");

				string RedistInstallDirectory = String.Format("{0}/Redist/MSVC", VCEnv.GetVCInstallDirectory());
				List<string> PotentialMSVCRedistPaths = new List<string>(Directory.EnumerateDirectories(RedistInstallDirectory));
				string? PrefferedMSVCRedistPath = null;
				string? FinalMSVCRedistPath = "";

				if (PotentialMSVCRedistPaths.Count == 0)
				{
					// We cannot possibly derive a FinalMSVCRedistPath, as all paths are predicated on there being at least 1 install directory within PotentialMSVCRedistPaths
					throw new Exception($"Couldn't find any installations under the VC Redist install directory: {RedistInstallDirectory}");
				}

				if (MsvcCRTRedistVersion.Length > 0)
				{
					PrefferedMSVCRedistPath = PotentialMSVCRedistPaths.Find(
						delegate (string str)
						{
							return str.Contains(MsvcCRTRedistVersion, StringComparison.Ordinal);
						});
				}

				if (PrefferedMSVCRedistPath == null)
				{
					// If upon Visual Studio installation the user selects "latest", two folders are created. If however the user only specifies one, only the selected folder is created.
					PrefferedMSVCRedistPath = PotentialMSVCRedistPaths.Count == 1 ? PotentialMSVCRedistPaths[0] : PotentialMSVCRedistPaths[^2];

					if (MsvcCRTRedistVersion.Length > 0)
					{
						Logger.LogInformation("Couldn't find redist path for given MsvcCRTRedistVersion {MsvcCRTRedistVersion}"
						+ " (in BuildConfiguration.xml). \n\t...Using this path instead: {PrefferedMSVCRedistPath}", MsvcCRTRedistVersion, PrefferedMSVCRedistPath);
					}
					else
					{
						Logger.LogInformation("Using path : {PrefferedMSVCRedistPath} for vccorlib_.dll (MSVC redist)..." +
							"\n\t...Add an entry for MsvcCRTRedistVersion in BuildConfiguration.xml to specify a version number", PrefferedMSVCRedistPath.ToString());
					}
				}

				PotentialMSVCRedistPaths = new List<string>(Directory.EnumerateDirectories(String.Format("{0}/{1}", PrefferedMSVCRedistPath, VCEnv.Architecture)));

				FinalMSVCRedistPath = PotentialMSVCRedistPaths.Find(x => x.Contains(".CRT", StringComparison.Ordinal));

				if (String.IsNullOrEmpty(FinalMSVCRedistPath))
				{
					FinalMSVCRedistPath = PrefferedMSVCRedistPath;
				}

				{
					AddText($"\t\t'$Root$/msvcp{platformVersionNumber}.dll'\n");
					AddText(String.Format("\t\t'{0}/vccorlib{1}.dll'\n", FinalMSVCRedistPath, platformVersionNumber));
					AddText($"\t\t'$Root$/tbbmalloc.dll'\n");

					//AddText(string.Format("\t\t'{0}/Redist/MSVC/{1}/x64/Microsoft.VC141.CRT/vccorlib{2}.dll'\n", VCEnv.GetVCInstallDirectory(), VCEnv.ToolChainVersion, platformVersionNumber));
				}

				AddText("\t}\n"); //End extra files

				AddText($"\t.CompilerFamily = 'msvc'\n");
				AddText("}\n\n"); //End compiler
			}

			if (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Mac)
			{
				AddText($".MacBaseSDKDir = '{ApplePlatformSDK.GetPlatformSDKDirectory("MacOSX")}'\n");
				AddText($".MacToolchainDir = '{ApplePlatformSDK.GetToolchainDirectory()}'\n");
				AddText($"Compiler('UEAppleCompiler') \n{{\n");
				AddText($"\t.Executable = '$MacToolchainDir$/clang++'\n");
				AddText($"\t.ClangRewriteIncludes = false\n"); // This is to fix an issue with iOS clang builds, __has_include, and Objective-C #imports
				AddText($"}}\n\n");
			}

			AddText("Settings \n{\n");

			if (bEnableCaching)
			{
				string? CachePath = GetCachePath();
				if (!String.IsNullOrEmpty(CachePath))
				{
					AddText($"\t.CachePath = '{CachePath}'\n");
				}
			}

			if (bEnableDistribution)
			{
				string? BrokeragePath = GetBrokeragePath();
				if (!String.IsNullOrEmpty(BrokeragePath))
				{
					AddText($"\t.BrokeragePath = '{BrokeragePath}'\n");
				}
			}

			//Start Environment
			AddText("\t.Environment = \n\t{\n");
			if (VCEnv != null)
			{
				AddText(String.Format("\t\t\"PATH={0}\\Common7\\IDE\\;{1};{2}\\bin\\{3}\\x64\",\n", VCEnv.GetVCInstallDirectory(), VCEnv.GetToolPath(), VCEnv.WindowsSdkDir, VCEnv.WindowsSdkVersion));
			}

			if (!IsApple())
			{
				if (envVars.ContainsKey("TMP"))
				{
					AddText($"\t\t\"TMP={envVars["TMP"]}\",\n");
				}

				if (envVars.ContainsKey("SystemRoot"))
				{
					AddText($"\t\t\"SystemRoot={envVars["SystemRoot"]}\",\n");
				}

				if (envVars.ContainsKey("INCLUDE"))
				{
					AddText($"\t\t\"INCLUDE={envVars["INCLUDE"]}\",\n");
				}

				if (envVars.ContainsKey("LIB"))
				{
					AddText($"\t\t\"LIB={envVars["LIB"]}\",\n");
				}
			}

			AddText("\t}\n"); //End environment
			AddText("}\n\n"); //End Settings
		}

		private void AddCompileAction(LinkedAction Action, IEnumerable<LinkedAction> DependencyActions, ILogger Logger)
		{
			string CompilerName = GetCompilerName();
			if (Action.CommandPath.FullName.Contains("rc.exe", StringComparison.Ordinal))
			{
				CompilerName = "UEResourceCompiler";
			}

			string[] SpecialCompilerOptions = { "/Fo", "/fo", "/Yc", "/Yu", "/Fp", "-o", "-dependencies=", "-compiler=" };
			Dictionary<string, string> ParsedCompilerOptions = ParseCommandLineOptions(Action.CommandPath.GetFileName(), Action.CommandArguments, SpecialCompilerOptions, Logger);

			string OutputObjectFileName = GetOptionValue(ParsedCompilerOptions, IsMSVC() ? "/Fo" : "-o", Action, Logger, ProblemIfNotFound: !IsMSVC());

			if (IsMSVC() && String.IsNullOrEmpty(OutputObjectFileName)) // Didn't find /Fo, try /fo
			{
				OutputObjectFileName = GetOptionValue(ParsedCompilerOptions, "/fo", Action, Logger, ProblemIfNotFound: true);
			}

			if (String.IsNullOrEmpty(OutputObjectFileName)) //No /Fo or /fo, we're probably in trouble.
			{
				throw new Exception("We have no OutputObjectFileName. Bailing. Our Action.CommandArguments were: " + Action.CommandArguments);
			}

			string IntermediatePath = Path.GetDirectoryName(OutputObjectFileName)!;
			if (String.IsNullOrEmpty(IntermediatePath))
			{
				throw new Exception("We have no IntermediatePath. Bailing. Our Action.CommandArguments were: " + Action.CommandArguments);
			}

			IntermediatePath = IsApple() ? IntermediatePath.Replace("\\", "/", StringComparison.Ordinal) : IntermediatePath;

			string InputFile = GetOptionValue(ParsedCompilerOptions, "InputFile", Action, Logger, ProblemIfNotFound: true);
			if (String.IsNullOrEmpty(InputFile))
			{
				throw new Exception("We have no InputFile. Bailing. Our Action.CommandArguments were: " + Action.CommandArguments);
			}

			AddText($"ObjectList('{ActionToActionString(Action)}')\n{{\n");
			AddText($"\t.Compiler = '{CompilerName}'\n");
			AddText($"\t.CompilerInputFiles = \"{InputFile}\"\n");
			AddText($"\t.CompilerOutputPath = \"{IntermediatePath}\"\n");

			if (!Action.bCanExecuteRemotely || !Action.bCanExecuteRemotelyWithSNDBS || ForceLocalCompileModules.Contains(Path.GetFileNameWithoutExtension(InputFile)))
			{
				AddText("\t.AllowDistribution = false\n");
			}

			string OtherCompilerOptions = GetOptionValue(ParsedCompilerOptions, "OtherOptions", Action, Logger);

			if (ForceOverwriteCompilerOptionModules.Any(x => Action.CommandArguments.Contains(x, StringComparison.OrdinalIgnoreCase)))
			{
				OtherCompilerOptions = OtherCompilerOptions.Replace("/WX", "", StringComparison.Ordinal);
			}

			string CompilerOutputExtension = ".unset";
			string CLFilterParams = "";
			string ShowIncludesParam = "";
			if (ParsedCompilerOptions.ContainsKey("CLFilterChildCommand"))
			{
				CLFilterParams = "-dependencies=\"%CLFilterDependenciesOutput\" -compiler=\"%5\" -stderronly -- \"%5\" ";
				ShowIncludesParam = "/showIncludes";
			}

			if (ParsedCompilerOptions.ContainsKey("/Yc")) //Create PCH
			{
				string PCHIncludeHeader = GetOptionValue(ParsedCompilerOptions, "/Yc", Action, Logger, ProblemIfNotFound: true);
				string PCHOutputFile = GetOptionValue(ParsedCompilerOptions, "/Fp", Action, Logger, ProblemIfNotFound: true);

				AddText($"\t.CompilerOptions = '{CLFilterParams}\"%1\" /Fo\"%2\" /Fp\"{PCHOutputFile}\" /Yu\"{PCHIncludeHeader}\" {OtherCompilerOptions} '\n");

				AddText($"\t.PCHOptions = '{CLFilterParams}\"%1\" /Fp\"%2\" /Yc\"{PCHIncludeHeader}\" {OtherCompilerOptions} /Fo\"{OutputObjectFileName}\"'\n");
				AddText($"\t.PCHInputFile = \"{InputFile}\"\n");
				AddText($"\t.PCHOutputFile = \"{PCHOutputFile}\"\n");
				CompilerOutputExtension = ".obj";
			}
			else if (ParsedCompilerOptions.ContainsKey("/Yu")) //Use PCH
			{
				string PCHIncludeHeader = GetOptionValue(ParsedCompilerOptions, "/Yu", Action, Logger, ProblemIfNotFound: true);
				string PCHOutputFile = GetOptionValue(ParsedCompilerOptions, "/Fp", Action, Logger, ProblemIfNotFound: true);
				string PCHToForceInclude = PCHOutputFile.Replace(".pch", "", StringComparison.Ordinal);
				AddText($"\t.CompilerOptions = '{CLFilterParams}\"%1\" /Fo\"%2\" /Fp\"{PCHOutputFile}\" /Yu\"{PCHIncludeHeader}\" /FI\"{PCHToForceInclude}\" {OtherCompilerOptions} {ShowIncludesParam} '\n");
				string InputFileExt = Path.GetExtension(InputFile);
				CompilerOutputExtension = InputFileExt + ".obj";
			}
			else if (Path.GetExtension(OutputObjectFileName) == ".gch") //Create PCH
			{
				AddText($"\t.CompilerOptions = '{OtherCompilerOptions} -D __BUILDING_WITH_FASTBUILD__ -fno-diagnostics-color -o \"%2\" \"%1\" '\n");
				AddText($"\t.PCHOptions = '{OtherCompilerOptions} -o \"%2\" \"%1\" '\n");
				AddText($"\t.PCHInputFile = \"{InputFile}\"\n");
				AddText($"\t.PCHOutputFile = \"{OutputObjectFileName}\"\n");
				CompilerOutputExtension = ".h.gch";
			}
			else
			{
				if (CompilerName == "UEResourceCompiler")
				{
					AddText($"\t.CompilerOptions = '{OtherCompilerOptions} /fo\"%2\" \"%1\" '\n");
					CompilerOutputExtension = Path.GetExtension(InputFile) + ".res";
				}
				else
				{
					if (IsMSVC())
					{
						AddText($"\t.CompilerOptions = '{CLFilterParams}{OtherCompilerOptions} /Fo\"%2\" \"%1\" {ShowIncludesParam} '\n");
						string InputFileExt = Path.GetExtension(InputFile);
						CompilerOutputExtension = InputFileExt + ".obj";
					}
					else
					{
						AddText($"\t.CompilerOptions = '{OtherCompilerOptions} -D __BUILDING_WITH_FASTBUILD__ -fno-diagnostics-color -o \"%2\" \"%1\" '\n");
						string InputFileExt = Path.GetExtension(InputFile);
						CompilerOutputExtension = InputFileExt + ".o";
					}
				}
			}

			AddText($"\t.CompilerOutputExtension = '{CompilerOutputExtension}' \n");
			AddPreBuildDependenciesText(DependencyActions);
			AddText("}\n\n");
		}

		private void AddExecAction(LinkedAction Action, IEnumerable<LinkedAction> DependencyActions)
		{
			AddText($"Exec('{ActionToActionString(Action)}')\n{{\n");
			AddText($"\t.ExecExecutable = '{Action.CommandPath.FullName}' \n");
			AddText($"\t.ExecArguments = '{Action.CommandArguments}' \n");
			AddText($"\t.ExecWorkingDir = '{Action.WorkingDirectory.FullName}' \n");
			AddText($"\t.ExecOutput = '{Action.ProducedItems.First().FullName}' \n");
			AddText($"\t.ExecAlways = true \n");
			AddPreBuildDependenciesText(DependencyActions);
			AddText($"}}\n\n");
		}

		private void PrintActionDetails(LinkedAction ActionToPrint, ILogger Logger)
		{
			Logger.LogInformation("{Action}", ActionToActionString(ActionToPrint));
			Logger.LogInformation("Action Type: {Type}", ActionToPrint.ActionType.ToString());
			Logger.LogInformation("Action CommandPath: {Path}", ActionToPrint.CommandPath.FullName);
			Logger.LogInformation("Action CommandArgs: {Args}", ActionToPrint.CommandArguments);
		}

		private MemoryStream? bffOutputMemoryStream = null;

		[SupportedOSPlatform("windows")]
		private bool CreateBffFile(IEnumerable<LinkedAction> Actions, string BffFilePath, ILogger Logger)
		{
			try
			{
				bffOutputMemoryStream = new MemoryStream();

				AddText(";*************************************************************************\n");
				AddText(";* Autogenerated bff - see FASTBuild.cs for how this file was generated. *\n");
				AddText(";*************************************************************************\n\n");

				WriteEnvironmentSetup(Logger); //Compiler, environment variables and base paths

				foreach (LinkedAction Action in Actions)
				{
					// Resolve the list of prerequisite items for this action to
					// a list of actions which produce these prerequisites
					IEnumerable<LinkedAction> DependencyActions = Action.PrerequisiteActions.Distinct();

					AddText($";** Function for Action {GetActionID(Action)} **\n");
					AddText($";** CommandPath: {Action.CommandPath.FullName}\n");
					AddText($";** CommandArguments: {Action.CommandArguments}\n");
					AddText("\n");

					if (Action.ActionType == ActionType.Compile && Action.bCanExecuteRemotely && Action.bCanExecuteRemotelyWithSNDBS)
					{
						AddCompileAction(Action, DependencyActions, Logger);
					}
					else
					{
						AddExecAction(Action, DependencyActions);
					}
				}

				string JoinedActions = Actions
					.Select(Action => ActionToDependencyString(Action))
					.DefaultIfEmpty(String.Empty)
					.Aggregate((str, obj) => str + "\n" + obj);

				AddText("Alias( 'all' ) \n{\n");
				AddText("\t.Targets = { \n");
				AddText(JoinedActions);
				AddText("\n\t}\n");
				AddText("}\n");

				using (FileStream bffOutputFileStream = new FileStream(BffFilePath, FileMode.Create, FileAccess.Write))
				{
					bffOutputMemoryStream.Position = 0;
					bffOutputMemoryStream.CopyTo(bffOutputFileStream);
				}

				bffOutputMemoryStream.Close();
			}
			catch (Exception e)
			{
				Logger.LogError("Exception while creating bff file: {Ex}", e.ToString());
				return false;
			}

			return true;
		}

		private bool ExecuteBffFile(string BffFilePath, ILogger Logger)
		{
			string CacheArgument = "";

			if (bEnableCaching)
			{
				switch (CacheMode)
				{
					case FASTBuildCacheMode.ReadOnly:
						CacheArgument = "-cacheread";
						break;
					case FASTBuildCacheMode.WriteOnly:
						CacheArgument = "-cachewrite";
						break;
					case FASTBuildCacheMode.ReadWrite:
						CacheArgument = "-cache";
						break;
				}
			}

			string DistArgument = bEnableDistribution ? "-dist" : "";
			string ForceRemoteArgument = bForceRemote ? "-forceremote" : "";
			string NoStopOnErrorArgument = bStopOnError ? "" : "-nostoponerror";
			string IDEArgument = IsApple() ? "" : "-ide";
			string MaxProcesses = "-j" + LocalExecutor.NumParallelProcesses;

			// Interesting flags for FASTBuild:
			// -nostoponerror, -verbose, -monitor (if FASTBuild Monitor Visual Studio Extension is installed!)
			// Yassine: The -clean is to bypass the FASTBuild internal
			// dependencies checks (cached in the fdb) as it could create some conflicts with UBT.
			// Basically we want FB to stupidly compile what UBT tells it to.
			string FBCommandLine = $"-monitor -summary {DistArgument} {CacheArgument} {IDEArgument} {MaxProcesses} -clean -config \"{BffFilePath}\" {NoStopOnErrorArgument} {ForceRemoteArgument}";

			Logger.LogInformation("FBuild Command Line Arguments: '{FBCommandLine}", FBCommandLine);

			string FBExecutable = GetExecutablePath()!;
			string WorkingDirectory = Path.GetFullPath(Path.Combine(Unreal.EngineDirectory.MakeRelativeTo(DirectoryReference.GetCurrentDirectory()), "Source"));

			ProcessStartInfo FBStartInfo = new ProcessStartInfo(FBExecutable, FBCommandLine);
			FBStartInfo.UseShellExecute = false;
			FBStartInfo.WorkingDirectory = WorkingDirectory;
			FBStartInfo.RedirectStandardError = true;
			FBStartInfo.RedirectStandardOutput = true;

			string? Coordinator = GetCoordinator();
			if (!String.IsNullOrEmpty(Coordinator) && !FBStartInfo.EnvironmentVariables.ContainsKey("FASTBUILD_COORDINATOR"))
			{
				FBStartInfo.EnvironmentVariables.Add("FASTBUILD_COORDINATOR", Coordinator);
			}
			FBStartInfo.EnvironmentVariables.Remove("FASTBUILD_BROKERAGE_PATH"); // remove stale serialized value and defer to GetBrokeragePath
			string? BrokeragePath = GetBrokeragePath();
			if (!String.IsNullOrEmpty(BrokeragePath) && !FBStartInfo.EnvironmentVariables.ContainsKey("FASTBUILD_BROKERAGE_PATH"))
			{
				FBStartInfo.EnvironmentVariables.Add("FASTBUILD_BROKERAGE_PATH", BrokeragePath);
			}
			string? CachePath = GetCachePath();
			if (!String.IsNullOrEmpty(CachePath) && !FBStartInfo.EnvironmentVariables.ContainsKey("FASTBUILD_CACHE_PATH"))
			{
				FBStartInfo.EnvironmentVariables.Add("FASTBUILD_CACHE_PATH", CachePath);
			}

			try
			{
				Process FBProcess = new Process();
				FBProcess.StartInfo = FBStartInfo;
				FBProcess.EnableRaisingEvents = true;

				DataReceivedEventHandler OutputEventHandler = (Sender, Args) =>
				{
					if (Args.Data != null)
					{
						Logger.LogInformation("{Output}", Args.Data);
					}
				};

				FBProcess.OutputDataReceived += OutputEventHandler;
				FBProcess.ErrorDataReceived += OutputEventHandler;

				FBProcess.Start();

				FBProcess.BeginOutputReadLine();
				FBProcess.BeginErrorReadLine();

				FBProcess.WaitForExit();
				return FBProcess.ExitCode == 0;
			}
			catch (Exception e)
			{
				Logger.LogError("Exception launching fbuild process. Is it in your path? {Ex}", e.ToString());
				return false;
			}
		}

		private bool _disposedValue;
		protected override void Dispose(bool disposing)
		{
			if (!_disposedValue)
			{
				if (disposing)
				{
					LocalExecutor.Dispose();
				}

				base.Dispose(disposing);
				_disposedValue = true;
			}
		}
	}
}
