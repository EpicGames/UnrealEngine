// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections;
using System.Collections.Generic;
using System.Diagnostics;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using System.Linq;
using System.Net;
using System.Net.NetworkInformation;
using System.Runtime.InteropServices;
using System.Runtime.Versioning;
using System.ServiceProcess;
using System.Text.Json;
using System.Text.RegularExpressions;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using Microsoft.Win32;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	sealed class SNDBS : ActionExecutor
	{
		internal const string ExecutorName = "SNDBS";

		public override string Name => ExecutorName;

		private static readonly string? ProgramFilesx86 = Environment.GetEnvironmentVariable("ProgramFiles(x86)");
		private static readonly string? SCERoot = Environment.GetEnvironmentVariable("SCE_ROOT_DIR");

		private static string FindDbsExe(string ExeName)
		{
			string InstallPath = Path.Combine(ProgramFilesx86 ?? String.Empty, "SCE", "Common", "SN-DBS", "bin", ExeName);
			if (File.Exists(InstallPath))
			{
				return InstallPath;
			}
			else
			{
				// Legacy install location using SCE_ROOT_DIR
				return Path.Combine(SCERoot ?? String.Empty, "Common", "SN-DBS", "bin", ExeName);
			}
		}

		private static string SNDBSBuildExe => FindDbsExe("dbsbuild.exe");
		private static string SNDBSUtilExe => FindDbsExe("dbsutil.exe");

		private static readonly DirectoryReference IntermediateDir = DirectoryReference.Combine(Unreal.EngineDirectory, "Intermediate", "Build", "SNDBS");
		private static readonly FileReference IncludeRewriteRulesFile = FileReference.Combine(IntermediateDir, "include-rewrite-rules.ini");
		private static readonly FileReference ScriptFile = FileReference.Combine(IntermediateDir, "sndbs.json");

		private readonly Dictionary<string, string> GlobalTemplates = new Dictionary<string, string>();

#pragma warning disable IDE0044 // Make field readonly - these private static fields are set by XML configuration.
		/// <summary>
		/// When enabled, dependency scanning is prioritized to avoid being slowed down by running local jobs.
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		bool bPrioritizeDependencyScanning = false;

		/// <summary>
		/// When enabled, SN-DBS will stop compiling targets after a compile error occurs.  Recommended, as it saves computing resources for others.
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		bool bStopSNDBSCompilationAfterErrors = false;

		/// <summary>
		/// When set to false, SNDBS will not be enabled when running connected to the coordinator over VPN. Configure VPN-assigned subnets via the VpnSubnets parameter.
		/// </summary>
		[XmlConfigFile(Category = "SNDBS")]
		static bool bAllowOverVpn = true;

		/// <summary>
		/// List of subnets containing IP addresses assigned by VPN
		/// </summary>
		[XmlConfigFile(Category = "SNDBS")]
		static string[]? VpnSubnets = null;
#pragma warning restore IDE0044

		private const string ProgressMarkupPrefix = "@action:";

		private readonly IReadOnlyList<TargetDescriptor> TargetDescriptors;

		public SNDBS(IEnumerable<TargetDescriptor> InTargetDescriptors, ILogger Logger)
			: base(Logger)
		{
			XmlConfig.ApplyTo(this);
			TargetDescriptors = [.. InTargetDescriptors];
		}

		public SNDBS AddTemplate(string ExeName, string TemplateContents)
		{
			GlobalTemplates.Add(ExeName, TemplateContents);
			return this;
		}

		static bool TryGetBrokerHost([NotNullWhen(true)] out string? OutBroker)
		{
			if (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Win64)
			{
				string BrokerHostName = "";
				Regex FindHost = new Regex(@"Active broker is ""(\S +)"" \((\S+)\)");
				Process LocalProcess = new Process();
				LocalProcess.StartInfo = new ProcessStartInfo(SNDBSUtilExe, $"-connected");
				LocalProcess.OutputDataReceived += (Sender, Args) =>
				{
					if (Args.Data != null)
					{
						Match Result = FindHost.Match(Args.Data);
						if (Result.Success)
						{
							BrokerHostName = Result.Groups[1].Value;
						}
					}
				};
				if (Utils.RunLocalProcess(LocalProcess) == 1 && BrokerHostName.Length > 0)
				{
					OutBroker = BrokerHostName;
					return true;
				}
			}

			OutBroker = null;
			return false;
		}

		[DllImport("iphlpapi")]
		static extern int GetBestInterface(uint dwDestAddr, ref int pdwBestIfIndex);

		static NetworkInterface? GetInterfaceForHost(string Host)
		{
			if (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Win64)
			{
				IPHostEntry HostEntry = Dns.GetHostEntry(Host);
				foreach (IPAddress HostAddress in HostEntry.AddressList)
				{
					int InterfaceIdx = 0;
					if (GetBestInterface(BitConverter.ToUInt32(HostAddress.GetAddressBytes(), 0), ref InterfaceIdx) == 0)
					{
						foreach (NetworkInterface Interface in NetworkInterface.GetAllNetworkInterfaces())
						{
							IPv4InterfaceProperties Properties = Interface.GetIPProperties().GetIPv4Properties();
							if (Properties.Index == InterfaceIdx)
							{
								return Interface;
							}
						}
					}
				}
			}
			return null;
		}

		public static bool IsHostOnVpn(string HostName, ILogger Logger)
		{
			// If there aren't any defined subnets, just early out
			if (VpnSubnets == null || VpnSubnets.Length == 0)
			{
				return false;
			}

			// Parse all the subnets from the config file
			List<Subnet> ParsedVpnSubnets = new List<Subnet>();
			foreach (string VpnSubnet in VpnSubnets)
			{
				ParsedVpnSubnets.Add(Subnet.Parse(VpnSubnet));
			}

			// Check if any network adapters have an IP within one of these subnets
			try
			{
				NetworkInterface? Interface = GetInterfaceForHost(HostName);
				if (Interface != null && Interface.OperationalStatus == OperationalStatus.Up)
				{
					IPInterfaceProperties Properties = Interface.GetIPProperties();
					foreach (UnicastIPAddressInformation UnicastAddressInfo in Properties.UnicastAddresses)
					{
						byte[] AddressBytes = UnicastAddressInfo.Address.GetAddressBytes();
						foreach (Subnet Subnet in ParsedVpnSubnets)
						{
							if (Subnet.Contains(AddressBytes))
							{
								if (!bAllowOverVpn)
								{
									Log.TraceInformationOnce("XGE coordinator {0} will be not be used over VPN (adapter '{1}' with IP {2} is in subnet {3}). Set <XGE><bAllowOverVpn>true</bAllowOverVpn></XGE> in BuildConfiguration.xml to override.", HostName, Interface.Description, UnicastAddressInfo.Address, Subnet);
								}
								return true;
							}
						}
					}
				}
			}
			catch (Exception Ex)
			{
				Logger.LogWarning("Unable to check whether host {HostName} is connected to VPN:\n{Ex}", HostName, ExceptionUtils.FormatExceptionDetails(Ex));
			}
			return false;
		}

		public static bool IsAvailable(ILogger Logger)
		{
			// Check the executable exists on disk
			if (!File.Exists(SNDBSBuildExe))
			{
				return false;
			}

			// Check the service is running
			if (OperatingSystem.IsWindows() && !ServiceController.GetServices().Any(s => OperatingSystem.IsWindows() && s.ServiceName.StartsWith("SNDBS", StringComparison.Ordinal) && s.Status == ServiceControllerStatus.Running))
			{
				return false;
			}

			// Check if we're connected over VPN
			if (!bAllowOverVpn && VpnSubnets != null && VpnSubnets.Length > 0)
			{
				string? BrokerHost;
				if (TryGetBrokerHost(out BrokerHost) && IsHostOnVpn(BrokerHost, Logger))
				{
					return false;
				}
			}

			return true;
		}

		private TelemetryExecutorEvent? telemetryEvent;

		/// <inheritdoc/>
		public override TelemetryExecutorEvent? GetTelemetryEvent() => telemetryEvent;

		/// <inheritdoc/>
		public override Task<bool> ExecuteActionsAsync(IEnumerable<LinkedAction> ActionsToExecute, ILogger Logger)
		{
			return Task.FromResult(ExecuteActions(ActionsToExecute, Logger));
		}

		[SupportedOSPlatform("windows")]
		private static string? GetInstalledSNDBSVersion()
		{
			return (string?)Registry.GetValue(@"HKEY_LOCAL_MACHINE\SOFTWARE\Wow6432Node\SCE\Common\SN-DBS", "Version", null);
		}

		bool ExecuteActions(IEnumerable<LinkedAction> Actions, ILogger Logger)
		{
			if (!Actions.Any())
			{
				return true;
			}

			if (!OperatingSystem.IsWindows())
			{
				throw new BuildException("SN-DBS is only supported on Windows.");
			}

			// UE requires SN-DBS 2.6.10.1020 or later to avoid issues that may arise when compiling multiple platforms
			// at once. To ease the pain of engine upgrades, we are temporarily maintaining support for older versions.
			// When using older versions, a build failure will occur if the scenario requiring the newer version is
			// encountered.
			bool bUsePerJobToolTemplates = false;

			string? SNDBSVersionString = GetInstalledSNDBSVersion();

			if (SNDBSVersionString != null)
			{
				Version InstalledVersion = new Version(SNDBSVersionString);
				Version RequiredVersion = new Version("2.6.10.1020");

				if (InstalledVersion >= RequiredVersion)
				{
					bUsePerJobToolTemplates = true;
				}
			}
			else
			{
				throw new BuildException("Failed to read SN-DBS version from registry.");
			}

			// Clean the intermediate directory in case there are any leftovers from previous builds
			if (DirectoryReference.Exists(IntermediateDir))
			{
				DirectoryReference.Delete(IntermediateDir, true);
			}

			DirectoryReference.CreateDirectory(IntermediateDir);
			if (!DirectoryReference.Exists(IntermediateDir))
			{
				throw new BuildException($"Failed to create directory \"{IntermediateDir}\".");
			}

			int IdCounter = 0;
			// Build the json script file to describe all the actions and their dependencies
			Dictionary<LinkedAction, string> ActionIds = Actions.ToDictionary(a => a, a => new Guid(++IdCounter, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0).ToString());
			JsonSerializerOptions JsonOption = new JsonSerializerOptions();
			JsonOption.Encoder = System.Text.Encodings.Web.JavaScriptEncoder.UnsafeRelaxedJsonEscaping;
			File.WriteAllText(ScriptFile.FullName, JsonSerializer.Serialize(new Dictionary<string, object>()
			{
				["jobs"] = Actions.Select(a =>
				{
					Dictionary<string, object> Job = new Dictionary<string, object>()
					{
						["id"] = ActionIds[a],
						["title"] = a.StatusDescription,
						["command"] = $"\"{a.CommandPath}\" {a.CommandArguments}",
						["working_directory"] = a.WorkingDirectory.FullName,
						["dependencies"] = a.PrerequisiteActions.Select(p => ActionIds[p]).ToArray(),
						["run_locally"] = !(a.bCanExecuteRemotely && a.bCanExecuteRemotelyWithSNDBS)
					};

					// Associate this job with the platform-specific template directory
					if (bUsePerJobToolTemplates)
					{
						if (a.Target != null)
						{
							// JSON format needs this to be an array, even though we only use one directory
							DirectoryReference[] ToolTemplateDirectories = new DirectoryReference[1];
							ToolTemplateDirectories[0] = DirectoryReference.Combine(IntermediateDir, a.Target.Platform.ToString());

							Job["tool_template_directories"] = ToolTemplateDirectories;
						}
					}

					if (a.PrerequisiteItems.Any())
					{
						Job["explicit_input_files"] = a.PrerequisiteItems
							.Where(i => !(i.AbsolutePath.EndsWith(".rsp", System.StringComparison.Ordinal) || i.AbsolutePath.EndsWith(".response", System.StringComparison.Ordinal)))
							.Select(i => new Dictionary<string, object>()
								{
									["filename"] = i.AbsolutePath
								})
							.ToList();
					}

					// Look for any prerequisite actions that produce .pch files and add any .cpp files they depend on
					List<Dictionary<string, object>>? ExplicitInputFiles = Job.ContainsKey("explicit_input_files") ? Job["explicit_input_files"] as List<Dictionary<string, object>> : null;

					ExplicitInputFiles?.AddRange(a.PrerequisiteActions
						.Where(Prereq => Prereq.ProducedItems.Any(Produced => Produced.AbsolutePath.EndsWith(".gch", System.StringComparison.Ordinal) || Produced.AbsolutePath.EndsWith(".pch", System.StringComparison.Ordinal)))
						.SelectMany(Prereq => Prereq.PrerequisiteItems, (_, PrereqFile) => PrereqFile.AbsolutePath)
						.Where(Path => Path.EndsWith(".cpp", StringComparison.Ordinal))
						.Select(Path => new Dictionary<string, object>()
						{
							["filename"] = Path
						}));

					// When using clang-cl, any time the PCH is included we need to include the corresponding .cpp file.
					// This is because the AST in the PCH file references this file and without it we get a "malformed
					// or corrupted AST file" error stating it could not find the .cpp file.
					//
					// The file is tiny and harmless to include when not using clang-cl.
					List<Dictionary<string, object>>? PrecompiledDependencies = ExplicitInputFiles?.Select(File => File["filename"].ToString())
						.Where(Path => Path != null && Path.EndsWith(".h.pch", System.StringComparison.Ordinal))
						.Select(Path => Path!.Replace(".h.pch", ".cpp", StringComparison.Ordinal))
						.Select(Path => new Dictionary<string, object>()
						{
							["filename"] = Path
						}).ToList();

					if (PrecompiledDependencies != null)
					{
						ExplicitInputFiles?.AddRange(PrecompiledDependencies);
					}

					string CommandDescription = String.IsNullOrWhiteSpace(a.CommandDescription) ? a.ActionType.ToString() : a.CommandDescription;
					Job["echo"] = $"{ProgressMarkupPrefix}{CommandDescription}:{a.StatusDescription}";

					return Job;
				}).ToArray()
			}, JsonOption));

			PrepareToolTemplates(bUsePerJobToolTemplates);
			bool bHasRewrites = GenerateSNDBSIncludeRewriteRules();

			IEnumerable<string> ConfigList = TargetDescriptors.Select(Descriptor => $"{Descriptor.Name}|{Descriptor.Platform}|{Descriptor.Configuration}");
			string ConfigDescription = String.Join(",", ConfigList);

			// The per-job "tool_template_directories" take precedent over the value passed to -templates. We use the
			// -templates for GlobalTemplates.
			ProcessStartInfo StartInfo = new ProcessStartInfo(
				SNDBSBuildExe,
				$"-q -p \"{ConfigDescription}\" -s \"{ScriptFile}\" -templates \"{IntermediateDir}\""
				);
			StartInfo.UseShellExecute = false;
			if (bHasRewrites)
			{
				StartInfo.Arguments += $" --include-rewrite-rules \"{IncludeRewriteRulesFile}\"";
			}
			if (!bStopSNDBSCompilationAfterErrors)
			{
				StartInfo.Arguments += " -k";
			}
			if (bPrioritizeDependencyScanning)
			{
				StartInfo.Arguments += " --prioritize-dependency-scanning";
			}
			return ExecuteProcessWithProgressMarkup(StartInfo, Actions.Count(), Logger);
		}

		/// <summary>
		/// Executes the process, parsing progress markup as part of the output.
		/// </summary>
		private bool ExecuteProcessWithProgressMarkup(ProcessStartInfo SnDbsStartInfo, int NumActions, ILogger Logger)
		{
			using (ProgressWriter Writer = new ProgressWriter("Compiling C++ source files...", false, Logger))
			{
				int NumCompletedActions = 0;
				string CurrentStatus = "";

				// Create a wrapper delegate that will parse the output actions
				DataReceivedEventHandler EventHandlerWrapper = (Sender, Args) =>
				{
					if (Args.Data != null)
					{
						string Text = Args.Data;
						if (Text.StartsWith(ProgressMarkupPrefix, StringComparison.Ordinal))
						{
							Writer.Write(++NumCompletedActions, NumActions);

							Text = Args.Data.Substring(ProgressMarkupPrefix.Length).Trim();
							string[] ActionInfo = Text.Split(':');
							Logger.LogInformation("[{NumCompletedActions}/{NumActions}] {ActionInfo0} {ActionInfo1}", NumCompletedActions, NumActions, ActionInfo[0], ActionInfo[1]);
							CurrentStatus = ActionInfo[1];
							return;
						}
						// Suppress redundant tool output of status we already printed (e.g., msvc cl prints compile unit name always)
						if (!Text.Equals(CurrentStatus, StringComparison.Ordinal))
						{
							WriteToolOutput(Text);
						}
					}
				};

				try
				{
					// Start the process, redirecting stdout/stderr if requested.
					DateTime startTimeUTC = DateTime.UtcNow;
					Process LocalProcess = new Process();
					LocalProcess.StartInfo = SnDbsStartInfo;
					SnDbsStartInfo.RedirectStandardError = true;
					SnDbsStartInfo.RedirectStandardOutput = true;
					LocalProcess.EnableRaisingEvents = true;
					LocalProcess.OutputDataReceived += EventHandlerWrapper;
					LocalProcess.ErrorDataReceived += EventHandlerWrapper;

					LocalProcess.Start();

					LocalProcess.BeginOutputReadLine();
					LocalProcess.BeginErrorReadLine();

					Logger.LogInformation("Distributing {NumAction} action{ActionS} to SN-DBS",
						NumActions,
						NumActions == 1 ? "" : "s");

					// Wait until the process is finished and return whether it all the tasks successfully executed.
					LocalProcess.WaitForExit();
					bool result = LocalProcess.ExitCode == 0;

					telemetryEvent = new TelemetryExecutorEvent(Name, startTimeUTC, result, NumActions, -1, -1, 0, 0, DateTime.UtcNow, 0, 0);
					return result;
				}
				catch (Exception Ex)
				{
					Log.WriteException(Ex, null);
					return false;
				}
			}
		}

		private void PrepareToolTemplates(bool bUsePerJobToolTemplates)
		{
			// Get distinct platforms being used in this build.
			List<UnrealTargetPlatform> TargetPlatforms = TargetDescriptors
				.Select(TargetDescriptor => TargetDescriptor.Platform)
				.Distinct()
				.ToList();

			if (bUsePerJobToolTemplates)
			{
				// Write templates to a per-platform directory. Each job will refer to the directory of the associated
				// platform. This allows different platforms to have different templates for a tool of the same name.
				foreach (UnrealTargetPlatform TargetPlatform in TargetPlatforms)
				{
					UEBuildPlatform? BuildPlatform = UEBuildPlatform.GetBuildPlatform(TargetPlatform);

					if (BuildPlatform == null)
					{
						continue;
					}

					// Create the directory even if there are no templates to avoid an error when the JSON references a
					// non-existant directory.
					DirectoryReference PlatformTemplateDir = DirectoryReference.Combine(IntermediateDir, BuildPlatform.Platform.ToString());
					DirectoryReference.CreateDirectory(PlatformTemplateDir);

					// Not all platforms specify their own templates
					IReadOnlyDictionary<string, string>? PlatformTemplates = BuildPlatform.GetSNDBSToolTemplates();

					if (PlatformTemplates == null)
					{
						continue;
					}

					WriteToolTemplates(PlatformTemplates, PlatformTemplateDir);
				}

				// Global templates are always written to the directory passed to 'dbsbuild -templates'. This gets
				// searched last, ensuring the per-job directories always override it.
				WriteToolTemplates(GlobalTemplates, IntermediateDir);
			}
			else
			{
				// Older versions of SN-DBS have no way to disambiguate templates for a tool of the same name. This is
				// only a problem when building multiple platforms at the same time that are using a different template
				// for a tool of the same name. To provide a smoother upgrade path, we support using older versions of
				// SN-DBS provided this scenario is not encountered.
				//
				// For example, if platforms A and B both have a tool named "compiler.exe" requiring a different
				// template, that will produce a build error indicating an upgrade is required. However, compiling A by
				// itself would work fine as would any other combination of platforms that did not include both A and B.
				//
				// If encountering this error, the recommended resolution is to upgrade SN-DBS to 2.6.10.1020 or later.

				Dictionary<string, string> MergedTemplates = new Dictionary<string, string>();

				foreach (UnrealTargetPlatform TargetPlatform in TargetPlatforms)
				{
					UEBuildPlatform? BuildPlatform = UEBuildPlatform.GetBuildPlatform(TargetPlatform);

					if (BuildPlatform == null)
					{
						continue;
					}

					IReadOnlyDictionary<string, string>? PlatformTemplates = BuildPlatform.GetSNDBSToolTemplates();

					if (PlatformTemplates == null)
					{
						continue;
					}

					// Merge all platform templates into a single set. Two platforms may specify an identical template
					// for the same tool.
					foreach (KeyValuePair<string, string> Template in PlatformTemplates)
					{
						string? PrevValue;
						if (MergedTemplates.TryGetValue(Template.Key, out PrevValue))
						{
							if (PrevValue != null && !PrevValue.Equals(Template.Value, StringComparison.Ordinal))
							{
								throw new BuildException($"Conflicting SN-DBS tool template. {BuildPlatform} specifies a template for \"{Template.Key}\", but another platform is using a different template for the same tool. Upgrade SN-DBS to version 2.6.10.1020 or later to resolve this problem.");
							}
						}
						else
						{
							MergedTemplates.Add(Template.Key, Template.Value);
						}
					}
				}

				// Now merge in any manually added templates
				foreach (KeyValuePair<string, string> Template in GlobalTemplates)
				{
					string? PrevValue;
					if (MergedTemplates.TryGetValue(Template.Key, out PrevValue))
					{
						throw new BuildException($"Conflicting SN-DBS tool templates. Template for \"{Template.Key}\" added via AddTemplate() conflicts with a platform-provided template.");
					}
					else
					{
						MergedTemplates.Add(Template.Key, Template.Value);
					}
				}

				WriteToolTemplates(MergedTemplates, IntermediateDir);
			}
		}

		private static void WriteToolTemplates(IReadOnlyDictionary<string, string> Templates, DirectoryReference Directory)
		{
			foreach (KeyValuePair<string, string> Template in Templates)
			{
				FileReference TemplateFile = FileReference.Combine(Directory, $"{Template.Key}.sn-dbs-tool.ini");
				string TemplateText = Template.Value;

				foreach (Nullable<DictionaryEntry> Variable in Environment.GetEnvironmentVariables(EnvironmentVariableTarget.Process))
				{
					if (Variable.HasValue)
					{
						TemplateText = TemplateText.Replace($"{{{Variable.Value.Key}}}", Variable.Value.Value!.ToString(), StringComparison.Ordinal);
					}
				}

				File.WriteAllText(TemplateFile.FullName, TemplateText);
			}
		}

		private bool GenerateSNDBSIncludeRewriteRules()
		{
			// Get all distinct platform names being used in this build.
			List<string> Platforms = TargetDescriptors
				.Select(TargetDescriptor => UEBuildPlatform.GetBuildPlatform(TargetDescriptor.Platform).GetPlatformName())
				.Distinct()
				.ToList();

			if (Platforms.Count > 0)
			{
				// language=regex
				string[] Lines = new[]
				{
					@"pattern1=^COMPILED_PLATFORM_HEADER\(\s*([^ ,]+)\s*\)",
					$"expansions1={String.Join("|", Platforms.Select(Name => $"{Name}/{Name}$1|{Name}$1"))}",

					@"pattern2=^COMPILED_PLATFORM_HEADER_GENERATED\(\s*([^ ,]+)\s*\)",
					$"expansions2={String.Join("|", Platforms.Select(Name => $"{Name}$1"))}",

					@"pattern3=^COMPILED_PLATFORM_HEADER_WITH_PREFIX\(\s*([^ ,]+)\s*,\s*([^ ,]+)\s*\)",
					$"expansions3={String.Join("|", Platforms.Select(Name => $"$1/{Name}/{Name}$2|$1/{Name}$2"))}",

					@"pattern4=^[A-Z]{5}_PLATFORM_HEADER_NAME\(\s*([^ ,]+)\s*\)",
					$"expansions4={String.Join("|", Platforms.Select(Name => $"{Name}/{Name}$1|{Name}$1"))}",

					@"pattern5=^[A-Z]{5}_PLATFORM_HEADER_NAME_WITH_PREFIX\(\s*([^ ,]+)\s*,\s*([^ ,]+)\s*\)",
					$"expansions5={String.Join("|", Platforms.Select(Name => $"$1/{Name}/{Name}$2|$1/{Name}$2"))}"
				};

				File.WriteAllText(IncludeRewriteRulesFile.FullName, String.Join(Environment.NewLine, new[] { "[computed-include-rules]" }.Concat(Lines)));
				return true;
			}
			else
			{
				return false;
			}
		}
	}
}
