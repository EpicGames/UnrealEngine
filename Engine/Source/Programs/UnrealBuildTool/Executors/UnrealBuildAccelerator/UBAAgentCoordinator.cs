// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.UBA;

namespace UnrealBuildTool
{
	// statusRow, statusColumn, statusText, statusType, statusLink
	using StatusUpdateAction = Action<uint, uint, string, LogEntryType, string?>;

	interface IUbaAgentCoordinatorScheduler
	{
		bool IsEmpty { get; }
		void GetProcessWeightThatCanRunRemotelyNow(out double totalWeight, out double crossArchitecture, out double crossPlatform);
		bool AddClient(string ip, int port, string crypto = "");
		bool DisableRemoteExecutionOnAgent(string agentName);
	}

	interface IUBAAgentCoordinator
	{
		DirectoryReference? GetUBARootDir();

		Task InitAsync(UBAExecutor executor, StatusUpdateAction updateStatus, CancellationToken cancellationToken);

		UnrealBuildAcceleratorCacheConfig? RequestCacheServer(CancellationToken cancellationToken);

		void Start(IUbaAgentCoordinatorScheduler scheduler, Func<LinkedAction, bool> canRunRemotely);

		void Stop();

		Task CloseAsync();

		void Done();
	}
}