// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Runtime.InteropServices;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Horde.Compute;
using EpicGames.Horde.Compute.Clients;
using EpicGames.Horde.Server;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	// statusRow, statusColumn, statusText, statusType, statusLink
	using StatusUpdateAction = Action<uint, uint, string, EpicGames.UBA.LogEntryType, string?>;

	internal class UBAAgentCoordinatorHorde : IUBAAgentCoordinator, IDisposable
	{
		public static string ProviderPrefix = "Uba.Provider.Horde";

		public static UnrealBuildAcceleratorHordeConfig CreateHordeConfig(string provider, CommandLineArguments? additionalArguments = null, DirectoryReference? projectDir = null)
		{
			UnrealBuildAcceleratorHordeConfig config = new();

			ConfigHierarchy engineIni = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, projectDir, BuildHostPlatform.Current.Platform);
			config.LoadConfigProvider(engineIni, provider);

			if (provider == ProviderPrefix)
			{
				XmlConfig.ApplyTo(config);
			}

			if (Unreal.IsBuildMachine())
			{
				string? hordeUrl = System.Environment.GetEnvironmentVariable("UE_HORDE_URL");
				if (!String.IsNullOrEmpty(hordeUrl))
				{
					config.HordeServer = hordeUrl;
				}
			}

			if (config.HordeEnabled?.Equals("False", StringComparison.OrdinalIgnoreCase) == true || (config.HordeEnabled?.Equals("BuildMachineOnly", StringComparison.OrdinalIgnoreCase) == true && !Unreal.IsBuildMachine()))
			{
				config.bDisableHorde = true;
			}

			additionalArguments?.ApplyTo(config);

			// Sentry is currently unsupported for non-Windows and non-x64
			if (!OperatingSystem.IsWindows() || RuntimeInformation.ProcessArchitecture != Architecture.X64)
			{
				config.UBASentryUrl = null;
			}

			// Normalize URLs
#pragma warning disable CA1308 // Normalize strings to uppercase - URLs are conventionally lowercased
			config.HordeServer = config.HordeServer?.TrimEnd('/').ToLowerInvariant();
			config.UBASentryUrl = config.UBASentryUrl?.TrimEnd('/').ToLowerInvariant();
#pragma warning restore CA1308

			return config;
		}

		public static IEnumerable<IUBAAgentCoordinator> Init(ILogger logger, IEnumerable<TargetDescriptor> targetDescriptors, UnrealBuildAcceleratorConfig ubaConfig, ref string remoteConnectionMode)
		{
			IEnumerable<string> providers = Unreal.IsBuildMachine()
				? [.. ubaConfig.BuildMachineProviders, .. ubaConfig.IniBuildMachineProviders]
				: [.. ubaConfig.Providers, .. ubaConfig.IniProviders];
			providers = providers.Distinct().Where(x => !String.IsNullOrEmpty(x) && x.StartsWith(ProviderPrefix, StringComparison.Ordinal));

			if (!providers.Any())
			{
				providers = [ProviderPrefix];
			}

			// We group configs based on hordeserver/token to be able to have only one connection for configs having matching server/token.

			List<IGrouping<string, UnrealBuildAcceleratorHordeConfig>> configGroups = [.. providers
				.SelectMany(provider => targetDescriptors.Select(targetDescriptor => CreateHordeConfig(provider, targetDescriptor.AdditionalArguments, targetDescriptor.ProjectFile?.Directory)))
				.Where(x => !x.bDisableHorde && !String.IsNullOrWhiteSpace(x.HordeServer))
				.DistinctBy(x => HashCode.Combine(x.HordeServer, x.HordeCondition, x.HordeCluster, x.HordePool, x.HordeConnectionMode, x.AgentPlatform))
				.GroupBy(x => String.Join(";", x.HordeServer, x.HordeToken))];

			List<UBAAgentCoordinatorHorde> coordinators = [];
			foreach (IGrouping<string, UnrealBuildAcceleratorHordeConfig> configGroup in configGroups)
			{
				coordinators.Add(new UBAAgentCoordinatorHorde(logger, ubaConfig, configGroup));
			}

			remoteConnectionMode = ConnectionMode.Direct.ToString();
			if (configGroups.Count > 0)
			{
				remoteConnectionMode = (Enum.TryParse(configGroups[0].First().HordeConnectionMode, true, out ConnectionMode cm) ? cm : ConnectionMode.Direct).ToString();
			}
			return coordinators;
		}

		public UBAAgentCoordinatorHorde(ILogger logger, UnrealBuildAcceleratorConfig ubaConfig, IEnumerable<UnrealBuildAcceleratorHordeConfig> hordeConfigs)
		{
			_logger = logger;
			_title = "Horde";
			_ubaConfig = ubaConfig;
			_hordeConfigs = hordeConfigs.ToArray();
			_timers = new Timer?[_hordeConfigs.Length];
		}

		public DirectoryReference? GetUBARootDir()
		{
			DirectoryReference? hordeSharedDir = DirectoryReference.FromString(Environment.GetEnvironmentVariable("UE_HORDE_SHARED_DIR"));
			if (hordeSharedDir != null)
			{
				return DirectoryReference.Combine(hordeSharedDir, "UbaHost");
			}
			return null;
		}

		public async Task InitAsync(UBAExecutor executor, StatusUpdateAction updateStatus, CancellationToken cancellationToken)
		{
			_updateStatus = updateStatus;
			_cancellationToken = cancellationToken;

			try
			{
				if (_ubaConfig.bDisableRemote)
				{
					return;
				}

				// All configs have the same server/token
				UnrealBuildAcceleratorHordeConfig firstConfig = _hordeConfigs.First()!;
				Uri? serverUrl = (firstConfig.HordeServer == null) ? null : new Uri(firstConfig.HordeServer);
				string? accessToken = firstConfig.HordeToken;

				ServiceCollection services = new();
				services.AddLogging(builder =>
				{
					builder.AddEpicDefault();
					if (firstConfig.VerboseAuthLogging)
					{
						builder.AddFilter("EpicGames.OIDC", LogLevel.Debug);
						builder.AddFilter("EpicGames.Horde.Auth", LogLevel.Debug);
					}
				});
				services.AddHorde(options =>
				{
					options.ServerUrl = serverUrl;
					options.AccessToken = accessToken;
				});
				_serviceProvider = services.BuildServiceProvider();

				IHordeClient hordeClient;
				using (ITimelineEvent _ = Timeline.ScopeEvent($"HordeGetService"))
				{
					hordeClient = _serviceProvider.GetRequiredService<IHordeClient>();
				}

				IComputeClient computeClient = hordeClient.Compute;
				using (HordeHttpClient httpClient = hordeClient.CreateHttpClient())
				{
					using ITimelineEvent _ = Timeline.ScopeEvent($"HordeConnect");
					GetServerInfoResponse serverInfo = await httpClient.GetServerInfoAsync(cancellationToken);
					_logger.LogInformation("Horde server: {ServerVersion}, agent: {AgentVersion}", serverInfo.ServerVersion, serverInfo.AgentVersion);
				}

				List<Task<(UBAHordeSession?, string)>> sessionTasks = [];

				for (uint i = 0; i != _hordeConfigs.Length; ++i)
				{
					UnrealBuildAcceleratorHordeConfig hordeConfig = _hordeConfigs[i];
					string? link = null;
					if (hordeConfig.HordeServer != null)
					{
						link = $"{hordeConfig.HordeServer!.TrimEnd('/')}/agents";
						if (!String.IsNullOrEmpty(hordeConfig.HordePool))
						{
							link += $"?agent={hordeConfig.HordePool}";
						}
					}
					_updateStatus(i, 1, _title, EpicGames.UBA.LogEntryType.Info, link);
					_updateStatus!(i, 6, "Connecting", EpicGames.UBA.LogEntryType.Info, null);

					sessionTasks.Add(UBAHordeSession.TryCreateHordeSessionAsync(_ubaConfig, hordeConfig, computeClient, executor.Crypto, _ubaConfig.bStrict, _logger, cancellationToken));
				}
				_hordeSessionTasks = sessionTasks.ToArray();

				for (uint i = 0; i != _hordeSessionTasks.Length; ++i)
				{
					(UBAHordeSession? session, string status) = await _hordeSessionTasks[i];
					_updateStatus!(i, 6, status, EpicGames.UBA.LogEntryType.Info, null);
					executor.AgentCoordinatorInitialized(_hordeSessionTasks[i].IsCompletedSuccessfully && session is not null);
				}
			}
			finally
			{
				_initDoneEvent.Set();
			}
			lock (_startLock)
			{
				_initDone = true;
				if (_scheduler != null)
				{
					StartTimer();
				}
			}
		}

		public UnrealBuildAcceleratorCacheConfig? RequestCacheServer(CancellationToken cancellationToken)
		{
			for (uint i = 0; i != _hordeConfigs.Length; ++i)
			{
				UnrealBuildAcceleratorHordeConfig hordeConfig = _hordeConfigs[i];
				if (!hordeConfig.bCacheEnabled)
				{
					continue;
				}

				try
				{
					_initDoneEvent.Wait(cancellationToken);
				}
				catch (OperationCanceledException)
				{
					return null;
				}
				if (i >= _hordeSessionTasks.Length)
				{
					return null;
				}

				(UBAHordeSession? hordeSession, string reason) = _hordeSessionTasks[i].Result;
				_updateStatus!(0, 6, reason, EpicGames.UBA.LogEntryType.Info, null);
				if (hordeSession == null)
				{
					continue;
				}

				return hordeSession.RequestCacheServer(hordeConfig.CacheDesiredConnectionCount, cancellationToken);
			}
			return null;
		}

		public void Start(IUbaAgentCoordinatorScheduler scheduler, Func<LinkedAction, bool> canRunRemotely)
		{
			lock (_startLock)
			{
				_scheduler = scheduler;
				if (_initDone)
				{
					StartTimer();
				}
			}
		}
		private void StartTimer()
		{
			for (uint i2 = 0; i2 != _hordeConfigs.Length; ++i2)
			{
				uint i = i2;
				UnrealBuildAcceleratorHordeConfig hordeConfig = _hordeConfigs[i];

				if (!hordeConfig.bAgentsEnabled)
				{
					continue;
				}

				int timerPeriod = 5000;
				bool shownNoAgentsFoundMessage = false;

				(UBAHordeSession? hordeSession, string reason) = _hordeSessionTasks[i].Result;

				if (hordeSession == null)
				{
					if (_updateStatus != null)
					{
						_updateStatus(0, 6, reason, EpicGames.UBA.LogEntryType.Info, null);
					}
					continue;
				}

				hordeSession._scheduler = _scheduler;
				hordeSession._updateStatus = (sr, sc, st, t, sl) => _updateStatus!(sr + i, sc, st, t, sl);
				hordeSession.UpdateHordeStatus(null);

				int requestCounter = 0;
				Timer timer = new(async (_) =>
				{
					_timers[i]?.Change(Timeout.Infinite, Timeout.Infinite);

					if (_hordeSessionTasks[i] == null)
					{
						return;
					}

					(UBAHordeSession? hordeSession, string? reason) = await _hordeSessionTasks[i];

					if (hordeSession == null)
					{
						return;
					}

					if (_scheduler!.IsEmpty || _cancellationToken.IsCancellationRequested)
					{
						hordeSession.UpdateHordeStatus(" - Requests stopped");
						return;
					}

					// We are assuming all active logical cores are already being used.. so queueWeight is essentially work that could be executed but can't because of bandwidth
					double queueThreshold = _ubaConfig.bForceBuildAllRemote ? 0 : 5;
					LogLevel logLevel = _ubaConfig.bStrict ? LogLevel.Warning : LogLevel.Information;

					try
					{
						while (true)
						{
							if (_cancellationToken.IsCancellationRequested)
							{
								hordeSession.UpdateHordeStatus(" - Requests stopped");
								return;
							}

							hordeSession.RemoveCompleteWorkersAsync();

							double totalWeight;
							double crossArchitecture;
							double crossPlatform;
							_scheduler.GetProcessWeightThatCanRunRemotelyNow(out totalWeight, out crossArchitecture, out crossPlatform);

							double queueWeight = totalWeight;

							// Hacky, needs to be fixed up
							if (UBAExecutor.GetHostPlatform() == OSPlatform.OSX && hordeSession.WorkerPlatform == OSPlatform.Linux)
							{
								queueWeight = crossPlatform;
							}

							int activeCores;
							int queuedCores;
							int activeAgents;
							int queuedAgents;
							hordeSession.GetCoreCount(out activeCores, out queuedCores, out activeAgents, out queuedAgents);

							queueWeight -= queuedCores;

							bool isSatisfied = queueWeight <= queueThreshold || (activeCores + queuedCores) >= hordeConfig.HordeMaxCores || _cancellationToken.IsCancellationRequested;

							hordeSession.UpdateHordeStatus(isSatisfied ? " - Requests paused" : $" - Requesting agent{("...."[..(requestCounter++ % 4)])}");

							if (isSatisfied)
							{
								break;
							}

							if (!await hordeSession.AddWorkerAsync(activeCores, _cancellationToken))
							{
								await Task.Delay(500); // Sleep a little bit to make sure previous horde status is visible
								string status = (queuedAgents + activeAgents) == 0 ? " - No agents available" : " - No additional agents available";
								hordeSession.UpdateHordeStatus(status);
								_logger.LogDebug("{HordeStatus}", status);
								break;
							}
						}
					}
					catch (NoComputeAgentsFoundException ex)
					{
						if (!shownNoAgentsFoundMessage)
						{
							_logger.Log(logLevel, KnownLogEvents.Systemic_Horde_Compute, ex, "No agents found matching requirements (cluster: {ClusterId}, requirements: {Requirements})", ex.ClusterId, ex.Requirements);
							shownNoAgentsFoundMessage = true;
						}
					}
					catch (ComputeClientException cce)
					{
						_logger.Log(logLevel, KnownLogEvents.Systemic_Horde_Compute, cce, "{ErrorMessage}", cce.Message);
					}
					catch (Exception ex)
					{
						if (!_cancellationToken.IsCancellationRequested)
						{
							hordeSession.UpdateHordeStatus(" - " + ex.Message);
							_logger.Log(logLevel, KnownLogEvents.Systemic_Horde_Compute, ex, "Unable to get worker: {Ex}", ex.Message);
						}
					}

					_timers[i]?.Change(timerPeriod, Timeout.Infinite);
				}, null, Timeout.Infinite, Timeout.Infinite);

				_timers[i] = timer;
				timer.Change(hordeConfig.HordeDelay * 1000, timerPeriod);
			}
		}

		public void Stop()
		{
			// Check for whether we can be cancelled, as if remote execution is disabled, then we'll have a default CancellationToken.
			if (!_cancellationToken.IsCancellationRequested && _cancellationToken.CanBeCanceled)
			{
				_logger.LogWarning("Cancellation should have been set");
			}
		}

		public async Task CloseAsync()
		{
			// Check for whether we can be cancelled, as if remote execution is disabled, then we'll have a default CancellationToken.
			if (!_cancellationToken.IsCancellationRequested && _cancellationToken.CanBeCanceled)
			{
				_logger.LogWarning("Cancellation should have been set");
			}

			for (uint i = 0; i != _hordeSessionTasks.Length; ++i)
			{
				if (_hordeSessionTasks[i] == null)
				{
					return;
				}

				(UBAHordeSession? hordeSession, _) = await _hordeSessionTasks[i];
				_hordeSessionTasks[i] = Task.FromResult(((UBAHordeSession?)null, ""));
				if (hordeSession != null)
				{
					await hordeSession.DisposeAsync();
				}
			}

			if (_serviceProvider != null)
			{
				await _serviceProvider.DisposeAsync();
			}
		}

		public void Done()
		{
			if (_updateStatus != null)
			{
				for (uint i = 0; i != _hordeSessionTasks?.Length; ++i)
				{
					_updateStatus(i, 6, "Done", EpicGames.UBA.LogEntryType.Info, null);
				}
			}
		}

		public void Dispose()
		{
			Stop();
			CloseAsync().Wait();
			Dispose(true);
			GC.SuppressFinalize(this);
		}

		protected virtual void Dispose(bool disposing)
		{
			if (disposing)
			{
				_initDoneEvent.Dispose();

				for (uint i = 0; i != _timers.Length; ++i)
				{
					_timers[i]?.Dispose();
					_timers[i] = null;
				}
			}
		}
		
		readonly ILogger _logger;
		readonly string _title;
		readonly UnrealBuildAcceleratorConfig _ubaConfig;
		readonly UnrealBuildAcceleratorHordeConfig[] _hordeConfigs;
		Task<(UBAHordeSession?, string)>[] _hordeSessionTasks = [];
		readonly Timer?[] _timers = [];
		readonly ManualResetEventSlim _initDoneEvent = new();

		bool _initDone;
		readonly Lock _startLock = new();
		IUbaAgentCoordinatorScheduler? _scheduler;

		ServiceProvider? _serviceProvider;
		CancellationToken _cancellationToken;
		StatusUpdateAction? _updateStatus;
	}
}
