// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Management;
using System.Runtime.InteropServices;
using System.Runtime.Versioning;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.UBA;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;
using UnrealBuildTool.Actions.ResultStore;
using UnrealBuildTool.Storage.Impl;
using ILogger = Microsoft.Extensions.Logging.ILogger;

namespace UnrealBuildTool
{
	class UBAExecutor : ActionExecutor
	{
		public UnrealBuildAcceleratorConfig UBAConfig { get; init; }

		public string Crypto { get; private set; } = String.Empty;

		readonly IEnumerable<TargetDescriptor> _targetDescriptors;
		readonly Task _initTask;
		readonly CancellationTokenSource _cancellationSource;
		readonly bool _bCompactOutput;

		IConfig? _config;
		IServer? _server;
		IStorageServer? _storageServer;
		ISessionServer? _session;
#pragma warning disable CA2213 // Disposable fields should be disposed - _trace can be a global trace sometimes, so not safe to dispose.
		ITrace? _trace;
#pragma warning restore CA2213
		string? _sessionHyperlink;

		readonly Lock _sessionLock = new();

		readonly Dictionary<UnrealBuildAcceleratorCacheConfig, ICacheClient> _cacheClients = [];
		readonly List<IUBAAgentCoordinator> _agentCoordinators = new();
		DirectoryReference? _rootDirRef;

		DateTime _ubaStartTimeUtc = DateTime.UtcNow;
		TimeSpan _ubaDurationWaitingForRemote = TimeSpan.Zero;

		// Tracking for successful coordinator connections
		int _successfulCoordinatorConnections = 0;
		// Tracking for failed coordinator connections
		int _failedCoordinatorConnections = 0;

		int _cacheableActionsLeft = 0;

		// Tracking for remote connection mode
		string _remoteConnectionMode = "Local";

		/// <summary>
		/// Telemetry event for this executor
		/// </summary>
		private TelemetryExecutorEvent? telemetryEvent;

		protected override void Dispose(bool disposing)
		{
			if (disposing)
			{
				_initTask.Wait();

				{
					using ITimelineEvent a2 = Timeline.ScopeEvent("DisconnectCache");
					DisposeCacheClients();
				}

				_session?.Dispose();

				{
					using ITimelineEvent a4 = Timeline.ScopeEvent("SaveCasTable");
					_storageServer?.Dispose();
				}

				_config?.Dispose();
				_server?.Dispose();

				_cancellationSource.Dispose();
			}
			base.Dispose(disposing);
		}

		void DisposeCacheClients()
		{
			if (_cacheClients.Count > 0)
			{
				foreach ((_, ICacheClient client) in _cacheClients)
				{
					client.Dispose();
				}
				_cacheClients.Clear();
			}
		}

		internal const string ExecutorName = "UBA";

		public override string Name => !UBAConfig.bDisableRemote && _agentCoordinators.Count > 0 ? "Unreal Build Accelerator" : "Unreal Build Accelerator local";

		public int NumParallelProcesses { get; }

		public static bool IsAvailable() => EpicGames.UBA.Utils.IsAvailable();

		public delegate void SessionCreatedDelegate(ISessionServer Session);

		// Event invoked when ISessionServer is created. Can be used to for example register virtual files etc
		public event SessionCreatedDelegate? OnSessionCreated;

		public static OSPlatform GetHostPlatform()
		{
			if (OperatingSystem.IsLinux())
			{
				return OSPlatform.Linux;
			}
			if (OperatingSystem.IsWindows())
			{
				return OSPlatform.Windows;
			}
			return OSPlatform.OSX;
		}

		public static string GetPlatformName(OSPlatform platform)
		{
			if (platform == OSPlatform.Windows)
			{
				return "Win64";
			}
			if (platform == OSPlatform.Linux)
			{
				return "Linux";
			}
			if (platform == OSPlatform.OSX)
			{
				return "Mac";
			}
			throw new PlatformNotSupportedException();
		}

		public static DirectoryReference GetUbaBinariesDir(OSPlatform platform)
		{
			if (platform == OSPlatform.Windows)
			{
				#pragma warning disable CA1308 // Normalize strings to uppercase
				return DirectoryReference.Combine(Unreal.EngineDirectory, "Binaries", "Win64", "UnrealBuildAccelerator", RuntimeInformation.ProcessArchitecture.ToString().ToLowerInvariant());
				#pragma warning restore CA1308 // Normalize strings to uppercase
			}
			else if (platform == OSPlatform.Linux)
			{
				//if (RuntimeInformation.ProcessArchitecture == Architecture.X64)
				{
					return DirectoryReference.Combine(Unreal.EngineDirectory, "Binaries", "Linux", "UnrealBuildAccelerator");
				}
				//else if (RuntimeInformation.ProcessArchitecture == Architecture.Arm64)
				//{
				//	return DirectoryReference.Combine(Unreal.EngineDirectory, "Binaries", "LinuxArm64", "UnrealBuildAccelerator");
				//}
			}
			else if (platform == OSPlatform.OSX)
			{
				return DirectoryReference.Combine(Unreal.EngineDirectory, "Binaries", "Mac", "UnrealBuildAccelerator");
			}
			throw new PlatformNotSupportedException();
		}

		internal static UnrealBuildAcceleratorConfig GetAppliedConfig(IEnumerable<TargetDescriptor> targetDescriptors)
		{
			UnrealBuildAcceleratorConfig config = new();
			XmlConfig.ApplyTo(config);

			foreach (TargetDescriptor targetDescriptor in targetDescriptors)
			{
				ConfigCache.ReadSettings(targetDescriptor.ProjectFile?.Directory, BuildHostPlatform.Current.Platform, config);
				targetDescriptor.AdditionalArguments.ApplyTo(config);
			}

			return config;
		}

		public UBAExecutor(UnrealBuildAcceleratorConfig config, int maxLocalActions, bool bAllCores, bool bCompactOutput, ILogger logger, IEnumerable<TargetDescriptor> targetDescriptors)
			: base(logger)
		{
			UBAConfig = config;

			// Figure out how many processors to use
			NumParallelProcesses = GetDefaultNumParallelProcesses(config, maxLocalActions, bAllCores, logger);

			_bCompactOutput = bCompactOutput;

			_targetDescriptors = targetDescriptors;
			_cancellationSource = new();
			_initTask = new(() => { Init(targetDescriptors, logger); }, TaskCreationOptions.LongRunning);
			_initTask.Start();
		}

		private void Init(IEnumerable<TargetDescriptor> targetDescriptors, ILogger logger)
		{
			XmlConfig.ApplyTo(this);
			CommandLine.ParseArguments(Environment.GetCommandLineArgs(), this, logger);

			foreach (TargetDescriptor targetDescriptor in targetDescriptors)
			{
				targetDescriptor.AdditionalArguments.ApplyTo(this);
			}

			_trace = ITrace.GlobalTrace;

			List<UnrealBuildAcceleratorCacheConfig> cacheConfigs = [];

			if (!UBAConfig.bForceNoCache)
			{
				// These are non-coordinator cache servers. Coordinators can also provide cache servers
				const string ProviderPrefix = "Uba.CacheProvider";

				IEnumerable<string> providers = Unreal.IsBuildMachine()
				? [.. UBAConfig.BuildMachineCacheProviders, .. UBAConfig.IniBuildMachineCacheProviders]
				: [.. UBAConfig.CacheProviders, .. UBAConfig.IniCacheProviders];

				if (!providers.Any())
				{
					providers = [ProviderPrefix];
				}
				foreach (string provider in providers.Distinct().Where(x => x.StartsWith(ProviderPrefix, StringComparison.Ordinal)))
				{
					foreach (TargetDescriptor targetDescriptor in targetDescriptors)
					{
						UnrealBuildAcceleratorCacheConfig cacheConfig = new();
						if (provider == ProviderPrefix)
						{
							XmlConfig.ApplyTo(cacheConfig);
							CommandLine.ParseArguments(Environment.GetCommandLineArgs(), this, logger);
						}
						ConfigHierarchy engineIni = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, targetDescriptor.ProjectFile?.Directory, BuildHostPlatform.Current.Platform);
						cacheConfig.LoadConfigProvider(engineIni, provider);
						targetDescriptor.AdditionalArguments.ApplyTo(cacheConfig);
						cacheConfigs.Add(cacheConfig);
					}
				}
				cacheConfigs = cacheConfigs.DistinctBy(x => x.CacheServer).Where(x => !String.IsNullOrEmpty(x.CacheServer)).ToList();
			}

			if (UBAConfig.bUseCrypto)
			{
				Crypto = CreateCrypto();
			}

			_agentCoordinators.AddRange(UBAAgentCoordinatorHorde.Init(logger, targetDescriptors, UBAConfig, ref _remoteConnectionMode));

			if (!String.IsNullOrEmpty(UBAConfig.RootDir))
			{
				_rootDirRef = DirectoryReference.FromString(UBAConfig.RootDir);
			}
			else
			{
				_rootDirRef = DirectoryReference.FromString(Environment.GetEnvironmentVariable("UBA_ROOT") ?? Environment.GetEnvironmentVariable("BOX_ROOT"));
			}

			if (_rootDirRef == null)
			{
				foreach (IUBAAgentCoordinator coordinator in _agentCoordinators)
				{
					_rootDirRef = _rootDirRef == null ? coordinator.GetUBARootDir() : _rootDirRef;
				}
			}

			if (_rootDirRef == null)
			{
				if (OperatingSystem.IsWindows())
				{
					_rootDirRef = DirectoryReference.Combine(DirectoryReference.GetSpecialFolder(Environment.SpecialFolder.CommonApplicationData)!, "Epic", "UnrealBuildAccelerator");
				}
				else
				{
					_rootDirRef = DirectoryReference.Combine(DirectoryReference.GetSpecialFolder(Environment.SpecialFolder.UserProfile)!, ".epic", "UnrealBuildAccelerator");
				}
			}

			List<Task> coordinatorInitTasks = new();
			if (!UBAConfig.bDisableRemote)
			{
				uint statusUpdateCounter = 10;
				foreach (IUBAAgentCoordinator coordinator in _agentCoordinators)
				{
					uint statusUpdateIndex = statusUpdateCounter;
					coordinatorInitTasks.Add(coordinator.InitAsync(this, (sr, sc, st, t, sl) => UpdateStatus(statusUpdateIndex + sr, sc, st, t, sl), _cancellationSource.Token));
					statusUpdateCounter += 10;
				}

				if (!UBAConfig.bForceNoCache)
				{
					foreach (IUBAAgentCoordinator coordinator in _agentCoordinators)
					{
						UnrealBuildAcceleratorCacheConfig? cacheConfig = coordinator.RequestCacheServer(_cancellationSource.Token);
						if (cacheConfig != null)
						{
							cacheConfigs.Add(cacheConfig);
						}
					}
				}
			}

			DirectoryReference.CreateDirectory(_rootDirRef);

			string? ubaTraceFile = ITrace.GlobalTrace?.Path;
			if (ubaTraceFile is null)
			{
				throw new Exception("UBA executor is not expected to be invoked from a recursive UBT call.");
			}

			DirectoryReference ubaBinariesDir = GetUbaBinariesDir(GetHostPlatform());

			FileReference configFile = FileReference.Combine(ubaBinariesDir, "UbaHost.toml");
			_config = EpicGames.UBA.IConfig.LoadConfig(configFile.FullName);
			if (!String.IsNullOrEmpty(UBAConfig.CompressionLevel))
			{
				_config.AddValue("Storage", "CompressionLevel", UBAConfig.CompressionLevel);
			}

			if (!String.IsNullOrEmpty(UBAConfig.SharedMemoryTempFile))
			{
				_config.AddValue("Storage", "SharedMemoryTempFile", UBAConfig.SharedMemoryTempFile);
			}

			if (!UBAConfig.bCleanupOnDispose)
			{
				_config.AddValue("Storage", "SkipCleanupOnShutdown", "true"); // For faster shutdown
			}

			if (UBAConfig.bWritePlaceholders)
			{
				_config.AddValue("Session", "WritePlaceholders", "true");
			}

			if (!String.IsNullOrEmpty(UBAConfig.SharedMemoryTempFile))
			{
				_config.AddValue("Storage", "SharedMemoryTempFile", UBAConfig.SharedMemoryTempFile);
			}

			if (!UBAConfig.bAllowDetour)
			{
				_config.AddValue("Session", "AllowMemoryMaps", "false");
			}

			// Register Apple cross-architecture paths BEFORE creating the session.
			// SessionServerImpl iterates Utils.CrossArchitecturePaths in its constructor,
			// so these paths must be registered before CreateSessionServer() is called.
			if (OperatingSystem.IsMacOS())
			{
				AppleToolChain.RegisterCrossArchitecturePathsForUBA(logger);
				_config.AddValue("Scheduler", "DeferRemoteCapableProcesses", "true");
			}
	
			if (UBAConfig.MaxRacingPercent > 0)
			{
				_config.AddValue("Scheduler", "MaxRacingPercent", UBAConfig.MaxRacingPercent.ToString());
			}

			EpicGames.UBA.ILogger ubaLogger = EpicGames.UBA.ILogger.GlobalLogger!;
			_server = IServer.CreateServer(UBAConfig.MaxWorkers, UBAConfig.SendSize, ubaLogger, UBAConfig.bUseQuic);
			_storageServer = IStorageServer.CreateStorageServer(_server, ubaLogger, new StorageServerCreateInfo(_rootDirRef.FullName, ((ulong)UBAConfig.StoreCapacityGb) * 1024 * 1024 * 1024, !UBAConfig.bStoreRaw, UBAConfig.Zone));
			using ISessionServerCreateInfo serverCreateInfo = ISessionServerCreateInfo.CreateSessionServerCreateInfo(_storageServer, _server, ubaLogger, new SessionServerCreateInfo(_rootDirRef.FullName, ubaTraceFile.Replace('\\', '/'), UBAConfig.bDisableCustomAlloc, false, UBAConfig.bResetCas, UBAConfig.bWriteToDisk, UBAConfig.bDetailedTrace, !UBAConfig.bDisableWaitOnMem, UBAConfig.bAllowKillOnMem, UBAConfig.bStoreObjFilesCompressed));
			_session = ISessionServer.CreateSessionServer(serverCreateInfo);
			_trace = _session.GetTrace();

			string? jobId = Environment.GetEnvironmentVariable("UE_HORDE_JOBID");
			if (jobId != null)
			{
				string hordeUrl = Environment.GetEnvironmentVariable("UE_HORDE_URL") ?? "";
				string stepId = Environment.GetEnvironmentVariable("UE_HORDE_STEPID") ?? "unknown";
				string streamId = Environment.GetEnvironmentVariable("UE_HORDE_STREAMID") ?? "unknown-stream";
				_sessionHyperlink = $"{hordeUrl}job/{jobId}?step={stepId} ({streamId})";

				_session.AddInfo(_sessionHyperlink);
			}

			Task? preloadTask = null;

			if (!UBAConfig.bDisableRemote)
			{
				preloadTask = Task.Run(() =>
				{
					using TraceTaskScope s = new(_trace, "LoadCasTable", "");
					_storageServer.PreloadCasTable();
				});
			}

			if (cacheConfigs.Count > 0)
			{
				_trace.UpdateStatus(1, 1, "Cache", LogEntryType.Info, null);
				_trace.UpdateStatus(1, 6, "Connecting...", LogEntryType.Info);

				using TraceTaskScope s = new(_trace, "CacheConnect", "");
				foreach (UnrealBuildAcceleratorCacheConfig cacheConfig in cacheConfigs)
				{
					ICacheClient cacheClient = ICacheClient.CreateCacheClient(_session, UBAConfig.bReportCacheMissReason, cacheConfig.Crypto, _sessionHyperlink ?? Environment.MachineName);
					string[] nameAndPort = cacheConfig.CacheServer.Split(':');
					int port = 1347;
					if (nameAndPort.Length > 1)
					{
						port = Int32.Parse(nameAndPort[1]);
					}

					System.Diagnostics.Stopwatch stopwatch = System.Diagnostics.Stopwatch.StartNew();
					bool cacheSuccess = cacheClient.Connect(nameAndPort[0], port, cacheConfig.DesiredConnectionCount);
					long totalMs = stopwatch.ElapsedMilliseconds;
					string successText = cacheSuccess ? "Connected to" : "Failed to connect to";
					logger.LogInformation("UbaCache - {SuccessText} {Name}:{Port}, CanWrite: {CanWrite}, RequireVFS: {Vfs}, Encrypted: {Encrypted} ({Seconds}.{Milliseconds}s)",
						successText, nameAndPort[0], port, cacheConfig.CanWrite, cacheConfig.bRequireVfs, !String.IsNullOrEmpty(cacheConfig.Crypto), totalMs / 1000, totalMs % 1000);
					if (cacheSuccess)
					{
						_cacheClients.Add(cacheConfig, cacheClient);
					}
					else
					{
						cacheClient.Dispose();
					}
				}
				_trace.UpdateStatus(1, 6, _cacheClients.Any() ? "Ready" : "Failed to connect", LogEntryType.Info);
			}

			preloadTask?.Wait();

			if (!UBAConfig.bDisableRemote)
			{
				using IConfig agentConfig = IConfig.LoadConfig(FileReference.Combine(ubaBinariesDir, "UbaAgent.toml").FullName);
				if (!UBAConfig.UseAgentStore)
				{
					agentConfig.AddValue("Session", "UseStorage", "false");
					agentConfig.AddValue("Storage", "WriteToDisk", "false");
				}
				_server.SetClientsConfig(agentConfig);

				_server.StartServer(UBAConfig.Host, UBAConfig.Port, Crypto);
			}
			else
			{
				_session.DisableRemoteExecution();
			}
		}

		internal static int GetDefaultNumParallelProcesses(UnrealBuildAcceleratorConfig config, int maxLocalActions, bool bAllCores, ILogger logger)
		{
			double MemoryPerActionBytesComputed = Math.Max(config.MemoryPerActionBytes, MemoryPerActionBytesOverride);
			if (MemoryPerActionBytesComputed > config.MemoryPerActionBytes)
			{
				logger.LogInformation("Overriding MemoryPerAction with target-defined value of {Memory} bytes", MemoryPerActionBytesComputed / 1024 / 1024 / 1024);
			}

			return Utils.GetMaxActionsToExecuteInParallel(maxLocalActions, bAllCores ? 1.0f : config.ProcessorCountMultiplier, bAllCores, Convert.ToInt64(MemoryPerActionBytesComputed));
		}

		public void UpdateStatus(uint statusRow, uint statusColumn, string statusText, LogEntryType statusType, string? statusLink)
		{
			_trace?.UpdateStatus(statusRow, statusColumn, statusText, statusType, statusLink);
		}

		private void PrintConfiguration(IEnumerable<LinkedAction> inputActions, ILogger logger)
		{
			long totalPhysicalBytes = SystemUtils.GetTotalSystemMemoryBytes();
			long totalCommittedMemory = SystemUtils.GetMaxCommittedMemoryBytes();
			long currentCommittedMemory = SystemUtils.GetCurrentCommittedMemoryBytes();

			logger.LogInformation("  CPU {Physical} physical cores, {Logical} logical cores", SystemUtils.GetPhysicalProcessorCount(), SystemUtils.GetLogicalProcessorCount());
			logger.LogInformation("  Memory {PhysicalTotal} physical, {CommittedCurrent}/{CommittedTotal} committed", StringUtils.FormatBytesString(totalPhysicalBytes), StringUtils.FormatBytesString(currentCommittedMemory), StringUtils.FormatBytesString(totalCommittedMemory));
			logger.LogInformation("  UBA Storage capacity {StoreCapacity}", StringUtils.FormatBytesString((ulong)UBAConfig.StoreCapacityGb * 1024 * 1024 * 1024));
			if (UBAConfig.bStoreObjFilesCompressed)
			{
				logger.LogInformation("  UBA Object Compression Allowed");
			}
			logger.LogDebug("  UBA RootDir {RootDir}", _rootDirRef);

			CaptureLogger recommendationLogger = new();
			if (_cacheableActionsLeft > 0 && !inputActions.Any(x => x.RootPaths.bUseVfs))
			{
				foreach (KeyValuePair<UnrealBuildAcceleratorCacheConfig, ICacheClient> pair in _cacheClients.Where(x => x.Key.bRequireVfs))
				{
					recommendationLogger.LogInformation("    Cache '{Server}' requires VFS but no actions found with VFS enabled. Please ensure TargetRules.bUseVFS is enabled.", pair.Key.CacheServer);
				}
			}

			if (recommendationLogger.Events.Count > 0)
			{
				logger.LogInformation("  Recommendations:");
				recommendationLogger.RenderTo(logger);
			}
		}

		private async Task WriteActionOutputFileAsync(IEnumerable<LinkedAction> inputActions, ILogger logger)
		{
			if (String.IsNullOrEmpty(UBAConfig.ActionsOutputFile))
			{
				return;
			}

			if (!UBAConfig.ActionsOutputFile.EndsWith(".yaml", StringComparison.OrdinalIgnoreCase))
			{
				logger.LogError("UBA actions output file needs to have extension .yaml for UbaCli to understand it");
			}
			using System.IO.StreamWriter streamWriter = new(UBAConfig.ActionsOutputFile);
			using System.CodeDom.Compiler.IndentedTextWriter writer = new(streamWriter, "  ");
			await writer.WriteAsync("environment: ");
			await writer.WriteLineAsync(Environment.GetEnvironmentVariable("PATH"));
			await writer.WriteLineAsync("processes:");
			writer.Indent++;
			int index = 0;
			foreach (LinkedAction action in inputActions)
			{
				action.SortIndex = index++;
				await writer.WriteLineAsync($"- id: {action.SortIndex}");
				writer.Indent++;
				await writer.WriteLineAsync($"app: {action.CommandPath}");
				await writer.WriteLineAsync($"arg: {action.CommandArguments}");
				await writer.WriteLineAsync($"dir: {action.WorkingDirectory}");
				await writer.WriteLineAsync($"desc: {action.StatusDescription}");
				// TODO: Add cache roots
				if (action.Weight != 1.0f)
				{
					await writer.WriteLineAsync($"weight: {action.Weight}");
				}
				if (!action.bCanExecuteInUBA)
				{
					await writer.WriteLineAsync("detour: false");
				}
				else if (!action.bCanExecuteRemotely)
				{
					await writer.WriteLineAsync("remote: false");
				}
				uint bucket = GetCacheBucket(logger, action);
				if (bucket != 0)
				{
					await writer.WriteLineAsync($"cache: {bucket}");
				}
				uint memoryGroup = GetMemoryGroup(action);
				if (memoryGroup != 0)
				{
					await writer.WriteLineAsync($"memgroup: {memoryGroup}");
				}
				if (action.PrerequisiteActions.Any())
				{
					await writer.WriteAsync("dep: [");
					await writer.WriteAsync(String.Join(", ", action.PrerequisiteActions.Select(x => x.SortIndex)));
					await writer.WriteLineAsync("]");
				}
				if (action.RootPaths.Any())
				{
					await writer.WriteAsync("vfs: [");
					bool isFirst = true;
					foreach ((uint id, DirectoryReference vfs, DirectoryReference local) in action.RootPaths)
					{
						if (!isFirst)
						{
							await writer.WriteAsync(", ");
						}

						isFirst = false;
						await writer.WriteAsync(vfs.FullName);
						await writer.WriteAsync(';');
						await writer.WriteAsync(local.FullName);
					}
					await writer.WriteLineAsync("]");
				}
				writer.Indent--;
				await writer.WriteLineNoTabsAsync(null);
			}
		}

		[System.Diagnostics.CodeAnalysis.SuppressMessage("Globalization", "CA1308:Normalize strings to uppercase", Justification = "lowercase crypto string")]
		public static string CreateCrypto()
		{
			byte[] bytes = new byte[16];
			using System.Security.Cryptography.RandomNumberGenerator random = System.Security.Cryptography.RandomNumberGenerator.Create();
			random.GetBytes(bytes);
			return BitConverter.ToString(bytes).Replace("-", "", StringComparison.OrdinalIgnoreCase).ToLowerInvariant(); // "1234567890abcdef1234567890abcdef";
		}

		public void AgentCoordinatorInitialized(bool successful)
		{
			if (successful)
			{
				Interlocked.Add(ref _successfulCoordinatorConnections, 1);
			}
			else
			{
				Interlocked.Add(ref _failedCoordinatorConnections, 1);
			}
		}

		/// <inheritdoc/>
		public override TelemetryExecutorEvent? GetTelemetryEvent() => telemetryEvent;

		public override async Task<bool> ExecuteActionsAsync(IEnumerable<LinkedAction> inputActions, Microsoft.Extensions.Logging.ILogger logger)
		{
			// Disable cache server when using hot reload path (there would likely never be any cache hits anyway)
			// Note we can't do this earlier because HotReloadMode is not set until actions are created
			foreach (TargetDescriptor targetDescriptor in _targetDescriptors)
			{
				UBAConfig.bForceNoCache |= targetDescriptor.HotReloadMode != HotReloadMode.Disabled;
			}

			if (!inputActions.Any())
			{
				await _cancellationSource.CancelAsync();
			}

			await _initTask;

			_trace?.UpdateStatus(1, 6, "Hits 0 Misses 0", LogEntryType.Info);

			if (!inputActions.Any())
			{
				return true;
			}

			if (UBAConfig.bForceNoCache)
			{
				DisposeCacheClients();

				// TOOD: We want to do this but it cause okta re-logins which are annoying
				//if (_agentCoordinators.Count > 0 && inputActions.Count() < NumParallelProcesses)
				//{
				//	_cancellationSource.Cancel();
				//}
			}
			else
			{
				_cacheableActionsLeft = inputActions.Count(x => x.ArtifactMode.HasFlag(ArtifactMode.Enabled));
			}

			PrintConfiguration(inputActions, logger);

			await WriteActionOutputFileAsync(inputActions, logger);

			try
			{
				if (UBAConfig.bLaunchVisualizer)
				{
					_ = Task.Run(LaunchVisualizer);
				}

				OnSessionCreated?.Invoke(_session!);

				bool success = await ExecuteActionsInternalAsync(logger, inputActions, _server!, _storageServer!, _session!, _cacheClients.Values);

				if (!UBAConfig.bDisableRemote)
				{
					_server!.StopServer();
				}

				if (UBAConfig.bPrintSummary)
				{
					_session!.PrintSummary();
				}

				return success;
			}
			finally
			{
				await _cancellationSource.CancelAsync();
				_agentCoordinators.ForEach(ac => ac.Stop());

				DisposeCacheClients();

				lock (_sessionLock)
				{
					_agentCoordinators.ForEach(ac => ac.Done());
				}

				foreach (IUBAAgentCoordinator coordinator in _agentCoordinators)
				{
					await coordinator.CloseAsync();
				}
			}
		}

		[SupportedOSPlatform("windows")]
		static void LaunchVisualizerWin()
		{
			FileReference visualizerPath = FileReference.Combine(GetUbaBinariesDir(OSPlatform.Windows), "UbaVisualizer.exe");
			FileReference tempPath = FileReference.Combine(new DirectoryReference(System.IO.Path.GetTempPath()), visualizerPath.GetFileName());
			if (!FileReference.Exists(visualizerPath))
			{
				return;
			}

			try
			{
				// Check if a listening visualizer is already running
				foreach (System.Diagnostics.Process process in System.Diagnostics.Process.GetProcessesByName(visualizerPath.GetFileNameWithoutAnyExtensions()))
				{
					using ManagementObjectSearcher searcher = new($"SELECT CommandLine FROM Win32_Process WHERE ProcessId = {process.Id}");
					using ManagementObjectCollection objects = searcher.Get();
					string args = objects.Cast<ManagementBaseObject>().SingleOrDefault()?["CommandLine"]?.ToString() ?? "";
					if (args.Contains("-listen", StringComparison.OrdinalIgnoreCase))
					{
						return;
					}
				}
				if (!FileReference.Exists(tempPath) || tempPath.ToFileInfo().LastWriteTime < visualizerPath.ToFileInfo().LastWriteTime)
				{
					FileReference.Copy(visualizerPath, tempPath, true);
				}
				if (FileReference.Exists(tempPath))
				{
					System.Diagnostics.ProcessStartInfo psi = new(BuildHostPlatform.Current.Shell.FullName, $" /C start \"\" \"{tempPath.FullName}\" -listen -nocopy")
					{
						WorkingDirectory = System.IO.Path.GetTempPath(),
						WindowStyle = System.Diagnostics.ProcessWindowStyle.Hidden,
						UseShellExecute = true,
					};
					System.Diagnostics.Process.Start(psi);
				}
			}
			catch (Exception)
			{
			}
		}

		[SupportedOSPlatform("macos")]
		static void LaunchVisualizerMac()
		{
			DirectoryReference visualizerPath = DirectoryReference.Combine(GetUbaBinariesDir(OSPlatform.OSX), "UbaVisualizer.app");
			if (!DirectoryReference.Exists(visualizerPath))
			{
				return;
			}

			System.Diagnostics.ProcessStartInfo psi = new()
			{
				FileName = "/usr/bin/open",
				UseShellExecute = false,
			};
			psi.ArgumentList.Add("-a");
			psi.ArgumentList.Add(visualizerPath.FullName);
			System.Diagnostics.Process.Start(psi);
		}

		static void LaunchVisualizer()
		{
			if (OperatingSystem.IsWindows())
			{
				LaunchVisualizerWin();
			}
			else if (OperatingSystem.IsMacOS())
			{
				LaunchVisualizerMac();
			}
		}

		public override bool VerifyOutputs => UBAConfig.bWriteToDisk;

		(byte[]?, uint) BuildKnownInputs(LinkedAction action)
		{
			if (!UBAConfig.bUseKnownInputs)
			{
				return (null, 0);
			}
			
			uint knownInputsCount = 0;
			int sizeOfChar = System.OperatingSystem.IsWindows() ? 2 : 1;

			int byteCount = 0;
			foreach (FileItem item in action.PrerequisiteItems)
			{
				byteCount += (item.FullName.Length + 1) * sizeOfChar;
				++knownInputsCount;
			}

			byte[] knownInputs = new byte[byteCount + sizeOfChar];

			int byteOffset = 0;
			foreach (FileItem item in action.PrerequisiteItems)
			{
				string str = item.FullName;
				int strBytes = str.Length * sizeOfChar;
				if (sizeOfChar == 1) // Unmanaged size uses ascii
				{
					System.Buffer.BlockCopy(System.Text.Encoding.ASCII.GetBytes(str.ToCharArray()), 0, knownInputs, byteOffset, strBytes);
				}
				else
				{
					System.Buffer.BlockCopy(str.ToCharArray(), 0, knownInputs, byteOffset, strBytes);
				}

				byteOffset += strBytes + sizeOfChar;
			}
			return (knownInputs, knownInputsCount);
		}

		class AgentCoordinatorScheduler : IUbaAgentCoordinatorScheduler
		{
			internal AgentCoordinatorScheduler(IScheduler scheduler, ISessionServer session, IServer server)
			{
				_scheduler = scheduler;
				_session = session;
				_server = server;
			}

			internal IScheduler? _scheduler;
			internal ISessionServer? _session;
			internal IServer? _server;
			private readonly Lock _lockObject = new();

			public void Reset()
			{
				lock (_lockObject)
				{
					_scheduler = null;
					_session = null;
					_server = null;
				}
			}

			public bool IsEmpty
			{
				get
				{
					lock (_lockObject)
					{
						return _scheduler == null || _scheduler.IsEmpty;
					}
				}
			}

			public void GetProcessWeightThatCanRunRemotelyNow(out double totalWeight, out double crossArchitecture, out double crossPlatform)
			{
				totalWeight = 0;
				crossArchitecture = 0;
				crossPlatform = 0;
				lock (_lockObject)
				{
					if (_scheduler != null)
					{
						_scheduler.GetProcessWeightThatCanRunRemotelyNow(out totalWeight, out crossArchitecture, out crossPlatform);
					}
				}
			}

			public bool AddClient(string ip, int port, string crypto = "")
			{
				lock (_lockObject)
				{
					if (_server != null)
					{
						return _server.AddClient(ip, port, crypto);
					}
				}
				return false;
			}
			
			public bool DisableRemoteExecutionOnAgent(string agentName)
			{
				lock (_lockObject)
				{
					if (_session != null)
					{
						return _session.DisableRemoteExecutionOnAgent(agentName);
					}
				}
				return false;
			}
		}

		class ProcessUserData
		{
			public required LinkedAction Action;
			public int RetryCount = 0;
		}

		/// <summary>
		/// Executes the provided actions
		/// </summary>
		/// <returns>True if all the tasks successfully executed, or false if any of them failed.</returns>
		private async Task<bool> ExecuteActionsInternalAsync(ILogger logger, IEnumerable<LinkedAction> inputActions, IServer server, IStorageServer storage, ISessionServer session, IEnumerable<ICacheClient> cache)
		{
			_ubaStartTimeUtc = DateTime.UtcNow;

			using ActionLogger actionLogger = new(inputActions.Count(), "Compiling C++ source code...", x => WriteToolOutput(x), () => FlushToolOutput(), logger)
			{
				ShowCompilationTimes = UBAConfig.bShowCompilationTimes,
				ShowCPUUtilization = UBAConfig.bShowCPUUtilization,
				PrintActionTargetNames = UBAConfig.bPrintActionTargetNames,
				LogActionCommandLines = UBAConfig.bLogActionCommandLines,
				ShowPerActionCompilationTimes = UBAConfig.bShowPerActionCompilationTimes,
				CompactOutput = _bCompactOutput,
			};

			using IScheduler scheduler = IScheduler.CreateScheduler(session, cache, _config!, NumParallelProcesses, UBAConfig.bForceBuildAllRemote);

			// This must not outlive the cancellation token source being cancelled in order for it to correctly save results.
			await using IActionResultStore resultStore = new ContentAddressableActionResultStore(new FileSystemStorageProvider(), logger);

			int totalActions = 0;
			int cacheTotalActions = 0;
			int cacheHitActions = 0;
			int succeededActions = 0;
			int failedActions = 0;
			int localProcessedActions = 0;
			int remoteProcessedActions = 0;
			int localRetryActions = 0;
			int forcedRetryActions = 0;
			ulong peakProcessMemory = 0;
			ulong peakTotalMemory = 0;
			bool shouldCancel = false;
			string cancelReason = "user cancellation";

			ConcurrentBag<Task> resultStoreTasks = [];

			scheduler.SetProcessFinishedCallback((info) =>
			{
				int exitCode = info.ExitCode;

				if (_cancellationSource.IsCancellationRequested || exitCode == 99999) // Process was cancelled by executor
				{
					return ProcessFinishedResponse.None;
				}

				List<string> logLines = info.LogLines;

				ProcessUserData userData = (ProcessUserData)info.UserData;
				LinkedAction action = userData.Action;

				string additionalDescription = String.Empty;

				ProcessExecutionType executionType = info.ExecutionType;

				switch (executionType)
				{
					case ProcessExecutionType.Skip:
						{
							return ProcessFinishedResponse.None;
						}

					case ProcessExecutionType.Remote:
						{
							if (_ubaDurationWaitingForRemote == TimeSpan.Zero)
							{
								_ubaDurationWaitingForRemote = DateTime.UtcNow - _ubaStartTimeUtc;
							}

							string executingHost = info.ExecutingHost ?? "Unknown";

							ProcessFinishedResponse rerun(string error)
							{
								logger.LogInformation("{Description} {StatusDescription} [RemoteExecutor: {ExecutingHost}]: Exited with error code {ExitCode} ({Error}). This action will retry locally", action.CommandDescription, action.StatusDescription, executingHost, exitCode, error);
								foreach (string line in logLines)
								{
									logger.LogInformation("{ErroredProcessLine}", line);
								}
								++localRetryActions;
								return ProcessFinishedResponse.RerunLocal;
							}

							if (exitCode != 0 && !logLines.Any())
							{
								logger.LogInformation("{Description} {StatusDescription} [RemoteExecutor: {ExecutingHost}]: Exited with error code {ExitCode} with no output. This action will retry locally", action.CommandDescription, action.StatusDescription, executingHost, exitCode);
								return ProcessFinishedResponse.RerunLocal;
							}
							else if ((uint)exitCode == 0xC0000005)
							{
								return rerun("Access violation");
							}
							else if ((uint)exitCode == 0xC0000409)
							{
								return rerun("Stack buffer overflow");
							}
							else if ((uint)exitCode == 0xC0000602)
							{
								return rerun("Fail Fast Exception");
							}
							else if (exitCode != 0 && logLines.Any(x => x.Contains(" C1001: ", StringComparison.Ordinal)))
							{
								return rerun("C1001");
							}
							else if (exitCode != 0 && logLines.Any(x => x.Contains("clang frontend command failed", StringComparison.Ordinal)))
							{
								return rerun("Clang Exception");
							}
							else if (exitCode != 0 && logLines.Any(x => x.Contains("cannot overwrite the original file", StringComparison.Ordinal)))
							{
								// We have seen this once on the farm and have no idea how it can happen.
								return rerun("Cannot overwrite original file");
							}
							else if (exitCode >= 9000 && exitCode < 10000)
							{
								return rerun("UBA error");
							}
							else if (exitCode != 0 && UBAConfig.bForcedRetryRemote)
							{
								return rerun("Force local retry");
							}
							additionalDescription = $"[RemoteExecutor: {executingHost}]";
							++remoteProcessedActions;
							break;
						}

					case ProcessExecutionType.Local:
						{
							if (UBAConfig.bAllowRetry && (exitCode != 0 && UBAConfig.bForcedRetry || (exitCode >= 9000 && exitCode < 10000)))
							{
								// If files are compressed (pch, obj etc), then native will not work so must retry detoured
								// If vfs paths are used, then native will not work
								bool mustRunDetoured = !UBAConfig.CanRetryNatively || UBAConfig.bStoreObjFilesCompressed || action.RootPaths.bUseVfs;

								++userData.RetryCount;

								if (mustRunDetoured && userData.RetryCount <= 3)
								{
									logger.LogInformation("{Description} {StatusDescription}: Exited with error code {ExitCode}. This action will retry (More info in uba trace)", action.CommandDescription, action.StatusDescription, exitCode);
									++forcedRetryActions;
									return ProcessFinishedResponse.RerunLocal;
								}
								else
								{
									logger.LogInformation("{Description} {StatusDescription}: Exited with error code {ExitCode}. This action will retry without UBA", action.CommandDescription, action.StatusDescription, exitCode);
									++forcedRetryActions;
									return ProcessFinishedResponse.RerunNative;
								}
							}
							++localProcessedActions;
							break;
						}

					case ProcessExecutionType.Native:
						{
							// If not detoured we manually have to report created files
							if (exitCode == 0)
							{
								session!.RegisterNewFiles(action.ProducedItems.Where(x => FileReference.Exists(x.Location)).Select(x => x.FullName).ToArray());
							}
							additionalDescription = "[NoUba]";
							break;
						}

					case ProcessExecutionType.Cache:
						{
							++cacheHitActions;
							additionalDescription = "[Cache]";
							break;
						}
				}

				TimeSpan processorTime = TimeSpan.Zero;
				TimeSpan executionTime = TimeSpan.Zero;
				ulong peakMemory = 0;
				if (executionType != ProcessExecutionType.Cache)
				{
					processorTime = info.TotalProcessorTime;
					executionTime = info.TotalWallTime;
					peakMemory = info.PeakMemoryUsed;
				}

				peakProcessMemory = Math.Max(peakProcessMemory, peakMemory);

				logLines.RemoveAll(line =>
					line.StartsWith("   Creating library ", StringComparison.Ordinal) ||
					line.StartsWith("   Creating object ", StringComparison.Ordinal) ||
					line.EndsWith("file(s) copied.", StringComparison.Ordinal));

				if (exitCode == 0)
				{
					if (action.bForceWarningsAsError && logLines.Any(x => x.Contains(": warning: ", StringComparison.OrdinalIgnoreCase)))
					{
						exitCode = 1;
						logLines = [.. logLines.Select(x => x.Replace(": warning: ", ": error: ", StringComparison.OrdinalIgnoreCase))];
					}
					else
					{
						if (executionType != ProcessExecutionType.Cache)
						{
							WriteToCache(logger, action, exitCode, info.ProcessHandle);
						}
						++succeededActions;
					}
				}

				if (exitCode != 0)
				{
					++failedActions;

					// Delete produced items on error if requested
					if (action.bDeleteProducedItemsOnError)
					{
						foreach (FileItem output in action.ProducedItems.Distinct().Where(x => FileReference.Exists(x.Location)))
						{
							FileReference.Delete(output.Location);
						}
					}

					if (UBAConfig.bStopCompilationAfterErrors || UBAConfig.StopCompilationAfterNumErrors <= 0)
					{
						cancelReason = "stop compilation after any errors";
						shouldCancel = true;
					}
					else if (failedActions >= UBAConfig.StopCompilationAfterNumErrors)
					{
						cancelReason = $"stop compilation after {UBAConfig.StopCompilationAfterNumErrors} errors";
						shouldCancel = true;
					}
				}

				ExecuteResults result = new(logLines, exitCode, executionTime, processorTime, peakMemory, action, additionalDescription);
				actionLogger.AddActionToLog(result);
				Task storeResultTask = Task.Run(
					() => resultStore.StoreResultAsync(action, ActionResult.From(result), _cancellationSource.Token),
					_cancellationSource.Token);
				resultStoreTasks.Add(storeResultTask);

				return ProcessFinishedResponse.None;
			});

			foreach (ICacheClient client in cache)
			{
				client.PopulatePathHashes();
			}

			scheduler.Start();

			ConsoleCancelEventHandler? cancelHandler = null;
			cancelHandler = (object? sender, ConsoleCancelEventArgs e) =>
			{
				Console.CancelKeyPress -= cancelHandler;
				cancelHandler = null;
				e.Cancel = true;
				shouldCancel = true;
			};
			Console.CancelKeyPress += cancelHandler;

			int index = 0;
			foreach (LinkedAction action in inputActions)
			{
				action.SortIndex = index++;
				ProcessStartInfo startInfo = GetActionStartInfo(action);
				startInfo.RootsHandle = GetActionRootsHandle(action);

				int[] prereqActionsSortIndex = [.. action.PrerequisiteActions.Select(a => a.SortIndex)];
				bool enableDetour = !ForceLocalNoDetour(action) && action.bCanExecuteInUBA;
				bool canRunRemotely = CanRunRemotely(action);
				bool canCrossArchitecture = action.bCanExecuteInUBACrossArchitecture;
				bool canCrossPlatform = canCrossArchitecture;
				(byte[]? knownInputs, uint knownInputsCount) = BuildKnownInputs(action);

				uint bucket = GetCacheBucket(logger, action);
				uint memoryGroup = GetMemoryGroup(action);
				ulong predictedMemoryUsage = 0; // uba would do a better job scheduling if we knew approx memory usage
				ProcessUserData userData = new ProcessUserData() { Action = action };
				scheduler.EnqueueProcess(startInfo, action.Weight, enableDetour, canRunRemotely, canCrossArchitecture, canCrossPlatform, prereqActionsSortIndex, knownInputs, knownInputsCount, bucket, memoryGroup, predictedMemoryUsage, userData);

				++totalActions;
				cacheTotalActions += bucket != 0 ? 0 : 1;
			}

			if (!UBAConfig.bForceBuildAllRemote)
			{
				scheduler.SetAllowDisableRemoteExecution();
			}

			logger.LogDebug("All processes queued");

			try
			{
				AgentCoordinatorScheduler coordinatorScheduler = new(scheduler, session, server);
				foreach (IUBAAgentCoordinator coordinator in _agentCoordinators)
				{
					coordinator.Start(coordinatorScheduler, CanRunRemotely);
				}

				while (!scheduler.IsEmpty)
				{
					if (shouldCancel && !_cancellationSource.IsCancellationRequested)
					{
						await _cancellationSource.CancelAsync();

						scheduler.Cancel();
						foreach (ICacheClient client in cache)
						{
							client.Disconnect();
						}

						server?.StopServer(); // Make sure all remove processes are returned. We can't have any callbacks after this
						session?.CancelAll(); // Cancel all processes native side
						storage?.SaveCasTable();

						// We need the lock here since things are happening in parallel.
						lock (_sessionLock)
						{
							foreach (IUBAAgentCoordinator coordinator in _agentCoordinators)
							{
								_agentCoordinators.ForEach(ac => ac.CloseAsync().Wait(2000)); // Give coordinators some time to close (this makes coordinators like horde return resources faster)
							}
						}
					}
					await Task.Delay(100);
				}

				bool noFailedActions = failedActions == 0;
				bool success = noFailedActions && !shouldCancel;

				actionLogger.TraceSummary(noFailedActions);

				coordinatorScheduler.Reset();

				int cacheMissActions = cacheTotalActions - cacheHitActions;

				if (!_cancellationSource.IsCancellationRequested)
				{
					telemetryEvent = new TelemetryExecutorUBAEvent(Name, _ubaStartTimeUtc, success, totalActions, succeededActions, failedActions, cacheHitActions, cacheMissActions,
						localProcessedActions, remoteProcessedActions,
						localRetryActions, forcedRetryActions,
						!UBAConfig.bDisableRemote ? _agentCoordinators.Count : 0, !UBAConfig.bDisableRemote ? _successfulCoordinatorConnections : 0, !UBAConfig.bDisableRemote ? _failedCoordinatorConnections : 0,
						!UBAConfig.bDisableRemote ? _ubaDurationWaitingForRemote : TimeSpan.Zero,
						!UBAConfig.bDisableRemote || _agentCoordinators.Count == 0 ? "Local" : _remoteConnectionMode,
						DateTime.UtcNow,
						peakProcessMemory, peakTotalMemory);
				}
				else
				{
					logger.LogInformation("Cancelled execution after {Actions}/{TotalActions} actions ({Reason})", succeededActions + failedActions + cacheHitActions, totalActions, cancelReason);
				}

				Task whenAllTask = Task.WhenAll(resultStoreTasks);
				try
				{
					await whenAllTask;
				}
				catch (Exception) when (whenAllTask.IsFaulted)
				{
					logger.LogWarning(whenAllTask.Exception, "Some action result store tasks did not succeed");
				}
				catch (OperationCanceledException)
				{
					// We've already logged the cancellation, we don't have to do anything more here, just don't throw the cancellation up the callstack.
				}

				return success;
			}
			finally
			{
				if (cancelHandler != null)
				{
					Console.CancelKeyPress -= cancelHandler;
					cancelHandler = null;
				}
			}
		}

		/// <summary>
		/// Determine if an action must be run locally and with no detouring
		/// </summary>
		/// <param name="action">The action to check</param>
		/// <returns>If this action must be local, non-detoured</returns>
		bool ForceLocalNoDetour(LinkedAction action)
		{
			if (!UBAConfig.bAllowDetour)
			{
				return true;
			}

			if (!OperatingSystem.IsMacOS()) // Below code is slow, so early out
			{
				return false;
			}
			// Don't let Mac run shell commands through Uba as interposing dylibs into
			// the shell results in dyld errors about no matching architecture.
			// The shell is used to run various commands during a build like copy/ditto.
			// So for these actions we need to make sure UBA is not used.
			bool bIsShellAction = action.CommandPath == BuildHostPlatform.Current.Shell;
			return bIsShellAction;
		}

		/// <summary>
		/// Determine if an action is able to be run remotely
		/// </summary>
		/// <param name="action">The action to check</param>
		/// <returns>If this action can be run remotely</returns>
		bool CanRunRemotely(LinkedAction action) =>
			action.bCanExecuteInUBA &&
			action.bCanExecuteRemotely &&
			(UBAConfig.bLinkRemote || action.ActionType != ActionType.Link) &&
			!ForceLocalNoDetour(action);

		ProcessStartInfo GetActionStartInfo(LinkedAction action)
		{
			string description = action.StatusDescription;
			if (!String.IsNullOrEmpty(action.CommandDescription))
			{
				description = $"{action.StatusDescription} ({action.CommandDescription})";
			}

			// Let pch compile at higher priority
			System.Diagnostics.ProcessPriorityClass priority = UBAConfig.ProcessPriority;
			if (priority == System.Diagnostics.ProcessPriorityClass.BelowNormal && action.StatusDescription.Contains("PCH.", StringComparison.Ordinal))
			{
					priority = System.Diagnostics.ProcessPriorityClass.Normal;
			}

			ProcessStartInfo startInfo = new()
			{
				Application = action.CommandPath.FullName,
				WorkingDirectory = action.WorkingDirectory.FullName,
				Arguments = action.CommandArguments,
				Priority = priority,
				UserData = action,
				Description = description,
				Configuration = action.bIsClangCompiler ? EpicGames.UBA.ProcessStartInfo.CommonProcessConfigs.CompileClang : EpicGames.UBA.ProcessStartInfo.CommonProcessConfigs.CompileMsvc,
				LogFile = UBAConfig.bLogEnabled ? (action.Inner.ProducedItems.First().Location.GetFileName() + ".log") : null,
			};

			return startInfo;
		}

		void DecrementCacheableAction()
		{
			if (Interlocked.Decrement(ref _cacheableActionsLeft) == 0)
			{
				DisposeCacheClients();
			}
		}

		readonly HashSet<uint> _usedBuckets = new();

		uint GetCacheBucket(ILogger logger, LinkedAction action)
		{
			if (!action.ArtifactMode.HasFlag(ArtifactMode.Enabled))
			{
				return 0;
			}

			if (!UBAConfig.bCacheLinkActions && action.ActionType == ActionType.Link)
			{
				return 0;
			}

			uint bucket = 0;
			if (action.CacheBucket != 0)
			{
				bucket = action.CacheBucket;
			}
			else if (action.Target == null)
			{
				bucket = 0;
			}
			else
			{
				using (Blake3.Hasher hasher = Blake3.Hasher.New())
				{
					// Use platform and config to chose bucket since there will never be any cache hits between platforms or configs
					hasher.Update(System.Text.Encoding.UTF8.GetBytes(action.Target.Platform.ToString()));
					hasher.Update(new byte[] { (byte)action.Target.Configuration });

					// Absolute path is set for actions that uses pch.
					// And since pch contains absolute paths we unfortunately can't share cache data between machines that have different paths
					if (action.ArtifactMode.HasFlag(ArtifactMode.AbsolutePath))
					{
						foreach ((uint id, DirectoryReference vfs, DirectoryReference local) in action.RootPaths)
						{
							hasher.Update(System.Text.Encoding.UTF8.GetBytes(local.FullName));
						}
					}

					bucket = (uint)IoHash.FromBlake3(hasher).GetHashCode();
				}
			}

			lock (_usedBuckets)
			{
				if (_usedBuckets.Add(bucket))
				{
					logger.LogDebug(
						"Cache bucket {CacheBucket} used. Action: {CommandDescription} {StatusDescription} ({GroupNames})",
						bucket,
						action.CommandDescription,
						action.StatusDescription,
						String.Join(" + ", action.GroupNames));
				}
			}

			return bucket;
		}

		private static uint GetStringHash(string str, uint hash = 5381)
		{
			for (int idx = 0; idx < str.Length; idx++)
			{
				hash += (hash << 5) + str[idx];
			}
			return hash;
		}

		private static uint GetMemoryGroup(LinkedAction action)
		{
			if (action.ActionType != ActionType.Compile)
			{
				return 0;
			}

			FileItem? pch = action.ProducedItems.FirstOrDefault(i => i.HasExtension(".pch"));

			if (pch == null)
			{
				pch = action.Inner.PrerequisiteItems.FirstOrDefault(i => i.HasExtension(".pch"));
			}

			if (pch != null)
			{
				return GetStringHash(pch.FullName);
			}
			return 1;
		}

		ulong GetActionRootsHandle(LinkedAction action)
		{
			using MemoryStream outputsMemory = new(1024);
			using (BinaryWriter writer = new(outputsMemory, System.Text.Encoding.UTF8, true))
			{
				bool useVfs = action.RootPaths.bUseVfs;
				foreach ((uint id, DirectoryReference vfs, DirectoryReference local) in action.RootPaths)
				{
					writer.Write((byte)id);
					if (useVfs)
					{
						writer.Write(vfs.FullName);
					}
					else
					{
						writer.Write("");
					}
					writer.Write(local.FullName);
				}
			}
			return _session!.RegisterRoots(outputsMemory.GetBuffer(), (uint)outputsMemory.Position);
		}

		public class DepsFile
		{
			public class DepsData
			{
				public string? Source { get; init; }
				public string? PCH { get; init; }
				public SortedSet<string>? Includes { get; init; }
			}
			public string? Version { get; init; }
			public DepsData? Data { get; init; }
		}

		bool WriteToCache(ILogger logger, LinkedAction action, int exitCode, nint processHandle)
		{
			if (exitCode != 0 || !action.ArtifactMode.HasFlag(ArtifactMode.Enabled) || !_cacheClients.Any(x => x.Key.CanWrite))
			{
				return true;
			}

			// If there are no outputs there is nothing to cache
			if (!action.ProducedItems.Any())
			{
				return true;
			}

			if (!UBAConfig.bCacheLinkActions && action.ActionType == ActionType.Link)
			{
				return true;
			}

			// Collect all inputs for action
			// We use prerequisite items plus what we find in dependency list file if it exists.

			using MemoryStream inputsMemory = new(1024);
			using (BinaryWriter writer = new(inputsMemory, System.Text.Encoding.UTF8, true))
			{
				writer.Write(action.CommandPath.FullName);

				foreach (FileItem f in action.PrerequisiteItems)
				{
					if (f.HasExtension(".lib")) // It seems like .lib files can change without dependencies relink
					{
						continue;
					}

					writer.Write(f.FullName);
				}

				if (action.DependencyListFile != null)
				{
					try
					{
						CppDependencyCache.DependencyInfo info = CppDependencyCache.ReadDependencyInfo(action.DependencyListFile);
						foreach (FileItem f in info.Files)
						{
							writer.Write(f.FullName);
						}
					}
					catch (Exception)
					{
						logger.LogInformation("{Description} {StatusDescription}: Failed to read dependency file {DependencyFile} for WriteCache", action.CommandDescription, action.StatusDescription, action.DependencyListFile);
						return false;
					}
				}
			}

			// Collect all outputs for action

			using MemoryStream outputsMemory = new(1024);
			using (BinaryWriter writer = new(outputsMemory, System.Text.Encoding.UTF8, true))
			{
				foreach (FileItem f in action.ProducedItems)
				{
					writer.Write(f.FullName);
				}
			}

			uint bucket = GetCacheBucket(logger, action);

			bool success = true;
			foreach (KeyValuePair<UnrealBuildAcceleratorCacheConfig, ICacheClient> item in _cacheClients.Where(x => x.Key.CanWrite))
			{
				if (item.Key.bRequireVfs && !action.RootPaths.bUseVfs)
				{
					continue;
				}

				success = item.Value.WriteToCache(bucket, processHandle, inputsMemory.GetBuffer(), (uint)inputsMemory.Position, outputsMemory.GetBuffer(), (uint)outputsMemory.Position) || success;
			}

			DecrementCacheableAction();

			return success;
		}
	}
}
