// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.Linq;
using System.Runtime.InteropServices;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Horde.Agents;
using EpicGames.Horde.Common;
using EpicGames.Horde.Compute;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Storage.Bundles;
using EpicGames.Horde.Storage.Nodes;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Logging.Abstractions;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	// statusRow, statusColumn, statusText, statusType, statusLink
	using StatusUpdateAction = Action<uint, uint, string, EpicGames.UBA.LogEntryType, string?>;

	internal class UBAHordeSession : IAsyncDisposable
	{
		/// <summary>
		/// ID for this Horde session
		/// </summary>
		readonly Guid _id = Guid.NewGuid();
		readonly IComputeClient _client;

		const string ResourceLogicalCores = "LogicalCores";
		ClusterId? _clusterId;

		readonly bool _strict;
		readonly ConnectionMode? _connectionMode;
		readonly Encryption? _encryption;
		readonly CancellationTokenSource _cancellationTokenSource = new();
		readonly ILogger _logger;

		readonly UnrealBuildAcceleratorConfig _config;
		readonly UnrealBuildAcceleratorHordeConfig _hordeConfig;
		internal StatusUpdateAction? _updateStatus;
		internal IUbaAgentCoordinatorScheduler? _scheduler;

		readonly BundleStorageNamespace _storage = BundleStorageNamespace.CreateInMemory(NullLogger.Instance);
		BlobLocator _ubaAgentLocator;

		readonly string _crypto;

		OSPlatform _targetWorkerPlatform;

		public class Worker
		{
			public string? Name { get; init; }
			public Stopwatch? StartTime { get; init; }
			public string? Ip { get; init; }
			public ConnectionMetadataPort? Port { get; init; }
			public ConnectionMetadataPort? ProxyPort { get; init; }

			public int NumLogicalCores { get; set; }
			public bool Active { get; set; }
			public Task? BackgroundTask { get; set; }
		}

		readonly List<Worker> _workers = new();

		public UBAHordeSession(UnrealBuildAcceleratorConfig config, UnrealBuildAcceleratorHordeConfig hordeConfig, IComputeClient client, string crypto, bool strict, ConnectionMode? connectionMode, Encryption? encryption, ILogger logger)
		{
			_config = config;
			_hordeConfig = hordeConfig;
			_client = client;
			_strict = strict;
			_connectionMode = connectionMode;
			_encryption = encryption;
			_logger = logger;
			_crypto = crypto;

			if (connectionMode == ConnectionMode.Relay && String.IsNullOrEmpty(_crypto))
			{
				_crypto = UBAExecutor.CreateCrypto();
				_encryption = Encryption.Ssl;
			}
		}

		public OSPlatform WorkerPlatform => _targetWorkerPlatform;

		public async ValueTask DisposeAsync()
		{
			if (_workers.Count > 0) // Should handle double-dispose, prevent cancelling twice
			{
				await _cancellationTokenSource.CancelAsync();

				for (int idx = _workers.Count - 1; idx >= 0; idx--)
				{
					await _workers[idx].BackgroundTask!;
					lock (_workers)
					{
						_workers.RemoveAt(idx);
					}
				}
			}

			// Set CPU resource need to zero (will also expire on the server if not updated)
			await UpdateCpuCoreNeedAsync(0);

			_cancellationTokenSource.Dispose();
		}

		public async Task InitAsync(bool useSentry, string? hordeCondition, CancellationToken cancellationToken)
		{
			if (!String.IsNullOrEmpty(_hordeConfig.AgentPlatform))
			{
				_targetWorkerPlatform = GetTargetWorkerPlatform(_hordeConfig.AgentPlatform);
			}
			else
			{
				_targetWorkerPlatform = GetTargetWorkerPlatform(hordeCondition);
			}

			DirectoryReference ubaDir = UBAExecutor.GetUbaBinariesDir(_targetWorkerPlatform);

			List<string> agentFiles = [];

			if (_targetWorkerPlatform == OSPlatform.Windows)
			{
				agentFiles.Add("UbaAgent.exe");
				//agentFiles.Add("UbaAgent.pdb");
				if (FileReference.Exists(FileReference.Combine(ubaDir, "UbaWine.dll.so")))
				{
					agentFiles.Add("UbaWine.dll.so"); // Not needed but used by linux/wine helpers for more detailed logging or optimizations
				}
				if (useSentry)
				{
					agentFiles.Add("crashpad_handler.exe");
					agentFiles.Add("sentry.dll");
				}
			}
			else if (_targetWorkerPlatform == OSPlatform.Linux)
			{
				agentFiles.Add("UbaAgent");
				bool bIsDebug = false;
				if (bIsDebug)
				{
					agentFiles.Add("UbaAgent.debug");
					agentFiles.Add("UbaAgent.sym");
					agentFiles.Add("libclang_rt.tsan.so"); // Needs to be copied from autosdk
					agentFiles.Add("llvm-symbolizer"); // Needs to be copied from autosdk
				}
			}
			else if (_targetWorkerPlatform == OSPlatform.OSX)
			{
				agentFiles.Add("UbaAgent");
			}
			else
			{
				throw new PlatformNotSupportedException();
			}

			_ubaAgentLocator = await CreateToolAsync(ubaDir, agentFiles.Select(x => FileReference.Combine(ubaDir, x)), cancellationToken);
			_logger.LogInformation("Created tool bundle with locator {UbaAgentLocator}", _ubaAgentLocator.ToString());
		}

		public UnrealBuildAcceleratorCacheConfig? RequestCacheServer(int desiredConnectionCount, CancellationToken cancellationToken)
		{
			ClusterId clusterId = new(UnrealBuildAcceleratorHordeConfig.ClusterDefault);
			using ITimelineEvent timelineEvent = Timeline.ScopeEvent($"HordeGetCache");
			UbaConfig ubaConfig;
			try
			{
				ubaConfig = _client.AllocateUbaCacheServerAsync(clusterId, cancellationToken).GetAwaiter().GetResult();
			}
			catch (OperationCanceledException)
			{
				// Might get cancelled if the actions finish faster than the call to allocate a cache server. This is fine.
				return null;
			}
			timelineEvent.Finish();
			if (String.IsNullOrEmpty(ubaConfig.CacheEndpoint))
			{
				return null;
			}
			bool writeAccess = ubaConfig.WriteAccess;
			return new UnrealBuildAcceleratorCacheConfig() { CacheServer = ubaConfig.CacheEndpoint, bRequireVfs = true, WriteCache = writeAccess.ToString(), Crypto = ubaConfig.CacheSessionKey, DesiredConnectionCount = desiredConnectionCount };
		}

		async Task<BlobLocator> CreateToolAsync(DirectoryReference baseDir, IEnumerable<FileReference> files, CancellationToken cancellationToken)
		{
			using ITimelineEvent __ = Timeline.ScopeEvent($"HordeCreateTool");

			// TODO: should drive this off API version reported by server
			BlobSerializerOptions serializerOptions = BlobSerializerOptions.Create(HordeApiVersion.Initial);

			await using IBlobWriter writer = _storage.CreateBlobWriter(serializerOptions: serializerOptions, cancellationToken: cancellationToken);
			DirectoryNode sandbox = new();
			await sandbox.AddFilesAsync(baseDir, files, writer, cancellationToken: cancellationToken);

			// If host is windows and we want to use non-windows helper, then UbaAgent does not have executable flag so we have to add it
			if (_targetWorkerPlatform != OSPlatform.Windows && UBAExecutor.GetHostPlatform() == OSPlatform.Windows)
			{
				sandbox.GetFileEntry("UbaAgent").Flags |= FileEntryFlags.Executable;
			}

			IHashedBlobRef <DirectoryNode> handle = await writer.WriteBlobAsync(sandbox, cancellationToken);
			await writer.FlushAsync(cancellationToken);
			return handle.GetLocator();
		}

		public async void RemoveCompleteWorkersAsync()
		{
			for (int idx = 0; idx < _workers.Count; idx++)
			{
				Worker worker = _workers[idx];
				if (worker.BackgroundTask!.IsCompleted)
				{
					await worker.BackgroundTask;
					lock (_workers)
					{
						_workers.RemoveAt(idx--);
					}
				}
			}
		}

		public void GetCoreCount(out int active, out int queued, out int activeAgents, out int queuedAgents)
		{
			active = 0;
			queued = 0;
			activeAgents = 0;
			queuedAgents = 0;
			lock (_workers)
			{
				foreach (Worker worker in _workers)
				{
					if (worker.NumLogicalCores == 0)
					{
						continue;
					}
					if (worker.Active)
					{
						++activeAgents;
						active += worker.NumLogicalCores;
					}
					else
					{
						++queuedAgents;
						queued += worker.NumLogicalCores;
					}
				}
			}
		}

		int _workerId = 0;

		static readonly HashSet<StringView> s_logProperties = new()
			{
				"ComputeIp",
				"CPU",
				"RAM",
				"DiskFreeSpace",
				"PhysicalCores",
				"LogicalCores",
				"EC2",
				"LeaseId",
				"aws-instance-type",
			};

		private static Requirements GetRequirements(UnrealBuildAcceleratorHordeConfig config)
		{
			Requirements requirements = new() { Exclusive = true };

			if (!String.IsNullOrEmpty(config.HordeCluster))
			{
				OSPlatform platform = GetTargetWorkerPlatform(config.AgentPlatform);

				// Apply filtering options when a cluster, either explicit or auto is specified, ensuring the cluster options take precidence.
				string condition;
				if (config.HordeCondition != null)
				{
					// Use explicitly configured condition, allowing cross-platform distribution (e.g., Mac host to Linux agents)
					condition = config.HordeCondition;
				}
				else if (platform == OSPlatform.Windows)
				{
					condition = $"({KnownPropertyNames.OsFamily} == 'Windows' || {KnownPropertyNames.WineEnabled} == 'true')";
				}
				else if (platform == OSPlatform.OSX)
				{
					condition = $"{KnownPropertyNames.OsFamily} == 'MacOS'";
				}
				else if (platform == OSPlatform.Linux)
				{
					condition = $"{KnownPropertyNames.OsFamily} == 'Linux'";
				}
				else
				{
					condition = "";
				}
				requirements.Condition = Condition.Parse(condition);
			}
			else
			{
				if (!String.IsNullOrEmpty(config.HordePool))
				{
					requirements.Pool = config.HordePool;
				}

				if (config.HordeCondition != null)
				{
					requirements.Condition = Condition.Parse(config.HordeCondition);
				}
			}

			return requirements;
		}

		public async Task<bool> AddWorkerAsync(int activeCores, CancellationToken cancellationToken)
		{
			if (_client == null)
			{
				throw new InvalidOperationException($"Session not initialized. Call {nameof(InitAsync)} first");
			}

			int workerId = _workerId;

			string agentNamePrefix = _targetWorkerPlatform != UBAExecutor.GetHostPlatform() ? UBAExecutor.GetPlatformName(_targetWorkerPlatform) : "";
			PrefixLogger workerLogger = new($"[{agentNamePrefix}Worker{workerId}]", _logger);

			const string UbaPortName = "UbaPort";
			const string UbaProxyPortName = "UbaProxyPort";
			const int UbaPort = 7001;
			const int UbaProxyPort = 7002;

			// Request ID that is unique per attempt to acquire the same compute lease/worker
			// Primarily for tracking worker demand on Horde server as UBAExecutor will repeatedly try adding a new worker
			string requestId = $"{_id}-worker-{workerId}";
			IComputeLease? lease = null;
			try
			{
				Stopwatch stopwatch = Stopwatch.StartNew();
				ConnectionMetadataRequest cmr = new()
				{
					ModePreference = _connectionMode,
					Encryption = _encryption,
					Ports = { { UbaPortName, UbaPort } },
					InactivityTimeoutMs = 90000
				};

				Requirements requirements = GetRequirements(_hordeConfig);

				await ResolveClusterIdAsync(requirements, requestId, cmr, workerLogger, cancellationToken);
				lease = await _client.TryAssignWorkerAsync(_clusterId, requirements, requestId, cmr, false, workerLogger, cancellationToken);
				if (lease == null)
				{
					_logger.LogDebug("Unable to assign a remote worker");

					int missingNumCores = Math.Max(0, _hordeConfig.HordeMaxCores - activeCores);
					await UpdateCpuCoreNeedAsync(missingNumCores, cancellationToken);
					return false;
				}

				++_workerId;

				workerLogger.LogDebug("Agent properties:");

				string agentName = String.Empty;
				int numLogicalCores = 24; // Assume 24 if something goes wrong here and property is not found
				string computeIp = String.Empty;
				string leaseId = String.Empty;
				string instanceType = String.Empty;
				foreach (string property in lease.Properties)
				{
					int equalsIdx = property.IndexOf('=', StringComparison.OrdinalIgnoreCase);
					StringView propertyName = new(property, 0, equalsIdx);
					if (s_logProperties.Contains(propertyName))
					{
						_logger.LogDebug("  {Property}", property);

						if (propertyName == ResourceLogicalCores && Int32.TryParse(property.AsSpan(equalsIdx + 1), out int value))
						{
							numLogicalCores = value;
						}
						else if (propertyName == "ComputeIp")
						{
							computeIp = property[(equalsIdx + 1)..];
						}
						else if (propertyName == "EC2")
						{
							agentName = $"{agentNamePrefix}Worker{workerId}";
						}
						else if (propertyName == "LeaseId")
						{
							leaseId = property[(equalsIdx + 1)..];
						}
						else if (propertyName == "aws-instance-type")
						{
							instanceType = property[(equalsIdx + 1)..];
						}
					}
				}

				string desc = instanceType;
				if (!String.IsNullOrEmpty(leaseId))
				{
					string link = $" {_hordeConfig.HordeServer!.TrimEnd('/')}/lease/{leaseId}";
					desc += link;
				}
				// When using relay connection mode, the IP will be relay server's IP
				string ip = String.IsNullOrEmpty(lease.Ip) ? computeIp : lease.Ip;

				if (!lease.Ports.TryGetValue(UbaPortName, out ConnectionMetadataPort? ubaPort))
				{
					ubaPort = new ConnectionMetadataPort(UbaPort, UbaPort);
				}

				if (!lease.Ports.TryGetValue(UbaProxyPortName, out ConnectionMetadataPort? ubaProxyPort))
				{
					ubaProxyPort = new ConnectionMetadataPort(UbaProxyPort, UbaProxyPort);
				}

				Worker worker = new()
				{
					Name = agentName,
					StartTime = stopwatch,
					NumLogicalCores = numLogicalCores,
					Ip = ip,
					Port = ubaPort,
					ProxyPort = ubaProxyPort,
				};
				worker.BackgroundTask = RunWorkerAsync(worker, lease, workerLogger, agentName, desc, _cancellationTokenSource.Token);
				lock (_workers)
				{
					_workers.Add(worker);
				}
				UpdateHordeStatus(null);
				lease = null; // Will be disposed by RunWorkerAsync

				return true;
			}
			finally
			{
				if (lease != null)
				{
					await lease.DisposeAsync();
				}
			}
		}

		int _targetCoreCount;

		async Task UpdateCpuCoreNeedAsync(int targetCoreCount, CancellationToken cancellationToken = default)
		{
			string? pool = _hordeConfig.HordePool;
			if (_client != null && pool != null && _clusterId != null && targetCoreCount != _targetCoreCount)
			{
				_targetCoreCount = targetCoreCount;

				_logger.LogDebug("Setting CPU core need to {TargetCoreCount}", targetCoreCount);
				Dictionary<string, int> resourceNeeds = new() { { ResourceLogicalCores, targetCoreCount } };
				try
				{
					await _client.DeclareResourceNeedsAsync(_clusterId.Value, pool, resourceNeeds, cancellationToken);
				}
				catch (Exception e)
				{
					_logger.Log(_strict ? LogLevel.Error : LogLevel.Debug, KnownLogEvents.Systemic_Horde_Compute, e, "Failed updating resource need to {TargetCoreCount} cores", targetCoreCount);
				}
			}
		}

		private async Task ResolveClusterIdAsync(Requirements requirements, string requestId, ConnectionMetadataRequest cmr, ILogger logger, CancellationToken cancellationToken = default)
		{
			if (_client == null)
			{
				throw new InvalidOperationException($"Session not initialized. Call {nameof(InitAsync)} first");
			}

			if (_clusterId == null)
			{
				if (_hordeConfig.HordeCluster == UnrealBuildAcceleratorHordeConfig.ClusterAuto)
				{
					_clusterId = await _client.GetClusterAsync(requirements, requestId, cmr, logger, cancellationToken);
				}
				else if (!String.IsNullOrEmpty(_hordeConfig.HordeCluster))
				{
					_clusterId = new ClusterId(_hordeConfig.HordeCluster);
				}
				else
				{
					_clusterId = new ClusterId(UnrealBuildAcceleratorHordeConfig.ClusterDefault);
				}
				_logger.LogInformation("Horde cluster resolved as '{ClusterId}'", _clusterId.ToString());
			}
		}

		public static async Task<(UBAHordeSession?, string)> TryCreateHordeSessionAsync(UnrealBuildAcceleratorConfig ubaConfig, UnrealBuildAcceleratorHordeConfig hordeConfig, IComputeClient client, string crypto, bool bStrictErrors, ILogger logger, CancellationToken cancellationToken = default)
		{
			using ITimelineEvent hordeCreateTimlineEvent = Timeline.ScopeEvent($"HordeCreate");

			if (hordeConfig.bDisableHorde)
			{
				logger.LogInformation("Horde disabled via command line option.");
				return (null, "Disabled via command line");
			}

			if (String.IsNullOrEmpty(hordeConfig.HordeServer) && HordeOptions.GetServerUrlFromEnvironment() == null && HordeOptions.GetDefaultServerUrl() == null)
			{
				logger.LogInformation("Horde disabled. Url not set.");
				return (null, "No server url set");
			}

			Uri? server = (hordeConfig.HordeServer == null) ? null : new Uri(hordeConfig.HordeServer);
			string? token = hordeConfig.HordeToken;

			ConnectionMode? connectionMode = Enum.TryParse(hordeConfig.HordeConnectionMode, true, out ConnectionMode cm) ? cm : null;
			Encryption? encryption = Enum.TryParse(hordeConfig.HordeEncryption, true, out Encryption enc) ? enc : null;

			// Default to SSL encryption for relay mode if unset
			if (connectionMode == ConnectionMode.Relay && encryption == null)
			{
				encryption = Encryption.Ssl;
			}

			logger.LogInformation("Horde URL: {Server}, Pool: {Pool}, Cluster: {Cluster}, Condition: {Condition}, Connection: {Connection}, Encryption: {Encryption}, MaxCores: {MaxCores}, MaxWorkers: {MaxWorkers}, MaxIdle: {MaxIdle}s",
				server, hordeConfig.HordePool ?? "(none)", hordeConfig.HordeCluster ?? "(none)", hordeConfig.HordeCondition ?? "(none)", connectionMode?.ToString() ?? "(none)", encryption?.ToString() ?? "(none)",
				hordeConfig.HordeMaxCores, hordeConfig.HordeMaxWorkers, hordeConfig.HordeMaxIdle);
			try
			{
				UBAHordeSession session = new(ubaConfig, hordeConfig, client, crypto, bStrictErrors, connectionMode, encryption, logger);
				hordeCreateTimlineEvent.Dispose();
				await session.InitAsync(useSentry: !String.IsNullOrEmpty(hordeConfig.UBASentryUrl), hordeConfig.HordeCondition, cancellationToken);
				return (session, "Ready");
			}
			catch (TaskCanceledException)
			{
				return (null, "Cancelled");
			}
			catch (Exception ex)
			{
				logger.Log(bStrictErrors ? LogLevel.Error : LogLevel.Information, ex, "Unable to create Horde session: {Message}", ex.Message);
				return (null, ex.Message);
			}
		}

		async Task RunWorkerAsync(Worker self, IComputeLease lease, ILogger logger, string agentName, string desc, CancellationToken cancellationToken)
		{
			logger.LogDebug("Running worker task..");
			try
			{
				await using (_ = lease)
				{
					// Create a message channel on channel id 0. The Horde Agent always listens on this channel for requests.
					const int PrimaryChannelId = 0;
					using (AgentMessageChannel channel = lease.Socket.CreateAgentMessageChannel(PrimaryChannelId, 4 * 1024 * 1024))
					{
						logger.LogDebug("Waiting for attach...");

						TimeSpan attachTimeout = TimeSpan.FromSeconds(20.0);
						try
						{
							Task attachTask = channel.WaitForAttachAsync(cancellationToken).AsTask();
							await attachTask.WaitAsync(attachTimeout, cancellationToken);
						}
						catch (TimeoutException)
						{
							logger.Log(_strict ? LogLevel.Error : LogLevel.Debug, KnownLogEvents.Systemic_Horde_Compute, "Waited {Time}s on attach message. Giving up", (int)attachTimeout.TotalSeconds);
							throw;
						}

						logger.LogDebug("Uploading files...");
						await channel.UploadFilesAsync("", _ubaAgentLocator, _storage.Backend, cancellationToken);

						string hordeHost = _config.Host;
						if (!String.IsNullOrEmpty(_hordeConfig.HordeHost))
						{
							hordeHost = _hordeConfig.HordeHost;
						}

						bool useListen = !String.IsNullOrEmpty(_hordeConfig.HordeHost);
						List<string> arguments = new();

						if (!String.IsNullOrEmpty(agentName))
						{
							arguments.Add($"-name={agentName}");
						}

						if (useListen)
						{
							arguments.Add($"-Host={hordeHost}:{_config.Port}");
						}
						else
						{
							arguments.Add($"-Listen={self.Port!.AgentPort}");
							arguments.Add("-ListenTimeout=10");
						}

						if (!String.IsNullOrEmpty(_crypto))
						{
							arguments.Add($"-crypto={_crypto}");
						}

						arguments.Add("-NoPoll");
						arguments.Add("-Quiet");
						if (!String.IsNullOrEmpty(_hordeConfig.UBASentryUrl))
						{
							arguments.Add($"-Sentry=\"{_hordeConfig.UBASentryUrl}\"");
						}
						arguments.Add("-ProxyPort=" + self.ProxyPort!.AgentPort);
						if (_config.bUseQuic)
						{
							arguments.Add("-quic");
						}
						//arguments.Add("-NoStore");
						//arguments.Add("-KillRandom"); // For debugging

						if (_targetWorkerPlatform == OSPlatform.OSX)
						{
							// we need to populate the cas with all known xcodes so we can serve all the ones we have installed
							string xcodeVersion = Utils.RunLocalProcessAndReturnStdOut("/bin/sh", "-c '/usr/bin/defaults read $(xcode-select -p)/../version.plist ProductBuildVersion");
							arguments.Add($"-populateCasFromXcodeVersion={xcodeVersion}");
							arguments.Add("-killtcphogs");
						}

						arguments.Add("-Dir=%UE_HORDE_SHARED_DIR%/Uba");
						arguments.Add("-Eventfile=%UE_HORDE_TERMINATION_SIGNAL_FILE%");
						arguments.Add("-MaxCpu=%UE_HORDE_CPU_COUNT%");
						arguments.Add("-MulCpu=%UE_HORDE_CPU_MULTIPLIER%");
						arguments.Add("-MaxCon=8");
						arguments.Add($"-MaxWorkers={_hordeConfig.HordeMaxWorkers}");
						arguments.Add($"-MaxIdle={_hordeConfig.HordeMaxIdle}");
						arguments.Add("-UseCrawler");
						// arguments.Add("-ProxyUseLocalStorage"); // If set then agent's cas storage will be used by proxy. [honk] This is disabled because we suspect a hang in this feature
						//arguments.Add("-UseIocp=4");
						//arguments.Add("-NoStore"); // This means that no cas files will be written to disk.. so no state is stored between runs

						if (!String.IsNullOrEmpty(desc))
						{
							arguments.Add($"-Description=\"{desc}\"");
						}

						if (_config.bLogEnabled)
						{
							arguments.Add("-Log");
						}

						LogLevel logLevel = _config.bDetailedLog ? LogLevel.Information : LogLevel.Debug;

						string executable = _targetWorkerPlatform == OSPlatform.Windows ? "UbaAgent.exe" : "UbaAgent";
						logger.Log(logLevel, "Executing child process: {Executable} {Arguments}", executable, CommandLineArguments.Join(arguments));

						bool allowWine = _hordeConfig.bHordeAllowWine && GetTargetWorkerPlatform(_hordeConfig.AgentPlatform) == OSPlatform.Windows;
						ExecuteProcessFlags execFlags = allowWine ? ExecuteProcessFlags.UseWine : ExecuteProcessFlags.None;
						await using AgentManagedProcess process = await channel.ExecuteAsync(executable, arguments, null, null, execFlags, cancellationToken);
						bool shouldConnect = !useListen;
						bool isFirstRead = true;
						string? line;

						while ((line = await process.ReadLineAsync(cancellationToken)) != null)
						{
							logger.Log(logLevel, "{Line}", line);

							if (shouldConnect && line.Contains("Listening on", StringComparison.OrdinalIgnoreCase)) // This log entry means that the agent is ready for connections.
							{
								long totalMs = self.StartTime!.ElapsedMilliseconds;
								logger.LogInformation("Connecting to UbaAgent on {Ip}:{Port} (local agent port {AgentPort}) {Seconds}.{Milliseconds} seconds after assigned",
									self.Ip, self.Port!.Port, self.Port.AgentPort, totalMs / 1000, totalMs % 1000);

								if (!_scheduler!.AddClient(self.Ip!, self.Port.Port, _crypto))
								{
									break;
								}
								shouldConnect = false;
							}

							if (isFirstRead)
							{
								isFirstRead = false;
								self.Active = true;
								UpdateHordeStatus(null);
							}
						}
						self.NumLogicalCores = 0;
						UpdateHordeStatus(null);
						logger.LogDebug("Shutting down process");
						int ExitCode = await process.WaitForExitAsync(cancellationToken);
						if (ExitCode != 0)
						{
							logger.LogInformation("UbaAgent exited with exit code: {ExitCode}", ExitCode);
						}
					}

					logger.LogDebug("Closing channel");
					await lease.CloseAsync(cancellationToken);
				}
			}
			catch (TimeoutException)
			{
				self.NumLogicalCores = 0;
				UpdateHordeStatus(null);
			}
			catch (ComputeExecutionCancelledException ex)
			{
				// Cancellations are expected to happen due to spot instance interruptions or unscheduled maintenance of agents
				self.NumLogicalCores = 0;
				UpdateHordeStatus(null);

				if (!cancellationToken.IsCancellationRequested)
				{
					logger.Log(_strict ? LogLevel.Information : LogLevel.Debug, KnownLogEvents.Systemic_Horde_Compute, ex, "Compute lease cancelled");
				}
			}
			catch (Exception ex)
			{
				self.NumLogicalCores = 0;
				UpdateHordeStatus(null);

				if (!cancellationToken.IsCancellationRequested)
				{
					logger.Log(_strict ? LogLevel.Error : LogLevel.Debug, KnownLogEvents.Systemic_Horde_Compute, ex, "Exception in worker task: {Ex}", ex.ToString());

					// Add additional properties to aid debugging
					logger.Log(_strict ? LogLevel.Information : LogLevel.Debug, KnownLogEvents.Systemic_Horde_Compute, ex, "UBA agent locator {UBAAgentLocator}", _ubaAgentLocator.ToString());
				}
			}
		}

		string _additionalStatus = "";
		string _lastStatus = "";

		internal bool DisableRemoteExecutionOnAgent()
		{
			lock (_workers)
			{
				for (int i = _workers.Count - 1; i >= 0; i--)
				{
					Worker worker = _workers[i];
					if (!worker.Active)
					{
						continue;
					}
					_scheduler?.DisableRemoteExecutionOnAgent(worker.Name!);
					return true;
				}
			}
			return false;
		}

		internal void UpdateHordeStatus(string? additionalStatus)
		{
			if (additionalStatus != null)
			{
				_additionalStatus = additionalStatus;
			}

			if (_updateStatus == null)
			{
				return;
			}

			int activeCores;
			int queuedCores;
			int activeAgents;
			int queuedAgents;
			GetCoreCount(out activeCores, out queuedCores, out activeAgents, out queuedAgents);
			string status = $"Running. {activeAgents} agent{(activeAgents != 1 ? "s" : "")} ({activeCores} cores){_additionalStatus} {(queuedAgents != 0 ? $"(Preparing {queuedAgents} agent{(queuedAgents != 1 ? "s" : "")})" : "")}";
			if (_lastStatus != status)
			{
				_lastStatus = status;
				_updateStatus(0, 6, status, EpicGames.UBA.LogEntryType.Info, null);
			}
		}

		static OSPlatform GetTargetWorkerPlatform(string? searchString)
		{
			if (!String.IsNullOrEmpty(searchString))
			{
				if (searchString.Contains("Linux", StringComparison.OrdinalIgnoreCase))
				{
					return OSPlatform.Linux;
				}
				if (searchString.Contains("Windows", StringComparison.OrdinalIgnoreCase))
				{
					return OSPlatform.Windows;
				}
				if (searchString.Contains("MacOS", StringComparison.OrdinalIgnoreCase))
				{
					return OSPlatform.OSX;
				}
			}

			return UBAExecutor.GetHostPlatform();
		}
	}
}
