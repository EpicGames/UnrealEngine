// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Reflection;
using EpicGames.Core;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// Configuration for Unreal Build Accelerator
	/// </summary>
	[System.Diagnostics.CodeAnalysis.SuppressMessage("Style", "IDE1006:Naming Styles", Justification = "UnrealBuildTool naming style")]
	class UnrealBuildAcceleratorCacheConfig
	{
		/// <summary>
		/// Address of the uba cache service. Will automatically use cache if connected
		/// </summary>
		[XmlConfigFile(Category = "UnrealBuildAccelerator", Name = "Cache")]
		[ConfigFile(ConfigHierarchyType.Engine, "Uba.CacheProvider", "Url")]
		[CommandLine("-UBACache=")]
		public string CacheServer { get; set; } = String.Empty;

		/// <summary>
		/// Set cache to write, expects one of [True, False, BuildMachineOnly]
		/// </summary>
		[XmlConfigFile(Category = "UnrealBuildAccelerator", Name = "WriteCache")]
		[ConfigFile(ConfigHierarchyType.Engine, "Uba.CacheProvider", "Write")]
		[CommandLine("-UBAWriteCache", Value = "true")]
		public string WriteCache { get; set; } = "BuildMachineOnly";

		/// <summary>
		/// Set cache to require VFS to be enabled
		/// </summary>
		[XmlConfigFile(Category = "UnrealBuildAccelerator", Name = "RequireVFS")]
		[ConfigFile(ConfigHierarchyType.Engine, "Uba.CacheProvider", "RequireVFS")]
		public bool bRequireVfs { get; set; } = false;

		/// <summary>
		/// If the cache can be written to
		/// </summary>
		public bool CanWrite => WriteCache.Equals("True", StringComparison.OrdinalIgnoreCase) || (Unreal.IsBuildMachine() && WriteCache.Equals("BuildMachineOnly", StringComparison.OrdinalIgnoreCase));

		/// <summary>
		/// Crypto used to connect to cache server.
		/// </summary>
		[XmlConfigFile(Category = "UnrealBuildAccelerator", Name = "CacheCrypto")]
		[ConfigFile(ConfigHierarchyType.Engine, "Uba.CacheProvider", "Crypto")]
		public string Crypto { get; set; } = String.Empty;

		/// <summary>
		/// Desired connection count. When there is latency it might be a good idea to have multiple tcp connections
		/// </summary>
		[XmlConfigFile(Category = "UnrealBuildAccelerator", Name = "CacheDesiredConnectionCount")]
		[ConfigFile(ConfigHierarchyType.Engine, "Uba.CacheProvider", "DesiredConnectionCount")]
		public int DesiredConnectionCount { get; set; } = 1;
	}

	#region Extension methods
	/// <summary>
	/// Extension methods for FileReference functionality
	/// </summary>
	internal static class UnrealBuildAcceleratorCacheConfigExtensionMethods
	{
		/// <summary>
		/// Load horde config from a different section name
		/// </summary>
		/// <param name="config"></param>
		/// <param name="ini"></param>
		/// <param name="section"></param>
		internal static void LoadConfigProvider(this UnrealBuildAcceleratorCacheConfig config, ConfigHierarchy ini, string section)
		{
			object? TryGetValue(Type propertyType, string keyName)
			{
				if (propertyType == typeof(string))
				{
					if (ini.TryGetValue(section, keyName, out string? value))
					{
						return value;
					}
				}
				else if (propertyType == typeof(bool))
				{
					if (ini.TryGetValue(section, keyName, out bool value))
					{
						return value;
					}
				}
				else if (propertyType == typeof(int))
				{
					if (ini.TryGetValue(section, keyName, out int value))
					{
						return value;
					}
				}
				else if (propertyType == typeof(float))
				{
					if (ini.TryGetValue(section, keyName, out float value))
					{
						return value;
					}
				}
				else if (propertyType == typeof(double))
				{
					if (ini.TryGetValue(section, keyName, out double value))
					{
						return value;
					}
				}
				else
				{
					throw new BuildLogEventException("UnrealBuildAcceleratorCacheConfig.LoadConfigProvider: Unsupported type {Type} for Key {KeyName}", propertyType, keyName);
				}
				return null;
			}

			IEnumerable<PropertyInfo> properties = config.GetType().GetProperties(BindingFlags.Instance | BindingFlags.Public | BindingFlags.NonPublic);
			foreach (PropertyInfo property in properties)
			{
				Type propertyType = property.PropertyType;
				ConfigFileAttribute? attribute = property.GetCustomAttributes<ConfigFileAttribute>().FirstOrDefault();
				if (attribute != null)
				{
					string keyName = attribute.KeyName ?? property.Name;
					object? value = TryGetValue(propertyType, keyName) ?? null;
					if (value != null)
					{
						property.SetValue(config, value);
					}
				}
			}
		}
	}
	#endregion
}