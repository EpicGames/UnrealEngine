// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using EpicGames.Core;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// Configuration for Unreal Build Accelerator
	/// </summary>
	[System.Diagnostics.CodeAnalysis.SuppressMessage("Style", "IDE1006:Naming Styles", Justification = "UnrealBuildTool naming style")]
	class UnrealBuildAcceleratorConfig
	{
		/// <summary>
		/// List of provider names for remote connections.
		/// </summary>
		[XmlConfigFile(Category = "UnrealBuildAccelerator")]
		public string[] Providers { get; set; } = [];

		/// <summary>
		/// List of provider names for remote connections from ini
		/// It is recommended to set Providers in ini so the setting can be shared with the uba controller in editor, such as
		/// Windows:	%ProgramData%/Epic/Unreal Engine/Engine/Config/SystemEngine.ini
		/// Mac:		~/Library/Application Support/Epic/Unreal Engine/Engine/Config/SystemEngine.ini
		/// Linux:		~/.config/Epic/Unreal Engine/Engine/Config/SystemEngine.ini
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "UbaController", "Providers")]
		[CommandLine("-UBAProviders=", ListSeparator = '+')]
		public List<string> IniProviders { get; set; } = [];

		/// <summary>
		/// List of provider names for remote connections
		/// </summary>
		[XmlConfigFile(Category = "UnrealBuildAccelerator")]
		public string[] BuildMachineProviders { get; set; } = [];

		/// <summary>
		/// List of provider names for remote connections from ini
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "UbaController", "BuildMachineProviders")]
		public List<string> IniBuildMachineProviders { get; set; } = [];

		/// <summary>
		/// List of provider names for remote cache.
		/// </summary>
		[XmlConfigFile(Category = "UnrealBuildAccelerator")]
		public string[] CacheProviders { get; set; } = [];

		/// <summary>
		/// List of provider names for remote cache from ini
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "UbaController", "CacheProviders")]
		[CommandLine("-UBACacheProviders=", ListSeparator = '+')]
		public List<string> IniCacheProviders { get; set; } = [];

		/// <summary>
		/// List of provider names for remote cache
		/// </summary>
		[XmlConfigFile(Category = "UnrealBuildAccelerator")]
		public string[] BuildMachineCacheProviders { get; set; } = [];

		/// <summary>
		/// List of provider names for remote cache from ini
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "UbaController", "BuildMachineCacheProviders")]
		public List<string> IniBuildMachineCacheProviders { get; set; } = [];

		private bool _disableRemote = false;
		/// <summary>
		/// When set to true, UBA will not use any remote help
		/// </summary>
		[XmlConfigFile(Category = "UnrealBuildAccelerator")]
		[CommandLine("-UBADisableRemote")]
		public bool bDisableRemote
		{
			// Remote must be disabled if we don't allow detouring.
			get => !bAllowDetour || _disableRemote;
			set => _disableRemote = value;
		}

		/// <summary>
		/// When set to true, UBA will force all actions that can be built remotely to be built remotely. This will hang if there are no remote agents available
		/// </summary>
		[XmlConfigFile(Category = "UnrealBuildAccelerator")]
		[CommandLine("-UBAForceRemote")]
		public bool bForceBuildAllRemote { get; set; } = false;

		private bool _allowRetry = true;
		/// <summary>
		/// When set to true, actions that fail locally with certain error codes will retry without uba.
		/// </summary>
		[XmlConfigFile(Category = "UnrealBuildAccelerator")]
		[CommandLine("-UBANoRetry", Value = "false")]
		public bool bAllowRetry
		{
			get => bForcedRetry ? true : _allowRetry;
			set => _allowRetry = value;
		}

		/// <summary>
		/// When set to true, all actions that fail locally with UBA will be retried without UBA.
		/// </summary>
		[XmlConfigFile(Category = "UnrealBuildAccelerator")]
		[CommandLine("-UBAForcedRetry")]
		public bool bForcedRetry { get; set; } = false;

		/// <summary>
		/// When set to true, actions that fail remotely with UBA will be retried locally with UBA.
		/// </summary>
		[XmlConfigFile(Category = "UnrealBuildAccelerator")]
		[CommandLine("-UBAForcedRetryRemote")]
		public bool bForcedRetryRemote { get; set; } = false;

		/// <summary>
		/// When set to true, all errors and warnings from UBA will be output at the appropriate severity level to the log (rather than being output as 'information' and attempting to continue regardless).
		/// </summary>
		[XmlConfigFile(Category = "UnrealBuildAccelerator")]
		[CommandLine("-UBAStrict")]
		public bool bStrict { get; set; } = false;

		/// <summary>
		/// If UBA should store cas compressed or raw
		/// </summary>
		[XmlConfigFile(Category = "UnrealBuildAccelerator")]
		[CommandLine("-UBAStoreRaw")]
		public bool bStoreRaw { get; set; } = false;

		/// <summary>
		/// If UBA should distribute linking to remote workers. This needs bandwidth but can be an optimization
		/// </summary>
		[XmlConfigFile(Category = "UnrealBuildAccelerator")]
		[CommandLine("-UBALinkRemote")]
		public bool bLinkRemote { get; set; } = false;

		/// <summary>
		/// The amount of gigabytes UBA is allowed to use to store workset and cached data. It is a good idea to have this >10gb
		/// </summary>
		[XmlConfigFile(Category = "UnrealBuildAccelerator")]
		[CommandLine("-UBAStoreCapacityGb")]
		public int StoreCapacityGb { get; set; } = 40;

		/// <summary>
		/// Max number of worker threads that can handle messages from remotes. 
		/// </summary>
		[XmlConfigFile(Category = "UnrealBuildAccelerator")]
		[CommandLine("-UBAMaxWorkers")]
		public int MaxWorkers { get; set; } = 192;

		/// <summary>
		/// Max size of each message sent from server to client
		/// </summary>
		[XmlConfigFile(Category = "UnrealBuildAccelerator")]
		[CommandLine("-UBASendSize")]
		public int SendSize { get; set; } = 256 * 1024;

		/// <summary>
		/// Which ip UBA server should listen to for connections
		/// </summary>
		[XmlConfigFile(Category = "UnrealBuildAccelerator")]
		[CommandLine("-UBAHost")]
		public string Host { get; set; } = "0.0.0.0";

		/// <summary>
		/// Which port UBA server should listen to for connections.
		/// </summary>
		[XmlConfigFile(Category = "UnrealBuildAccelerator")]
		[CommandLine("-UBAPort")]
		public int Port { get; set; } = 1345;

		/// <summary>
		/// Which directory to store files for UBA.
		/// </summary>
		[XmlConfigFile(Category = "UnrealBuildAccelerator")]
		[CommandLine("-UBARootDir")]
		public string? RootDir { get; set; } = null;

		/// <summary>
		/// Use Quic protocol instead of Tcp (experimental)
		/// </summary>
		[XmlConfigFile(Category = "UnrealBuildAccelerator")]
		[CommandLine("-UBAQuic", Value = "true")]
		public bool bUseQuic { get; set; } = false;

		/// <summary>
		/// Enable logging of UBA processes
		/// </summary>
		[XmlConfigFile(Category = "UnrealBuildAccelerator")]
		[CommandLine("-UBALog", Value = "true")]
		public bool bLogEnabled { get; set; } = false;

		/// <summary>
		/// Prints summary of UBA stats at end of build
		/// </summary>
		[XmlConfigFile(Category = "UnrealBuildAccelerator")]
		[CommandLine("-UBAPrintSummary", Value = "true")]
		public bool bPrintSummary { get; set; } = false;

		/// <summary>
		/// Launch visualizer application which shows build progress
		/// </summary>
		[XmlConfigFile(Category = "UnrealBuildAccelerator")]
		[ConfigFile(ConfigHierarchyType.Engine, "UbaController", "bLaunchVisualizer")]
		[CommandLine("-UBAVisualizer", Value = "true")]
		public bool bLaunchVisualizer { get; set; } = false;

		/// <summary>
		/// Resets the cas cache
		/// </summary>
		[XmlConfigFile(Category = "UnrealBuildAccelerator")]
		[CommandLine("-UBAResetCas", Value = "true")]
		public bool bResetCas { get; set; } = false;

		/// <summary>
		/// Provide custom path for trace output file
		/// </summary>
		[XmlConfigFile(Category = "UnrealBuildAccelerator")]
		[CommandLine("-UBATraceOutputFile")]
		public string TraceFile { get; set; } = String.Empty;

		/// <summary>
		/// Add verbose details to the UBA trace
		/// </summary>
		[XmlConfigFile(Category = "UnrealBuildAccelerator")]
		[CommandLine("-UBADetailedTrace", Value = "true")]
		public bool bDetailedTrace { get; set; }

		/// <summary>
		/// Disable UBA waiting on available memory before spawning new processes
		/// </summary>
		[XmlConfigFile(Category = "UnrealBuildAccelerator")]
		[CommandLine("-UBADisableWaitOnMem", Value = "true")]
		public bool bDisableWaitOnMem { get; set; }

		/// <summary>
		/// Let UBA kill running processes when close to out of memory
		/// </summary>
		[XmlConfigFile(Category = "UnrealBuildAccelerator")]
		[CommandLine("-UBAAllowKillOnMem", Value = "true")]
		public bool bAllowKillOnMem { get; set; }

		/// <summary>
		/// Store object (.obj) compressed on disk. Requires uba to do link step where it will decompress obj files again
		/// </summary>
		[XmlConfigFile(Category = "UnrealBuildAccelerator")]
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/BuildSettings.BuildSettings", "bStoreObjFilesCompressed")]
		[CommandLine("-UBAStoreObjFilesCompressed", Value = "true")]
		[CommandLine("-UBADisableStoreObjFilesCompressed", Value = "false")]
		[CommandLine("-nocompress", Value = "false")]
		public bool bStoreObjFilesCompressed { get; set; }

		/// <summary>
		/// Skip writing intermediate and output files to disk. Useful for validation builds where we don't need the output
		/// </summary>
		[XmlConfigFile(Category = "UnrealBuildAccelerator")]
		[CommandLine("-UBANoWrite", Value = "false")]
		public bool bWriteToDisk { get; set; } = true;

		/// <summary>
		/// Set to true to disable mimalloc and detouring of memory allocations.
		/// </summary>
		[XmlConfigFile(Category = "UnrealBuildAccelerator")]
		[CommandLine("-UBANoCustomMalloc", Value = "true")]
		public bool bDisableCustomAlloc { get; set; } = false;

		private string _zone = String.Empty;
		/// <summary>
		/// The zone to use for UBA.
		/// </summary>
		[XmlConfigFile(Category = "UnrealBuildAccelerator")]
		[CommandLine("-UBAZone=")]
		public string Zone
		{
			get => bDisableRemote ? "local" : _zone;
			set => _zone = value;
		}

		/// <summary>
		/// Set to true to enable encryption when transferring files over the network.
		/// </summary>
		[XmlConfigFile(Category = "UnrealBuildAccelerator")]
		[CommandLine("-UBACrypto", Value = "true")]
		public bool bUseCrypto { get; set; } = false;

		/// <summary>
		/// Set to true to provide known inputs to processes that are run remote. This is an experimental feature to speed up build times when ping is higher
		/// </summary>
		[XmlConfigFile(Category = "UnrealBuildAccelerator")]
		[CommandLine("-UBAUseKnownInputs", Value = "true")]
		public bool bUseKnownInputs { get; set; } = true;

		/// <summary>
		/// Write yaml file with all actions that are queued for build. This can be used to replay using "UbaCli.exe local file.yaml"
		/// </summary>
		[XmlConfigFile(Category = "UnrealBuildAccelerator")]
		[CommandLine("-UBAActionsOutputFile")]
		public string ActionsOutputFile { get; set; } = String.Empty;

		/// <summary>
		/// Set to true to see more info about what is happening inside uba and also log output from agents
		/// </summary>
		[XmlConfigFile(Category = "UnrealBuildAccelerator")]
		[CommandLine("-UBADetailedLog", Value = "true")]
		public bool bDetailedLog { get; set; } = false;

		/// <summary>
		/// Disable all detouring making all actions run outside uba
		/// </summary>
		[XmlConfigFile(Category = "UnrealBuildAccelerator", Name = "AllowDetour")]
		[CommandLine("-UBANoDetour", Value = "false")]
		public bool bAllowDetour { get; set; } = true;

		/// <summary>
		/// Write zero-byte placeholder files for exe/dll/pdb instead of the full files. This can be useful for validation builds or builds just populating cache
		/// </summary>
		[XmlConfigFile(Category = "UnrealBuildAccelerator", Name = "WritePlaceholders")]
		[CommandLine("-UBAWritePlaceholders", Value = "true")]
		public bool bWritePlaceholders { get; set; } = false;

		/// <summary>
		/// Makes the shared memory used in UBA to be backed by a Temp+DeleteOnClose file. This trades committed memory charge for the risk of more I/O under RAM pressure
		/// </summary>
		[XmlConfigFile(Category = "UnrealBuildAccelerator", Name = "SharedMemoryTempFile")]
		[CommandLine("-UBASharedMemoryTempFile")]
		public string SharedMemoryTempFile { get; set; } = String.Empty;

		/// <summary>
		/// With this enabled local machine will start local processes to race remotes when utilization goes under percent number
		/// So if local machine can run 20 actions and only run 10, that means that it is 50% utilized. If MaxRacingPercent
		/// is set to 70%, local machine will look at what actions remotes are running and start running them too until it reaches
		/// 70% utilization
		/// </summary>
		[XmlConfigFile(Category = "UnrealBuildAccelerator", Name = "MaxRacingPercent")]
		[CommandLine("-UBAMaxRacingPercent")]
		public int MaxRacingPercent { get; set; } = 0;

		/// <summary>
		/// Max number of cache download tasks that can execute in parallel
		/// </summary>
		[XmlConfigFile(Category = "UnrealBuildAccelerator", Name = "CacheMaxWorkers")]
		[CommandLine("-UBACacheMaxWorkers")]
		public int CacheMaxWorkers { get; set; } = 32;

		/// <summary>
		/// Shuffle the list of cache servers before each read attempt
		/// </summary>
		[XmlConfigFile(Category = "UnrealBuildAccelerator", Name = "CacheShuffle")]
		[CommandLine("-UBAShuffleCache")]
		public bool bCacheShuffle { get; set; } = false;

		/// <summary>
		/// Shuffle the list of cache servers before each read attempt
		/// </summary>
		[XmlConfigFile(Category = "UnrealBuildAccelerator", Name = "ForceNoCache")]
		[CommandLine("-nocache")]
		public bool bForceNoCache { get; set; } = false;

		/// <summary>
		/// Report reason a cache miss happened. Useful when searching for determinism/portability issues
		/// </summary>
		[XmlConfigFile(Category = "UnrealBuildAccelerator", Name = "ReportCacheMissReason")]
		[CommandLine("-UBAReportCacheMissReason")]
		public bool bReportCacheMissReason { get; set; }

		/// <summary>
		/// Control if link actions should be downloaded from cache or not
		/// </summary>
		[XmlConfigFile(Category = "UnrealBuildAccelerator", Name = "CacheLinkActions")]
		public bool bCacheLinkActions { get; set; } = true;

		/// <summary>
		/// Compression level. Options are None, SuperFast, VeryFast, Fast, Normal, Optimal1 to Optimal5
		/// </summary>
		[XmlConfigFile(Category = "UnrealBuildAccelerator")]
		[CommandLine("-UBACompressionLevel")]
		public string CompressionLevel { get; set; } = String.Empty;

		/// <summary>
		/// Processor count multiplier for local execution. Can be below 1 to reserve CPU for other tasks.
		/// When using the local executor (not XGE), run a single action on each CPU core. Note that you can set this to a larger value
		/// to get slightly faster build times in many cases, but your computer's responsiveness during compiling may be much worse.
		/// This value is ignored if the CPU does not support hyper-threading.
		/// </summary>
		[XmlConfigFile(Category = "ParallelExecutor")]
		public double ProcessorCountMultiplier { get; set; } = 1.0;

		/// <summary>
		/// Free memory per action in bytes, used to limit the number of parallel actions if the machine is memory starved.
		/// Set to 0 to disable free memory checking.
		/// </summary>
		[XmlConfigFile(Category = "ParallelExecutor")]
		public double MemoryPerActionBytes { get; set; } = 1.5 * 1024 * 1024 * 1024;

		/// <summary>
		/// The priority to set for spawned processes.
		/// Valid Settings: Idle, BelowNormal, Normal, AboveNormal, High
		/// Default: BelowNormal or Normal for an Asymmetrical processor as BelowNormal can cause scheduling issues.
		/// </summary>
		[XmlConfigFile(Category = "ParallelExecutor")]
		public ProcessPriorityClass ProcessPriority { get; set; } = SystemUtils.IsAsymmetricalProcessor() ? ProcessPriorityClass.Normal : ProcessPriorityClass.BelowNormal;

		/// <summary>
		/// When enabled, will stop compiling targets after a compile error occurs.
		/// </summary>
		[XmlConfigFile(Category = "ParallelExecutor")]
		public bool bStopCompilationAfterErrors { get; set; } = false;

		/// <summary>
		/// When set, will stop compiling targets after a set number of errors occurs.
		/// </summary>
		[XmlConfigFile(Category = "ParallelExecutor")]
		public int StopCompilationAfterNumErrors { get; set; } = Int32.MaxValue;

		/// <summary>
		/// Whether to show compilation times along with worst offenders or not.
		/// </summary>
		[XmlConfigFile(Category = "ParallelExecutor")]
		public bool bShowCompilationTimes { get; set; } = Unreal.IsBuildMachine();

		/// <summary>
		/// Whether to show compilation times for each executed action
		/// </summary>
		[XmlConfigFile(Category = "ParallelExecutor")]
		public bool bShowPerActionCompilationTimes { get; set; } = Unreal.IsBuildMachine();

		/// <summary>
		/// Whether to log command lines for actions being executed
		/// </summary>
		[XmlConfigFile(Category = "ParallelExecutor")]
		public bool bLogActionCommandLines { get; set; } = false;

		/// <summary>
		/// Add target names for each action executed
		/// </summary>
		[XmlConfigFile(Category = "ParallelExecutor")]
		public bool bPrintActionTargetNames { get; set; } = false;

		/// <summary>
		/// Whether to show CPU utilization after the work is complete.
		/// </summary>
		[XmlConfigFile(Category = "ParallelExecutor")]
		public bool bShowCPUUtilization { get; set; } = Unreal.IsBuildMachine();

		/// <summary>
		/// Whether to cleanup the UBA executor's resources on dispose. This will result in resources being freed sooner,
		/// rather than waiting until process shutdown, at the expense of taking longer before the UBA process terminates.
		/// Unless you know your UBT process will significantly outlive the UBA executor and need to free up resources now,
		/// it's suggested you leave this as false.
		/// </summary>
		public bool bCleanupOnDispose { get; set; } = false;

		/// <summary>
		/// Whether UBA can retry actions natively if they fail when detoured locally
		/// </summary>
		public bool CanRetryNatively { get; set; } = true;

		/// <summary>
		/// Disable this to stop UbaAgent instances to use casdb store
		/// </summary>
		public bool UseAgentStore { get; set; } = true;
	}
}