// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Reflection;
using EpicGames.Core;
using EpicGames.Horde.Compute;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// Configuration for Unreal Build Accelerator Horde session
	/// </summary>
	[System.Diagnostics.CodeAnalysis.SuppressMessage("Style", "IDE1006:Naming Styles", Justification = "UnrealBuildTool naming style")]
	class UnrealBuildAcceleratorHordeConfig
	{
		/// <summary>
		/// Uri of the Horde server
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "Uba.Provider.Horde", "ServerUrl")]
		[XmlConfigFile(Category = "Horde", Name = "Server")]
		[CommandLine("-UBAHorde=")]
		public string? HordeServer { get; set; }

		/// <summary>
		/// Auth token for the Horde server
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "Uba.Provider.Horde", "UbaToken")]
		[XmlConfigFile(Category = "Horde", Name = "Token")]
		[CommandLine("-UBAHordeToken=")]
		public string? HordeToken { get; set; }

		/// <summary>
		/// OIDC id for the login to use
		/// </summary>
		[Obsolete("Deprecated in UE5.8. The OidcProvider option is no longer used")]
		[XmlConfigFile(Category = "Horde", Name = "OidcProvider")]
		[CommandLine("-UBAHordeOidc=")]
		public string? HordeOidcProvider { get; set; }

		/// <summary>
		/// Pool for the Horde agent to assign, calculated based on platform and overrides
		/// </summary>
		public string? HordePool
		{
			get
			{
				string? Pool = DefaultHordePool;
				if (OverrideHordePool != null)
				{
					Pool = OverrideHordePool;
				}
				else if (OperatingSystem.IsWindows() && WindowsHordePool != null)
				{
					Pool = WindowsHordePool;
				}
				else if (OperatingSystem.IsMacOS() && MacHordePool != null)
				{
					Pool = MacHordePool;
				}
				else if (OperatingSystem.IsLinux() && LinuxHordePool != null)
				{
					Pool = LinuxHordePool;
				}

				return Pool;
			}
		}

		/// <summary>
		/// Pool for the Horde agent to assign if no override current platform doesn't have it set
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "Uba.Provider.Horde", "UbaPool")]
		[XmlConfigFile(Category = "Horde", Name = "Pool")]
		public string? DefaultHordePool { get; set; }

		/// <summary>
		/// Pool for the Horde agent to assign, only used for commandline override
		/// </summary>
		[CommandLine("-UBAHordePool=")]
		public string? OverrideHordePool { get; set; }

		/// <summary>
		/// Pool for the Horde agent to assign when on Linux
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "Uba.Provider.Horde", "UbaLinuxPool")]
		[XmlConfigFile(Category = "Horde", Name = "LinuxPool")]
		public string? LinuxHordePool { get; set; }

		/// <summary>
		/// Pool for the Horde agent to assign when on Mac
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "Uba.Provider.Horde", "UbaMacPool")]
		[XmlConfigFile(Category = "Horde", Name = "MacPool")]
		public string? MacHordePool { get; set; }

		/// <summary>
		/// Pool for the Horde agent to assign when on Windows
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "Uba.Provider.Horde", "UbaWindowsPool")]
		[XmlConfigFile(Category = "Horde", Name = "WindowsPool")]
		public string? WindowsHordePool { get; set; }

		/// <summary>
		/// Requirements for the Horde agent to assign
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "Uba.Provider.Horde", "UbaRequirements")]
		[XmlConfigFile(Category = "Horde", Name = "Requirements")]
		[CommandLine("-UBAHordeRequirements=")]
		public string? HordeCondition { get; set; }

		/// <summary>
		/// Default compute cluster ID if nothing is set
		/// </summary>
		public const string ClusterDefault = "default";

		/// <summary>
		/// Compute cluster ID for automatically resolving the most suitable cluster
		/// </summary>
		public const string ClusterAuto = "_auto";

		/// <summary>
		/// Compute cluster ID to use in Horde. Set to "_auto" to let Horde server resolve a suitable cluster.
		/// In multi-region setups this is can simplify configuration of UBT/UBA a lot.
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "Uba.Provider.Horde", "UbaCluster")]
		[XmlConfigFile(Category = "Horde", Name = "Cluster")]
		[CommandLine("-UBAHordeCluster=")]
		public string? HordeCluster { get; set; }

		/// <summary>
		/// Which ip UBA server should give to agents. This will invert so host listens and agents connect
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "Uba.Provider.Horde", "UbaLocalHost")]
		[XmlConfigFile(Category = "Horde", Name = "LocalHost")]
		[CommandLine("-UBAHordeHost")]
		public string HordeHost { get; set; } = String.Empty;

		/// <summary>
		/// Max cores allowed to be used by build session
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "Uba.Provider.Horde", "MaxCores")]
		[XmlConfigFile(Category = "Horde", Name = "MaxCores")]
		[CommandLine("-UBAHordeMaxCores")]
		public int HordeMaxCores { get; set; } = 576;

		/// <summary>
		/// Max workers allowed to be used by build session
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "Uba.Provider.Horde", "MaxWorkers")]
		[XmlConfigFile(Category = "Horde", Name = "MaxWorkers")]
		[CommandLine("-UBAHordeMaxWorkers")]
		public int HordeMaxWorkers { get; set; } = 64;

		/// <summary>
		/// Max idle time in seconds to maintain a connection to a worker when no pending actions are currently available due to waiting on dependencies
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "Uba.Provider.Horde", "MaxIdle")]
		[XmlConfigFile(Category = "Horde", Name = "MaxIdle")]
		[CommandLine("-UBAHordeMaxIdle")]
		public int HordeMaxIdle { get; set; } = 15;

		/// <summary>
		/// How long UBT should wait to ask for help. Useful in build configs where machine can delay remote work and still get same wall time results (pch dependencies etc)
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "Uba.Provider.Horde", "UbaDelay")]
		[XmlConfigFile(Category = "Horde", Name = "StartupDelay")]
		[CommandLine("-UBAHordeDelay")]
		public int HordeDelay { get; set; } = 0;

		/// <summary>
		/// Allow use of Wine. Only applicable to Horde agents running Linux. Can still be ignored if Wine executable is not set on agent.
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "Uba.Provider.Horde", "UbaAllowWine")]
		[XmlConfigFile(Category = "Horde", Name = "AllowWine")]
		[CommandLine("-UBAHordeAllowWine", Value = "true")]
		public bool bHordeAllowWine { get; set; } = true;

		/// <summary>
		/// Connection mode for agent/compute communication
		/// <see cref="ConnectionMode" /> for valid modes.
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "Uba.Provider.Horde", "UbaConnectionMode")]
		[XmlConfigFile(Category = "Horde", Name = "ConnectionMode")]
		[CommandLine("-UBAHordeConnectionMode=")]
		public string? HordeConnectionMode { get; set; }

		/// <summary>
		/// Encryption to use for agent/compute communication. Note that UBA agent uses its own encryption.
		/// <see cref="Encryption" /> for valid modes.
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "Uba.Provider.Horde", "UbaEncryption")]
		[XmlConfigFile(Category = "Horde", Name = "Encryption")]
		[CommandLine("-UBAHordeEncryption=")]
		public string? HordeEncryption { get; set; }

		/// <summary>
		/// Sentry URL to send box data to. Optional.
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "Uba.Provider.Horde", "UbaSentryUrl")]
		[XmlConfigFile(Category = "Horde")]
		[CommandLine("-UBASentryUrl=")]
		public string? UBASentryUrl { get; set; }

		/// <summary>
		/// Disable horde all together
		/// </summary>
		[XmlConfigFile(Category = "UnrealBuildAccelerator")]
		[CommandLine("-UBADisableHorde")]
		public bool bDisableHorde { get; set; } = false;

		/// <summary>
		/// Enabled parameter for use by INI config, expects one of [True, False, BuildMachineOnly]
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "Uba.Provider.Horde", "UbaEnabled")]
		public string? HordeEnabled { get; set; } = null;

		/// <summary>
		/// Agents enabled parameter for use by INI config, expects one of [True, False, BuildMachineOnly]
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "Uba.Provider.Horde", "UbaAgentsEnabled")]
		public bool bAgentsEnabled { get; set; } = true;

		/// <summary>
		/// Enabled parameter for use by INI config, expects one of [True, False, BuildMachineOnly]
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "Uba.Provider.Horde", "UbaCacheEnabled")]
		[CommandLine("-UBAHordeCacheEnabled")]
		public bool bCacheEnabled { get; set; } = false;

		/// <summary>
		/// Number of desired connections when connecting to cache
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "Uba.Provider.Horde", "UbaCacheDesiredConnectionCount")]
		[CommandLine("-UBAHordeCacheDesiredConnectionCount=")]
		public int CacheDesiredConnectionCount { get; set; } = 1;

		/// <summary>
		/// Set which platform the agents requested should have. If not set, default will be host platform
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "Uba.Provider.Horde", "UbaAgentPlatform")]
		[XmlConfigFile(Category = "Horde", Name = "AgentPlatform")]
		public string? AgentPlatform { get; set; }

		/// <summary>
		/// When true, enables verbose OIDC auth logging for diagnostics
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "Uba.Provider.Horde", "UbaVerboseAuthLogging")]
		[XmlConfigFile(Category = "Horde", Name = "VerboseAuthLogging")]
		[CommandLine("-UBAHordeVerboseAuthLogging")]
		public bool VerboseAuthLogging { get; set; } = true;
	}

	#region Extension methods
	/// <summary>
	/// Extension methods for FileReference functionality
	/// </summary>
	internal static class UnrealBuildAcceleratorHordeConfigExtensionMethods
	{
		/// <summary>
		/// Load horde config from a different section name
		/// </summary>
		/// <param name="config"></param>
		/// <param name="ini"></param>
		/// <param name="section"></param>
		internal static void LoadConfigProvider(this UnrealBuildAcceleratorHordeConfig config, ConfigHierarchy ini, string section)
		{
			object? TryGetValue(Type propertyType, string keyName)
			{
				if (propertyType == typeof(string))
				{
					if (ini.TryGetValue(section, keyName, out string? value))
					{
						return value;
					}
				}
				else if (propertyType == typeof(bool))
				{
					if (ini.TryGetValue(section, keyName, out bool value))
					{
						return value;
					}
				}
				else if (propertyType == typeof(int))
				{
					if (ini.TryGetValue(section, keyName, out int value))
					{
						return value;
					}
				}
				else if (propertyType == typeof(float))
				{
					if (ini.TryGetValue(section, keyName, out float value))
					{
						return value;
					}
				}
				else if (propertyType == typeof(double))
				{
					if (ini.TryGetValue(section, keyName, out double value))
					{
						return value;
					}
				}
				else
				{
					throw new BuildLogEventException("UnrealBuildAcceleratorHordeConfig.LoadConfigProvider: Unsupported type {Type} for Key {KeyName}", propertyType, keyName);
				}
				return null;
			}

			IEnumerable<PropertyInfo> properties = config.GetType().GetProperties(BindingFlags.Instance | BindingFlags.Public | BindingFlags.NonPublic);
			foreach (PropertyInfo property in properties)
			{
				Type propertyType = property.PropertyType;
				ConfigFileAttribute? attribute = property.GetCustomAttributes<ConfigFileAttribute>().FirstOrDefault();
				if (attribute != null)
				{
					string keyName = attribute.KeyName ?? property.Name;
					object? value = TryGetValue(propertyType, keyName) ?? (keyName.StartsWith("Uba", StringComparison.Ordinal) ? TryGetValue(propertyType, keyName[3..]) : null);
					if (value != null)
					{
						property.SetValue(config, value);
					}
				}
			}
		}
	}
	#endregion
}