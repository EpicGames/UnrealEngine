// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections;
using System.Collections.Generic;
using System.Diagnostics;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using System.Linq;
using System.Net;
using System.Net.NetworkInformation;
using System.Runtime.InteropServices;
using System.Runtime.Versioning;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using Microsoft.Win32;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	class XGE : ActionExecutor
	{
#pragma warning disable IDE0044 // Make field readonly - these private static fields are set by XML configuration.
		/// <summary>
		/// Whether to use the no_watchdog_thread option to prevent VS2015 toolchain stalls.
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		bool bXGENoWatchdogThread = false;

		/// <summary>
		/// Whether to display the XGE build monitor.
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		bool bShowXGEMonitor = false;

		/// <summary>
		/// When enabled, XGE will stop compiling targets after a compile error occurs.  Recommended, as it saves computing resources for others.
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		bool bStopXGECompilationAfterErrors = false;

		/// <summary>
		/// When set to false, XGE will not be enabled when running connected to the coordinator over VPN. Configure VPN-assigned subnets via the VpnSubnets parameter.
		/// </summary>
		[XmlConfigFile(Category = ExecutorName)]
		static bool bAllowOverVpn = true;

		/// <summary>
		/// List of subnets containing IP addresses assigned by VPN
		/// </summary>
		[XmlConfigFile(Category = ExecutorName)]
		static string[]? VpnSubnets = null;

		/// <summary>
		/// Whether to allow remote linking
		/// </summary>
		[XmlConfigFile(Category = ExecutorName)]
		static bool bAllowRemoteLinking = false;

		/// <summary>
		/// Whether to enable the VCCompiler=true setting. This requires an additional license for VC tools. 
		/// </summary>
		[XmlConfigFile(Category = ExecutorName)]
		static bool bUseVCCompilerMode = false;

		/// <summary>
		/// Minimum number of actions to use XGE execution.
		/// </summary>
		[XmlConfigFile(Category = ExecutorName)]
		public static int MinActions = 2;

		/// <summary>
		/// Check for a concurrent XGE build and treat the XGE executor as unavailable if it's in use.
		/// This will allow UBT to fall back to another executor such as the parallel executor. 
		/// </summary>
		[XmlConfigFile(Category = ExecutorName)]
		static bool bUnavailableIfInUse = false;
#pragma warning restore IDE0044

		private const string ProgressMarkupPrefix = "@action";

		private static readonly List<string> CompileAutoRecover = new List<string> {
			"C1060", // C1060: compiler is out of heap space
			"C1076", // C1076: compiler limit: internal heap limit reached
			"C2855", // C2855: command-line option 'X' inconsistent with precompiled header
			"C3435", // C3435: character set 'X' is not supported
			"C3859", // C3859: Failed to create virtual memory for PCH
		};

		private static readonly List<string> LinkAutoRecover = new List<string> {
			"Unexpected PDB error; OK (0)"
		};

		public XGE(ILogger Logger)
			: base(Logger)
		{
			XmlConfig.ApplyTo(this);
		}

		internal const string ExecutorName = "XGE";

		public override string Name => ExecutorName;

		public static bool TryGetXgConsoleExecutable([NotNullWhen(true)] out string? OutXgConsoleExe)
		{
			// Try to get the path from the registry
			if (OperatingSystem.IsWindows())
			{
				string? XgConsoleExe;
				if (TryGetXgConsoleExecutableFromRegistry(RegistryView.Registry32, out XgConsoleExe))
				{
					OutXgConsoleExe = XgConsoleExe;
					return true;
				}
				if (TryGetXgConsoleExecutableFromRegistry(RegistryView.Registry64, out XgConsoleExe))
				{
					OutXgConsoleExe = XgConsoleExe;
					return true;
				}
			}

			// Get the name of the XgConsole executable.
			string XgConsole = "xgConsole";
			if (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Win64)
			{
				XgConsole = "xgConsole.exe";
			}
			else if (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Linux)
			{
				XgConsole = "ib_console";
			}

			// Search the path for it
			string? PathVariable = Environment.GetEnvironmentVariable("PATH");
			if (PathVariable != null)
			{
				foreach (string SearchPath in PathVariable.Split(Path.PathSeparator))
				{
					try
					{
						string PotentialPath = Path.Combine(SearchPath, XgConsole);
						if (File.Exists(PotentialPath))
						{
							OutXgConsoleExe = PotentialPath;
							return true;
						}
					}
					catch (ArgumentException)
					{
						// PATH variable may contain illegal characters; just ignore them.
					}
				}
			}

			OutXgConsoleExe = null;
			return false;
		}

		[SupportedOSPlatform("windows")]
		private static bool TryGetXgConsoleExecutableFromRegistry(RegistryView View, [NotNullWhen(true)] out string? OutXgConsoleExe)
		{
			try
			{
				using (RegistryKey BaseKey = RegistryKey.OpenBaseKey(RegistryHive.LocalMachine, View))
				{
					using (RegistryKey? Key = BaseKey.OpenSubKey("SOFTWARE\\Xoreax\\IncrediBuild\\Builder", false))
					{
						if (Key != null)
						{
							string? Folder = Key.GetValue("Folder", null) as string;
							if (!String.IsNullOrEmpty(Folder))
							{
								string FileName = Path.Combine(Folder, "xgConsole.exe");
								if (File.Exists(FileName))
								{
									OutXgConsoleExe = FileName;
									return true;
								}
							}
						}
					}
				}
			}
			catch (Exception Ex)
			{
				Log.WriteException(Ex, null);
			}

			OutXgConsoleExe = null;
			return false;
		}

		[SupportedOSPlatform("windows")]
		static bool TryReadRegistryValue(RegistryHive Hive, RegistryView View, string KeyName, string ValueName, [NotNullWhen(true)] out string? OutCoordinator)
		{
			using (RegistryKey BaseKey = RegistryKey.OpenBaseKey(Hive, View))
			{
				using (RegistryKey? SubKey = BaseKey.OpenSubKey(KeyName))
				{
					if (SubKey != null)
					{
						string? Coordinator = SubKey.GetValue(ValueName) as string;
						if (!String.IsNullOrEmpty(Coordinator))
						{
							OutCoordinator = Coordinator;
							return true;
						}
					}
				}
			}

			OutCoordinator = null;
			return false;
		}

		static bool TryGetCoordinatorHost([NotNullWhen(true)] out string? OutCoordinator)
		{
			if (OperatingSystem.IsWindows())
			{
				const string KeyName = @"SOFTWARE\Xoreax\IncrediBuild\BuildService";
				const string ValueName = "CoordHost";

				return TryReadRegistryValue(RegistryHive.CurrentUser, RegistryView.Registry64, KeyName, ValueName, out OutCoordinator) ||
					TryReadRegistryValue(RegistryHive.CurrentUser, RegistryView.Registry32, KeyName, ValueName, out OutCoordinator) ||
					TryReadRegistryValue(RegistryHive.LocalMachine, RegistryView.Registry64, KeyName, ValueName, out OutCoordinator) ||
					TryReadRegistryValue(RegistryHive.LocalMachine, RegistryView.Registry32, KeyName, ValueName, out OutCoordinator);
			}
			else
			{
				OutCoordinator = null;
				return false;
			}
		}

		[DllImport("iphlpapi")]
		static extern int GetBestInterface(uint dwDestAddr, ref int pdwBestIfIndex);

		static NetworkInterface? GetInterfaceForHost(string Host)
		{
			if (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Win64)
			{
				IPHostEntry HostEntry = Dns.GetHostEntry(Host);
				foreach (IPAddress HostAddress in HostEntry.AddressList)
				{
					int InterfaceIdx = 0;
					if (GetBestInterface(BitConverter.ToUInt32(HostAddress.GetAddressBytes(), 0), ref InterfaceIdx) == 0)
					{
						foreach (NetworkInterface Interface in NetworkInterface.GetAllNetworkInterfaces())
						{
							IPv4InterfaceProperties Properties = Interface.GetIPProperties().GetIPv4Properties();
							if (Properties.Index == InterfaceIdx)
							{
								return Interface;
							}
						}
					}
				}
			}
			return null;
		}

		public static bool IsHostOnVpn(string HostName, ILogger Logger)
		{
			if (!OperatingSystem.IsWindows())
			{
				return false;
			}

			// If there aren't any defined subnets, just early out
			if (VpnSubnets == null || VpnSubnets.Length == 0)
			{
				return false;
			}

			// Parse all the subnets from the config file
			List<Subnet> ParsedVpnSubnets = new List<Subnet>();
			foreach (string VpnSubnet in VpnSubnets)
			{
				ParsedVpnSubnets.Add(Subnet.Parse(VpnSubnet));
			}

			// Check if any network adapters have an IP within one of these subnets
			try
			{
				NetworkInterface? Interface = GetInterfaceForHost(HostName);
				if (Interface != null && Interface.OperationalStatus == OperationalStatus.Up)
				{
					IPInterfaceProperties Properties = Interface.GetIPProperties();
					foreach (UnicastIPAddressInformation UnicastAddressInfo in Properties.UnicastAddresses)
					{
						byte[] AddressBytes = UnicastAddressInfo.Address.GetAddressBytes();
						foreach (Subnet Subnet in ParsedVpnSubnets)
						{
							if (Subnet.Contains(AddressBytes))
							{
								if (!bAllowOverVpn)
								{
									Log.TraceInformationOnce("XGE coordinator {0} will be not be used over VPN (adapter '{1}' with IP {2} is in subnet {3}). Set <XGE><bAllowOverVpn>true</bAllowOverVpn></XGE> in BuildConfiguration.xml to override.", HostName, Interface.Description, UnicastAddressInfo.Address, Subnet);
								}
								return true;
							}
						}
					}
				}
			}
			catch (Exception Ex)
			{
				Logger.LogWarning("Unable to check whether host {Host} is connected to VPN:\n{Ex}", HostName, ExceptionUtils.FormatExceptionDetails(Ex));
			}
			return false;
		}

		public static bool IsAvailable(ILogger Logger)
		{
			string? XgConsoleExe;
			if (!TryGetXgConsoleExecutable(out XgConsoleExe))
			{
				return false;
			}

			// on windows check the service is actually running
			if (OperatingSystem.IsWindows())
			{
				try
				{
					// will throw if the service doesn't exist, which it should if IB is present but just incase...
					System.ServiceProcess.ServiceController SC = new System.ServiceProcess.ServiceController("Incredibuild Agent");
					if (SC.Status != System.ServiceProcess.ServiceControllerStatus.Running)
					{
						return false;
					}
				}
				catch (Exception Ex)
				{
					Logger.LogDebug("Unable to query for status of Incredibuild service: {Ex}", ExceptionUtils.FormatExceptionDetails(Ex));
					return false;
				}
			}

			// Check if we're connected over VPN
			if (!bAllowOverVpn && VpnSubnets != null && VpnSubnets.Length > 0)
			{
				string? CoordinatorHost;
				if (TryGetCoordinatorHost(out CoordinatorHost) && IsHostOnVpn(CoordinatorHost, Logger))
				{
					return false;
				}
			}

			// Check if there's an XGE build already running 
			if (bUnavailableIfInUse)
			{
				Process XGEProcess = new Process()
				{
					StartInfo = new ProcessStartInfo(
					XgConsoleExe,
					"/Command=Unused /nowait /silent")  // The actual command here doesn't matter - it will fail with a different error code (1) than "in use" (4)
					{
						UseShellExecute = false
					}
				};
				if (Utils.RunLocalProcess(XGEProcess) == 4)
				{
					Logger.LogWarning("Unable to use Incredibuild executor because a build is already in progress");
					return false;

				}
			}

			return true;
		}

		private TelemetryExecutorEvent? telemetryEvent;

		/// <inheritdoc/>
		public override TelemetryExecutorEvent? GetTelemetryEvent() => telemetryEvent;

		// precompile the Regex needed to parse the XGE output (the ones we want are of the form "File (Duration at +time)"
		//private static Regex XGEDurationRegex = new Regex(@"(?<Filename>.*) *\((?<Duration>[0-9:\.]+) at [0-9\+:\.]+\)", RegexOptions.ExplicitCapture);

		public static void ExportActions(List<LinkedAction> ActionsToExecute, ILogger Logger)
		{
			for (int FileNum = 0; ; FileNum++)
			{
				string OutFile = Path.Combine(Unreal.EngineDirectory.FullName, "Intermediate", "Build", String.Format("UBTExport.{0}.xge.xml", FileNum.ToString("D3")));
				if (!File.Exists(OutFile))
				{
					ExportActions(ActionsToExecute, OutFile, Logger);
					break;
				}
			}
		}

		public static void ExportActions(List<LinkedAction> ActionsToExecute, string OutFile, ILogger Logger)
		{
			WriteTaskFile(ActionsToExecute, OutFile, ProgressWriter.bWriteMarkup, bXGEExport: true, Logger);
			Logger.LogInformation("XGEEXPORT: Exported '{OutFile}'", OutFile);
		}

		/// <inheritdoc/>
		public override Task<bool> ExecuteActionsAsync(IEnumerable<LinkedAction> ActionsToExecute, ILogger Logger)
		{
			return Task.FromResult(ExecuteActions(ActionsToExecute, Logger));
		}

		bool ExecuteActions(IEnumerable<LinkedAction> Actions, ILogger Logger)
		{
			if (!Actions.Any())
			{
				return true;
			}

			// Write the actions to execute to a XGE task file.
			string XGETaskFilePath = FileReference.Combine(Unreal.EngineDirectory, "Intermediate", "Build", "XGETasks.xml").FullName;
			WriteTaskFile(Actions, XGETaskFilePath, true, false, Logger);

			return ExecuteTaskFileWithProgressMarkup(XGETaskFilePath, Actions.ToArray(), Logger);
		}

		/// <summary>
		/// Writes a XGE task file containing the specified actions to the specified file path.
		/// </summary>
		static void WriteTaskFile(IEnumerable<LinkedAction> InActions, string TaskFilePath, bool bProgressMarkup, bool bXGEExport, ILogger Logger)
		{
			bool HostOnVpn = TryGetCoordinatorHost(out string? CoordinatorHost) && IsHostOnVpn(CoordinatorHost, Logger);

			Dictionary<string, string> ExportEnv = new Dictionary<string, string>();

			List<LinkedAction> Actions = InActions.ToList();
			if (bXGEExport)
			{
				IDictionary CurrentEnvironment = Environment.GetEnvironmentVariables();
				foreach (Nullable<System.Collections.DictionaryEntry> Pair in CurrentEnvironment)
				{
					if (Pair.HasValue)
					{
						if (!UnrealBuildTool.InitialEnvironment!.Contains(Pair.Value.Key) || (string)(UnrealBuildTool.InitialEnvironment[Pair.Value.Key]!) != (string)(Pair.Value.Value!))
						{
							ExportEnv.Add((string)(Pair.Value.Key), (string)(Pair.Value.Value!));
						}
					}
				}
			}

			XmlDocument XGETaskDocument = new XmlDocument();

			// <BuildSet FormatVersion="1">...</BuildSet>
			XmlElement BuildSetElement = XGETaskDocument.CreateElement("BuildSet");
			XGETaskDocument.AppendChild(BuildSetElement);
			BuildSetElement.SetAttribute("FormatVersion", "1");

			// <Environments>...</Environments>
			XmlElement EnvironmentsElement = XGETaskDocument.CreateElement("Environments");
			BuildSetElement.AppendChild(EnvironmentsElement);

			// <Environment Name="Default">...</CompileEnvironment>
			XmlElement EnvironmentElement = XGETaskDocument.CreateElement("Environment");
			EnvironmentsElement.AppendChild(EnvironmentElement);
			EnvironmentElement.SetAttribute("Name", "Default");

			// <Tools>...</Tools>
			XmlElement ToolsElement = XGETaskDocument.CreateElement("Tools");
			EnvironmentElement.AppendChild(ToolsElement);

			if (ExportEnv.Count > 0)
			{
				// <Variables>...</Variables>
				XmlElement VariablesElement = XGETaskDocument.CreateElement("Variables");
				EnvironmentElement.AppendChild(VariablesElement);

				foreach (KeyValuePair<string, string> Pair in ExportEnv)
				{
					// <Variable>...</Variable>
					XmlElement VariableElement = XGETaskDocument.CreateElement("Variable");
					VariablesElement.AppendChild(VariableElement);
					VariableElement.SetAttribute("Name", Pair.Key);
					VariableElement.SetAttribute("Value", Pair.Value);
				}
			}

			for (int ActionIndex = 0; ActionIndex < Actions.Count; ActionIndex++)
			{
				LinkedAction Action = Actions[ActionIndex];

				// Don't allow remote linking if on VPN.
				bool CanExecuteRemotely = Action.bCanExecuteRemotely && Action.bCanExecuteRemotelyWithXGE;
				if (CanExecuteRemotely && Action.ActionType == ActionType.Link)
				{
					if (HostOnVpn || !bAllowRemoteLinking)
					{
						CanExecuteRemotely = false;
					}
				}

				// <Tool ... />
				XmlElement ToolElement = XGETaskDocument.CreateElement("Tool");
				ToolsElement.AppendChild(ToolElement);
				ToolElement.SetAttribute("Name", String.Format("Tool{0}", ActionIndex));
				ToolElement.SetAttribute("AllowRemote", CanExecuteRemotely.ToString());

				// The XGE documentation says that 'AllowIntercept' must be set to 'true' for all tools where 'AllowRemote' is enabled
				ToolElement.SetAttribute("AllowIntercept", CanExecuteRemotely.ToString());

				string OutputPrefix = "";
				if (bProgressMarkup)
				{
					OutputPrefix += $"{ProgressMarkupPrefix}_{ActionIndex} ";
				}
				if (Action.bShouldOutputStatusDescription)
				{
					OutputPrefix += Action.StatusDescription;
				}
				if (OutputPrefix.Length > 0)
				{
					ToolElement.SetAttribute("OutputPrefix", OutputPrefix);
				}
				if (Action.GroupNames.Count > 0)
				{
					ToolElement.SetAttribute("GroupPrefix", String.Format("** For {0} **", String.Join(" + ", Action.GroupNames)));
				}

				ToolElement.SetAttribute("Params", Action.CommandArguments);
				ToolElement.SetAttribute("Path", Action.CommandPath.FullName);
				ToolElement.SetAttribute("SkipIfProjectFailed", "true");
				if (Action.ActionType == ActionType.Compile && bUseVCCompilerMode)
				{
					string FileName = Action.CommandPath.GetFileName();
					if (FileName.Equals("cl.exe", StringComparison.OrdinalIgnoreCase) || FileName.Equals("cl-filter.exe", StringComparison.OrdinalIgnoreCase))
					{
						ToolElement.SetAttribute("VCCompiler", "true");
					}
				}
				if (Action.bIsClangCompiler)
				{
					ToolElement.SetAttribute("AutoReserveMemory", "*.gch");
				}
				else
				{
					ToolElement.SetAttribute("AutoReserveMemory", "*.pch");
				}
				ToolElement.SetAttribute(
					"OutputFileMasks",
					String.Join(
						",",
						Action.ProducedItems.Select(
							delegate (FileItem ProducedItem) { return ProducedItem.Location.GetFileName(); }
							).ToArray()
						)
					);

				if (Action.ActionType == ActionType.Compile)
				{
					ToolElement.SetAttribute("AutoRecover", String.Join(',', CompileAutoRecover));
				}
				else if (Action.ActionType == ActionType.Link)
				{
					ToolElement.SetAttribute("AutoRecover", String.Join(',', LinkAutoRecover));
				}
			}

			// <Project Name="Default" Env="Default">...</Project>
			XmlElement ProjectElement = XGETaskDocument.CreateElement("Project");
			BuildSetElement.AppendChild(ProjectElement);
			ProjectElement.SetAttribute("Name", "Default");
			ProjectElement.SetAttribute("Env", "Default");

			for (int ActionIndex = 0; ActionIndex < Actions.Count; ActionIndex++)
			{
				LinkedAction Action = Actions[ActionIndex];

				// <Task ... />
				XmlElement TaskElement = XGETaskDocument.CreateElement("Task");
				ProjectElement.AppendChild(TaskElement);
				TaskElement.SetAttribute("SourceFile", "");
				TaskElement.SetAttribute("Caption", Action.StatusDescription);
				TaskElement.SetAttribute("Name", String.Format("Action{0}", ActionIndex));
				TaskElement.SetAttribute("Tool", String.Format("Tool{0}", ActionIndex));
				TaskElement.SetAttribute("WorkingDir", Action.WorkingDirectory.FullName);
				TaskElement.SetAttribute("SkipIfProjectFailed", "true");
				TaskElement.SetAttribute("AllowRestartOnLocal", "true");

				// Create a semi-colon separated list of the other tasks this task depends on the results of.
				List<string> DependencyNames = new List<string>();
				foreach (LinkedAction PrerequisiteAction in Action.PrerequisiteActions)
				{
					if (Actions.Contains(PrerequisiteAction))
					{
						DependencyNames.Add(String.Format("Action{0}", Actions.IndexOf(PrerequisiteAction)));
					}
				}

				if (DependencyNames.Count > 0)
				{
					TaskElement.SetAttribute("DependsOn", String.Join(";", DependencyNames.ToArray()));
				}
			}

			// Write the XGE task XML to a temporary file.
			using (FileStream OutputFileStream = new FileStream(TaskFilePath, FileMode.Create, FileAccess.Write))
			{
				XGETaskDocument.Save(OutputFileStream);
			}
		}

		/// <summary>
		/// The possible result of executing tasks with XGE.
		/// </summary>
		enum ExecutionResult
		{
			Unavailable,
			TasksFailed,
			TasksSucceeded,
		}

		/// <summary>
		/// Executes the tasks in the specified file.
		/// </summary>
		/// <param name="TaskFilePath">- The path to the file containing the tasks to execute in XGE XML format.</param>
		/// <param name="OutputEventHandler"></param>
		/// <param name="ActionCount"></param>
		/// <param name="Logger"></param>
		/// <returns>Indicates whether the tasks were successfully executed.</returns>
		[SuppressMessage("Interoperability", "CA1416:Validate platform compatibility", Justification = "Registry only checked on Windows HostPlatform")]
		bool ExecuteTaskFile(string TaskFilePath, DataReceivedEventHandler OutputEventHandler, int ActionCount, ILogger Logger)
		{
			// A bug in the UCRT can cause XGE to hang on VS2015 builds. Figure out if this hang is likely to effect this build and workaround it if able.
			// @todo: There is a KB coming that will fix this. Once that KB is available, test if it is present. Stalls will not be a problem if it is.
			//
			// Stalls are possible. However there is a workaround in XGE build 1659 and newer that can avoid the issue.
			string? XGEVersion = (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Win64) ? (string?)Registry.GetValue(@"HKEY_LOCAL_MACHINE\SOFTWARE\Wow6432Node\Xoreax\IncrediBuild\Builder", "Version", null) : null;
			if (XGEVersion != null)
			{
				int XGEBuildNumber;
				if (Int32.TryParse(XGEVersion, out XGEBuildNumber))
				{
					// Per Xoreax support, subtract 1001000 from the registry value to get the build number of the installed XGE.
					if (XGEBuildNumber - 1001000 >= 1659)
					{
						bXGENoWatchdogThread = true;
					}
					// @todo: Stalls are possible and we don't have a workaround. What should we do? Most people still won't encounter stalls, we don't really
					// want to disable XGE on them if it would have worked.
				}
			}

			string? XgConsolePath;
			if (!TryGetXgConsoleExecutable(out XgConsolePath))
			{
				throw new BuildException("Unable to find xgConsole executable.");
			}

			bool bSilentCompileOutput = false;
			string SilentOption = bSilentCompileOutput ? "/Silent" : "";

			ProcessStartInfo XGEStartInfo = new ProcessStartInfo(
				XgConsolePath,
				String.Format("\"{0}\" /Rebuild /NoWait {1} /NoLogo {2} /ShowAgent /ShowTime {3}",
					TaskFilePath,
					bStopXGECompilationAfterErrors ? "/StopOnErrors" : "",
					SilentOption,
					bXGENoWatchdogThread ? "/no_watchdog_thread" : "")
				);
			XGEStartInfo.UseShellExecute = false;
			XGEStartInfo.Arguments += " /Title=\"UnrealBuildTool Compile\"";

			// Use the IDE-integrated Incredibuild monitor to display progress.
			XGEStartInfo.Arguments += " /UseIdeMonitor";

			// Optionally display the external XGE monitor.
			if (bShowXGEMonitor)
			{
				XGEStartInfo.Arguments += " /OpenMonitor";
			}

			try
			{
				// Start the process, redirecting stdout/stderr if requested.
				Process XGEProcess = new Process();
				XGEProcess.StartInfo = XGEStartInfo;
				bool bShouldRedirectOuput = OutputEventHandler != null;
				if (bShouldRedirectOuput)
				{
					XGEStartInfo.RedirectStandardError = true;
					XGEStartInfo.RedirectStandardOutput = true;
					XGEProcess.EnableRaisingEvents = true;
					XGEProcess.OutputDataReceived += OutputEventHandler;
					XGEProcess.ErrorDataReceived += OutputEventHandler;
				}
				XGEProcess.Start();
				if (bShouldRedirectOuput)
				{
					XGEProcess.BeginOutputReadLine();
					XGEProcess.BeginErrorReadLine();
				}

				Logger.LogInformation("Distributing {NumAction} action{ActionS} to XGE",
					ActionCount,
					ActionCount == 1 ? "" : "s");

				// Wait until the process is finished and return whether it all the tasks successfully executed.
				XGEProcess.WaitForExit();
				return XGEProcess.ExitCode == 0;
			}
			catch (Exception Ex)
			{
				Log.WriteException(Ex, null);
				return false;
			}
		}

		/// <summary>
		/// Executes the tasks in the specified file, parsing progress markup as part of the output.
		/// </summary>
		bool ExecuteTaskFileWithProgressMarkup(string TaskFilePath, LinkedAction[] Actions, ILogger Logger)
		{
			int NumActions = Actions.Length;
			using (ProgressWriter Writer = new ProgressWriter("Compiling C++ source files...", false, Logger))
			{
				int NumCompletedActions = 0;
				string ProgressText = String.Empty;
				string CommandDescription = String.Empty;
				HashSet<int> ReportedActionIndices = new();

				// Create a wrapper delegate that will parse the output actions
				DataReceivedEventHandler EventHandlerWrapper = (Sender, Args) =>
				{
					if (Args.Data != null)
					{
						string Text = Args.Data;
						if (Text.StartsWith(ProgressMarkupPrefix, StringComparison.Ordinal))
						{
							// Code below should not need to be tested for success but if some logging from XGE is wrong we just gracefully ignore it and accept that counting might end up wrong
							int ActionIndex = -1;
							int MarkupLength = ProgressMarkupPrefix.Length;
							int EndOfMarkupPrefix = Text.IndexOf(' ', MarkupLength);
							if (EndOfMarkupPrefix != -1)
							{
								MarkupLength = EndOfMarkupPrefix + 1;
								if (Int32.TryParse(Text.Substring(ProgressMarkupPrefix.Length + 1, EndOfMarkupPrefix - ProgressMarkupPrefix.Length - 1), out ActionIndex))
								{
									// We keep track of the actions that we have already reported so NumCompletedActions match up with NumActions
									if (ReportedActionIndices.Add(ActionIndex))
									{
										Writer.Write(++NumCompletedActions, NumActions);
									}
								}
							}
							// Flush old progress text
							if (!String.IsNullOrEmpty(ProgressText))
							{
								Logger.LogInformation("[{NumCompletedActions}/{NumActions}] Complete {ProgressText}", NumCompletedActions, NumActions, ProgressText);
								ProgressText = String.Empty;
							}

							CommandDescription = ActionIndex != -1 ? Actions[ActionIndex].CommandDescription + " " : String.Empty;

							// Strip out anything that is just an XGE timer. Some programs don't output anything except the progress text.
							Text = Args.Data.Substring(MarkupLength);
							if (Text.StartsWith(" (", StringComparison.Ordinal) && Text.EndsWith(')'))
							{
								// Write the progress text with the next line of output if the current doesn't have any status.
								ProgressText = Text.Trim();
								return;
							}

							Logger.LogInformation("[{NumCompletedActions}/{NumActions}] {CommandDescription}{Text}", NumCompletedActions, NumActions, CommandDescription, Text);
							return;
						}
						if (!String.IsNullOrEmpty(ProgressText))
						{
							Logger.LogInformation("[{NumCompletedActions}/{NumActions}] {CommandDescription}{Text} {ProgressText}", NumCompletedActions, NumActions, CommandDescription, Text, ProgressText);
							ProgressText = String.Empty;
							CommandDescription = String.Empty;
							return;
						}
						WriteToolOutput(Text);
					}
				};

				DateTime startTimeUTC = DateTime.UtcNow;

				// Run through the standard XGE executor
				bool result = ExecuteTaskFile(TaskFilePath, EventHandlerWrapper, NumActions, Logger);

				telemetryEvent = new TelemetryExecutorEvent(Name, startTimeUTC, result, NumActions, -1, -1, 0, 0, DateTime.UtcNow, 0, 0);

				return result;
			}
		}
	}
}
