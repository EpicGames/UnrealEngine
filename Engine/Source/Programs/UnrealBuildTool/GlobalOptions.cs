// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using EpicGames.Core;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// Global options for UBT (any modes)
	/// </summary>
	internal record GlobalOptions
	{
		/// <summary>
		/// User asked for help
		/// </summary>
		[CommandLine(Prefix = "-Help", Description = "Display this help.")]
		[CommandLine(Prefix = "-h")]
		[CommandLine(Prefix = "--help")]
		public bool GetHelp = false;

		/// <summary>
		/// The amount of detail to write to the log
		/// </summary>
		[CommandLine(Prefix = "-Verbose", Value = "Verbose", Description = "Increase output verbosity")]
		[CommandLine(Prefix = "-VeryVerbose", Value = "VeryVerbose", Description = "Increase output verbosity more")]
		public LogEventType LogOutputLevel = LogEventType.Log;

		/// <summary>
		/// Specifies the path to a log file to write. Note that the default mode (eg. building, generating project files) will create a log file by default if this not specified.
		/// </summary>
		[CommandLine(Prefix = "-Log", Description = "Specify a log file location instead of the default Engine/Programs/UnrealBuildTool/Log.txt")]
		public FileReference? LogFileName = null;

		/// <summary>
		/// Log all attempts to write to the specified file
		/// </summary>
		[CommandLine(Prefix = "-TraceWrites", Description = "Trace writes requested to the specified file")]
		public FileReference? TraceWrites = null;

		/// <summary>
		/// Whether to include timestamps in the log
		/// </summary>
		[CommandLine(Prefix = "-Timestamps", Description = "Include timestamps in the log")]
		public bool bLogTimestamps = false;

		/// <summary>
		/// Whether to format messages in MsBuild format
		/// </summary>
		[CommandLine(Prefix = "-FromMsBuild", Description = "Format messages for msbuild")]
		public bool LogFromMsBuild { get; set; } = false;

		/// <summary>
		/// Whether or not to suppress warnings of missing SDKs from warnings to LogEventType.Log in UEBuildPlatformSDK.cs 
		/// </summary>
		[CommandLine(Prefix = "-SuppressSDKWarnings", Description = "Missing SDKs error verbosity level will be reduced from warning to log")]
		public bool ShouldSuppressSDKWarnings { get; set; } = false;

		/// <summary>
		/// Whether to write progress markup in a format that can be parsed by other programs
		/// </summary>
		[CommandLine(Prefix = "-Progress", Description = "Write progress messages in a format that can be parsed by other programs")]
		public bool WriteProgressMarkup { get; set; } = false;

		/// <summary>
		/// Whether to ignore the mutex
		/// </summary>
		[CommandLine(Prefix = "-NoMutex", Description = "Allow more than one instance of the program to run at once")]
		public bool NoMutex { get; set; } = false;

		/// <summary>
		/// Whether to wait for the mutex rather than aborting immediately
		/// </summary>
		[CommandLine(Prefix = "-WaitMutex", Description = "Wait for another instance to finish and then start, rather than aborting immediately")]
		public bool WaitMutex { get; set; } = false;

		/// <summary>
		/// </summary>
		[CommandLine(Prefix = "-RemoteIni", Description = "Remote tool ini directory")]
		public string RemoteIni { get; set; } = String.Empty;

		/// <summary>
		/// The mode to execute
		/// </summary>
		[CommandLine("-Mode=", Description = "Tool mode to select. Default tool mode is 'Build'")]

		[CommandLine("-Clean", Value = "Clean", Description = "Clean build products. Equivalent to -Mode=Clean")]

		[CommandLine("-ProjectFiles", Value = "GenerateProjectFiles", Description = "Generate project files based on IDE preference. Equivalent to -Mode=GenerateProjectFiles")]
		[CommandLine("-ProjectFileFormat=", Value = "GenerateProjectFiles", Description = "Generate project files in specified format. May be used multiple times.")]
		[CommandLine("-Makefile", Value = "GenerateProjectFiles", Description = "Generate Makefile")]
		[CommandLine("-CMakefile", Value = "GenerateProjectFiles", Description = "Generate project files for CMake")]
		[CommandLine("-QMakefile", Value = "GenerateProjectFiles", Description = "Generate project files for QMake")]
		[CommandLine("-KDevelopfile", Value = "GenerateProjectFiles", Description = "Generate project files for KDevelop")]
		[CommandLine("-CodeliteFiles", Value = "GenerateProjectFiles", Description = "Generate project files for Codelite")]
		[CommandLine("-XCodeProjectFiles", Value = "GenerateProjectFiles", Description = "Generate project files for XCode")]
		[CommandLine("-EddieProjectFiles", Value = "GenerateProjectFiles", Description = "Generate project files for Eddie")]
		[CommandLine("-VSCode", Value = "GenerateProjectFiles", Description = "Generate project files for Visual Studio Code")]
		[CommandLine("-VSMac", Value = "GenerateProjectFiles", Description = "Generate project files for Visual Studio Mac")]
		[CommandLine("-CLion", Value = "GenerateProjectFiles", Description = "Generate project files for CLion")]
		[CommandLine("-Rider", Value = "GenerateProjectFiles", Description = "Generate project files for Rider")]
		[CommandLine("-AndroidStudio", Value = "GenerateProjectFiles", Description = "Generate project files for Android Studio")]
#if __VPROJECT_AVAILABLE__
		[CommandLine("-VProject", Value = "GenerateProjectFiles")]
#endif
		public string? Mode { get; set; } = null;

		// The following Log settings exists in this location because, at the time of writing, EpicGames.Core does
		// not have access to XmlConfigFileAttribute.

		/// <summary>
		/// Whether to backup an existing log file, rather than overwriting it.
		/// </summary>
		[XmlConfigFile(Category = "Log", Name = "bBackupLogFiles")]
		public bool BackupLogFiles { get; set; } = Log.BackupLogFiles;

		/// <summary>
		/// The number of log file backups to preserve. Older backups will be deleted.
		/// </summary>
		[XmlConfigFile(Category = "Log")]
		public int LogFileBackupCount { get; set; } = Log.LogFileBackupCount;

		/// <summary>
		/// If set and tool execution was successful, then display an unreal build tool script execution timeline summary.
		/// If unset or the tool execution failed, print the same information silently to the log.
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration", Name = "bShowTimeline")]
		public bool ShowTimeline { get; set; } = Unreal.IsBuildMachine();

		/// <summary>
		/// If set TMP\TEMP will be overidden to this directory, each process will create a unique subdirectory in this folder.
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		public string? TempDirectory { get; set; } = null;

		/// <summary>
		/// If set the application temp directory will be deleted on exit, only when running with a single instance mutex.
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration", Name = "bDeleteTempDirectory")]
		public bool DeleteTempDirectory { get; set; } = false;

		/// <summary>
		/// Providers to load opt-in telemetry connection information from ini. If unset, or the provider categories do not contain connection info, no telemetry will be sent.
		/// </summary>
		[XmlConfigFile(Category = "Telemetry", Name = "Providers")]
		public string[] TelemetryProviders { get; set; } = [];

		/// <summary>
		/// Additional command line providers to load opt-in telemetry connection information from ini.
		/// </summary>
		[CommandLine(Prefix = "-TelemetryProvider", Description = "List of ini providers for telemetry", ListSeparator = '+')]
		public List<string> CmdTelemetryProviders { get; set; } = [];

		/// <summary>
		/// Session identifier for this run of UBT, if unset defaults to a random Guid
		/// </summary>
		[CommandLine(Prefix = "-Session", Description = "Session identifier for this run of UBT, if unset defaults to a random Guid")]
		public string? TelemetrySession { get; set; } = null;

		/// <summary>
		/// Whether telemetry/horde should use ipv6 or not. If network does not support ipv6 then using ipv6 can cause multi-second stalls
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration", Name = "bUseIpv6")]
		public bool UseIpv6 { get; set; } = true;

		/// <summary>
		/// Initialize the options with the given command line arguments
		/// </summary>
		/// <param name="Arguments"></param>
		public GlobalOptions(CommandLineArguments Arguments)
		{
			Arguments.ApplyTo(this);
			if (!String.IsNullOrEmpty(RemoteIni))
			{
				UnrealBuildTool.SetRemoteIniPath(RemoteIni);
			}
		}
	}
}
