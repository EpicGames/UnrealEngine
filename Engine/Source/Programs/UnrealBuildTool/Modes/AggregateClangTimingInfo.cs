// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.IO;
using System.IO.Compression;
using System.Linq;
using System.Text.Json;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace UnrealBuildTool
{
	/// <summary>
	/// Aggregates clang timing information files together into one monolithic breakdown file.
	/// </summary>
	internal sealed class AggregateClangTimingInfo : IToolMode<AggregateClangTimingInfo>
	{
		public static string Name => "AggregateClangTimingInfo";
		public static ToolModeOptions Options => ToolModeOptions.None;

		public class ClangTrace
		{
			public class TraceEvent
			{
				public long pid { get; set; }
				public long tid { get; set; }
				public string? ph { get; set; }
				public long ts { get; set; }
				public long dur { get; set; }
				public string? name { get; set; }
				public Dictionary<string, object>? args { get; set; }
			}

			public List<TraceEvent>? traceEvents { get; set; }
			public long beginningOfTime { get; set; }
		}

		public class TraceData
		{
			FileReference SourceFile { get; init; }
			public string Module => SourceFile.Directory.GetDirectoryName();
			public string Name => SourceFile.GetFileName();

			public long TotalExecuteCompiler { get; init; }

			// Subset of execute compiler
			public long TotalFrontend { get; init; }
			public long TotalBackend { get; init; }

			// Subset of frontend
			public long TotalSource { get; init; }
			public long TotalInstantiateFunction { get; init; }
			public long TotalCodeGenFunction { get; init; }

			// Subset of backend
			public long TotalModuleToFunctionPassAdaptor { get; init; }
			public long TotalModuleInlinerWrapperPass { get; init; }
			public long TotalOptModule { get; init; }

			// Frontend entry counts
			public long SourceEntries { get; init; }
			public long InstantiateFunctionEntries { get; init; }
			public long CodeGenFunctionEntries { get; init; }

			// Other
			public long ObjectBytes { get; init; }
			public long DependencyIncludes { get; init; }

			public TraceData(FileReference inputFile, ClangTrace? trace)
			{
				SourceFile = inputFile;

				TotalExecuteCompiler = trace?.traceEvents?.FindLast(x => String.Equals(x.name, "Total ExecuteCompiler", StringComparison.Ordinal))?.dur ?? 0;

				// Subset of execute compiler
				TotalFrontend = trace?.traceEvents?.FindLast(x => String.Equals(x.name, "Total Frontend", StringComparison.Ordinal))?.dur ?? 0;
				TotalBackend = trace?.traceEvents?.FindLast(x => String.Equals(x.name, "Total Backend", StringComparison.Ordinal))?.dur ?? 0;

				// Subset of frontend
				TotalSource = trace?.traceEvents?.FindLast(x => String.Equals(x.name, "Total Source", StringComparison.Ordinal))?.dur ?? 0;
				TotalInstantiateFunction = trace?.traceEvents?.FindLast(x => String.Equals(x.name, "Total InstantiateFunction", StringComparison.Ordinal))?.dur ?? 0;
				TotalCodeGenFunction = trace?.traceEvents?.FindLast(x => String.Equals(x.name, "Total CodeGen Function", StringComparison.Ordinal))?.dur ?? 0;

				// Subset of backend
				TotalModuleToFunctionPassAdaptor = trace?.traceEvents?.FindLast(x => String.Equals(x.name, "Total ModuleToFunctionPassAdaptor", StringComparison.Ordinal))?.dur ?? 0;
				TotalModuleInlinerWrapperPass = trace?.traceEvents?.FindLast(x => String.Equals(x.name, "Total ModuleInlinerWrapperPass", StringComparison.Ordinal))?.dur ?? 0;
				TotalOptModule = trace?.traceEvents?.FindLast(x => String.Equals(x.name, "Total OptModule", StringComparison.Ordinal))?.dur ?? 0;

				// Frontend entry counts
				SourceEntries = trace?.traceEvents?.Where(x => String.Equals(x.name, "Source", StringComparison.Ordinal)).LongCount() ?? 0;
				InstantiateFunctionEntries = trace?.traceEvents?.Where(x => String.Equals(x.name, "InstantiateFunction", StringComparison.Ordinal)).LongCount() ?? 0;
				CodeGenFunctionEntries = trace?.traceEvents?.Where(x => String.Equals(x.name, "CodeGen Function", StringComparison.Ordinal)).LongCount() ?? 0;

				// Other
				ObjectBytes = GetObjectSize();
				DependencyIncludes = CountIncludes();
			}

			static readonly string[] CsvColumns = new string[] {
				"Module",
				"Name",

				"TotalExecuteCompiler",

				// Subset of execute compiler
				"TotalFrontend",
				"TotalBackend",

				// Subset of frontend
				"TotalSource",
				"TotalInstantiateFunction",
				"TotalCodeGenFunction",

				// Subset of backend
				"TotalModuleToFunctionPassAdaptor",
				"TotalModuleInlinerWrapperPass",
				"TotalOptModule",

				// Frontend entry counts
				"SourceEntries",
				"InstantiateFunctionEntries",
				"CodeGenFunctionEntries",

				// Other
				"ObjectBytes",
				"DependencyIncludes",
			};

			static readonly string[] ObjectExtensions = new string[]
			{
				".o",
				".obj",
				".gch",
				".pch",
			};

			public static string CsvHeader => String.Join(',', CsvColumns);
			public string CsvLine => String.Join(',', CsvColumns.Select(x => GetType().GetProperty(x)!.GetValue(this)!.ToString()));

			private long GetObjectSize()
			{
				foreach (string Extension in ObjectExtensions)
				{
					FileReference ObjectFile = new FileReference($"{SourceFile.FullName}{Extension}");
					if (FileReference.Exists(ObjectFile))
					{
						return ObjectFile.ToFileInfo().Length;
					}
				}
				return 0;
			}

			private long CountIncludes()
			{
				FileReference DependsFile = new FileReference($"{SourceFile.FullName}.d");
				if (!FileReference.Exists(DependsFile))
				{
					DependsFile = new FileReference($"{SourceFile.FullName}.txt"); // ispc depends
				}
				if (!FileReference.Exists(DependsFile))
				{
					return 0;
				}
				// Subtract 1 for the header line
				return Math.Max(0, File.ReadLines(DependsFile.FullName).Count() - 1);
			}
		}

		private readonly ConcurrentDictionary<FileReference, ClangTrace> ClangTraceCache = new();

		public async Task<int> ExecuteAsync(CommandLineArguments Arguments, ILogger Logger)
		{
			FileReference ManifestFile = Arguments.GetFileReference("-ManifestFile=");
			IEnumerable<FileReference> SourceFiles = (await FileReference.ReadAllLinesAsync(ManifestFile)).Select(x => new FileReference(x));

			// Create aggregate summary.
			FileReference? AggregateFile = Arguments.GetFileReferenceOrDefault("-AggregateFile=", null);
			FileReference? HeadersFile = Arguments.GetFileReferenceOrDefault("-HeadersFile=", null);

			if (AggregateFile != null)
			{
				Task<TraceData[]> Tasks = Task.WhenAll(SourceFiles.Select(x => GetTraceData(x, Logger)));
				TraceData[] RawTraceData = await Tasks;
				List<TraceData> TraceDatas = RawTraceData.OrderBy(x => x.Module).ThenBy(x => x.Name).ToList();

				string TempFilePath = Path.Join(Path.GetTempPath(), AggregateFile.GetFileName() + ".tmp");
				using (StreamWriter Writer = new StreamWriter(TempFilePath))
				{
					await Writer.WriteLineAsync(TraceData.CsvHeader);
					foreach (TraceData Data in TraceDatas)
					{
						await Writer.WriteLineAsync(Data.CsvLine);
					}
				}
				File.Move(TempFilePath, AggregateFile.FullName, true);
			}

			if (HeadersFile != null)
			{
				Task<ClangTrace[]> Tasks = Task.WhenAll(SourceFiles.Select(x => ParseTimingDataFileAsync(x, Logger)));
				ClangTrace[] RawTraceData = await Tasks;
				List<ClangTrace> ClangTraces = RawTraceData.ToList();

				Dictionary<FileReference, List<long>> Sources = new Dictionary<FileReference, List<long>>();
				foreach (ClangTrace ClangTrace in ClangTraces.Where(x => x.traceEvents != null))
				{
					foreach (ClangTrace.TraceEvent Event in ClangTrace.traceEvents!.Where(x => String.Equals(x.name, "Source", StringComparison.Ordinal) && x.args?.ContainsKey("detail") == true))
					{
						FileReference SourceFile = new FileReference(Event.args!["detail"].ToString()!);
						if (!Sources.ContainsKey(SourceFile))
						{
							Sources.Add(SourceFile, new List<long>());
						}

						Sources[SourceFile].Add(Event.dur);
					}
				}

				string TempFilePath = Path.Join(Path.GetTempPath(), HeadersFile.GetFileName() + ".tmp");
				using (StreamWriter Writer = new StreamWriter(TempFilePath))
				{
					await Writer.WriteLineAsync("Source,Count,Total,Min,Max,Average");
					foreach (KeyValuePair<FileReference, List<long>> Data in Sources.OrderBy(x => x.Key.FullName).Where(x => x.Key.HasExtension(".h") || x.Key.HasExtension(".inl")))
					{
						await Writer.WriteLineAsync($"{Data.Key.FullName},{Data.Value.Count},{Data.Value.Sum()},{Data.Value.Min()},{Data.Value.Max()},{Data.Value.Average()}");
					}
				}
				File.Move(TempFilePath, HeadersFile.FullName, true);
			}

			// Write out aggregate archive if requested.
			FileReference? ArchiveFile = Arguments.GetFileReferenceOrDefault("-ArchiveFile=", null);
			if (ArchiveFile != null)
			{
				Logger.LogDebug("Writing {OutputFile} Archive", ArchiveFile);
				string TempFilePath = Path.Join(Path.GetTempPath(), ArchiveFile.GetFileName() + ".tmp");
				using (ZipArchive ZipArchive = new ZipArchive(File.Open(TempFilePath, FileMode.Create), ZipArchiveMode.Create))
				{
					if (AggregateFile != null)
					{
						ZipArchive.CreateEntryFromFile_CrossPlatform(AggregateFile.FullName, AggregateFile.GetFileName(), CompressionLevel.Optimal);
					}
					if (HeadersFile != null)
					{
						ZipArchive.CreateEntryFromFile_CrossPlatform(HeadersFile.FullName, HeadersFile.GetFileName(), CompressionLevel.Optimal);
					}
					foreach (FileReference SourceFile in SourceFiles)
					{
						FileReference JsonFile = new FileReference($"{SourceFile.FullName}.json");
						string EntryName = $"{JsonFile.Directory.GetDirectoryName()}/{JsonFile.GetFileName()}";
						ZipArchive.CreateEntryFromFile_CrossPlatform(JsonFile.FullName, EntryName, CompressionLevel.Optimal);
					}
				}
				File.Move(TempFilePath, ArchiveFile.FullName, true);
			}

			return 0;
		}

		private async Task<ClangTrace> ParseTimingDataFileAsync(FileReference SourceFile, ILogger Logger)
		{
			if (!ClangTraceCache.ContainsKey(SourceFile))
			{
				Logger.LogDebug("Parsing {SourceFile}", SourceFile.FullName);
				FileReference JsonFile = new FileReference($"{SourceFile.FullName}.json");
				ClangTrace? Trace = await JsonSerializer.DeserializeAsync<ClangTrace>(File.OpenRead(JsonFile.FullName));
				if (Trace == null)
				{
					throw new NullReferenceException($"Unable to deserialize {JsonFile}");
				}
				ClangTraceCache[SourceFile] = Trace;
			}

			return ClangTraceCache[SourceFile];
		}

		private async Task<TraceData> GetTraceData(FileReference SourceFile, ILogger Logger)
		{
			ClangTrace Trace = await ParseTimingDataFileAsync(SourceFile, Logger);
			return new TraceData(SourceFile, Trace);
		}
	}
}
