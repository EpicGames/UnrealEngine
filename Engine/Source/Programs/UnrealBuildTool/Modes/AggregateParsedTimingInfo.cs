// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.IO;
using System.IO.Compression;
using System.Linq;
using System.Text.Json;
using System.Text.RegularExpressions;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace UnrealBuildTool
{
	/// <summary>
	/// Aggregates parsed Visual C++ timing information files together into one monolithic breakdown file.
	/// </summary>
	internal sealed class AggregateParsedTimingInfo : IToolMode<AggregateParsedTimingInfo>
	{
		public static string Name => "AggregateParsedTimingInfo";
		public static ToolModeOptions Options => ToolModeOptions.None;

		private TimingData? FileTimingData;
		private readonly ConcurrentDictionary<string, int> DecompressedFileSizes = new ConcurrentDictionary<string, int>();
		private readonly ConcurrentDictionary<string, byte[]> CompressedFiles = new ConcurrentDictionary<string, byte[]>();
		private readonly ConcurrentBag<TimingData> AggregateIncludes = new ConcurrentBag<TimingData>();
		private readonly ConcurrentBag<TimingData> AggregateClasses = new ConcurrentBag<TimingData>();
		private readonly ConcurrentBag<TimingData> AggregateFunctions = new ConcurrentBag<TimingData>();
		private readonly Lock AddFileLock = new();

		public async Task<int> ExecuteAsync(CommandLineArguments Arguments, ILogger Logger)
		{
			FileReference ManifestFile = Arguments.GetFileReference("-ManifestFile=");
			string[] ParsedFileNames = await FileReference.ReadAllLinesAsync(ManifestFile);

			// Load all the file timing data and summarize them for the aggregate.
			FileTimingData = new TimingData("Files", TimingDataType.Summary);
			Parallel.ForEach(ParsedFileNames, new ParallelOptions() { MaxDegreeOfParallelism = 4 }, x => ParseTimingDataFile(x, Logger));

			// Create aggregate summary. Duration is the duration of the files in the aggregate.
			string AggregateName = Arguments.GetString("-Name=");
			TimingData AggregateData = new TimingData(AggregateName, TimingDataType.Aggregate);
			AggregateData.AddChild(FileTimingData);

			// Group the includes, classes, and functions by name and sum them up then add to the aggregate.
			GroupTimingDataOnName(AggregateData, "Include Timings", AggregateIncludes);
			GroupTimingDataOnName(AggregateData, "Class Timings", AggregateClasses);
			GroupTimingDataOnName(AggregateData, "Function Timings", AggregateFunctions);

			// Write out aggregate summary.
			string OutputFile = Path.Combine(ManifestFile.Directory.FullName, String.Format("{0}.cta", AggregateName));
			Logger.LogDebug("Writing {OutputFile}", OutputFile);
			using (BinaryWriter Writer = new BinaryWriter(File.Open(OutputFile, FileMode.Create)))
			{
				// Write out the aggregate data.
				Writer.Write(AggregateData);

				// Write the look up table for the compressed binary blobs.
				int Offset = 0;
				Writer.Write(CompressedFiles.Count);
				foreach (KeyValuePair<string, byte[]> CompressedFile in CompressedFiles)
				{
					Writer.Write(CompressedFile.Key);
					Writer.Write(Offset);
					Writer.Write(CompressedFile.Value.Length);
					Writer.Write(DecompressedFileSizes[CompressedFile.Key]);
					Offset += CompressedFile.Value.Length;
				}

				// Write the compressed binary blobs.
				foreach (KeyValuePair<string, byte[]> CompressedFile in CompressedFiles)
				{
					Writer.Write(CompressedFile.Value);
				}
			}

			if (Arguments.HasValue("-CompileTimingFile="))
			{
				FileReference CompileTimingFile = Arguments.GetFileReference("-CompileTimingFile=");
				Dictionary<string, double> CompileTimes = new Dictionary<string, double>();
				foreach (KeyValuePair<string, TimingData> TimingData in FileTimingData.Children)
				{
					CompileTimes.Add(JsonWriter.EscapeString(TimingData.Key), TimingData.Value.InclusiveDuration);
				}
				string JsonCompileTimes = JsonSerializer.Serialize(CompileTimes);
				await FileReference.WriteAllTextAsync(CompileTimingFile, JsonCompileTimes);
			}

			return 0;
		}

		private static void GroupTimingDataOnName(TimingData AggregateData, string TimingName, IEnumerable<TimingData> UngroupedData)
		{
			string GroupName = TimingName;
			TimingData GroupedTimingData = new TimingData(GroupName, TimingDataType.Summary);
			IEnumerable<IGrouping<string, TimingData>> Groups = UngroupedData.GroupBy(i => i.Name).OrderByDescending(g => g.Sum(d => d.ExclusiveDuration)).ToList();
			foreach (IGrouping<string, TimingData> Group in Groups)
			{
				TimingData GroupedData = new TimingData(Group.Key, TimingDataType.None) { ExclusiveDuration = Group.Sum(d => d.ExclusiveDuration), Count = Group.Sum(d => d.Count) };
				GroupedTimingData.Children.Add(Group.Key, GroupedData);
			}

			AggregateData.AddChild(GroupedTimingData);
			AggregateData.ExclusiveDuration -= GroupedTimingData.InclusiveDuration;
		}

		private void ParseTimingDataFile(string ParsedFileName, ILogger Logger)
		{
			Logger.LogDebug("Parsing {ParsedFileName}", ParsedFileName);
			// Convert input file back into summary objects.
			using (BinaryReader Reader = new BinaryReader(File.Open(ParsedFileName, FileMode.Open, FileAccess.Read)))
			{
				TimingData ParsedTimingData = new TimingData(Reader);

				Task CompressDataTask = Task.Run(() =>
				{
					Reader.BaseStream.Seek(0, SeekOrigin.Begin);
					using (MemoryStream CompressedMemoryStream = new MemoryStream())
					{
						using (GZipStream CompressionStream = new GZipStream(CompressedMemoryStream, CompressionMode.Compress))
						{
							Reader.BaseStream.CopyTo(CompressionStream);
						}

						CompressedFiles.TryAdd(ParsedTimingData.Name, CompressedMemoryStream.ToArray());
						DecompressedFileSizes.TryAdd(ParsedTimingData.Name, (int)Reader.BaseStream.Length);
					}
				});

				TimingData SummarizedTimingData = new TimingData(ParsedTimingData.Name, TimingDataType.Summary) { Parent = FileTimingData };
				foreach (TimingData Include in ParsedTimingData.Children["IncludeTimings"].Children.Values)
				{
					SummarizedTimingData.AddChild(Include.Clone());
				}
				SummarizedTimingData.ExclusiveDuration += ParsedTimingData.Children["ClassTimings"].Children.Values.Sum(d => d.InclusiveDuration);
				SummarizedTimingData.ExclusiveDuration += ParsedTimingData.Children["FunctionTimings"].Children.Values.Sum(d => d.InclusiveDuration);

				// Gather and update child timing data.
				Task QueueIncludesTask = Task.Run(() =>
				{
					// Flatten the includes for aggregation.
					IEnumerable<TimingData> Includes = ParsedTimingData.Children["IncludeTimings"].Children.Values;
					Dictionary<string, TimingData> FlattenedIncludes = new Dictionary<string, TimingData>();
					FlattenIncludes(FlattenedIncludes, Includes);
					foreach (TimingData Include in FlattenedIncludes.Values)
					{
						AggregateIncludes.Add(Include);
					}
				});

				Task QueueClassesTask = Task.Run(() =>
				{
					// Collapse templates into single entries.
					IEnumerable<TimingData> CollapsedClasses = GroupChildren(ParsedTimingData.Children["ClassTimings"].Children.Values, TimingDataType.Class);
					foreach (TimingData Class in CollapsedClasses)
					{
						AggregateClasses.Add(new TimingData(Class.Name, TimingDataType.Class) { ExclusiveDuration = Class.InclusiveDuration });
					}
				});

				Task QueueFunctionsTask = Task.Run(() =>
				{
					// Collapse templates into single entries.
					IEnumerable<TimingData> CollapsedFunctions = GroupChildren(ParsedTimingData.Children["FunctionTimings"].Children.Values, TimingDataType.Function);
					foreach (TimingData Function in CollapsedFunctions)
					{
						AggregateFunctions.Add(new TimingData(Function.Name, TimingDataType.Function) { ExclusiveDuration = Function.InclusiveDuration });
					}
				});

				while (!CompressDataTask.IsCompleted || !QueueIncludesTask.IsCompleted || !QueueClassesTask.IsCompleted || !QueueFunctionsTask.IsCompleted)
				{
					Thread.Sleep(TimeSpan.FromMilliseconds(50));
				}

				lock (AddFileLock)
				{
					FileTimingData!.AddChild(SummarizedTimingData);
				}
			}
		}

		private static void FlattenIncludes(Dictionary<string, TimingData> Includes, IEnumerable<TimingData> IncludeData)
		{
			foreach (TimingData Include in IncludeData)
			{
				TimingData? AggregatedInclude;
				if (Includes.TryGetValue(Include.Name, out AggregatedInclude))
				{
					AggregatedInclude.ExclusiveDuration += Include.ExclusiveDuration;
					AggregatedInclude.Count += 1;
				}
				else
				{
					Includes.Add(Include.Name, new TimingData(Include.Name, TimingDataType.Include) { ExclusiveDuration = Include.ExclusiveDuration });
				}

				FlattenIncludes(Includes, Include.Children.Values);
			}
		}

		private static IEnumerable<TimingData> GroupChildren(IEnumerable<TimingData> Children, TimingDataType Type)
		{
			List<TimingData> GroupedChildren = new List<TimingData>();
			Dictionary<string, List<TimingData>> ChildGroups = new Dictionary<string, List<TimingData>>();
			foreach (TimingData Child in Children)
			{
				// See if this is a templated class. If not, add it as is.
				Match Match = Regex.Match(Child.Name, @"^([^<]*)(?<Template><.*>)");
				if (!Match.Success)
				{
					GroupedChildren.Add(Child);
				}
				else
				{
					// Generate group name from template.
					int TemplateParamCount = Match.Groups["Template"].Value.Count(c => c == ',') + 1;
					List<string> TemplateParamSig = new List<string>(TemplateParamCount);
					for (int i = 0; i < TemplateParamCount; ++i)
					{
						TemplateParamSig.Add("...");
					}
					string GroupName = Child.Name.Replace(Match.Groups["Template"].Value, String.Format("<{0}>", String.Join(", ", TemplateParamSig)), StringComparison.Ordinal);

					// See if we have a group for this template already. If not, add it.
					if (!ChildGroups.ContainsKey(GroupName))
					{
						ChildGroups.Add(GroupName, new List<TimingData>());
					}

					ChildGroups[GroupName].Add(Child);
				}
			}

			// Add grouped children.
			foreach (KeyValuePair<string, List<TimingData>> Group in ChildGroups)
			{
				if (Group.Value.Count == 1)
				{
					GroupedChildren.Add(Group.Value.First());
					continue;
				}

				TimingData NewViewModel = new TimingData(Group.Key, Type);

				foreach (TimingData Child in Group.Value)
				{
					TimingData NewChild = new TimingData(Child.Name, Child.Type) { Parent = NewViewModel, ExclusiveDuration = Child.ExclusiveDuration };
					NewViewModel.AddChild(NewChild);
				}

				GroupedChildren.Add(NewViewModel);
			}

			return GroupedChildren;
		}
	}
}
