// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Collections.Immutable;
using System.IO;
using System.Linq;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace UnrealBuildTool.Modes
{
	/// <summary>
	/// Outputs information about the given target, including a module dependecy graph (in .gefx format and list of module references)
	/// </summary>
	internal sealed class AnalyzeMode : IToolMode<AnalyzeMode>
	{
		public static string Name => "Analyze";
		public static ToolModeOptions Options => ToolModeOptions.XmlConfig | ToolModeOptions.BuildPlatforms | ToolModeOptions.SingleInstance | ToolModeOptions.StartPrefetchingEngine | ToolModeOptions.ShowExecutionTime;

		/// <inheritdoc/>
		public Task<int> ExecuteAsync(CommandLineArguments arguments, ILogger logger)
		{
			arguments.ApplyTo(this);

			// Create the build configuration object, and read the settings
			BuildConfiguration buildConfiguration = new();
			XmlConfig.ApplyTo(buildConfiguration);
			arguments.ApplyTo(buildConfiguration);

			// Parse all the target descriptors
			List<TargetDescriptor> targetDescriptors = TargetDescriptor.ParseCommandLine(arguments, buildConfiguration, logger);

			// Generate the compile DB for each target
			using (ISourceFileWorkingSet workingSet = new EmptySourceFileWorkingSet())
			{
				// Find the compile commands for each file in the target
				Dictionary<FileReference, string> fileToCommand = [];
				foreach (TargetDescriptor targetDescriptor in targetDescriptors)
				{
					AnalyzeTarget(targetDescriptor, buildConfiguration, logger);
				}
			}

			return Task.FromResult(0);
		}

		private class ModuleInfo(UEBuildModule module, params string[] includeChain)
		{
			public UEBuildModule Module = module;
			public string[] IncludeChain = includeChain;
			public HashSet<UEBuildModule> InwardRefs = [];
			public HashSet<UEBuildModule> UniqueInwardRefs = [];
			public HashSet<UEBuildModule> OutwardRefs = [];
			public HashSet<UEBuildModule> UniqueOutwardRefs = [];

			public List<FileReference> ObjectFiles = [];
			public long ObjSize = 0;
			public List<FileReference> BinaryFiles = [];
			public long BinSize = 0;
			public ProcessedExternalModuleMetadata? ExternalMetadata = null;

			public string Chain => String.Join(" -> ", IncludeChain.Where(x => !String.IsNullOrEmpty(x)));
		}

		private static void AnalyzeModuleChains(
			string moduleName,
			List<string> parentChain,
			UEBuildTarget target,
			Dictionary<UEBuildModule, ModuleInfo> moduleToInfo,
			HashSet<UEBuildModule> visited,
			ILogger logger)
		{
			// Prevent recursive includes, they'll never be shorter
			if (parentChain.Contains(moduleName))
			{
				return;
			}

			UEBuildModule module = target.GetModuleByName(moduleName);

			List<string> currentChain = [.. parentChain, module.Name];

			if (!moduleToInfo.TryGetValue(module, out ModuleInfo? moduleInfo))
			{
				moduleInfo = new ModuleInfo(module, [.. currentChain]);
				moduleToInfo[module] = moduleInfo;
			}

			if (moduleInfo.IncludeChain.Length > currentChain.Count)
			{
				// Now we need to recheck all downstream dependencies again because the chain may be shorter
				List<UEBuildModule> recheckModules = [];
				module.GetAllDependencyModules(recheckModules, [], bIncludeDynamicallyLoaded: true, bForceCircular: true, bOnlyDirectDependencies: true);
				visited.ExceptWith(recheckModules);
				logger.LogDebug("Found shorter chain for {Module} {Prev} -> {New}, rechecking {Count} already visited dependencies", module.Name, moduleInfo.IncludeChain.Length, currentChain.Count, recheckModules.Count);
				moduleInfo.IncludeChain = [.. currentChain];
			}
			else if (visited.Contains(module))
			{
				return;
			}

			visited.Add(module);

			List<UEBuildModule> targetModules = [];
			module.GetAllDependencyModules(targetModules, [], bIncludeDynamicallyLoaded: true, bForceCircular: true, bOnlyDirectDependencies: true);
			targetModules.ForEach(x => AnalyzeModuleChains(x.Name, currentChain, target, moduleToInfo, visited, logger));
		}

		private static void AnalyzeTarget(TargetDescriptor targetDescriptor, BuildConfiguration buildConfiguration, ILogger logger)
		{
			// Create a makefile for the target
			UEBuildTarget target = UEBuildTarget.Create(targetDescriptor, buildConfiguration, logger);
			DirectoryReference.CreateDirectory(target.ReceiptFileName.Directory);

			// Find the shortest path from the target to each module
			HashSet<UEBuildModule> visited = [];
			Dictionary<UEBuildModule, ModuleInfo> moduleToInfo = [];

			if (target.Rules.LaunchModuleName != null)
			{
				AnalyzeModuleChains(target.Rules.LaunchModuleName, ["target"], target, moduleToInfo, visited, logger);
			}

			foreach (string rootModuleName in target.Rules.ExtraModuleNames)
			{
				AnalyzeModuleChains(rootModuleName, ["target"], target, moduleToInfo, visited, logger);
			}

			// Also enable all the plugin modules
			foreach (UEBuildPlugin plugin in target.BuildPlugins!)
			{
				foreach (UEBuildModule module in plugin.Modules)
				{
					if (!moduleToInfo.ContainsKey(module))
					{
						AnalyzeModuleChains(module.Name, ["target", plugin.ReferenceChain, plugin.File.GetFileName()], target, moduleToInfo, visited, logger);
					}
				}
			}

			if (target.Rules.bBuildAllModules)
			{
				foreach (UEBuildBinary binary in target.Binaries)
				{
					foreach (UEBuildModule module in binary.Modules)
					{
						if (!moduleToInfo.ContainsKey(module))
						{
							// quick hack to make allmodules always worse (empty entries are ignored when writing)
							List<string> includeChain = [.. Enumerable.Repeat(String.Empty, 1000)];
							includeChain.Add("allmodules option");
							if (module.Rules.Plugin != null)
							{
								includeChain.Add(module.Rules.Plugin.File.GetFileName());
								AnalyzeModuleChains(module.Name, includeChain, target, moduleToInfo, visited, logger);
							}
							else
							{
								AnalyzeModuleChains(module.Name, includeChain, target, moduleToInfo, visited, logger);
							}
						}
					}
				}
			}

			// Find all the outward dependencies of each module
			foreach ((UEBuildModule sourceModule, ModuleInfo sourceModuleInfo) in moduleToInfo)
			{
				sourceModuleInfo.OutwardRefs.Add(sourceModule);
				sourceModule.GetAllDependencyModules([], sourceModuleInfo.OutwardRefs, bIncludeDynamicallyLoaded: false, bForceCircular: true, bOnlyDirectDependencies: false);
				sourceModuleInfo.OutwardRefs.Remove(sourceModule);
			}

			// Find the unique output dependencies of each module
			foreach ((UEBuildModule sourceModule, ModuleInfo sourceModuleInfo) in moduleToInfo)
			{
				sourceModuleInfo.UniqueOutwardRefs = [.. sourceModuleInfo.OutwardRefs];

				foreach (UEBuildModule targetModule in sourceModuleInfo.OutwardRefs)
				{
					HashSet<UEBuildModule> visitedTargetModules = [sourceModule];

					List<UEBuildModule> dependencyModules = [];
					targetModule.GetAllDependencyModules(dependencyModules, visitedTargetModules, bIncludeDynamicallyLoaded: false, bForceCircular: true, bOnlyDirectDependencies: false);
					dependencyModules.Remove(targetModule);

					sourceModuleInfo.UniqueOutwardRefs.ExceptWith(dependencyModules);
				}
			}

			// Find the direct inward dependencies of each module
			foreach ((UEBuildModule sourceModule, ModuleInfo sourceModuleInfo) in moduleToInfo)
			{
				foreach (UEBuildModule targetModule in sourceModuleInfo.OutwardRefs)
				{
					moduleToInfo[targetModule].InwardRefs.Add(sourceModule);
				}
				foreach (UEBuildModule targetModule in sourceModuleInfo.UniqueOutwardRefs)
				{
					moduleToInfo[targetModule].UniqueInwardRefs.Add(sourceModule);
				}
			}

			// Estimate the size of object files for each module
			foreach ((UEBuildModule sourceModule, ModuleInfo sourceModuleInfo) in moduleToInfo)
			{
				if (DirectoryReference.Exists(sourceModule.IntermediateDirectory))
				{
					foreach (FileReference intermediateFile in DirectoryReference.EnumerateFiles(sourceModule.IntermediateDirectory, "*", SearchOption.AllDirectories))
					{
						if (intermediateFile.HasExtension(".obj") || intermediateFile.HasExtension(".o"))
						{
							sourceModuleInfo.ObjectFiles.Add(intermediateFile);
							sourceModuleInfo.ObjSize += intermediateFile.ToFileInfo().Length;
						}
					}
				}
			}

			HashSet<UEBuildModule> missingModules = [];
			foreach (UEBuildBinary binary in target.Binaries)
			{
				long binSize = 0;
				foreach (FileReference outputFilePath in binary.OutputFilePaths)
				{
					FileInfo outputFileInfo = outputFilePath.ToFileInfo();
					if (outputFileInfo.Exists)
					{
						binSize += outputFileInfo.Length;
					}
				}
				foreach (UEBuildModule module in binary.Modules)
				{
					if (!moduleToInfo.TryGetValue(module, out ModuleInfo? moduleInfo))
					{
						missingModules.Add(module);
						continue;
					}

					moduleInfo.BinaryFiles.AddRange(binary.OutputFilePaths);
					moduleInfo.BinSize += binSize;
				}
			}

			// Warn about any missing modules
			foreach (UEBuildModule missingModule in missingModules.OrderBy(x => x.Name))
			{
				logger.LogInformation("Missing module '{MissingModuleName}'", missingModule.Name);
			}

			Dictionary<string, string> variables = target.GetTargetVariables(null);
			ImmutableArray<ModuleInfo> externalModules = [.. moduleToInfo.Values
				.Where(x => x.Module.Rules.Type == ModuleRules.ModuleType.External)
				.OrderBy(x => x.Module.Name)
			];
			foreach (ModuleInfo moduleInfo in externalModules)
			{
				try
				{
					moduleInfo.ExternalMetadata = ProcessExternalModuleMetadata(moduleInfo, variables);
				}
				catch (Exception ex)
				{
					logger.LogInformation(ex, "Failed to process external module metadata for module '{ModuleName}'.", moduleInfo.Module.Name);
				}
				if (moduleInfo.ExternalMetadata is null)
				{
					logger.LogInformation("Module '{ModuleName}' is external but has no ExternalMetadata set.", moduleInfo.Module.Name);
					continue;
				}

				if (String.Equals(moduleInfo.ExternalMetadata.TpsFilePath.GetFileName(), ExternalModuleMetadata.TpsMissing, StringComparison.OrdinalIgnoreCase))
				{
					logger.LogInformation("Module '{ModuleName}' is missing a TPS file.", moduleInfo.Module.Name);
				}
				else if (String.Equals(moduleInfo.ExternalMetadata.TpsFilePath.GetFileName(), ExternalModuleMetadata.TpsNotRequired, StringComparison.OrdinalIgnoreCase))
				{
					// Nothing to log here, it's declared as not required and that's fine.
				}
				else if (!FileReference.Exists(moduleInfo.ExternalMetadata.TpsFilePath))
				{
					logger.LogInformation("Module '{ModuleName}' points to TPS file '{TpsFile}' that does not exist.", moduleInfo.Module.Name, moduleInfo.ExternalMetadata.TpsFilePath);
				}
			}

			List<KeyValuePair<FileReference, BuildProductType>> analyzeProducts = [];

			// Generate the dependency graph between modules
			FileReference dependencyGraphFile = target.ReceiptFileName.ChangeExtension(".Dependencies.gexf");
			logger.LogInformation("Writing dependency graph to {DependencyGraphFile}...", dependencyGraphFile);
			WriteDependencyGraph(target, moduleToInfo, dependencyGraphFile);
			analyzeProducts.Add(new(dependencyGraphFile, BuildProductType.BuildResource));

			// Generate the dependency graph between modules
			FileReference shortestPathGraphFile = target.ReceiptFileName.ChangeExtension(".ShortestPath.gexf");
			logger.LogInformation("Writing shortest-path graph to {ShortestPathGraphFile}...", shortestPathGraphFile);
			WriteShortestPathGraph(target, moduleToInfo, shortestPathGraphFile);
			analyzeProducts.Add(new(shortestPathGraphFile, BuildProductType.BuildResource));

			// Write all the target stats as a text file
			FileReference textFile = target.ReceiptFileName.ChangeExtension(".txt");
			logger.LogInformation("Writing module information to {TextFile}", textFile);
			WriteInformationTxt(target, moduleToInfo, textFile);
			analyzeProducts.Add(new(textFile, BuildProductType.BuildResource));

			// Write all the target stats as a CSV file
			FileReference csvFile = target.ReceiptFileName.ChangeExtension(".csv");
			logger.LogInformation("Writing module information to {CsvFile}", csvFile);
			WriteInformationCsv(moduleToInfo, csvFile);
			analyzeProducts.Add(new(csvFile, BuildProductType.BuildResource));

			// Write all the external module dependencies as a CSV file.
			FileReference externalCsvFile = target.ReceiptFileName.ChangeExtension(".External.csv");
			logger.LogInformation("Writing external dependencies information to {CsvFile}", externalCsvFile);
			WriteExternalCsv(externalModules, externalCsvFile);
			analyzeProducts.Add(new(externalCsvFile, BuildProductType.BuildResource));

			foreach (FileReference manifestFileName in target.Rules.ManifestFileNames)
			{
				target.GenerateManifest(manifestFileName, analyzeProducts, logger);
			}
		}

		private static ProcessedExternalModuleMetadata? ProcessExternalModuleMetadata(ModuleInfo moduleInfo, Dictionary<string, string> variables)
		{
			if (moduleInfo.Module.Rules.ExternalMetadata is not ExternalModuleMetadata metadata)
			{
				return null;
			}

			string tpsPath = Utils.ExpandVariables(metadata.TpsFilePath, variables, true);
			FileReference tpsReference = FileReference.Combine(moduleInfo.Module.ModuleDirectory, tpsPath);
			return new(metadata.Uri, metadata.Version, tpsReference);
		}

		private static void WriteDependencyList(TextWriter writer, string prefix, HashSet<UEBuildModule> modules)
		{
			if (modules.Count == 0)
			{
				writer.WriteLine("{0} 0", prefix);
			}
			else
			{
				writer.WriteLine("{0} {1} ({2})", prefix, modules.Count, String.Join(", ", modules.Select(x => x.Name).OrderBy(x => x)));
			}
		}

		private static void WriteDependencyGraph(UEBuildTarget target, Dictionary<UEBuildModule, ModuleInfo> moduleToInfo, FileReference fileName)
		{
			List<GraphNode> nodes = [];

			Dictionary<UEBuildModule, GraphNode> moduleToNode = [];
			foreach (ModuleInfo moduleInfo in moduleToInfo.Values)
			{
				GraphNode node = new(moduleInfo.Module.Name);

				long size = target.ShouldCompileMonolithic() ? moduleInfo.ObjSize : moduleInfo.BinSize;
				node.Size = 1.0f + (size / (50.0f * 1024.0f * 1024.0f));
				nodes.Add(node);
				moduleToNode[moduleInfo.Module] = node;
			}

			List<GraphEdge> edges = [];
			foreach ((UEBuildModule sourceModule, ModuleInfo sourceModuleInfo) in moduleToInfo)
			{
				GraphNode sourceNode = moduleToNode[sourceModule];
				foreach (UEBuildModule targetModule in sourceModuleInfo.UniqueOutwardRefs)
				{
					ModuleInfo targetModuleInfo = moduleToInfo[targetModule];

					if (moduleToNode.TryGetValue(targetModule, out GraphNode? targetNode))
					{
						edges.Add(new(sourceNode, targetNode)
						{
							Thickness = targetModuleInfo.InwardRefs.Count
						});
					}
				}
			}

			GraphVisualization.WriteGraphFile(fileName, $"Module dependency graph for {target.TargetName}", nodes, edges);
		}

		private static void WriteShortestPathGraph(UEBuildTarget target, Dictionary<UEBuildModule, ModuleInfo> moduleToInfo, FileReference fileName)
		{
			Dictionary<string, GraphNode> nameToNode = new(StringComparer.Ordinal);

			HashSet<(GraphNode, GraphNode)> edgesSet = [];
			List<GraphEdge> edges = [];

			foreach ((UEBuildModule module, ModuleInfo moduleInfo) in moduleToInfo)
			{
				string[] parts = moduleInfo.Chain.Split(" -> ");

				GraphNode? prevNode = null;
				foreach (string part in parts)
				{
					if (!nameToNode.TryGetValue(part, out GraphNode? nextNode))
					{
						nextNode = new GraphNode(part);
						nameToNode[part] = nextNode;
					}
					if (prevNode != null && edgesSet.Add((prevNode, nextNode)))
					{
						edges.Add(new(prevNode, nextNode));
					}
					prevNode = nextNode;
				}
			}

			GraphVisualization.WriteGraphFile(fileName, $"Module dependency graph for {target.TargetName}", [.. nameToNode.Values], edges);
		}

		private static void WriteInformationTxt(UEBuildTarget target, Dictionary<UEBuildModule, ModuleInfo> moduleToInfo, FileReference textFile)
		{
			using (StreamWriter writer = new(textFile.FullName))
			{
				writer.WriteLine("All modules in {0}, ordered by number of indirect references", target.TargetName);

				foreach (ModuleInfo moduleInfo in moduleToInfo.Values.OrderByDescending(x => x.InwardRefs.Count).ThenBy(x => x.BinSize))
				{
					writer.WriteLine("");
					writer.WriteLine("Module:                  \"{0}\"", moduleInfo.Module.Name);
					writer.WriteLine("Shortest path:           {0}", moduleInfo.Chain);
					WriteDependencyList(writer, "Unique inward refs:     ", moduleInfo.UniqueInwardRefs);
					WriteDependencyList(writer, "Unique outward refs:    ", moduleInfo.UniqueOutwardRefs);
					WriteDependencyList(writer, "Recursive inward refs:  ", moduleInfo.InwardRefs);
					WriteDependencyList(writer, "Recursive outward refs: ", moduleInfo.OutwardRefs);
					writer.WriteLine("Object size:             {0:n0}kb", (moduleInfo.ObjSize + 1023) / 1024);
					writer.WriteLine("Object files:            {0}", String.Join(", ", moduleInfo.ObjectFiles.Select(x => x.GetFileName())));
					writer.WriteLine("Binary size:             {0:n0}kb", (moduleInfo.BinSize + 1023) / 1024);
					writer.WriteLine("Binary files:            {0}", String.Join(", ", moduleInfo.BinaryFiles.Select(x => x.GetFileName())));
				}
			}
		}

		private static void WriteInformationCsv(Dictionary<UEBuildModule, ModuleInfo> moduleToInfo, FileReference csvFile)
		{
			using (StreamWriter writer = new(csvFile.FullName))
			{
				List<string> columns =
				[
					"Module",
					"ShortestPath",
					"NumUniqueInwardRefs",
					"UniqueInwardRefs",
					"NumRecursiveInwardRefs",
					"RecursiveInwardRefs",
					"NumUniqueOutwardRefs",
					"UniqueOutwardRefs",
					"NumRecursiveOutwardRefs",
					"RecursiveOutwardRefs",
					"ObjSize",
					"ObjFiles",
					"BinSize",
					"BinFiles",
				];
				writer.WriteLine(String.Join(",", columns));

				foreach (ModuleInfo moduleInfo in moduleToInfo.Values.OrderByDescending(x => x.InwardRefs.Count).ThenBy(x => x.BinSize))
				{
					columns.Clear();
					columns.Add(moduleInfo.Module.Name);
					columns.Add(moduleInfo.Chain);
					columns.Add($"{moduleInfo.UniqueInwardRefs.Count}");
					columns.Add($"\"{String.Join(", ", moduleInfo.UniqueInwardRefs.Select(x => x.Name))}\"");
					columns.Add($"{moduleInfo.InwardRefs.Count}");
					columns.Add($"\"{String.Join(", ", moduleInfo.InwardRefs.Select(x => x.Name))}\"");
					columns.Add($"{moduleInfo.UniqueOutwardRefs.Count}");
					columns.Add($"\"{String.Join(", ", moduleInfo.UniqueOutwardRefs.Select(x => x.Name))}\"");
					columns.Add($"{moduleInfo.OutwardRefs.Count}");
					columns.Add($"\"{String.Join(", ", moduleInfo.OutwardRefs.Select(x => x.Name))}\"");
					columns.Add($"{moduleInfo.ObjSize}");
					columns.Add($"\"{String.Join(", ", moduleInfo.ObjectFiles.Select(x => x.GetFileName()))}\"");
					columns.Add($"{moduleInfo.BinSize}");
					columns.Add($"\"{String.Join(", ", moduleInfo.BinaryFiles.Select(x => x.GetFileName()))}\"");
					writer.WriteLine(String.Join(",", columns));
				}
			}
		}

		private static void WriteExternalCsv(ImmutableArray<ModuleInfo> externalModules, FileReference csvFile)
		{
			using (StreamWriter writer = new(csvFile.FullName))
			{
				List<string> columns =
				[
					"Module",
					"Uri",
					"Version",
					"TpsFile",
				];
				writer.WriteLine(String.Join(",", columns));

				foreach (ModuleInfo moduleInfo in externalModules)
				{
					columns.Clear();
					columns.Add(moduleInfo.Module.Name);
					columns.Add(moduleInfo.ExternalMetadata?.Uri.ToString() ?? "<unavailable>");
					columns.Add(moduleInfo.ExternalMetadata?.Version ?? "<unavailable>");
					columns.Add(moduleInfo.ExternalMetadata?.TpsFilePath.ToString() ?? "<unavailable>");
					writer.WriteLine(String.Join(",", columns));
				}
			}
		}
	}
}
