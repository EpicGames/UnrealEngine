// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Diagnostics;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using OpenTracing.Util;
using UnrealBuildBase;
using UnrealBuildTool.Executors;

namespace UnrealBuildTool
{
	/// <summary>
	/// Options controlling how a target is built
	/// </summary>
	[Flags]
	enum BuildOptions
	{
		/// <summary>
		/// Default options
		/// </summary>
		None = 0,

		/// <summary>
		/// Don't build anything, just do target setup and terminate
		/// </summary>
		SkipBuild = 1,

		/// <summary>
		/// Just output a list of XGE actions; don't build anything
		/// </summary>
		XGEExport = 2,

		/// <summary>
		/// Fail if any engine files would be modified by the build
		/// </summary>
		NoEngineChanges = 4,
	}

	/// <summary>
	/// Builds a target
	/// </summary>
	internal sealed class BuildMode : IToolMode<BuildMode>
	{
		public static string Name => "Build";
		public static ToolModeOptions Options => ToolModeOptions.XmlConfig | ToolModeOptions.BuildPlatforms | ToolModeOptions.SingleInstance | ToolModeOptions.StartPrefetchingEngine | ToolModeOptions.ShowExecutionTime | ToolModeOptions.UseStartupTraceListener;

		/// <summary>
		/// Specifies the file to use for logging.
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		public string? BaseLogFileName;

		/// <summary>
		/// Whether to skip checking for files identified by the junk manifest.
		/// </summary>
		[XmlConfigFile]
		[CommandLine("-IgnoreJunk")]
		public bool bIgnoreJunk = false;

		/// <summary>
		/// Skip building; just do setup and terminate.
		/// </summary>
		[CommandLine("-SkipBuild")]
		public bool bSkipBuild = false;

		/// <summary>
		/// Skip pre build targets; just do the main target.
		/// </summary>
		[CommandLine("-SkipPreBuildTargets")]
		public bool bSkipPreBuildTargets = false;

		/// <summary>
		/// Whether we should just export the XGE XML and pretend it succeeded
		/// </summary>
		[CommandLine("-XGEExport")]
		public bool bXGEExport = false;

		/// <summary>
		/// Do not allow any engine files to be output (used by compile on startup functionality)
		/// </summary>
		[CommandLine("-NoEngineChanges")]
		public bool bNoEngineChanges = false;

		/// <summary>
		/// Whether we should just export the outdated actions list
		/// </summary>
		[CommandLine("-WriteOutdatedActions=")]
		public FileReference? WriteOutdatedActionsFile = null;

		/// <summary>
		/// An optional directory to copy crash dump files into
		/// </summary>
		[CommandLine("-SaveCrashDumps=")]
		public DirectoryReference? SaveCrashDumpDirectory = null;

		/// <summary>
		/// If specified, then only this type of action will execute
		/// </summary>
		[CommandLine("-ActionTypeFilter=")]
		public string? ActionTypeFilter = null;

		/// <summary>
		/// Main entry point
		/// </summary>
		/// <param name="Arguments">Command-line arguments</param>
		/// <returns>One of the values of ECompilationResult</returns>
		/// <param name="Logger"></param>
		public async Task<int> ExecuteAsync(CommandLineArguments Arguments, ILogger Logger)
		{
			Arguments.ApplyTo(this);

			// Write the command line
			Logger.LogDebug("Command line: {EnvironmentCommandLine}", Environment.CommandLine);

			// Grab the environment.
			UnrealBuildTool.InitialEnvironment = Environment.GetEnvironmentVariables();
			if (UnrealBuildTool.InitialEnvironment.Count < 1)
			{
				throw new BuildException("Environment could not be read");
			}

			// Read the XML configuration files
			XmlConfig.ApplyTo(this);

			// Apply to architecture configs that need to read commandline arguments and didn't have the Arguments passed in during construction
			foreach (UnrealArchitectureConfig Config in UnrealArchitectureConfig.AllConfigs())
			{
				Arguments.ApplyTo(Config);
			}

			// Fixup the log path if it wasn't overridden by a config file
			BaseLogFileName ??= FileReference.Combine(Unreal.EngineProgramSavedDirectory, "UnrealBuildTool", "Log.txt").FullName;

			// Create the log file, and flush the startup listener to it
			if (!Arguments.HasOption("-NoLog") && !Log.HasFileWriter())
			{
				Log.AddFileWriter("DefaultLogTraceListener", new(BaseLogFileName));
			}
			else
			{
				Log.RemoveStartupTraceListener();
			}

			// Create the build configuration object, and read the settings
			BuildConfiguration BuildConfiguration = new BuildConfiguration();
			XmlConfig.ApplyTo(BuildConfiguration);
			Arguments.ApplyTo(BuildConfiguration);

			// Check the root path length isn't too long
			if (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Win64 && Unreal.RootDirectory.FullName.Length > BuildConfiguration.MaxRootPathLength)
			{
				Logger.LogWarning("Running from a path with a long directory name (\"{Path}\" = {NumChars} characters). Root paths shorter than {MaxChars} characters are recommended to avoid exceeding maximum path lengths on Windows.", Unreal.RootDirectory, Unreal.RootDirectory.FullName.Length, BuildConfiguration.MaxRootPathLength);
			}

			// now that we know the available platforms, we can delete other platforms' junk. if we're only building specific modules from the editor, don't touch anything else (it may be in use).
			if (!bIgnoreJunk && !Unreal.IsEngineInstalled())
			{
				using (GlobalTracer.Instance.BuildSpan("DeleteJunk()").StartActive())
				{
					JunkDeleter.DeleteJunk(Logger);
				}
			}

			bool warnOnNoPartition = false;
#if DEBUG
			warnOnNoPartition = true; // Pvs target writes three files that is not covered by partition
#endif

			CppDependencyCache cppDependencyCache = new();
			FileHashCache fileHashCache = new(warnOnNoPartition, true);
			List<Task> tasks = [];
			ActionExecutor? executor = null;

			// Parse and build the targets
			try
			{
				List<TargetDescriptor> TargetDescriptors;

				// Parse all the target descriptors
				using (GlobalTracer.Instance.BuildSpan("TargetDescriptor.ParseCommandLine()").StartActive())
				{
					using ITimelineEvent _ = Timeline.ScopeEvent("ParseCommandLine");

					TargetDescriptors = TargetDescriptor.ParseCommandLine(Arguments, BuildConfiguration, Logger);

					// Handle BuildConfiguration arguments nested inside -Target= args
					TargetDescriptors.ForEach(x => x.AdditionalArguments.ApplyTo(BuildConfiguration));
				}

				// Clean any target that wanted to be cleaned before being rebuilt
				if (TargetDescriptors.Any(D => D.bRebuild))
				{
					CleanMode CleanMode = new CleanMode();
					CleanMode.bSkipPreBuildTargets = bSkipPreBuildTargets;
					CleanMode.Clean(TargetDescriptors.Where(D => D.bRebuild).ToList(), BuildConfiguration, Logger);
				}

				// If we're using the UBA executor (very likely), we want to get a headstart on creating it, so it'll be ready when it needs it.
				// Based on not-particularly-rigorous profiling, this saves maybe ~0.4s per execution of UBT by doing this here -
				// enough to justify this slightly ugly pre-creation logic.
				executor = ExecutorFactory.Instance.PrecreateExecutor(BuildConfiguration, TargetDescriptors, Logger);

				// Handle remote builds
				for (int Idx = 0; Idx < TargetDescriptors.Count; ++Idx)
				{
					TargetDescriptor TargetDesc = TargetDescriptors[Idx];
					if (RemoteMac.HandlesTargetPlatform(TargetDesc.Platform))
					{
						FileReference BaseLogFile = Log.OutputFile ?? new FileReference(BaseLogFileName);
						FileReference RemoteLogFile = FileReference.Combine(BaseLogFile.Directory, BaseLogFile.GetFileNameWithoutExtension() + "_Remote.txt");

						int RemoteIniIndex = TargetDesc.AdditionalArguments.FindIndex(x => x.StartsWith("-remoteini=", StringComparison.OrdinalIgnoreCase));
						string? RemoteIni = RemoteIniIndex == -1 ? null : new string(TargetDesc.AdditionalArguments[RemoteIniIndex].Substring(11));
						RemoteMac RemoteMac = new RemoteMac(TargetDesc.ProjectFile, Logger, RemoteIni);
						if (!RemoteMac.Build(TargetDesc, RemoteLogFile, bSkipPreBuildTargets, Logger))
						{
							return (int)CompilationResult.Unknown;
						}

						TargetDescriptors.RemoveAt(Idx--);
					}
				}

				// Handle local builds
				if (TargetDescriptors.Count > 0)
				{
					// Get a set of all the project directories
					HashSet<DirectoryReference> ProjectDirs = new HashSet<DirectoryReference>();
					foreach (TargetDescriptor TargetDesc in TargetDescriptors)
					{
						if (TargetDesc.ProjectFile != null)
						{
							DirectoryReference ProjectDirectory = TargetDesc.ProjectFile.Directory;
							Task task = FileMetadataPrefetch.QueueProjectDirectory(ProjectDirectory);
							ProjectDirs.Add(ProjectDirectory);
						}

						// print out SDK info for only the platforms that are being compiled
						UEBuildPlatformSDK.GetSDKForPlatform(TargetDesc.Platform.ToString())?.PrintSDKInfoAndReturnValidity();
					}

					// Get all the build options
					BuildOptions Options = BuildOptions.None;
					if (bSkipBuild)
					{
						Options |= BuildOptions.SkipBuild;
					}
					if (bXGEExport)
					{
						Options |= BuildOptions.XGEExport;
					}
					if (bNoEngineChanges)
					{
						Options |= BuildOptions.NoEngineChanges;
					}

					// Create the working set provider per group.
					using (ISourceFileWorkingSet WorkingSet = SourceFileWorkingSet.Create(Unreal.RootDirectory, ProjectDirs, Logger))
					{
						List<List<(TargetDescriptor, TargetMakefile)>> targetsPasses = CreateMakefiles(BuildConfiguration, TargetDescriptors, WorkingSet, Logger, true, false, tasks, cppDependencyCache, fileHashCache);
						if (fileHashCache.HasErrors)
						{
							throw new BuildException("FileHashCache had errors");
						}

						bool usedExecutor = false;
						foreach (List<(TargetDescriptor, TargetMakefile)> pass in targetsPasses)
						{
							// We need a fresh executor for each pass, as each one is only expected to have its ExecuteAsync method called once.
							if (usedExecutor)
							{
								executor?.Dispose();
								executor = ExecutorFactory.Instance.PrecreateExecutor(BuildConfiguration, TargetDescriptors, Logger);
							}

							await BuildAsync(pass, BuildConfiguration, Options, WriteOutdatedActionsFile, cppDependencyCache, Logger, ActionTypeFilter, executor);
							usedExecutor = true;
						}
					}
				}
			}
			finally
			{
				executor?.Dispose();

				// Check if anything failed during our run, and act accordingly.
				ProcessCoreDumps(SaveCrashDumpDirectory, Logger);

				await Task.WhenAll(tasks.ToArray());

				// Save all the caches
				using (Timeline.ScopeEvent("Saving caches"))
				{
					fileHashCache.SaveAll(Logger);
					SourceFileMetadataCache.SaveAll();
					cppDependencyCache.SaveAll();
				}
			}
			return 0;
		}

		/// <summary>
		/// Creates scripts for executing the init scripts (only supported on project descriptors)
		/// </summary>
		/// <param name="TargetDescriptor">The current target</param>
		/// <returns>List of created script files</returns>
		public static FileReference[] CreateInitScripts(TargetDescriptor TargetDescriptor)
		{
			if (TargetDescriptor.ProjectFile == null)
			{
				return Array.Empty<FileReference>();
			}

			ProjectDescriptor ProjectDescriptor = ProjectDescriptor.FromFile(TargetDescriptor.ProjectFile);
			List<string[]> InitCommandBatches = new List<string[]>();

			if (ProjectDescriptor != null && ProjectDescriptor.InitSteps != null)
			{
				if (ProjectDescriptor.InitSteps.TryGetCommands(BuildHostPlatform.Current.Platform, out string[]? Commands))
				{
					InitCommandBatches.Add(Commands);
				}
			}

			if (InitCommandBatches.Count == 0)
			{
				return Array.Empty<FileReference>();
			}

			DirectoryReference ProjectDirectory = DirectoryReference.FromFile(TargetDescriptor.ProjectFile);
			string PlatformIntermediateFolder = UEBuildTarget.GetPlatformIntermediateFolder(TargetDescriptor.Platform, TargetDescriptor.Architectures, false);

			DirectoryReference ProjectIntermediateDirectory = DirectoryReference.Combine(ProjectDirectory, PlatformIntermediateFolder, TargetDescriptor.Name, TargetDescriptor.Configuration.ToString());

			return WriteInitScripts(TargetDescriptor, ProjectIntermediateDirectory, "Init", InitCommandBatches);
		}

		/// <summary>
		/// Write scripts containing the custom build steps for the host platform
		/// </summary>
		/// <param name="TargetDescriptor">The current target</param>
		/// <param name="Directory">The output directory for the scripts</param>
		/// <param name="FilePrefix">Bare prefix for all the created script files</param>
		/// <param name="CommandBatches">List of custom build steps</param>
		/// <returns>List of created script files</returns>
		private static FileReference[] WriteInitScripts(TargetDescriptor TargetDescriptor, DirectoryReference Directory, string FilePrefix, List<string[]> CommandBatches)
		{
			List<FileReference> ScriptFiles = new List<FileReference>();
			foreach (string[] CommandBatch in CommandBatches)
			{
				// Find all the standard variables
				Dictionary<string, string> Variables = GetTargetVariables(TargetDescriptor);

				// Get the output path to the script
				string ScriptExtension = OperatingSystem.IsWindows() ? ".bat" : ".sh";
				FileReference ScriptFile = FileReference.Combine(Directory, String.Format("{0}-{1}{2}", FilePrefix, ScriptFiles.Count + 1, ScriptExtension));

				// Write it to disk
				List<string> Contents = new List<string>();
				if (OperatingSystem.IsWindows())
				{
					Contents.Insert(0, "@echo off");
				}
				foreach (string Command in CommandBatch)
				{
					Contents.Add(Utils.ExpandVariables(Command, Variables));
				}
				if (!DirectoryReference.Exists(ScriptFile.Directory))
				{
					DirectoryReference.CreateDirectory(ScriptFile.Directory);
				}
				FileReference.WriteAllLines(ScriptFile, Contents);

				// Add the output file to the list of generated scripts
				ScriptFiles.Add(ScriptFile);
			}
			return ScriptFiles.ToArray();
		}

		/// <summary>
		/// Gets a list of variables that can be expanded in paths referenced by this target
		/// </summary>
		/// <param name="TargetDescriptor">The current target</param>
		/// <returns>Map of variable names to values</returns>
		private static Dictionary<string, string> GetTargetVariables(TargetDescriptor TargetDescriptor)
		{
			Dictionary<string, string> Variables = new Dictionary<string, string>();
			Variables.Add("RootDir", Unreal.RootDirectory.FullName);
			Variables.Add("EngineDir", Unreal.EngineDirectory.FullName);
			Variables.Add("TargetName", TargetDescriptor.Name);
			Variables.Add("TargetPlatform", TargetDescriptor.Platform.ToString());
			Variables.Add("TargetConfiguration", TargetDescriptor.Configuration.ToString());
			if (TargetDescriptor.ProjectFile != null)
			{
				Variables.Add("ProjectDir", TargetDescriptor.ProjectFile.Directory.FullName);
				Variables.Add("ProjectFile", TargetDescriptor.ProjectFile.FullName);
			}
			if (BuildVersion.TryRead(BuildVersion.GetDefaultFileName(), out BuildVersion? Version))
			{
				Variables.Add("EngineVersion", String.Format("{0}.{1}.{2}", Version.MajorVersion, Version.MinorVersion, Version.PatchVersion));
			}
			return Variables;
		}

		/// <summary>
		/// Build a list of targets
		/// </summary>
		/// <param name="TargetDescriptors">Target descriptors</param>
		/// <param name="BuildConfiguration">Current build configuration</param>
		/// <param name="WorkingSet">The source file working set</param>
		/// <param name="Options">Additional options for the build</param>
		/// <param name="WriteOutdatedActionsFile">Files to write the list of outdated actions to (rather than building them)</param>
		/// <param name="cppDependencyCache"></param>
		/// <param name="Logger">Logger for output</param>
		/// <param name="ActionTypeFilter">If specified, only actions of this type will be run</param>
		/// <returns>Result from the compilation</returns>
		public static async Task BuildAsync(List<TargetDescriptor> TargetDescriptors, BuildConfiguration BuildConfiguration, ISourceFileWorkingSet WorkingSet, BuildOptions Options, FileReference? WriteOutdatedActionsFile, CppDependencyCache cppDependencyCache, ILogger Logger, string? ActionTypeFilter = null)
		{
			List<Task> tasks = [];
			try
			{
				List<List<(TargetDescriptor, TargetMakefile)>> targetsPasses = CreateMakefiles(BuildConfiguration, TargetDescriptors, WorkingSet, Logger, true, true, tasks, cppDependencyCache, new(false, true));
				foreach (List<(TargetDescriptor, TargetMakefile)> targets in targetsPasses)
				{
					await BuildAsync(targets, BuildConfiguration, Options, WriteOutdatedActionsFile, cppDependencyCache, Logger, ActionTypeFilter, null);
				}
			}
			finally
			{
				await Task.WhenAll(tasks.ToArray());
			}
		}

		/// <summary>
		/// Build a list of targets with a given set of makefiles.
		/// </summary>
		/// <param name="Targets">Targets</param>
		/// <param name="BuildConfiguration">Current build configuration</param>
		/// <param name="Options">Additional options for the build</param>
		/// <param name="WriteOutdatedActionsFile">Files to write the list of outdated actions to (rather than building them)</param>
		/// <param name="Logger">Logger for output</param>
		/// <param name="ActionTypeFilter">If specified, only actions of this type will be run</param>
		/// <param name="cppDependencyCache"></param>
		/// <param name="executor"></param>
		/// <returns>Result from the compilation</returns>
		internal static async Task BuildAsync(List<(TargetDescriptor Descriptor, TargetMakefile Makefile)> Targets, BuildConfiguration BuildConfiguration, BuildOptions Options, FileReference? WriteOutdatedActionsFile, CppDependencyCache cppDependencyCache, ILogger Logger, string? ActionTypeFilter = null, ActionExecutor? executor = null)
		{
			List<TargetDescriptor> TargetDescriptors = Targets.Select(T => T.Descriptor).ToList();
			TargetMakefile[] Makefiles = Targets.Select(T => T.Makefile).ToArray();

			// Execute the build
			if ((Options & BuildOptions.SkipBuild) == 0)
			{
				// Create the C++ dependency cache
				for (int TargetIdx = 0; TargetIdx < Targets.Count; TargetIdx++)
				{
					cppDependencyCache.Mount(Targets[TargetIdx].Descriptor, Makefiles[TargetIdx].TargetType, Logger);
				}

				using ITimelineEvent ActionGraphTimer = Timeline.ScopeEvent($"Preparing ActionGraph");
				// Make sure that none of the actions conflict with any other (producing output files differently, etc...)
				using (GlobalTracer.Instance.BuildSpan("ActionGraph.CheckForConflicts").StartActive())
				{
					// TODO: Skipping conflicts for WriteMetadata is a hack to maintain parity with the BuildGraph executor which allowed
					// exporting mulitple conflicing WriteMetadata actions.
					// This is technically buggy behavior and should be properly fixed in UEBuildTarget.cs
					IEnumerable<IExternalAction> CheckActions = Targets.Select(T => T.Makefile).SelectMany(x => x.Actions).Where(x => !x.IgnoreConflicts());
					ActionGraph.CheckForConflicts(CheckActions, Logger);
				}

				// Check we don't exceed the nominal max path length
				using (GlobalTracer.Instance.BuildSpan("ActionGraph.CheckPathLengths").StartActive())
				{
					ActionGraph.CheckPathLengths(BuildConfiguration, Targets.Select(T => T.Makefile).SelectMany(x => x.Actions), Logger);
				}

				HashSet<FileItem> MergedOutputItems = new HashSet<FileItem>();

				// Create a QueuedAction instance for each action in the makefiles
				List<LinkedAction>[] QueuedActions = new List<LinkedAction>[Targets.Count];
				for (int Idx = 0; Idx < Targets.Count; Idx++)
				{
					TargetDescriptor TargetDescriptor = Targets[Idx].Descriptor;
					TargetMakefile Makefile = Targets[Idx].Makefile;

					QueuedActions[Idx] = Makefile.Actions.ConvertAll(x => new LinkedAction(x, TargetDescriptor));

					if (TargetDescriptor.SpecificFilesToCompile.Count != 0)
					{
						// Filter out SpecificFilesToCompile so only source files and headers included in the target are considered
						HashSet<FileReference> TargetFiles = Makefile.SourceAndHeaderFiles.Select(x => x.Location).ToHashSet();
						List<FileReference> FilesToBuild = TargetDescriptor.SpecificFilesToCompile
							.Where(x => TargetFiles.Contains(x))
							.ToList();

						// If we have specific files to build we need to put those as MergedOutputItems (and create special single file actions if needed)
						if (FilesToBuild.Count != 0)
						{
							// If there are headers in the list, expand the FilesToBuild to also include all files that include those headers
							if (TargetDescriptor.bSingleFileBuildDependents)
							{
								FilesToBuild = GetAllSourceFilesIncludingHeader(FilesToBuild, TargetDescriptor.ProjectFile, Makefile.Actions, Logger);
							}

							// We have specific files to compile so we will only queue up those files.
							List<FileItem> ProducedItems = CreateLinkedActionsFromFileList(TargetDescriptor, BuildConfiguration, FilesToBuild, QueuedActions[Idx], Logger);
							MergedOutputItems.UnionWith(ProducedItems);
						}
					}
				}

				// Clean up any previous hot reload runs, and reapply the current state if it's already active
				Dictionary<FileReference, FileReference>? InitialPatchedOldLocationToNewLocation = null;
				for (int TargetIdx = 0; TargetIdx < Targets.Count; TargetIdx++)
				{
					InitialPatchedOldLocationToNewLocation = HotReload.Setup(Targets[TargetIdx].Descriptor, Targets[TargetIdx].Makefile, QueuedActions[TargetIdx], BuildConfiguration, Logger);
				}

				// Merge the action graphs together
				List<LinkedAction> MergedActions;
				if (TargetDescriptors.Count == 1)
				{
					MergedActions = QueuedActions[0];
				}
				else
				{
					MergedActions = MergeActionGraphs(TargetDescriptors, QueuedActions);
				}

				// Gather all the prerequisite actions that are part of the targets
				for (int TargetIdx = 0; TargetIdx < TargetDescriptors.Count; TargetIdx++)
				{
					GatherOutputItems(TargetDescriptors[TargetIdx], Makefiles[TargetIdx], MergedOutputItems);
				}

				// Link all the actions together (No need sorting, we'll do that later again)
				ActionGraph.Link(MergedActions, Logger, Sort: false);

				// Get all the actions that are prerequisites for these targets. This forms the list of actions that we want executed.
				List<LinkedAction> PrerequisiteActions = ActionGraph.GatherPrerequisiteActions(MergedActions, MergedOutputItems);

				// Create the action history
				ActionHistory History = new ActionHistory();
				for (int TargetIdx = 0; TargetIdx < TargetDescriptors.Count; TargetIdx++)
				{
					using (GlobalTracer.Instance.BuildSpan("Reading action history").StartActive())
					{
						TargetDescriptor TargetDescriptor = TargetDescriptors[TargetIdx];
						if (TargetDescriptor.ProjectFile != null)
						{
							History.Mount(TargetDescriptor.ProjectFile.Directory);
						}
					}
				}

				// Wait for cpp dependencies task to finish
				cppDependencyCache.WaitAsync();

				// Pre-process module interfaces to generate dependency files
				List<LinkedAction> ModuleDependencyActions = PrerequisiteActions.Where(x => x.ActionType == ActionType.GatherModuleDependencies).ToList();
				if (ModuleDependencyActions.Count > 0)
				{
					ConcurrentDictionary<LinkedAction, bool> PreprocessActionToOutdatedFlag = new ConcurrentDictionary<LinkedAction, bool>();
					ActionGraph.GatherAllOutdatedActions(ModuleDependencyActions, History, PreprocessActionToOutdatedFlag, cppDependencyCache, BuildConfiguration.bIgnoreOutdatedImportLibraries, Logger);

					List<LinkedAction> PreprocessActions = PreprocessActionToOutdatedFlag.Where(x => x.Value).Select(x => x.Key).ToList();
					if (PreprocessActions.Count > 0)
					{
						Logger.LogInformation("Updating module dependencies...");
						await ActionGraph.ExecuteActionsAsync(BuildConfiguration, PreprocessActions, TargetDescriptors, Logger, null);

						foreach (FileItem ProducedItem in PreprocessActions.SelectMany(x => x.ProducedItems))
						{
							ProducedItem.ResetCachedInfo();
						}
					}
				}

				// Figure out which actions need to be built
				ConcurrentDictionary<LinkedAction, bool> ActionToOutdatedFlag = new ConcurrentDictionary<LinkedAction, bool>();
				for (int TargetIdx = 0; TargetIdx < TargetDescriptors.Count; TargetIdx++)
				{
					TargetDescriptor TargetDescriptor = TargetDescriptors[TargetIdx];

					// Update the module dependencies
					Dictionary<string, FileItem> ModuleOutputs = new Dictionary<string, FileItem>(StringComparer.Ordinal);
					if (ModuleDependencyActions.Count > 0)
					{
						Dictionary<FileItem, List<string>> ModuleImports = new Dictionary<FileItem, List<string>>();

						List<FileItem> CompiledModuleInterfaces = new List<FileItem>();
						foreach (LinkedAction ModuleDependencyAction in ModuleDependencyActions)
						{
							ICppCompileAction? CppModulesAction = ModuleDependencyAction.Inner as ICppCompileAction;
							if (CppModulesAction != null && CppModulesAction.CompiledModuleInterfaceFile != null)
							{
								string? ProducedModule;
								if (cppDependencyCache.TryGetProducedModule(ModuleDependencyAction.DependencyListFile!, Logger, out ProducedModule))
								{
									ModuleOutputs[ProducedModule] = CppModulesAction.CompiledModuleInterfaceFile;
								}

								List<(string Name, string BMI)>? ImportedModules;
								if (cppDependencyCache.TryGetImportedModules(ModuleDependencyAction.DependencyListFile!, Logger, out ImportedModules))
								{
									ModuleImports[CppModulesAction.CompiledModuleInterfaceFile] = ImportedModules.Select(x => x.Name).ToList();
								}

								CompiledModuleInterfaces.Add(CppModulesAction.CompiledModuleInterfaceFile);
							}
						}

						Dictionary<FileItem, string> ModuleInputs = ModuleOutputs.ToDictionary(x => x.Value, x => x.Key);
						foreach (LinkedAction PrerequisiteAction in PrerequisiteActions)
						{
							if (PrerequisiteAction.ActionType == ActionType.CompileModuleInterface)
							{
								ICppCompileAction CppModulesAction = (ICppCompileAction)PrerequisiteAction.Inner;

								List<string>? ImportedModules;
								if (ModuleImports.TryGetValue(CppModulesAction.CompiledModuleInterfaceFile!, out ImportedModules))
								{
									foreach (string ImportedModule in ImportedModules)
									{
										FileItem? ModuleOutput;
										if (ModuleOutputs.TryGetValue(ImportedModule, out ModuleOutput))
										{
											Action NewAction = new Action(PrerequisiteAction.Inner);
											NewAction.PrerequisiteItems.Add(ModuleOutput);
											NewAction.CommandArguments += String.Format(" /reference \"{0}={1}\"", ImportedModule, ModuleOutput.FullName);
											PrerequisiteAction.Inner = NewAction;
										}
										else
										{
											throw new BuildException("Unable to find interface for module '{0}'", ImportedModule);
										}
									}
								}
							}
							else if (PrerequisiteAction.ActionType == ActionType.Compile)
							{
								foreach (FileItem PrerequisiteItem in PrerequisiteAction.PrerequisiteItems)
								{
									string? ModuleName;
									if (ModuleInputs.TryGetValue(PrerequisiteItem, out ModuleName))
									{
										Action NewAction = new Action(PrerequisiteAction.Inner);
										NewAction.CommandArguments += String.Format(" /reference \"{0}={1}\"", ModuleName, PrerequisiteItem.AbsolutePath);
										PrerequisiteAction.Inner = NewAction;
									}
								}
							}
							else
							{
								if (PrerequisiteAction.ActionType != ActionType.GatherModuleDependencies)
								{
									Action NewAction = new Action(PrerequisiteAction.Inner);
									NewAction.PrerequisiteItems.UnionWith(CompiledModuleInterfaces);
									PrerequisiteAction.Inner = NewAction;
								}
							}
						}
					}

					// Plan the actions to execute for the build. For single file compiles, always rebuild the source file regardless of whether it's out of date.
					ActionGraph.GatherAllOutdatedActions(PrerequisiteActions, History, ActionToOutdatedFlag, cppDependencyCache, BuildConfiguration.bIgnoreOutdatedImportLibraries, Logger);
				}

				// Link the action graph again to sort it
				List<LinkedAction> MergedActionsToExecute = ActionToOutdatedFlag.Where(x => x.Value).Select(x => x.Key).ToList();

				// if we want to only run one type of action, filter it now
				if (ActionTypeFilter != null)
				{
					ActionType Filter;
					if (Enum.TryParse<ActionType>(ActionTypeFilter, out Filter))
					{
						MergedActionsToExecute = MergedActionsToExecute.Where(x => x.ActionType == Filter).ToList();
					}
					else
					{
						throw new BuildException($"Unknown ActionType for ActionTypeFilter: {ActionTypeFilter}");
					}
				}

				ActionGraph.Link(MergedActionsToExecute, Logger);

				// Allow hot reload to override the actions
				int HotReloadTargetIdx = -1;
				for (int Idx = 0; Idx < TargetDescriptors.Count; Idx++)
				{
					if (TargetDescriptors[Idx].HotReloadMode != HotReloadMode.Disabled)
					{
						if (HotReloadTargetIdx != -1)
						{
							throw new BuildException("Unable to perform hot reload with multiple targets.");
						}
						else
						{
							MergedActionsToExecute = HotReload.PatchActionsForTarget(BuildConfiguration, TargetDescriptors[Idx], Makefiles[Idx], PrerequisiteActions, MergedActionsToExecute, InitialPatchedOldLocationToNewLocation, Logger);
						}
						HotReloadTargetIdx = Idx;
					}
					else if (MergedActionsToExecute.Count > 0)
					{
						HotReload.CheckForLiveCodingSessionActive(TargetDescriptors[Idx], Makefiles[Idx], BuildConfiguration, Logger);
					}
				}

				if (HotReloadTargetIdx != -1)
				{
					Logger.LogDebug("Re-evaluating action graph");
					// Re-check the graph to remove any LiveCoding actions added by PatchActionsForTarget() that are already up to date.
					ConcurrentDictionary<LinkedAction, bool> LiveActionToOutdatedFlag = new ConcurrentDictionary<LinkedAction, bool>(Environment.ProcessorCount, MergedActionsToExecute.Count);
					ActionGraph.GatherAllOutdatedActions(MergedActionsToExecute, History, LiveActionToOutdatedFlag, cppDependencyCache, BuildConfiguration.bIgnoreOutdatedImportLibraries, Logger);
					List<LinkedAction> LiveCodingActionsToExecute = LiveActionToOutdatedFlag.Where(x => x.Value).Select(x => x.Key).ToList();
					ActionGraph.Link(LiveCodingActionsToExecute, Logger);
					MergedActionsToExecute = LiveCodingActionsToExecute;
				}

				// Make sure we're not modifying any existing engine files while allowing new engine files to be created
				if (Options.HasFlag(BuildOptions.NoEngineChanges))
				{
					IEnumerable<FileItem> EngineChanges = MergedActionsToExecute.SelectMany(x => x.ProducedItems).Where(x => x.Location.IsUnderDirectory(Unreal.EngineDirectory) && x.Exists).Distinct().OrderBy(x => x.AbsolutePath);
					if (EngineChanges.Any())
					{
						StringBuilder Result = new("Building would modify the following existing engine files:");
						Result.AppendLine();
						foreach (FileItem EngineChange in EngineChanges)
						{
							Result.AppendLine(EngineChange.FullName);
						}
						Result.AppendLine();
						Result.AppendLine("Please build from an IDE instead.");
						Logger.LogError("{Result}", Result.ToString());
						throw new CompilationResultException(CompilationResult.FailedDueToEngineChange);
					}
				}

				// Make sure the appropriate executor is selected
				foreach (TargetDescriptor TargetDescriptor in TargetDescriptors)
				{
					UEBuildPlatform BuildPlatform = UEBuildPlatform.GetBuildPlatform(TargetDescriptor.Platform);
					BuildConfiguration.bAllowXGE &= BuildPlatform.CanUseXGE();
					BuildConfiguration.bAllowFASTBuild &= BuildPlatform.CanUseFASTBuild();
					BuildConfiguration.bAllowSNDBS &= BuildPlatform.CanUseSNDBS();
				}

				// Delete produced items that are outdated.
				ActionGraph.DeleteOutdatedProducedItems(MergedActionsToExecute, Logger);

				// Save all the action histories now that files have been removed. We have to do this after deleting produced items to ensure that any
				// items created during the build don't have the wrong command line.
				{
					using ITimelineEvent __ = Timeline.ScopeEvent($"SaveActionHistory");
					History.Save();
				}

				// Create directories for the outdated produced items.
				ActionGraph.CreateDirectoriesForProducedItems(MergedActionsToExecute);

				ActionGraphTimer.Finish();

				// Execute the actions
				if ((Options & BuildOptions.XGEExport) != 0)
				{
					OutputToolchainInfo(TargetDescriptors, Makefiles, Logger);

					// Just export to an XML file
					using (GlobalTracer.Instance.BuildSpan("XGE.ExportActions()").StartActive())
					{
						using ITimelineEvent XGETimer = Timeline.ScopeEvent("XGEExport");
						XGE.ExportActions(MergedActionsToExecute, Logger);
					}
				}
				else if (WriteOutdatedActionsFile != null)
				{
					OutputToolchainInfo(TargetDescriptors, Makefiles, Logger);

					// Write actions to an output file
					using (GlobalTracer.Instance.BuildSpan("ActionGraph.WriteActions").StartActive())
					{
						using ITimelineEvent WriteActionsTimer = Timeline.ScopeEvent("Writing actions");
						ActionGraph.ExportJson(MergedActionsToExecute, WriteOutdatedActionsFile);
						Logger.LogInformation("Exported build actions to {WriteOutdatedActionsFile}", WriteOutdatedActionsFile);
					}
				}
				else
				{
					// Execute the actions
					if (MergedActionsToExecute.Count != 0)
					{
						if (TargetDescriptors.Any(x => !x.bQuiet))
						{
							Logger.LogInformation("Building {Targets}...", StringUtils.FormatList(TargetDescriptors.Select(x => x.Name).Distinct()));
						}

						OutputToolchainInfo(TargetDescriptors, Makefiles, Logger);

						ActionExecutor.SetMemoryPerActionOverride(Makefiles.Select(x => x.MemoryPerActionGB).Max());
					}

					//using ITimelineEvent ExcuteTimer = Timeline.ScopeEvent("Executing ActionGraph");
					using (GlobalTracer.Instance.BuildSpan("ActionGraph.ExecuteActions()").StartActive())
					{
						await ActionGraph.ExecuteActionsAsync(BuildConfiguration, MergedActionsToExecute, TargetDescriptors, Logger, executor);
					}

					// Log the output binary path for each target
					foreach (TargetMakefile Makefile in Makefiles)
					{
						Logger.LogInformation("Output binary: {ExecutableFile}", Makefile.ExecutableFile);
					}

					// these are parsed by external tools wishing to open this file directly
					foreach (LinkedAction BuildAction in MergedActionsToExecute.Where(BuildAction => BuildAction.ActionType == ActionType.Compile && (BuildAction.Inner as VCCompileAction)?.bIsAnalyzing != true))
					{
						FileItem? PreprocessedFile = BuildAction.ProducedItems.FirstOrDefault(x => x.HasExtension(".i"));
						if (PreprocessedFile != null)
						{
							Logger.LogInformation("PreProcessPath: {File}", PreprocessedFile);
						}

						FileItem? AssemblyPath = BuildAction.ProducedItems.FirstOrDefault(x => x.HasExtension(".asm"));
						if (AssemblyPath != null)
						{
							Logger.LogInformation("AssemblyPath: {File}", AssemblyPath);
						}
					}

					// Run the deployment steps
					using ITimelineEvent DeployTimer = Timeline.ScopeEvent("Deploying targets");
					foreach (TargetMakefile Makefile in Makefiles)
					{
						// Receipt file may not exist when compiling specific files
						if (Makefile.bDeployAfterCompile && FileReference.Exists(Makefile.ReceiptFile))
						{
							TargetReceipt Receipt = TargetReceipt.Read(Makefile.ReceiptFile);
							Logger.LogInformation("Deploying {ReceiptTargetName} {ReceiptPlatform} {ReceiptConfiguration}...", Receipt.TargetName, Receipt.Platform, Receipt.Configuration);

							UEBuildPlatform.GetBuildPlatform(Receipt.Platform).Deploy(Receipt);
						}
					}
				}
			}
			else
			{
				using ITimelineEvent MetadataTimer = Timeline.ScopeEvent("Writing Target Metadata");
				foreach (TargetMakefile Makefile in Makefiles)
				{
					FileReference TargetInfoFile = FileReference.Combine(Makefile.ProjectIntermediateDirectory, "TargetMetadata.json");
					if (FileReference.Exists(TargetInfoFile))
					{
						List<string> ArgumentsList = new List<string>()
						{
							$"-Input={TargetInfoFile}",
							$"-Version={WriteMetadataMode.CurrentVersionNumber}"
						};
						CommandLineArguments Arguments = new CommandLineArguments(ArgumentsList.ToArray());
						WriteMetadataMode MetadataMode = new WriteMetadataMode();
						await MetadataMode.ExecuteAsync(Arguments, Logger);
					}
				}
			}
		}

		internal static List<FileItem> CreateLinkedActionsFromFileList(TargetDescriptor Target, BuildConfiguration BuildConfiguration, List<FileReference> FileList, List<LinkedAction> Actions, ILogger Logger)
		{
			// We have specific files to compile so we will only queue up those files.
			// We will also add them to the MergedOutputItems to make sure they are not skipped
			List<FileItem> ProducedItems = new();

			// First, find all the SpecificFileActions.
			// These are used to create a custom action for the source file in case it is inside a unity or normally not part of the build (headers for example)
			Dictionary<DirectoryReference, ISpecificFileAction> SpecificFileActions = new();
			foreach (ISpecificFileAction SpecificFileAction in Actions.Select(x => x.Inner).OfType<ISpecificFileAction>())
			{
				SpecificFileActions.TryAdd(SpecificFileAction.RootDirectory, SpecificFileAction);
			}

			SourceFileMetadataCache MetadataCache = SourceFileMetadataCache.CreateHierarchy(null, Logger);

			// Now traverse all specific files and make sure we find or create actions for them.
			foreach (FileItem SourceFile in FileList.Select(x => FileItem.GetItemByFileReference(x)).Where(x => x.Exists))
			{
				if (SourceFile.HasExtension(".h") && MetadataCache.GetHeaderUnitType(SourceFile) != HeaderUnitType.Valid)
				{
					continue;
				}

				ISpecificFileAction? SpecificFileAction = null;

				// Traverse upwards from file directory to try to find a SpecificFileAction.
				DirectoryItem? Dir = SourceFile.Directory;
				while (Dir != null)
				{
					if (SpecificFileActions.TryGetValue(Dir.Location, out SpecificFileAction))
					{
						break;
					}
					Dir = Dir.GetParentDirectoryItem();
				}

				// We found an action to take care of this file.
				if (SpecificFileAction != null)
				{
					IExternalAction? NewAction = SpecificFileAction.CreateAction(SourceFile, Logger);
					if (NewAction != null)
					{
						Actions.Add(new LinkedAction(NewAction, Target));
						ProducedItems.AddRange(NewAction.ProducedItems);
						continue;
					}
				}

				// There is no special action for this file.. let's look for any actions that has this exact file as input and use that (for example ispc files)
				IEnumerable<LinkedAction> prereqActions = Actions.Where(x => x.PrerequisiteItems.Contains(SourceFile));

				if (prereqActions.Any())
				{
					// Hack to get PVS working with -SingleFile
					LinkedAction? pvsFinalizeAction = Actions.FirstOrDefault(x => x.CommandDescription == "Process PVS-Studio Results");
					if (pvsFinalizeAction != null)
					{
						prereqActions = prereqActions.Except([pvsFinalizeAction]);
						IExternalAction? newPvsFinalizeAction = PVSToolChain.SingleFileFinalizeOutput(pvsFinalizeAction, prereqActions);
						if (newPvsFinalizeAction != null)
						{
							Actions.Add(new LinkedAction(newPvsFinalizeAction, Target));
							ProducedItems.AddRange(newPvsFinalizeAction.ProducedItems);
						}
					}

					ProducedItems.AddRange(prereqActions.SelectMany(x => x.ProducedItems));
				}
				else if (!BuildConfiguration.bIgnoreInvalidFiles)
				{
					Logger.LogError("ERROR: Failed to find an Action that can be used to build \"{Path}\" (does target use this file?)", SourceFile);
				}
			}

			return ProducedItems;
		}

		/// <summary>
		/// If there are headers in the specific list, this function will expand the list to include all files that include those headers
		/// </summary>
		/// <param name="SpecificFilesToCompile">List of files to compile</param>
		/// <param name="ProjectFile">Project file if there is one</param>
		/// <param name="Actions">Actions to filter out expansion of files</param>
		/// <param name="Logger">Logger for output</param>
		internal static List<FileReference> GetAllSourceFilesIncludingHeader(List<FileReference> SpecificFilesToCompile, FileReference? ProjectFile, List<IExternalAction> Actions, ILogger Logger)
		{
			Func<FileReference, bool> IsCode = x => IsHeader(x) || x.HasExtension(".cpp") || x.HasExtension(".c");
			IEnumerable<FileReference> SpecificHeaderFiles = SpecificFilesToCompile.Where(IsHeader);
			if (!SpecificHeaderFiles.Any())
			{
				return SpecificFilesToCompile;
			}

			Logger.LogInformation("Building dependency cache for specified headers");
			foreach (FileReference HeaderFile in SpecificHeaderFiles)
			{
				Logger.LogDebug("  {HeaderFile}", HeaderFile);
			}

			// TODO: This code below works very similar to the code before this changelist but is not a great solution for figuring out which files to compile.
			// Suggested approach is to:
			// 1. Write out a file with additional include paths per module.
			// 2. Use the special single file actions to figure out if the specific headers are included by the source files using additional include paths
			// 3. Fix so CppIncludeLookup is including headers... Current code does not handle this: A.h <- B.h <- C.cpp... C.cpp is not added.
			// 4. Cache which files unity files include to reduce number of files needed to be read

			List<DirectoryReference> BaseDirs = [Unreal.EngineDirectory];
			DirectoryReference CacheDir = Unreal.EngineDirectory;
			if (ProjectFile != null)
			{
				BaseDirs.Add(ProjectFile.Directory);
				CacheDir = ProjectFile.Directory;
			}

			// Ignore generated files
			IEnumerable<FileItem> CodeFiles = 
				Actions
				.SelectMany(x => x.PrerequisiteItems)
				.Distinct()
				.Where(x => IsCode(x.Location) && !x.Name.StartsWith(Unity.ModulePrefix, StringComparison.Ordinal) && !x.HasExtension(".gen.cpp"));

			FileReference IncludeCache = FileReference.Combine(CacheDir, "Intermediate", "HeaderLookup.dat");
			CppIncludeLookup IncludeLookup = new CppIncludeLookup(IncludeCache);
			IncludeLookup.Load();
			IncludeLookup.Update(BaseDirs, CodeFiles);
			IncludeLookup.Save();

			// Find all files that can be part of this build
			IEnumerable<FileReference> UsedFiles = CodeFiles.Select(x => x.Location);
			IEnumerable<FileReference> FoundFiles = IncludeLookup.FindFiles(SpecificFilesToCompile.Select(x => x.GetFileName()).Distinct()).Select(x => x.Location);

			// Found files can be a lot (Commandline.h gives 130.000+ files). And we only want to build the ones that are in existing actions
			IEnumerable<FileReference> NewFiles = FoundFiles.Intersect(UsedFiles).Distinct();

			Logger.LogInformation("Found {NewFilesCount} source files", NewFiles.Count());
			foreach (FileReference NewFile in NewFiles.Order())
			{
				Logger.LogTrace("  {NewFile}", NewFile);
			}

			return NewFiles.Concat(SpecificFilesToCompile).Distinct().Order().ToList();
		}

		/// <summary>
		/// Outputs the toolchain used to build each target
		/// </summary>
		/// <param name="TargetDescriptors">List of targets being built</param>
		/// <param name="Makefiles">Matching array of makefiles for each target</param>
		/// <param name="Logger">Logger for output</param>
		static void OutputToolchainInfo(List<TargetDescriptor> TargetDescriptors, TargetMakefile[] Makefiles, ILogger Logger)
		{
			Dictionary<(string, UnrealTargetPlatform, UnrealTargetConfiguration), IEnumerable<string>> DistinctTargets = [];
			for (int Idx = 0; Idx < TargetDescriptors.Count; Idx++)
			{
				if (!TargetDescriptors[Idx].bQuiet && Makefiles[Idx].Diagnostics.Count > 0)
				{
					DistinctTargets.TryAdd((TargetDescriptors[Idx].Name, TargetDescriptors[Idx].Platform, TargetDescriptors[Idx].Configuration), Makefiles[Idx].Diagnostics);
				}
			}

			if (DistinctTargets.Count == 0)
			{
				return;
			}

			Logger.LogInformation("===== Toolchain Information =====");
			if (DistinctTargets.Count == 1)
			{
				foreach (string Diagnostic in DistinctTargets.First().Value)
				{
					Logger.LogInformation("{Diagnostic}", Diagnostic);
				}
			}
			else
			{
				foreach (KeyValuePair<(string, UnrealTargetPlatform, UnrealTargetConfiguration), IEnumerable<string>> DistinctTarget in DistinctTargets)
				{
					Logger.LogInformation("{Name} {Platform} {Configuration}:", DistinctTarget.Key.Item1, DistinctTarget.Key.Item2, DistinctTarget.Key.Item3);
					foreach (string Diagnostic in DistinctTarget.Value)
					{
						Logger.LogInformation("  {Diagnostic}", Diagnostic);
					}
				}
			}
			Logger.LogInformation("=================================");
		}

		/// <summary>
		/// Returns true if File is header
		/// </summary>
		/// <param name="File">File to check</param>
		internal static bool IsHeader(FileReference File) { return File.HasExtension(".h") || File.HasExtension(".hpp") || File.HasExtension(".hxx"); }

		/// <summary>
		/// Creates the makefile for a target. If an existing, valid makefile already exists on disk, loads that instead.
		/// </summary>
		/// <param name="BuildConfiguration">The build configuration</param>
		/// <param name="TargetDescriptor">Target being built</param>
		/// <param name="WorkingSet">Set of source files which are part of the working set</param>
		/// <param name="Logger">Logger for output</param>
		/// <returns>Makefile for the given target</returns>
		internal static async Task<TargetMakefile> CreateMakefileAsync(BuildConfiguration BuildConfiguration, TargetDescriptor TargetDescriptor, ISourceFileWorkingSet WorkingSet, ILogger Logger)
		{
			await Task.CompletedTask;

			List<Task> tasks = [];
			try
			{
				List<List<(TargetDescriptor, TargetMakefile Makefile)>> targetsPasses = CreateMakefiles(BuildConfiguration, [TargetDescriptor], WorkingSet, Logger, false, false, tasks, null, new(false, true));
				return targetsPasses[0][0].Makefile;
			}
			finally
			{
				await Task.WhenAll(tasks.ToArray());
			}
		}

		// Function that will load makefile and check if it is valid. (First round of targets will have their makefiles loaded in parallel with other work)
		private static (TargetMakefile? makefile, bool ranPrebuildScripts) LoadMakefile(FileReference location, TargetDescriptor descriptor, ISourceFileWorkingSet workingSet, ILogger logger)
		{
			string? ReasonNotLoaded;
			TargetMakefile? makefile = TargetMakefile.Load(location, descriptor.ProjectFile, descriptor.Platform, descriptor.AdditionalArguments.GetRawArray(), logger, out ReasonNotLoaded);
			if (makefile == null)
			{
				logger.LogInformation("Creating makefile for {DescriptorName} ({ReasonNotLoaded})", descriptor.Name, ReasonNotLoaded);
				return (null, false);
			}

			// Execute the scripts.
			Utils.ExecuteCustomBuildSteps(makefile.PreBuildScripts, logger);

			// Check that the makefile is still valid
			string? Reason;
			if (!TargetMakefile.IsValidForSourceFiles(makefile, descriptor.ProjectFile, descriptor.Platform, workingSet, logger, out Reason))
			{
				logger.LogInformation("Invalidating makefile for {DescriptorName} ({Reason})", descriptor.Name, Reason);
				return (null, true);
			}

			return (makefile, true);
		}

		private static List<List<(TargetDescriptor, TargetMakefile)>> CreateMakefiles(BuildConfiguration buildConfiguration, List<TargetDescriptor> descriptors, ISourceFileWorkingSet workingSet, ILogger logger, bool runInitScripts, bool skipPreBuildTargets, List<Task> outTasks, CppDependencyCache? cppDependencyCache, FileHashCache fileHashCache)
		{
			List<List<(TargetDescriptor descriptor, TargetMakefile makefile)>> outTargetsPasses = new();

			int helperCount = buildConfiguration.bParallelMakefileGeneration ? -1 : 0;

			Dictionary<FileReference, Task<SourceFileMetadataCache>> sourceFileMetadataCaches = [];

			// Mac toolchain unzips frameworks to a folder outside of Intermediate/Build and also write a .sh file. So we need to mount this directory
			if (OperatingSystem.IsMacOS())
			{
				fileHashCache.Mount(DirectoryReference.Combine(Unreal.WritableEngineDirectory, "Intermediate", "UnzippedFrameworks"), logger);
			}

			int prebuildTargetsSanityCounter = 0;

			while (descriptors.Count != 0) // descriptors list will be reallocated for PrebuildTargets. hence the while-loop
			{
				if (prebuildTargetsSanityCounter++ > 20)
				{
					logger.LogError("Error: Prebuild targets has looped 20 times. Is there a prebuild target pointing back to itself somewhere? ({Descriptors})", String.Join(' ', descriptors.Select(d => d.Name)));
					throw new CompilationResultException(CompilationResult.Unknown);
				}

				List<(TargetDescriptor descriptor, TargetMakefile makefile)> outTargets = new();

				int descriptorCount = descriptors.Count;
				bool skipRulesCompile = buildConfiguration.bSkipRulesCompile;
				bool forceRulesCompile = buildConfiguration.bForceRulesCompile;
				bool usePrecompiled = buildConfiguration.bUsePrecompiled;

				FileReference?[] makefileLocations = new FileReference[descriptorCount];
				UEBuildTarget[] buildTargets = new UEBuildTarget[descriptorCount];
				IReadOnlyList<FileReference>[] manifestFileNames = new IReadOnlyList<FileReference>[descriptorCount];

				// Start loading unique makefiles
				// Note, there are (very) rare instances of this being the wrong makefile
				// and that is when target rules change name of target. so there is a tiny chance we will waste some time loading a makefile that is not being used
				Dictionary<FileReference, Task<(TargetMakefile?, bool)>> preLoadedMakefiles = [];
				if (buildConfiguration.bUseUBTMakefiles && buildConfiguration.bParallelMakefileGeneration) // Only preload if parallel is enabled to prevent workingSet from being called from multiple threads
				{
					for (int index = 0; index != descriptorCount; ++index)
					{
						TargetDescriptor descriptor = descriptors[index];
						FileReference makefileLocation = TargetMakefile.GetLocation(descriptor.ProjectFile, descriptor.Name, descriptor.Platform, descriptor.Architectures, descriptor.Configuration, descriptor.IntermediateEnvironment);
						if (!preLoadedMakefiles.ContainsKey(makefileLocation))
						{
							preLoadedMakefiles.Add(makefileLocation, Task.Run(() => { return LoadMakefile(makefileLocation, descriptor, workingSet, logger); }));
						}
					}
				}

				// All appnames have unique folders. So if appnames are different we can run them in parallel.
				// The reason we can't run different targets with same appname in parallel is because of codegen (since codegen is not in target folder)
				// LyraEditor, QAGameEditor etc has the same appname (UnrealEditor) but same codegen folder (for engine part at least)
				Dictionary<string, List<int>> appNameTargetGroups = [];

				// Build all rules objects and populate groups. Start loading makefiles
				Parallel2.For(0, descriptorCount, (index) =>
				{
					TargetDescriptor descriptor = descriptors[index];

					// Create task to load source file metadata cache already here, it will be used later on
					if (descriptor.ProjectFile != null)
					{
						lock (sourceFileMetadataCaches)
						{
							if (!sourceFileMetadataCaches.ContainsKey(descriptor.ProjectFile))
							{
								sourceFileMetadataCaches.Add(descriptor.ProjectFile, Task.Run(() => SourceFileMetadataCache.CreateHierarchy(descriptor.ProjectFile, logger)));
							}
						}
					}

					// Compile/load assembly and create target rules
					(TargetRules rules, RulesAssembly assembly) = UEBuildTarget.CreateRulesObject(descriptor, skipRulesCompile, forceRulesCompile, usePrecompiled, descriptor.IntermediateEnvironment, logger);

					// Create the target. Ctor does not do much so we can create it regardless if we are building it or not
					UEBuildTarget buildTarget = new(descriptor, new ReadOnlyTargetRules(rules), assembly);
					buildTargets[index] = buildTarget;

					// Queue up cppdependencies loading
					if (cppDependencyCache != null)
					{
						cppDependencyCache.MountAsync(descriptor, buildTarget.AppName, buildTarget.IntermediateEnvironment, logger);
					}

					fileHashCache.Mount(descriptor, buildTarget.AppName, buildTarget.IntermediateEnvironment, logger);

					// Add target to proper app name group
					string appName = $"{buildTarget.AppName}_{buildTarget.Platform}";
					lock (appNameTargetGroups)
					{
						if (!appNameTargetGroups.TryGetValue(appName, out List<int>? appNameTarget))
						{
							appNameTarget = new();
							appNameTargetGroups.Add(appName, appNameTarget);
						}

						appNameTarget.Add(index);
					}
				}, helperCount);

				UEBuildTarget.BuildContext?[] buildContexts = new UEBuildTarget.BuildContext?[descriptorCount];
				TargetMakefile?[] makefiles = new TargetMakefile?[descriptorCount];
				ITimelineEvent?[] targetTimelines = new ITimelineEvent?[descriptorCount];

				// Will be populated with prebuild targets if there are any
				List<TargetDescriptor> newDescriptors = [];

				// Run all appname groups in parallel
				Parallel2.ForEach(appNameTargetGroups, kv =>
				{
					List<int> appNameTargets = kv.Value;
					appNameTargets.Sort(); // To build in same order as registered

					// All targets with same code gen folder but different engine intermediate dir can run in parallel
					// but they need to sync up for when code gen happens (UHT/VNI)

					Dictionary<DirectoryReference, List<int>> intermediateFolderGroups = [];
					int maxTargetsInIntermediateCount = 0;

					for (int i = 0; i != appNameTargets.Count; ++i)
					{
						int index = appNameTargets[i];
						UEBuildTarget buildTarget = buildTargets[index];
						DirectoryReference dir = buildTarget.EngineIntermediateDirectory;
						if (!intermediateFolderGroups.TryGetValue(dir, out List<int>? intermediateFolderGroup))
						{
							intermediateFolderGroup = new();
							intermediateFolderGroups.Add(dir, intermediateFolderGroup);
						}
						intermediateFolderGroup.Add(index);
						maxTargetsInIntermediateCount = Math.Max(maxTargetsInIntermediateCount, intermediateFolderGroup.Count);
					}

					// We need to track save makefile task so we can wait for it in case we need to load it again.
					// TODO: Just reuse previous one. we don't need to save until we are on the last one.
					ConcurrentDictionary<FileReference, Task> lastSaveMakefileTasks = []; // This must be allocated in here because it is iterated when all passes are done

					List<UEBuildTarget.CodeGenContext>? lastContextsForCodeGen = null;

					try
					{
						// Split up in passes. We run one target from each intermediate folder.
						for (int passIt = 0; passIt != maxTargetsInIntermediateCount; ++passIt)
						{
							List<UEBuildTarget.CodeGenContext> contextsForCodeGen = [];

							// Do everyting up to code gen in parallel
							Parallel2.ForEach(intermediateFolderGroups.Values, intermediateFolderGroup =>
							{
								if (passIt >= intermediateFolderGroup.Count)
								{
									return; // This group is done
								}

								int index = intermediateFolderGroup[passIt];
								TargetDescriptor descriptor = descriptors[index];

								if (runInitScripts)
								{
									// Create and execute the init scripts
									FileReference[] InitScripts = CreateInitScripts(descriptor);
									if (InitScripts.Length > 0)
									{
										Utils.ExecuteCustomBuildSteps(InitScripts, logger);
									}
								}

								TargetMakefile? makefile = null;
								bool ranPrebuildScripts = false;

								UEBuildTarget buildTarget = buildTargets[index];
								ReadOnlyTargetRules targetRules = buildTarget.Rules;

								// Calculating real makefile location. it is likely the same as the preload but not if appname is changed in target rules
								FileReference? makefileLocation = TargetMakefile.GetLocation(targetRules.ProjectFile, targetRules.Name, targetRules.Platform, targetRules.Architectures, targetRules.Configuration, buildTarget.IntermediateEnvironment);
								makefileLocations[index] = makefileLocation;

								if (makefileLocation != null)
								{
									Task<(TargetMakefile?, bool)>? makefileTask = null;
									lock (preLoadedMakefiles)
									{
										if (preLoadedMakefiles.TryGetValue(makefileLocation, out makefileTask))
										{
											preLoadedMakefiles.Remove(makefileLocation);
										}
									}
									if (makefileTask != null)
									{
										(makefile, ranPrebuildScripts) = makefileTask.Result;
									}
									else
									{
										// Need to make sure to wait on previous run's makefile
										lastSaveMakefileTasks.TryRemove(makefileLocation, out Task? lastMakefileSave);
										lastMakefileSave?.Wait();
										(makefile, ranPrebuildScripts) = LoadMakefile(makefileLocation, descriptor, workingSet, logger);
									}
								}

								targetTimelines[index] = Timeline.ScopeEvent($"{descriptor.Name} {descriptor.Platform} {descriptor.Configuration}");

								List<UEBuildTarget.GenerateHeaderFunc> generatedHeaderFuncs = [];

								// If we couldn't load a valid makefile, create a new one
								if (makefile == null)
								{
									CaptureLogger captureLogger = new();
									try
									{
										buildTarget.PreBuildSetup(captureLogger);

										// Create the pre-build scripts
										FileReference[] PreBuildScripts = buildTarget.CreatePreBuildScripts();

										// Execute the pre-build scripts
										if (!ranPrebuildScripts)
										{
											Utils.ExecuteCustomBuildSteps(PreBuildScripts, captureLogger);
											ranPrebuildScripts = true;
										}

										SourceFileMetadataCache sourceFileMetaDataCache = descriptor.ProjectFile != null
											? sourceFileMetadataCaches[descriptor.ProjectFile].Result
											: SourceFileMetadataCache.CreateHierarchy(null, logger);

										// Build the target
										UEBuildTarget.BuildContext context = buildTarget.BuildMakefile1(buildConfiguration, descriptor, captureLogger, sourceFileMetaDataCache);
										buildContexts[index] = context;

										makefile = context.Makefile;

										makefile.PreBuildScripts = PreBuildScripts;

										makefile.MemoryPerActionGB = buildTarget.Rules.MemoryPerActionGB;

										// Save the additional command line arguments
										makefile.AdditionalArguments = descriptor.AdditionalArguments.GetRawArray();

										generatedHeaderFuncs = context.GenerateHeaderFuncs;
										makefile.LogEvents = [.. captureLogger.Events];
									}
									finally
									{
										captureLogger.RenderTo(logger);
									}
								}
								else
								{
									CaptureLogger captureLogger = new();
									captureLogger.Events.AddRange(makefile.LogEvents);
									captureLogger.RenderTo(logger);
								}

								lock (contextsForCodeGen)
								{
									contextsForCodeGen.Add(new UEBuildTarget.CodeGenContext()
									{
										Descriptor = descriptor,
										Makefile = makefile,
										TargetName = buildTarget.TargetName,
										GenerateHeaderFuncs = generatedHeaderFuncs,
									});
								}
								makefiles[index] = makefile;
							}, helperCount);

							// Make sure to wait for previous pass codegen and timestamp writes
							if (lastContextsForCodeGen != null)
							{
								UEBuildTarget.WaitCodeGen(lastContextsForCodeGen, false);
								Task.WaitAll(UEBuildTarget.GetTimestampTasks(lastContextsForCodeGen).ToArray());
							}

							// Sort codegen based on config with Shipping last.
							// Since shipping often has fewer modules, that means previous configs will include later ones
							contextsForCodeGen.SortBy(context => context.Descriptor.Configuration);

							// Run codegen for all targets
							UEBuildTarget.RunCodeGen(buildConfiguration, contextsForCodeGen, logger);

							// Go wide again and finish the work
							Parallel2.ForEach(intermediateFolderGroups.Values, intermediateFolderGroup =>
							{
								if (passIt >= intermediateFolderGroup.Count)
								{
									return; // This group is done
								}

								int index = intermediateFolderGroup[passIt];
								TargetDescriptor descriptor = descriptors[index];

								TargetMakefile makefile = makefiles[index]!;
								UEBuildTarget.BuildContext? buildContext = buildContexts[index];

								if (buildContext != null)
								{
									UEBuildTarget buildTarget = buildTargets[index];
									manifestFileNames[index] = (index == 0) ? buildTarget.Rules.ManifestFileNames : [.. buildTarget.Rules.ManifestFileNames.Select( x => x.ChangeExtension($".{index}.xml") )]; // generate unique names for the manifests for other threads

									buildTarget.BuildMakefile2(workingSet, logger, buildContext, fileHashCache, manifestFileNames[index]);

									// Save the makefile for next time
									FileReference? makefileLocation = makefileLocations[index];
									if (makefileLocation != null)
									{
										// Save the environment variables. Toolchains etc might set environment variables
										// and we want to be able to restore them on runs where makefile is not invalidated.
										// We know they can contain other target's environment variables too which is bad but by-design atm.
										foreach (System.Collections.DictionaryEntry? EnvironmentVariable in Environment.GetEnvironmentVariables())
										{
											if (EnvironmentVariable != null)
											{
												makefile.EnvironmentVariables.Add(Tuple.Create((string)EnvironmentVariable.Value.Key, (string)EnvironmentVariable.Value.Value!));
											}
										}

										Task saveTask = Task.Run(() =>
										{
											using ITimelineEvent _ = Timeline.ScopeEvent($"SavingMakefile");
											makefile.Save(makefileLocation!);
										});

										if (!lastSaveMakefileTasks.TryAdd(makefileLocation, saveTask))
										{
											throw new Exception("Can't add savemakefile task. This should never fail");
										}
									}
								}
								else
								{
									// Restore the environment variables
									foreach (Tuple<string, string> EnvironmentVariable in makefile.EnvironmentVariables)
									{
										Environment.SetEnvironmentVariable(EnvironmentVariable.Item1, EnvironmentVariable.Item2);
									}
								}

								lock (outTargets)
								{
									outTargets.Add((descriptor, makefile));
								}

								if (!skipPreBuildTargets)
								{
									foreach (TargetInfo PreBuildTarget in makefile.PreBuildTargets)
									{
										lock (newDescriptors)
										{
											newDescriptors.Add(TargetDescriptor.FromTargetInfo(PreBuildTarget));
										}
									}
								}

								targetTimelines[index]?.Finish();
							}, helperCount);

							lastContextsForCodeGen = contextsForCodeGen;
						}
					}
					catch
					{
						// Make sure makefile saves are waited on when exception occurs
						Task.WaitAll(lastSaveMakefileTasks.Values.ToArray());
						throw;
					}

					// All passes are done.

					if (lastContextsForCodeGen != null)
					{
						List<Task> timestampTasks = UEBuildTarget.GetTimestampTasks(lastContextsForCodeGen);
						UEBuildTarget.WaitCodeGen(lastContextsForCodeGen, true); // We want to prefetch generated dirs since actiongraph will read file info

						lock (outTasks)
						{
							outTasks.AddRange(timestampTasks);
						}
					}

					lock (outTasks)
					{
						outTasks.AddRange(lastSaveMakefileTasks.Values);
					}
				}, helperCount);

				descriptors = newDescriptors;

				outTargetsPasses.Insert(0, outTargets);

				// In the very rare scenario we preload the wrong makefiles we want to make sure to wait for them in case there are prebuildtargets that use them
				foreach (Task<(TargetMakefile?, bool)> task in preLoadedMakefiles.Values)
				{
					task?.Wait();
				}

				// Make sure to finish all tasks if we have prebuild targets
				if (descriptors.Count != 0)
				{
					Task.WaitAll(outTasks.ToArray());
					outTasks.Clear();
				}

				// Merge the manifest files back together. @todo: could do better
				for (int index = 1; index != descriptorCount; ++index)
				{
					if (manifestFileNames[index] != null && manifestFileNames[index].Count > 0)
					{
						IReadOnlyList<FileReference> targetManifests = buildTargets[index].Rules.ManifestFileNames;
						IReadOnlyList<FileReference> workingSetManifests = manifestFileNames[index];
						if (workingSetManifests.Count != targetManifests.Count)
						{
							throw new BuildException("Manifest count mismatch for {0} ... {1} != {2}", buildTargets[index].TargetName, workingSetManifests.Count, targetManifests.Count);
						}
						for (int manifestIndex = 0; manifestIndex < workingSetManifests.Count; ++manifestIndex)
						{
							FileReference[] sourceManifests = [targetManifests[manifestIndex], workingSetManifests[manifestIndex]];
							UEBuildTarget.CombineManifests( targetManifests[manifestIndex], sourceManifests, logger);
						}
					}
				}
			}

			return outTargetsPasses;
		}

		/// <summary>
		/// Determines all the actions that should be executed for a target (filtering for single module/file, etc..)
		/// </summary>
		/// <param name="TargetDescriptor">The target being built</param>
		/// <param name="Makefile">Makefile for the target</param>
		/// <param name="OutputItems">Set of all output items</param>
		/// <returns>List of actions that need to be executed</returns>
		static void GatherOutputItems(TargetDescriptor TargetDescriptor, TargetMakefile Makefile, HashSet<FileItem> OutputItems)
		{
			if (TargetDescriptor.SpecificFilesToCompile.Count > 0)
			{
			}
			else if (TargetDescriptor.OnlyModuleNames.Count > 0)
			{
				// Find the output items for this module
				foreach (string OnlyModuleName in TargetDescriptor.OnlyModuleNames)
				{
					FileItem[]? OutputItemsForModule;
					if (!Makefile.ModuleNameToOutputItems.TryGetValue(OnlyModuleName, out OutputItemsForModule))
					{
						throw new BuildException("Unable to find output items for module '{0}'", OnlyModuleName);
					}
					OutputItems.UnionWith(OutputItemsForModule);
				}
			}
			else
			{
				// Use all the output items from the target
				OutputItems.UnionWith(Makefile.OutputItems);
			}
		}

		/// <summary>
		/// Merge action graphs for multiple targets into a single set of actions. Sets group names on merged actions to indicate which target they belong to.
		/// </summary>
		/// <param name="TargetDescriptors">List of target descriptors</param>
		/// <param name="TargetActions">Array of actions for each target</param>
		/// <returns>List of merged actions</returns>
		static List<LinkedAction> MergeActionGraphs(List<TargetDescriptor> TargetDescriptors, List<LinkedAction>[] TargetActions)
		{
			// Set of all output items. Knowing that there are no conflicts in produced items, we use this to eliminate duplicate actions.
			Dictionary<FileItem, LinkedAction> OutputItemToProducingAction = new Dictionary<FileItem, LinkedAction>();
			HashSet<LinkedAction> IgnoreConflictActions = new HashSet<LinkedAction>();
			for (int TargetIdx = 0; TargetIdx < TargetDescriptors.Count; TargetIdx++)
			{
				string GroupPrefix = String.Format("{0}-{1}-{2}", TargetDescriptors[TargetIdx].Name, TargetDescriptors[TargetIdx].Platform, TargetDescriptors[TargetIdx].Configuration);
				foreach (LinkedAction TargetAction in TargetActions[TargetIdx])
				{
					if (!TargetAction.ProducedItems.Any())
					{
						continue;
					}

					if (TargetAction.IgnoreConflicts())
					{
						IgnoreConflictActions.Add(TargetAction);
						TargetAction.GroupNames.Add(GroupPrefix);
						continue;
					}

					FileItem ProducedItem = TargetAction.ProducedItems.First();

					LinkedAction? ExistingAction;
					if (!OutputItemToProducingAction.TryGetValue(ProducedItem, out ExistingAction))
					{
						ExistingAction = new LinkedAction(TargetAction, TargetDescriptors[TargetIdx]);
						OutputItemToProducingAction[ProducedItem] = ExistingAction;
					}
					ExistingAction.GroupNames.Add(GroupPrefix);
				}
			}
			List<LinkedAction> Results = new List<LinkedAction>(OutputItemToProducingAction.Values);
			Results.AddRange(IgnoreConflictActions);
			return Results;
		}

		private static void ProcessCoreDumps(DirectoryReference? SaveCrashDumpDirectory, ILogger Logger)
		{
			if (SaveCrashDumpDirectory == null)
			{
				return;
			}

			if (!OperatingSystem.IsWindows())
			{
				return;
			}

			// set to true to have this code create some files in expected locations to report 
			bool bTestDumpDetection = false;

			using OpenTracing.IScope Scope = GlobalTracer.Instance.BuildSpan("Processing crash dump files").StartActive();

			DateTime UBTStartTime = Process.GetCurrentProcess().StartTime;

			List<FileReference> FoundCrashDumps = new List<FileReference>();

			// examine the contents of %LOCALAPPDATA%\CrashDumps
			string? LocalAppData = Environment.GetEnvironmentVariable("LOCALAPPDATA");
			if (LocalAppData != null)
			{
				DirectoryReference CrashDumpsDirectory = DirectoryReference.Combine(DirectoryReference.FromString(LocalAppData)!, "CrashDumps");
				if (DirectoryReference.Exists(CrashDumpsDirectory))
				{
					if (bTestDumpDetection)
					{
						System.IO.File.Create(System.IO.Path.Combine(CrashDumpsDirectory.FullName, Guid.NewGuid().ToString() + ".dmp")).Close();
					}

					foreach (FileReference CrashDump in DirectoryReference.EnumerateFiles(CrashDumpsDirectory))
					{
						if (FileReference.GetLastWriteTime(CrashDump) > UBTStartTime)
						{
							Logger.LogWarning("Crash dump {CrashDump} was created duing UnrealBuildTool execution", CrashDump);
							FoundCrashDumps.Add(CrashDump);
						}
					}
				}
			}

			// examine the contents of %TMP% (on CI agents, this should not be unreasonably slow as the tmp dir gets cleaned between builds
			string? TMP = Environment.GetEnvironmentVariable("TMP");
			if (TMP != null)
			{
				DirectoryReference TmpDir = DirectoryReference.FromString(TMP)!;

				if (bTestDumpDetection)
				{
					System.IO.File.Create(System.IO.Path.Combine(TmpDir.FullName, Guid.NewGuid().ToString() + ".dmp")); //.Close(); Intentionally not closed, to trigger catch() block below
				}

				List<DirectoryReference> AccessibleTmpDirectories = new List<DirectoryReference>();

				AccessibleTmpDirectories.Add(TmpDir);

				for (int I = 0; I < AccessibleTmpDirectories.Count; ++I)
				{
					try
					{
						// Manual recursion to avoid the case where an inaccessible file prevents us from iterating reachable parts of the dir
						AccessibleTmpDirectories.AddRange(DirectoryReference.EnumerateDirectories(AccessibleTmpDirectories[I]));

						foreach (FileReference TmpFile in DirectoryReference.EnumerateFiles(AccessibleTmpDirectories[I]))
						{
							if (TmpFile.HasExtension(".dmp") && FileReference.GetLastWriteTime(TmpFile) > UBTStartTime)
							{
								Logger.LogWarning("Crash dump {TmpFile} was created duing UnrealBuildTool execution", TmpFile);
								FoundCrashDumps.Add(TmpFile);
							}
						}
					}
					catch
					{
						// silently ignore inaccessible directories
					}
				}
			}

			if (FoundCrashDumps.Count > 0)
			{
				DirectoryReference.CreateDirectory(SaveCrashDumpDirectory);
				foreach (FileReference CrashDump in FoundCrashDumps)
				{
					Logger.LogInformation("Copying {CrashDump} to {SaveCrashDumpDirectory}", CrashDump, SaveCrashDumpDirectory);
					try
					{
						FileReference.Copy(CrashDump, FileReference.Combine(SaveCrashDumpDirectory, CrashDump.GetFileName()));
					}
					catch (Exception Ex)
					{
						// don't stop if there was a problem copying one of the files
						Logger.LogWarning("Failed to copy crash dump {CrashDump}: {Ex}", CrashDump, ExceptionUtils.FormatException(Ex));
					}
				}
			}
		}
	}
}

