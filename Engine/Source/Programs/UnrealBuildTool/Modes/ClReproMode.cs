// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text.Json;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool.Modes
{
	internal sealed class ClReproMode : IToolMode<ClReproMode>
	{
		public static string Name => "ClRepro";
		public static ToolModeOptions Options => ToolModeOptions.UseStartupTraceListener;

		[CommandLine("-InputCpp=", Required = true)]
		public FileReference? InputCpp { get; set; } = null;

		[CommandLine("-OutputDir=", Required = true)]
		public DirectoryReference? OutputDir { get; set; } = null;

		public class ClDepends
		{
			public class ClDependsData
			{
				public string? Source { get; init; }
				public string? PCH { get; init; }
				public HashSet<string>? Includes { get; init; }
			}
			public string? Version { get; init; }
			public ClDependsData? Data { get; init; }
		}

		private static string GetCaseSensitivePath(string path)
		{
			try
			{
				string root = Path.GetPathRoot(path)?.ToUpperInvariant() ?? String.Empty;
				foreach (string name in path.Substring(root.Length).Split(System.IO.Path.DirectorySeparatorChar))
				{
					root = System.IO.Directory.GetFileSystemEntries(root, name).First();
				}
				return root;
			}
			catch (Exception)
			{
			}
			return path;
		}

		private async Task<int> PrepareFullRepro(ILogger logger)
		{
			if (InputCpp == null || OutputDir == null)
			{
				return 1;
			}

			FileReference inputDepFile = InputCpp.ChangeExtension(".cpp.dep.json");

			ClDepends? depends;
			using (System.IO.FileStream fstream = FileReference.Open(inputDepFile, System.IO.FileMode.Open))
			{
				depends = await JsonSerializer.DeserializeAsync<ClDepends>(fstream);
			}

			if (depends == null || depends.Data == null || String.IsNullOrEmpty(depends.Data.Source))
			{
				logger.LogError("Unable to deserialize {Dep}", inputDepFile);
				return 1;
			}

			if (!String.IsNullOrEmpty(depends.Data.PCH))
			{
				logger.LogError("PCH repro not currently supported, please use -NoPCH when compiling ({Dep})", inputDepFile);
				return 1;
			}

			FileReference inputRspFile = InputCpp.ChangeExtension(".cpp.obj.rsp");
			FileReference inputSharedRspFile = DirectoryReference.EnumerateFiles(InputCpp.Directory, "*.Shared.rsp", SearchOption.TopDirectoryOnly).First();
			SortedSet<FileReference> allFiles = new();
			allFiles.Add(new FileReference(GetCaseSensitivePath(depends.Data.Source)));
			if (depends.Data.Includes != null)
			{
				allFiles.UnionWith(depends.Data.Includes.Select(x => new FileReference(GetCaseSensitivePath(x))));
			}

			DirectoryReference.CreateDirectory(OutputDir);

			List<string> rootPaths = new();
			rootPaths.Add(Unreal.RootDirectory.FullName + "\\");
			logger.LogInformation("Copying {Count} files to {OutputDir}...", allFiles.Count, OutputDir);
			foreach (FileReference file in allFiles.Where(x => FileReference.Exists(x)))
			{
				FileReference dest;
				if (file.IsUnderDirectory(Unreal.RootDirectory))
				{
					dest = FileReference.Combine(OutputDir, file.MakeRelativeTo(Unreal.RootDirectory));
					if (!file.IsUnderDirectory(Unreal.EngineDirectory))
					{
						logger.LogWarning("{File} is not under Engine directory and may not be safe to share", dest);
					}
				}
				else
				{
					DirectoryReference rootDir = new DirectoryReference(Path.GetPathRoot(file.FullName)!);
					if (!rootPaths.Contains(rootDir.FullName))
					{
						rootPaths.Add(rootDir.FullName);
					}
					dest = FileReference.Combine(OutputDir, file.MakeRelativeTo(rootDir));
				}
				DirectoryReference.CreateDirectory(dest.Directory);
				FileReference.Copy(file, dest);
				logger.LogDebug("Copied {Input} to {Output}", file, dest);
			}

			FileReference batchFile = FileReference.Combine(OutputDir, "Run.bat");

			FileReference outputRspFile = FileReference.Combine(OutputDir, inputRspFile.MakeRelativeTo(Unreal.RootDirectory));
			FileReference outputSharedRspFile = FileReference.Combine(OutputDir, inputSharedRspFile.MakeRelativeTo(Unreal.RootDirectory));

			string FixPaths(string line)
			{
				foreach (string root in rootPaths)
				{
					line = line.Replace($"\"{root}", "\"..\\..\\", StringComparison.InvariantCultureIgnoreCase);
				}
				return line;
			}

			// Update include paths in the rsp files
			await FileReference.WriteAllLinesAsync(outputRspFile, (await FileReference.ReadAllLinesAsync(inputRspFile)).Select(x => FixPaths(x)));
			logger.LogInformation("Rewrote absolute paths for {Input} to {Output}", inputRspFile, outputRspFile);
			await FileReference.WriteAllLinesAsync(outputSharedRspFile, (await FileReference.ReadAllLinesAsync(inputSharedRspFile)).Select(x => FixPaths(x)));
			logger.LogInformation("Rewrote absolute paths for {Input} to {Output}", inputSharedRspFile, outputSharedRspFile);

			DirectoryReference outputEngineSourceDir = DirectoryReference.Combine(OutputDir, "Engine", "Source");
			await FileReference.WriteAllLinesAsync(batchFile, new string[]
			{
				$"@echo off",
				$"setlocal",
				$"call \"%ProgramFiles%\\Microsoft Visual Studio\\2022\\Professional\\Common7\\Tools\\VsDevCmd.bat\"",
				$"cd /d \"%~dp0\\Engine\\Source\"",
				$"cl @\"{outputRspFile.MakeRelativeTo(outputEngineSourceDir)}\"",
				$"endlocal",
			});
			logger.LogInformation("Wrote batch file {Batch}", batchFile);

			return 0;
		}

		public Task<int> ExecuteAsync(CommandLineArguments arguments, ILogger logger)
		{
			arguments.ApplyTo(this);

			DirectoryReference logDirectory = DirectoryReference.Combine(Unreal.EngineProgramSavedDirectory, "UnrealBuildTool");
			DirectoryReference.CreateDirectory(logDirectory);
			FileReference logFile = FileReference.Combine(logDirectory, "Log_ClRepro.txt");
			Log.AddFileWriter("DefaultLogTraceListener", logFile);

			return PrepareFullRepro(logger);
		}
	}
}
