// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using OpenTracing;
using OpenTracing.Util;
using UnrealBuildBase;
using static UnrealBuildTool.DataDrivenPlatformInfo;

namespace UnrealBuildTool
{
	/// <summary>
	/// Cleans build products and intermediates for the target. This deletes files which are named consistently with the target being built
	/// (e.g. UnrealEditor-Foo-Win64-Debug.dll) rather than an actual record of previous build products.
	/// </summary>
	internal sealed class CleanMode : IToolMode<CleanMode>
	{
		public static string Name => "Clean";
		public static ToolModeOptions Options => ToolModeOptions.XmlConfig | ToolModeOptions.BuildPlatforms | ToolModeOptions.SingleInstance;

#pragma warning disable IDE0044 // Make field readonly - these private static fields are set by command-line parsing.
		/// <summary>
		/// Whether to avoid cleaning targets
		/// </summary>
		[CommandLine("-SkipRulesCompile")]
		bool bSkipRulesCompile = false;
#pragma warning restore IDE0044

		/// <summary>
		/// Skip pre build targets; just do the main target.
		/// </summary>
		[CommandLine("-SkipPreBuildTargets")]
		public bool bSkipPreBuildTargets = false;

		/// <summary>
		/// If true, lists entries to be cleaned but does not clean them.
		/// </summary>
		[CommandLine("-DryRun")]
		public bool bDryRun { get; set; } = false;

		/// <summary>
		/// Main entry point
		/// </summary>
		/// <param name="Arguments">Command-line arguments</param>
		/// <returns>One of the values of ECompilationResult</returns>
		/// <param name="Logger"></param>
		public Task<int> ExecuteAsync(CommandLineArguments Arguments, ILogger Logger)
		{
			Arguments.ApplyTo(this);

			// Create the build configuration object, and read the settings
			BuildConfiguration BuildConfiguration = new BuildConfiguration();
			XmlConfig.ApplyTo(BuildConfiguration);
			Arguments.ApplyTo(BuildConfiguration);

			// Parse all the targets being built
			List<TargetDescriptor> TargetDescriptors = TargetDescriptor.ParseCommandLine(Arguments, BuildConfiguration, Logger);

			Clean(TargetDescriptors, BuildConfiguration, Logger);

			return Task.FromResult(0);
		}

		public void Clean(List<TargetDescriptor> TargetDescriptors, BuildConfiguration BuildConfiguration, ILogger Logger)
		{
			using ScopedTimer CleanTimer = new ScopedTimer("CleanMode.Clean()", Logger);
			using IScope Scope = GlobalTracer.Instance.BuildSpan("CleanMode.Clean()").StartActive();

			using ITimelineEvent _ = Timeline.ScopeEvent("Cleaning");

			if (TargetDescriptors.Count == 0)
			{
				throw new BuildException("No targets specified to clean");
			}

			// Print out warning in Xcode to stop user from trying to make build folder deletable
			if (Environment.GetEnvironmentVariable("UE_BUILD_FROM_XCODE") == "1" && Unreal.IsEngineInstalled())
			{
				Logger.LogInformation("~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~");
				Logger.LogInformation("NOTICE: Please disregard the Xcode prepare clean failed message, we do NOT want Engine/Binaries/Mac to be deletable.");
				Logger.LogInformation("NOTICE: Cleaning UnrealEditor.app will cause UE fail to launch, you will then need to repair it from Epic Games Launcher.");
				Logger.LogInformation("~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~");
			}

			// Output the list of targets that we're cleaning
			Logger.LogInformation("Cleaning {TargetNames} binaries...", StringUtils.FormatList(TargetDescriptors.Select(x => x.Name).Distinct()));

			// Loop through all the targets, and clean them all
			HashSet<FileReference> FilesToDelete = new HashSet<FileReference>();
			HashSet<DirectoryReference> DirectoriesToDelete = new HashSet<DirectoryReference>();

			using (ScopedTimer GatherTimer = new ScopedTimer("Find paths to clean", Logger))
			{
				for (int Idx = 0; Idx < TargetDescriptors.Count; ++Idx)
				{
					TargetDescriptor TargetDescriptor = TargetDescriptors[Idx];

					// Create the rules assembly
					RulesAssembly RulesAssembly = RulesCompiler.CreateTargetRulesAssembly(TargetDescriptor.ProjectFile, TargetDescriptor.Name, bSkipRulesCompile, BuildConfiguration.bForceRulesCompile, BuildConfiguration.bUsePrecompiled, TargetDescriptor.ForeignPlugin, TargetDescriptor.bBuildPluginAsLocal, Logger);

					// Create the rules object
					ReadOnlyTargetRules Target = new ReadOnlyTargetRules(RulesAssembly.CreateTargetRules(TargetDescriptor.Name, TargetDescriptor.Platform, TargetDescriptor.Configuration, TargetDescriptor.Architectures, TargetDescriptor.ProjectFile, TargetDescriptor.AdditionalArguments, Logger, TargetDescriptor.IsTestsTarget));

					// Get the intermediate environment for this target
					UnrealIntermediateEnvironment IntermediateEnvironment = TargetDescriptor.IntermediateEnvironment != UnrealIntermediateEnvironment.Default ? TargetDescriptor.IntermediateEnvironment : Target.IntermediateEnvironment;

					if (!bSkipPreBuildTargets && Target.PreBuildTargets.Count > 0)
					{
						foreach (TargetInfo PreBuildTarget in Target.PreBuildTargets)
						{
							TargetDescriptor NewTarget = TargetDescriptor.FromTargetInfo(PreBuildTarget);
							if (!TargetDescriptors.Contains(NewTarget))
							{
								TargetDescriptors.Add(NewTarget);
							}
						}
					}

					RestrictedFolder[] RestrictedFolders =
					[
						RestrictedFolder.EpicInternal,
						RestrictedFolder.CarefullyRedist,
						RestrictedFolder.LimitedAccess,
						RestrictedFolder.NotForLicensees,
						RestrictedFolder.NoRedist,
					];

					// Find the base folders that can contain binaries
					HashSet<DirectoryReference> BaseDirs = [Unreal.EngineDirectory];

					foreach (RestrictedFolder Folder in RestrictedFolders)
					{
						BaseDirs.Add(DirectoryReference.Combine(Unreal.EngineDirectory, "Restricted", Folder.ToString()));
					}

					foreach (string Platform in GetAllPlatformInfos().Keys)
					{
						BaseDirs.Add(DirectoryReference.Combine(Unreal.EngineDirectory, "Platforms", Platform));
					}

					foreach (FileReference Plugin in PluginsBase.EnumeratePlugins(Target.ProjectFile))
					{
						BaseDirs.Add(Plugin.Directory);
					}

					if (Target.ProjectFile != null)
					{
						BaseDirs.Add(Target.ProjectFile.Directory);
					}

					// If we're running a precompiled build, remove anything under the engine folder
					List<DirectoryReference> ReadOnlyDirectories = BaseDirs.Where(x => RulesAssembly.IsReadOnly(x)).ToList();
					BaseDirs.ExceptWith(ReadOnlyDirectories);

					// Get all the names which can prefix build products
					List<string> NamePrefixes = new List<string>();
					if (Target.Type != TargetType.Program)
					{
						NamePrefixes.Add(UEBuildTarget.GetAppNameForTargetType(Target.Type));
					}
					NamePrefixes.Add(Target.Name);

					// Get the suffixes for this configuration
					List<string> NameSuffixes = new List<string>();
					if (Target.Configuration == Target.UndecoratedConfiguration)
					{
						NameSuffixes.Add("");
					}
					NameSuffixes.Add(String.Format("-{0}-{1}", Target.Platform.ToString(), Target.Configuration.ToString()));
					if (UnrealArchitectureConfig.ForPlatform(Target.Platform).RequiresArchitectureFilenames(Target.Architectures))
					{
						NameSuffixes.AddRange(NameSuffixes.ToArray().Select(x => x + Target.Architecture.ToString()));
					}

					// Add all the makefiles and caches to be deleted
					FilesToDelete.Add(TargetMakefile.GetLocation(Target.ProjectFile, Target.Name, Target.Platform, Target.Architectures, Target.Configuration, IntermediateEnvironment));
					FilesToDelete.UnionWith(SourceFileMetadataCache.GetFilesToClean(Target.ProjectFile));

					// Add all the intermediate folders to be deleted
					foreach (DirectoryReference BaseDir in BaseDirs)
					{
						foreach (string NamePrefix in NamePrefixes)
						{
							string NamePrefixWithEnv = UEBuildTarget.GetTargetIntermediateFolderName(NamePrefix, IntermediateEnvironment);
							// This is actually wrong.. the generated code is not in this dir.. if changing "Target.Architectures" parameter to null it will delete the right files.
							// However, this also means that it will cause a rebuild for both unity, nonunity and iwyu targets since they share this folder.
							DirectoryReference GeneratedCodeDir = DirectoryReference.Combine(BaseDir, UEBuildTarget.GetPlatformIntermediateFolder(Target.Platform, Target.Architectures, false), NamePrefixWithEnv, "Inc");
							if (DirectoryReference.Exists(GeneratedCodeDir))
							{
								DirectoriesToDelete.Add(GeneratedCodeDir);
							}

							DirectoryReference IntermediateDir = DirectoryReference.Combine(BaseDir, UEBuildTarget.GetPlatformIntermediateFolder(Target.Platform, Target.Architectures, false), NamePrefixWithEnv, Target.Configuration.ToString());
							if (DirectoryReference.Exists(IntermediateDir))
							{
								DirectoriesToDelete.Add(IntermediateDir);
							}
						}
					}

					// todo: handle external plugin intermediates, written to the Project's Intermediate/External directory

					// List of additional files and directories to clean, specified by the target platform
					List<FileReference> AdditionalFilesToDelete = new List<FileReference>();
					List<DirectoryReference> AdditionalDirectoriesToDelete = new List<DirectoryReference>();

					// Add all the build products from this target
					string[] NamePrefixesArray = NamePrefixes.Distinct().ToArray();
					string[] NameSuffixesArray = NameSuffixes.Distinct().ToArray();
					foreach (DirectoryReference BaseDir in BaseDirs)
					{
						DirectoryReference BinariesDir = DirectoryReference.Combine(BaseDir, "Binaries", Target.Platform.ToString());
						if (DirectoryReference.Exists(BinariesDir))
						{
							UEBuildPlatform.GetBuildPlatform(Target.Platform).FindBuildProductsToClean(BinariesDir, NamePrefixesArray, NameSuffixesArray, AdditionalFilesToDelete, AdditionalDirectoriesToDelete);
						}
					}

					// Get all the additional intermediate folders created by this platform
					UEBuildPlatform.GetBuildPlatform(Target.Platform).FindAdditionalBuildProductsToClean(Target, AdditionalFilesToDelete, AdditionalDirectoriesToDelete);

					// Add the platform's files and directories to the main list
					FilesToDelete.UnionWith(AdditionalFilesToDelete);
					DirectoriesToDelete.UnionWith(AdditionalDirectoriesToDelete);
				}
			}

			// Ensure no overlap between directories
			using (ScopedTimer Timer = new ScopedTimer("Ensure no directory overlap", Logger))
			{
				HashSet<DirectoryReference> SubdirectoriesToDelete = new(DirectoriesToDelete.Count);
				foreach (DirectoryReference Directory in DirectoriesToDelete)
				{
					// Is this directory a subdirectory of some other directory that is being deleted?
					for (DirectoryReference? DirectoryWalker = Directory.ParentDirectory; DirectoryWalker != null; DirectoryWalker = DirectoryWalker.ParentDirectory)
					{
						if (DirectoriesToDelete.Contains(DirectoryWalker))
						{
							SubdirectoriesToDelete.Add(Directory);
							break;
						}
					}
				}
				DirectoriesToDelete.ExceptWith(SubdirectoriesToDelete);
			}

			// Remove any files that are contained within one of the directories
			using (ScopedTimer Time = new ScopedTimer("Ensure no file overlap", Logger))
			{
				FilesToDelete.RemoveWhere(File =>
				{
					// Is this file in a subdirectory of some directory that is being deleted?
					for (DirectoryReference? DirectoryWalker = File.Directory; DirectoryWalker != null; DirectoryWalker = DirectoryWalker.ParentDirectory)
					{
						if (DirectoriesToDelete.Contains(DirectoryWalker))
						{
							return true;
						}
					}
					return false;
				});
			}

			LogLevel DeleteLogLevel = bDryRun ? LogLevel.Information : LogLevel.Debug;

			Task DeleteDirectories = Task.Run(() =>
			{
				using ScopedTimer Timer = new ScopedTimer($"Delete {DirectoriesToDelete.Count} directories", Logger);
				Parallel.ForEach(DirectoriesToDelete, DirectoryToDelete =>
				{
					using ScopedTimer Timer = new ScopedTimer($"Delete directory '{DirectoryToDelete}'", Logger, bIncreaseIndent: false);

					if (DirectoryReference.Exists(DirectoryToDelete))
					{
						Logger.Log(DeleteLogLevel, "    Deleting {DirectoryToDelete}...", $"{DirectoryToDelete}{Path.DirectorySeparatorChar}");

						if (bDryRun)
						{
							return;
						}

						try
						{
							FileUtils.ForceDeleteDirectory(DirectoryToDelete);
						}
						catch (Exception Ex)
						{
							throw new BuildException(Ex, "Unable to delete {0} ({1})", DirectoryToDelete, Ex.Message.TrimEnd());
						}
					}
				});
			});

			Task DeleteFiles = Task.Run(() =>
			{
				using ScopedTimer Timer = new ScopedTimer($"Delete {FilesToDelete.Count} files", Logger);
				Parallel.ForEach(FilesToDelete, FileToDelete =>
				{
					if (FileReference.Exists(FileToDelete))
					{
						Logger.Log(DeleteLogLevel, "    Deleting {FileToDelete}...", FileToDelete);

						if (bDryRun)
						{
							return;
						}

						try
						{
							FileUtils.ForceDeleteFile(FileToDelete);
						}
						catch (Exception Ex)
						{
							throw new BuildException(Ex, "Unable to delete {0} ({1})", FileToDelete, Ex.Message.TrimEnd());
						}
					}
				});
			});

			DeleteDirectories.Wait();
			DeleteFiles.Wait();

			if (bDryRun)
			{
				return;
			}

			// Also clean all the remote targets
			for (int Idx = 0; Idx < TargetDescriptors.Count; Idx++)
			{
				TargetDescriptor TargetDescriptor = TargetDescriptors[Idx];
				if (RemoteMac.HandlesTargetPlatform(TargetDescriptor.Platform))
				{
					RemoteMac RemoteMac = new RemoteMac(TargetDescriptor.ProjectFile, Logger);
					RemoteMac.Clean(TargetDescriptor, Logger);
				}
			}
		}
	}
}

