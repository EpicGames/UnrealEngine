// Copyright Epic Games, Inc. All Rights Reserved.

using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace UnrealBuildTool
{
	/// <summary>
	/// Invokes the deployment handler for a target.
	/// </summary>
	internal sealed class DeployMode : IToolMode<DeployMode>
	{
		public static string Name => "Deploy";
		public static ToolModeOptions Options => ToolModeOptions.BuildPlatforms;

		/// <summary>
		/// If we are just running the deployment step, specifies the path to the given deployment settings
		/// </summary>
		[CommandLine("-Receipt", Required = true)]
		public FileReference? ReceiptFile = null;

		/// <summary>
		/// Execute the tool mode
		/// </summary>
		/// <param name="Arguments">Command line arguments</param>
		/// <returns>Exit code</returns>
		/// <param name="Logger"></param>
		public Task<int> ExecuteAsync(CommandLineArguments Arguments, ILogger Logger)
		{
			// Apply the arguments
			Arguments.ApplyTo(this);
			Arguments.CheckAllArgumentsUsed();

			// Execute the deploy
			TargetReceipt Receipt = TargetReceipt.Read(ReceiptFile!);
			Logger.LogInformation("Deploying {ReceiptTargetName} {ReceiptPlatform} {ReceiptConfiguration}...", Receipt.TargetName, Receipt.Platform, Receipt.Configuration);
			UEBuildPlatform.GetBuildPlatform(Receipt.Platform).Deploy(Receipt);

			return Task.FromResult((int)CompilationResult.Succeeded);
		}
	}
}
