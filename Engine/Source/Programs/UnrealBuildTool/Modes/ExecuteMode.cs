// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using OpenTracing.Util;

namespace UnrealBuildTool
{
	/// <summary>
	/// Builds a target
	/// </summary>
	internal sealed class ExecuteMode : IToolMode<ExecuteMode>
	{
		public static string Name => "Execute";
		public static ToolModeOptions Options => ToolModeOptions.XmlConfig | ToolModeOptions.BuildPlatforms | ToolModeOptions.SingleInstance | ToolModeOptions.StartPrefetchingEngine | ToolModeOptions.ShowExecutionTime;

		/// <summary>
		/// Whether we should just export the outdated actions list
		/// </summary>
		[CommandLine("-Actions=", Required = true)]
		public FileReference? ActionsFile = null;

		/// <summary>
		/// Main entry point
		/// </summary>
		/// <param name="Arguments">Command-line arguments</param>
		/// <returns>One of the values of ECompilationResult</returns>
		/// <param name="Logger"></param>
		public async Task<int> ExecuteAsync(CommandLineArguments Arguments, ILogger Logger)
		{
			Arguments.ApplyTo(this);

			// Read the XML configuration files
			XmlConfig.ApplyTo(this);

			// Create the build configuration object, and read the settings
			BuildConfiguration BuildConfiguration = new BuildConfiguration();
			XmlConfig.ApplyTo(BuildConfiguration);
			Arguments.ApplyTo(BuildConfiguration);

			// Read the actions file
			List<LinkedAction> Actions;
			using (GlobalTracer.Instance.BuildSpan("ActionGraph.ReadActions()").StartActive())
			{
				Actions = ActionGraph.ImportJson(ActionsFile!).ConvertAll(x => new LinkedAction(x, null));
			}

			// Link the action graph
			using (GlobalTracer.Instance.BuildSpan("ActionGraph.Link()").StartActive())
			{
				ActionGraph.Link(Actions, Logger);
			}

			// Execute the actions
			using (GlobalTracer.Instance.BuildSpan("ActionGraph.ExecuteActions()").StartActive())
			{
				List<TargetDescriptor> TargetDescriptors = TargetDescriptor.ParseCommandLine(Arguments, BuildConfiguration, Logger);
				ActionGraph.CreateDirectoriesForProducedItems(Actions);
				await ActionGraph.ExecuteActionsAsync(BuildConfiguration, Actions, TargetDescriptors, Logger);
			}

			return 0;
		}
	}
}

