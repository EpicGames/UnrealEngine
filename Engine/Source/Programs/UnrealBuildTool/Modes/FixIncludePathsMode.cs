// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.Linq;
using System.Text.RegularExpressions;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// Fixes the include paths found in a header and source file
	/// </summary>
	internal sealed class FixIncludePathsMode : IToolMode<FixIncludePathsMode>
	{
		public static string Name => "FixIncludePaths";
		public static ToolModeOptions Options => ToolModeOptions.XmlConfig | ToolModeOptions.BuildPlatforms | ToolModeOptions.SingleInstance | ToolModeOptions.StartPrefetchingEngine | ToolModeOptions.ShowExecutionTime;

		/// <summary>
		/// Regex that matches #include statements.
		/// </summary>
		static readonly Regex IncludeRegex = new Regex("^[ \t]*#[ \t]*include[ \t]*[<\"](?<HeaderFile>[^\">]*)[\">]", RegexOptions.Compiled | RegexOptions.Singleline | RegexOptions.ExplicitCapture);

		static readonly string UnrealRootDirectory = Unreal.RootDirectory.FullName.Replace('\\', '/');
		static readonly string[] PreferredPaths = { "/Public/", "/Private/", "/Classes/", "/Internal/", "/UHT/", "/VNI/" };
		static readonly string[] PublicDirectories = { "Public", "Classes", };

#pragma warning disable IDE0044 // Make field readonly - these private static fields are set by command-line parsing.
		[CommandLine("-Filter=", Description = "Set of filters for files to include in the database. Relative to the root directory, or to the project file.")]
		List<string> FilterRules = new List<string>();

		[CommandLine("-IncludeFilter=", Description = "Set of filters for #include'd lines to allow updating. Relative to the root directory, or to the project file.")]
		List<string> IncludeFilterRules = new List<string>();
#pragma warning restore IDE0044

		[CommandLine("-CheckoutWithP4", Description = "Flags that this task should use p4 to check out the file before updating it.")]
		public bool bCheckoutWithP4 = false;

		[CommandLine("-NoOutput", Description = "Flags that the updated files shouldn't be saved.")]
		public bool bNoOutput = false;

		[CommandLine("-NoIncludeSorting", Description = "Flags that includes should not be sorted.")]
		public bool bNoIncludeSorting = false;

		[CommandLine("-ForceUpdate", Description = "Forces updating and sorting the includes.")]
		public bool bForceUpdate = false;

		/// <summary>
		/// Execute the command
		/// </summary>
		/// <param name="Arguments">Command line arguments</param>
		/// <returns>Exit code</returns>
		/// <param name="Logger"></param>
		public async Task<int> ExecuteAsync(CommandLineArguments Arguments, ILogger Logger)
		{
			Arguments.ApplyTo(this);

			// Create the build configuration object, and read the settings
			BuildConfiguration BuildConfiguration = new BuildConfiguration();
			XmlConfig.ApplyTo(BuildConfiguration);
			Arguments.ApplyTo(BuildConfiguration);

			// Parse the filter argument
			FileFilter? FileFilter = null;
			if (FilterRules.Count > 0)
			{
				FileFilter = new FileFilter(FileFilterType.Exclude);
				foreach (string FilterRule in FilterRules)
				{
					FileFilter.AddRules(FilterRule.Split(';'));
				}
			}

			// Parse the include filter argument
			FileFilter? IncludeFilter = null;
			if (IncludeFilterRules.Count > 0)
			{
				IncludeFilter = new FileFilter(FileFilterType.Exclude);
				foreach (string FilterRule in IncludeFilterRules)
				{
					IncludeFilter.AddRules(FilterRule.Split(';'));
				}
			}

			// Force C++ modules to always include their generated code directories
			UEBuildModuleCPP.bForceAddGeneratedCodeIncludePath = true;

			// Parse all the target descriptors
			List<TargetDescriptor> TargetDescriptors = TargetDescriptor.ParseCommandLine(Arguments, BuildConfiguration, Logger);

			// Generate the compile DB for each target
			using (ISourceFileWorkingSet WorkingSet = new EmptySourceFileWorkingSet())
			{
				HashSet<UEBuildModule> ScannedModules = new();

				// Find the compile commands for each file in the target
				Dictionary<FileReference, string> FileToCommand = new Dictionary<FileReference, string>();
				foreach (TargetDescriptor TargetDescriptor in TargetDescriptors)
				{
					// Create a makefile for the target
					UEBuildTarget Target = UEBuildTarget.Create(TargetDescriptor, BuildConfiguration, Logger);

					// Get InputLists and build Include Filter if necessary
					HashSet<FileReference> IncludeFileList = new HashSet<FileReference>();
					Dictionary<UEBuildModuleCPP, List<FileReference>> ModuleToFiles = new Dictionary<UEBuildModuleCPP, List<FileReference>>();
					foreach (UEBuildBinary Binary in Target.Binaries)
					{
						foreach (UEBuildModuleCPP Module in Binary.Modules.OfType<UEBuildModuleCPP>())
						{
							UEBuildModuleCPP.InputFileCollection InputFileCollection = Module.FindInputFiles(Target.Platform, new Dictionary<DirectoryItem, FileItem[]>(), Logger);
							List<FileItem> InputFiles = new List<FileItem>();
							InputFiles.AddRange(InputFileCollection.HeaderFiles);
							InputFiles.AddRange(InputFileCollection.CPPFiles);
							InputFiles.AddRange(InputFileCollection.CCFiles);
							InputFiles.AddRange(InputFileCollection.CFiles);

							List<FileReference> FileList = new List<FileReference>();
							foreach (FileItem InputFile in InputFiles)
							{
								if (FileFilter == null || FileFilter.Matches(InputFile.Location.MakeRelativeTo(Unreal.RootDirectory)))
								{
									FileReference fileRef = new FileReference(InputFile.AbsolutePath);
									FileList.Add(fileRef);
								}
							}
							ModuleToFiles[Module] = FileList;

							foreach (FileItem InputFile in InputFiles)
							{
								if (IncludeFilter == null || IncludeFilter.Matches(InputFile.Location.MakeRelativeTo(Unreal.RootDirectory)))
								{
									FileReference fileRef = new FileReference(InputFile.AbsolutePath);
									IncludeFileList.Add(fileRef);
								}
							}
						}
					}

					// List of modules that are allowed to be processed (not ThirdParty and have unfiltered files)
					HashSet<UEBuildModuleCPP> AllModules = ModuleToFiles.Where(Item => !Item.Key.RulesFile.ContainsName("ThirdParty", Unreal.RootDirectory) && Item.Value.Any()).Select(Item => Item.Key).ToHashSet();

					// Keep track of progress
					int Index = 0;
					int Total = AllModules.Count;

					// Create the global compile environment for this target
					CppCompileEnvironment GlobalCompileEnvironment = Target.CreateCompileEnvironmentForProjectFiles(Logger);

					// Create all the binaries and modules
					foreach (UEBuildBinary Binary in Target.Binaries)
					{
						CppCompileEnvironment BinaryCompileEnvironment = Binary.CreateBinaryCompileEnvironment(GlobalCompileEnvironment);

						foreach (UEBuildModuleCPP Module in Binary.Modules.OfType<UEBuildModuleCPP>().Where(Module => AllModules.Contains(Module)))
						{
							List<FileReference> FileList = ModuleToFiles[Module];

							Dictionary<string, string?> PreferredPathCache = new();
							CppCompileEnvironment env = Module.CreateCompileEnvironmentForIntellisense(Target.Rules, BinaryCompileEnvironment, Logger);

							foreach (FileReference InputFile in FileList)
							{
								List<int> LinesUpdated = new();
								string[] Text = await FileReference.ReadAllLinesAsync(InputFile);
								bool UpdatedText = false;

								for (int i = 0; i < Text.Length; i++)
								{
									string Line = Text[i];
									int LineNumber = i + 1;
									Match IncludeMatch = IncludeRegex.Match(Line);
									if (IncludeMatch.Success)
									{
										string Include = IncludeMatch.Groups[1].Value;

										char pathSep = System.IO.Path.DirectorySeparatorChar;
										if (Include.Contains("/Private/", StringComparison.Ordinal) && PublicDirectories.Any(dir => InputFile.FullName.Contains($"{pathSep}{dir}{pathSep}", StringComparison.Ordinal)))
										{
											Logger.LogError("{FileName}({LineNumber}): Can not update #include '{Include}' in the public file because it may break external code that uses it.", InputFile.FullName, LineNumber, Include);
											continue;
										}

										if (Include.Contains("..", StringComparison.Ordinal))
										{
											Logger.LogError("{FileName}({LineNumber}): Can not update #include '{Include}', relative pathing is not currently handled.", InputFile.FullName, LineNumber, Include);
											continue;
										}

										//Debugger.Launch();

										string? PreferredInclude = null;
										if (!PreferredPathCache.TryGetValue(Include, out PreferredInclude))
										{
											List<DirectoryReference> IncludePaths = new();
											IncludePaths.Add(new DirectoryReference(System.IO.Directory.GetParent(InputFile.FullName)!));
											IncludePaths.AddRange(env.UserIncludePaths);
											IncludePaths.AddRange(env.SystemIncludePaths);

											// search include paths
											FileReference? FoundIncludeFile = null;
											DirectoryReference? FoundIncludePath = null;
											foreach (DirectoryReference IncludePath in IncludePaths)
											{
												string Path = System.IO.Path.GetFullPath(System.IO.Path.Combine(IncludePath.FullName, Include));
												if (System.IO.File.Exists(Path))
												{
													FoundIncludeFile = new(Path);
													FoundIncludePath = IncludePath;
													break;
												}
											}

											if (FoundIncludeFile != null && !IncludeFileList.Contains(FoundIncludeFile))
											{
												Logger.LogInformation("{FileName}({LineNumber}): Skipping '{Include}' because it is filtered out.", InputFile.FullName, LineNumber, Include);
												PreferredInclude = Include;
												PreferredPathCache[Include] = PreferredInclude;
												continue;
											}

											if (FoundIncludeFile != null)
											{
												string FullPath = FoundIncludeFile.FullName.Replace('\\', '/');
												if (FullPath.Contains("ThirdParty", StringComparison.Ordinal))
												{
													Logger.LogInformation("{FileName}({LineNumber}): Skipping '{Include}' because it is a third party header.", InputFile.FullName, LineNumber, Include);
													PreferredInclude = Include;
													PreferredPathCache[Include] = PreferredInclude;
													continue;
												}

												// if the include and the source file live in the same directory then it is OK to be relative
												if (String.Equals(System.IO.Directory.GetParent(FullPath)?.FullName, System.IO.Directory.GetParent(InputFile.FullName)?.FullName, StringComparison.OrdinalIgnoreCase) &&
													String.Equals(Include, System.IO.Path.GetFileName(FullPath), StringComparison.OrdinalIgnoreCase))
												{
													Logger.LogInformation("{FileName}({LineNumber}): Using '{Include}' because it is in the same directory.", InputFile.FullName, LineNumber, Include);
													PreferredInclude = Include;
												}
												else
												{
													if (!FullPath.Contains(UnrealRootDirectory, StringComparison.Ordinal))
													{
														Logger.LogInformation("{FileName}({LineNumber}): Skipping '{Include}' because it isn't under the Unreal root directory.", InputFile.FullName, LineNumber, Include);
													}
													else
													{
														string? FoundPreferredPath = PreferredPaths.FirstOrDefault(path => FullPath.Contains(path, StringComparison.Ordinal));
														if (FoundPreferredPath != null)
														{
															int end = FullPath.LastIndexOf(FoundPreferredPath, StringComparison.Ordinal) + FoundPreferredPath.Length;
															PreferredInclude = FullPath.Substring(end);

															// Is the current include a shortened version of the preferred include path?
															if (PreferredInclude != Include && PreferredInclude.Contains(Include, StringComparison.Ordinal))
															{
																Logger.LogInformation("{FileName}({LineNumber}): Using '{Include}' because it is shorter than '{PreferredInclude}'.", InputFile.FullName, LineNumber, Include, PreferredInclude);
																PreferredInclude = Include;
															}
														}
														else
														{
															PreferredInclude = null;

															string ModulePath = FullPath;
															FileReference IncludeFileReference = new(FullPath);
															DirectoryReference? TempDirectory = IncludeFileReference.Directory;
															DirectoryReference? FoundDirectory = null;
															// find the module this include is part of
															while (TempDirectory != null)
															{
																if (DirectoryReference.EnumerateFiles(TempDirectory, $"*.build.cs").Any())
																{
																	FoundDirectory = TempDirectory;
																	break;
																}

																TempDirectory = TempDirectory.ParentDirectory;
															}

															if (FoundDirectory != null)
															{
																PreferredInclude = FullPath.Substring(FoundDirectory.FullName.Length + 1);
															}
														}
													}

													PreferredPathCache[Include] = PreferredInclude;
												}
											}

											if (PreferredInclude == null)
											{
												Logger.LogInformation("{FileName}({LineNumber}): Could not find path to '{IncludePath}'", InputFile.FullName, LineNumber, Include);
												continue;
											}
										}

										if (bForceUpdate || (PreferredInclude != null && Include != PreferredInclude))
										{
											Logger.LogInformation("{FileName}({LineNumber}): Updated '{OldInclude}' -> '{NewInclude}'", InputFile.FullName, LineNumber, Include, PreferredInclude);
											Text[i] = Line.Replace(Include, PreferredInclude, StringComparison.Ordinal);
											UpdatedText = true;
											LinesUpdated.Add(i);
										}
									}
								}

								if (UpdatedText)
								{
									if (!bNoIncludeSorting)
									{
										SortIncludes(InputFile, LinesUpdated, Text);
									}

									if (!bNoOutput)
									{
										Logger.LogInformation("Updating {IncludePath}", InputFile.FullName);
										try
										{
											if (bCheckoutWithP4)
											{
												System.Diagnostics.Process Process = new System.Diagnostics.Process();
												System.Diagnostics.ProcessStartInfo StartInfo = new System.Diagnostics.ProcessStartInfo();
												Process.StartInfo.WindowStyle = System.Diagnostics.ProcessWindowStyle.Hidden;
												Process.StartInfo.FileName = "p4.exe";
												Process.StartInfo.Arguments = $"edit {InputFile.FullName}";
												Process.Start();
												await Process.WaitForExitAsync();
											}
											System.IO.File.WriteAllLines(InputFile.FullName, Text);
										}
										catch (Exception ex)
										{
											Logger.LogWarning("Failed to write to file: {Exception}", ex);
										}
									}
								}
							}

							Logger.LogInformation("[{Index}/{Total}] Processed Module {Name} ({Files} files)", ++Index, Total, Module.Name, FileList.Count);

							ScannedModules.Add(Module);
						}
					}
				}
			}

			return 0;
		}

		class HeaderSortComparison : IComparer<string>
		{
			private readonly string IWYUFileName;

			public HeaderSortComparison(string IWYUFileName)
			{
				this.IWYUFileName = IWYUFileName;
			}

			public int Compare(string? x, string? y)
			{
				if (String.IsNullOrEmpty(x) && String.IsNullOrEmpty(y))
				{
					return 0;
				}

				if (String.IsNullOrEmpty(y))
				{
					return -1;
				}

				if (String.IsNullOrEmpty(x))
				{
					return 1;
				}

				// IWYU header
				if (x.Contains(IWYUFileName, StringComparison.Ordinal))
				{
					return -1;
				}
				if (y.Contains(IWYUFileName, StringComparison.Ordinal))
				{
					return 1;
				}

				// system includes
				if (x.Contains('<', StringComparison.Ordinal))
				{
					return 1;
				}
				if (y.Contains('<', StringComparison.Ordinal))
				{
					return -1;
				}

				// generated header
				if (x.Contains(".generated.h", StringComparison.Ordinal))
				{
					return 1;
				}
				if (y.Contains(".generated.h", StringComparison.Ordinal))
				{
					return -1;
				}

				return String.Compare(x, y, StringComparison.Ordinal);
			}
		}

		private static void SortIncludes(FileReference File, List<int> LinesUpdated, string[] Text)
		{
			HeaderSortComparison HeaderSort = new HeaderSortComparison(File.GetFileNameWithoutExtension() + ".h");
			foreach (int LineIndex in LinesUpdated)
			{
				int FirstIncludeIndex = LineIndex;
				for (int i = LineIndex - 1; i >= 0; i--)
				{
					Match IncludeMatch = IncludeRegex.Match(Text[i]);
					if (IncludeMatch.Success)
					{
						FirstIncludeIndex = i;
					}
					else
					{
						break;
					}
				}

				int LastIncludeIndex = LineIndex;
				for (int i = LineIndex + 1; i < Text.Length; i++)
				{
					Match IncludeMatch = IncludeRegex.Match(Text[i]);
					if (IncludeMatch.Success)
					{
						LastIncludeIndex = i;
					}
					else
					{
						break;
					}
				}
				Array.Sort(Text, FirstIncludeIndex, LastIncludeIndex - FirstIncludeIndex + 1, HeaderSort);
			}
		}
	}
}
