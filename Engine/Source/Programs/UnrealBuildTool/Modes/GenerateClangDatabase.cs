// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// Generate a clang compile_commands file for a target
	/// </summary>
	internal sealed class GenerateClangDatabase : IToolMode<GenerateClangDatabase>
	{
		public static string Name => "GenerateClangDatabase";
		public static ToolModeOptions Options => ToolModeOptions.XmlConfig | ToolModeOptions.BuildPlatforms | ToolModeOptions.SingleInstance | ToolModeOptions.StartPrefetchingEngine | ToolModeOptions.ShowExecutionTime;

#pragma warning disable IDE0044 // Make field readonly - these private static fields are set by command-line parsing.
		/// <summary>
		/// Optional set of source files to include in the compile database. If this is empty, all files will be included by default and -Exclude can be used to exclude some.
		/// Relative to the root directory, or to the project file.
		/// </summary>
		[CommandLine("-Include=")]
		List<string> IncludeRules = new List<string>();

		/// <summary>
		/// Optional set of source files to exclude from the compile database. 
		/// Relative to the root directory, or to the project file.
		/// </summary>
		[CommandLine("-Exclude=")]
		List<string> ExcludeRules = new List<string>();
#pragma warning restore IDE0044

		/// <summary>
		/// Execute any actions which result in code generation (eg. ISPC compilation)
		/// </summary>
		[CommandLine("-ExecCodeGenActions")]
		[CommandLine("-NoExecCodeGenActions", Value = "false")]
		public bool bExecCodeGenActions = true;

		/// <summary>
		/// Optionally override the output filename for handling multiple targets
		/// </summary>
		[CommandLine("-OutputFilename=")]
		public string OutputFilename = "compile_commands.json";

		/// <summary>
		/// Optionally overrite the output directory for the compile_commands file.
		/// </summary>
		[CommandLine("-OutputDir=")]
		public string OutputDir = Unreal.RootDirectory.ToString();

		/// <summary>
		/// Execute the command
		/// </summary>
		/// <param name="Arguments">Command line arguments</param>
		/// <returns>Exit code</returns>
		/// <param name="Logger"></param>
		public async Task<int> ExecuteAsync(CommandLineArguments Arguments, ILogger Logger)
		{
			Arguments.ApplyTo(this);

			// Create the build configuration object, and read the settings
			BuildConfiguration BuildConfiguration = new BuildConfiguration();
			XmlConfig.ApplyTo(BuildConfiguration);
			Arguments.ApplyTo(BuildConfiguration);

			// If we have specific rules on what to include, exclude anything not covered by such a rule.
			// Otherwise, include everything not covered by explicit Exclude rules
			FileFilter FileFilter = new FileFilter(IncludeRules.Count == 0 ? FileFilterType.Include : FileFilterType.Exclude);
			foreach (string Include in IncludeRules)
			{
				FileFilter.Include(Include.Split(';'));
			}	
			foreach (string Exclude in ExcludeRules)
			{
				FileFilter.Exclude(Exclude.Split(';'));
			}

			// Force C++ modules to always include their generated code directories
			UEBuildModuleCPP.bForceAddGeneratedCodeIncludePath = true;

			// Parse all the target descriptors
			List<TargetDescriptor> TargetDescriptors = TargetDescriptor.ParseCommandLine(Arguments, BuildConfiguration, Logger);

			// Generate the compile DB for each target
			using (ISourceFileWorkingSet WorkingSet = new EmptySourceFileWorkingSet())
			{
				// Find the compile commands for each file in the target
				Dictionary<Tuple<string, string>, string> FileToCommand = new();
				foreach (TargetDescriptor TargetDescriptor in TargetDescriptors)
				{
					// Disable PCHs and unity builds for the target
					TargetDescriptor.bUseUnityBuild = false;
					TargetDescriptor.IntermediateEnvironment = UnrealIntermediateEnvironment.GenerateClangDatabase;
					TargetDescriptor.AdditionalArguments = TargetDescriptor.AdditionalArguments.Append(new string[] { "-NoPCH", "-NoVFS" });
					// Default the compiler to clang
					if (!TargetDescriptor.AdditionalArguments.Any(x => x.StartsWith("-Compiler=", StringComparison.OrdinalIgnoreCase)))
					{
						TargetDescriptor.AdditionalArguments = TargetDescriptor.AdditionalArguments.Append(new string[] { "-Compiler=Clang" });
					}

					// Create a makefile for the target
					Logger.LogInformation("Creating target...");
					UEBuildTarget Target = UEBuildTarget.Create(TargetDescriptor, BuildConfiguration, Logger);
					UEToolChain TargetToolChain = Target.CreateToolchain(Target.Platform, Logger);

					// Create the makefile
					TargetMakefile Makefile = await Target.BuildAsync(BuildConfiguration, WorkingSet, TargetDescriptor, Logger);
					List<LinkedAction> Actions = Makefile.Actions.ConvertAll(x => new LinkedAction(x, TargetDescriptor));
					ActionGraph.Link(Actions, Logger);

					if (bExecCodeGenActions)
					{
						// Filter all the actions to execute
						HashSet<FileItem> PrerequisiteItems = new HashSet<FileItem>(Makefile.Actions.SelectMany(x => x.ProducedItems).Where(x => x.HasExtension(".h") || x.HasExtension(".cpp") || x.HasExtension(".cc") || x.HasExtension(".cxx") || x.HasExtension(".c")));
						List<LinkedAction> PrerequisiteActions = ActionGraph.GatherPrerequisiteActions(Actions, PrerequisiteItems);

						Utils.ExecuteCustomBuildSteps(Makefile.PreBuildScripts, Logger);

						// Execute code generation actions
						if (PrerequisiteActions.Any())
						{
							Logger.LogInformation("Executing actions that produce source files...");
							ActionGraph.CreateDirectoriesForProducedItems(PrerequisiteActions);
							await ActionGraph.ExecuteActionsAsync(BuildConfiguration, PrerequisiteActions, new List<TargetDescriptor> { TargetDescriptor }, Logger);
						}
					}

					Logger.LogInformation("Filtering compile actions...");

					IEnumerable<IExternalAction> CompileActions = Actions
						.Where(x => x.ActionType == ActionType.Compile)
						.Where(x => x.PrerequisiteItems.Any());

					if (CompileActions.Any())
					{
						foreach (IExternalAction Action in CompileActions)
						{
							FileItem? SourceFile = Action.PrerequisiteItems.FirstOrDefault(x => x.HasExtension(".cpp") || x.HasExtension(".cc") || x.HasExtension(".cxx") || x.HasExtension(".c")) ?? Action.PrerequisiteItems.FirstOrDefault(x => x.HasExtension(".h"));
							FileItem? OutputFile = Action.ProducedItems.FirstOrDefault(x => x.HasExtension(".obj") | x.HasExtension(".o"));
							if (SourceFile == null || OutputFile == null)
							{
								continue;
							}

							if (!FileFilter.Matches(SourceFile.Location.FullName))
							{
								continue;
							}

							// Create the command
							StringBuilder CommandBuilder = new StringBuilder();
							string CommandPath = Action.CommandPath.FullName.Contains(' ', StringComparison.Ordinal) ? Utils.MakePathSafeToUseWithCommandLine(Action.CommandPath) : Action.CommandPath.FullName;
							CommandBuilder.AppendFormat("{0} {1}", CommandPath, Action.CommandArguments);

							foreach (string ExtraArgument in GetExtraPlatformArguments(TargetToolChain))
							{
								CommandBuilder.AppendFormat(" {0}", ExtraArgument);
							}

							FileToCommand[Tuple.Create(SourceFile.FullName, OutputFile.FullName)] = CommandBuilder.ToString();
						}
					}
				}

				Logger.LogInformation("Writing database...");

				// Write the compile database
				DirectoryReference DatabaseDirectory = new DirectoryReference(OutputDir);
				FileReference DatabaseFile = FileReference.Combine(DatabaseDirectory, OutputFilename);
				using (JsonWriter Writer = new JsonWriter(DatabaseFile))
				{
					Writer.WriteArrayStart();
					foreach (KeyValuePair<Tuple<string, string>, string> FileCommandPair in FileToCommand.OrderBy(x => x.Key.Item1))
					{
						Writer.WriteObjectStart();
						Writer.WriteValue("file", FileCommandPair.Key.Item1.Replace('\\', '/'));
						Writer.WriteValue("command", FileCommandPair.Value.Replace('\\', '/'));
						Writer.WriteValue("directory", Unreal.EngineSourceDirectory.FullName.Replace('\\', '/'));
						Writer.WriteValue("output", FileCommandPair.Key.Item2.Replace('\\', '/'));
						Writer.WriteObjectEnd();
					}
					Writer.WriteArrayEnd();
				}
				Logger.LogInformation("ClangDatabase written to {DatabasePath}", DatabaseFile.FullName);

			}

			return 0;
		}

		private static IEnumerable<string> GetExtraPlatformArguments(UEToolChain TargetToolChain)
		{
			IList<string> ExtraPlatformArguments = new List<string>();

			ClangToolChain? ClangToolChain = TargetToolChain as ClangToolChain;
			ClangToolChain?.AddExtraToolArguments(ExtraPlatformArguments);

			return ExtraPlatformArguments;
		}
	}
}
