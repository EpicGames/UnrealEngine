// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Reflection;
using System.Text;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// Generates project files for one or more projects
	/// </summary>
	internal sealed class GenerateProjectFilesMode : IToolMode<GenerateProjectFilesMode>
	{
		public static string Name => "GenerateProjectFiles";
		public static ToolModeOptions Options => ToolModeOptions.XmlConfig | ToolModeOptions.BuildPlatforms | ToolModeOptions.SingleInstance | ToolModeOptions.UseStartupTraceListener | ToolModeOptions.StartPrefetchingEngine | ToolModeOptions.ShowExecutionTime;

#pragma warning disable IDE0044 // Make field readonly - these private static fields are set by command-line parsing.
		/// <summary>
		/// Types of project files to generate
		/// </summary>
		[CommandLine("-ProjectFileFormat")]
		[CommandLine("-VisualStudio", Value = nameof(ProjectFileFormat.VisualStudio))]
		[CommandLine("-2022", Value = nameof(ProjectFileFormat.VisualStudio2022))] // + override compiler
		[CommandLine("-2026", Value = nameof(ProjectFileFormat.VisualStudio2026))] // + override compiler
		[CommandLine("-Makefile", Value = nameof(ProjectFileFormat.Make))]
		[CommandLine("-CMakefile", Value = nameof(ProjectFileFormat.CMake))]
		[CommandLine("-QMakefile", Value = nameof(ProjectFileFormat.QMake))]
		[CommandLine("-KDevelopfile", Value = nameof(ProjectFileFormat.KDevelop))]
		[CommandLine("-CodeLiteFiles", Value = nameof(ProjectFileFormat.CodeLite))]
		[CommandLine("-XCodeProjectFiles", Value = nameof(ProjectFileFormat.XCode))]
		[CommandLine("-EddieProjectFiles", Value = nameof(ProjectFileFormat.Eddie))]
		[CommandLine("-VSCode", Value = nameof(ProjectFileFormat.VisualStudioCode))]
		[CommandLine("-VSWorkspace", Value = nameof(ProjectFileFormat.VisualStudioWorkspace))]
		[CommandLine("-CLion", Value = nameof(ProjectFileFormat.CLion))]
		[CommandLine("-Rider", Value = nameof(ProjectFileFormat.Rider))]
		[CommandLine("-AndroidStudio", Value = nameof(ProjectFileFormat.AndroidStudio))]
#if __VPROJECT_AVAILABLE__
		[CommandLine("-VProject", Value = nameof(ProjectFileFormat.VProject))]
#endif
		HashSet<ProjectFileFormat> ProjectFileFormats = new HashSet<ProjectFileFormat>();

		/// <summary>
		/// Disable native project file generators for platforms. Platforms with native project file generators typically require IDE extensions to be installed.
		/// </summary>
		[XmlConfigFile(Category = "ProjectFileGenerator")]
		string[]? DisablePlatformProjectGenerators = null;

		/// <summary>
		/// Whether this command is being run in an automated mode
		/// </summary>
		[CommandLine("-Automated")]
		bool bAutomated = false;
#pragma warning restore IDE0044

		/// <summary>
		/// Execute the tool mode
		/// </summary>
		/// <param name="Arguments">Command line arguments</param>
		/// <returns>Exit code</returns>
		/// <param name="Logger"></param>
		public Task<int> ExecuteAsync(CommandLineArguments Arguments, ILogger Logger)
		{
			// Apply any command line arguments to this class
			Arguments.ApplyTo(this);

			// Apply the XML config to this class
			XmlConfig.ApplyTo(this);

			// Apply to architecture configs that need to read commandline arguments and didn't have the Arguments passed in during construction
			foreach (UnrealArchitectureConfig Config in UnrealArchitectureConfig.AllConfigs())
			{
				Arguments.ApplyTo(Config);
			}

			// set up logging (taken from BuildMode)
			if (!Log.HasFileWriter())
			{
				FileReference LogFile = FileReference.Combine(Unreal.EngineProgramSavedDirectory, "UnrealBuildTool", "Log_GPF.txt");
				Log.AddFileWriter("DefaultLogTraceListener", LogFile);
			}
			else
			{
				Log.RemoveStartupTraceListener();
			}

			// Parse rocket-specific arguments.
			FileReference? ProjectFile;
			Utils.TryParseProjectFileArgument(Arguments, Logger, out ProjectFile);

			// Apply the XML config again with a project specific BuildConfiguration.xml 
			XmlConfig.ReadConfigFiles(null, ProjectFile?.Directory, Logger);
			XmlConfig.ApplyTo(this);

			// Warn if there are explicit project file formats specified
			if (ProjectFileFormats.Count > 0 && !bAutomated)
			{
				StringBuilder Configuration = new StringBuilder();
				Configuration.Append("Project file formats specified via the command line will be ignored when generating\n");
				Configuration.Append("project files from the editor and other engine tools.\n");
				Configuration.Append('\n');
				Configuration.Append("Consider setting your desired IDE from the editor preferences window, or modify your\n");
				Configuration.Append("BuildConfiguration.xml file with:\n");
				Configuration.Append('\n');
				Configuration.Append("<?xml version=\"1.0\" encoding=\"utf-8\" ?>\n");
				Configuration.Append("<Configuration xmlns=\"https://www.unrealengine.com/BuildConfiguration\">\n");
				Configuration.Append("  <ProjectFileGenerator>\n");
				foreach (ProjectFileFormat ProjectFileFormat in ProjectFileFormats)
				{
					Configuration.AppendFormat("    <Format>{0}</Format>\n", ProjectFileFormat);
				}
				Configuration.Append("  </ProjectFileGenerator>\n");
				Configuration.Append("</Configuration>\n");
				Logger.LogWarning("{Configuration}", Configuration.ToString());
			}

			// If there aren't any formats set, read the default project file format from the config file
			if (ProjectFileFormats.Count == 0)
			{
				// Read from the XML config
				if (!String.IsNullOrEmpty(ProjectFileGeneratorSettings.Format))
				{
					ProjectFileFormats.UnionWith(ProjectFileGeneratorSettings.ParseFormatList(ProjectFileGeneratorSettings.Format, Logger));
				}

				// Read from the editor config
				ProjectFileFormat PreferredSourceCodeAccessor;
				if (ProjectFileGenerator.GetPreferredSourceCodeAccessor(ProjectFile, out PreferredSourceCodeAccessor))
				{
					ProjectFileFormats.Add(PreferredSourceCodeAccessor);
				}

				// If there's still nothing set, get the default project file format for this platform
				if (ProjectFileFormats.Count == 0)
				{
					ProjectFileFormats.UnionWith(BuildHostPlatform.Current.GetDefaultProjectFileFormats());
				}
			}

			ValidateProjectFileFormats(Logger);

			// Register all the platform project generators
			PlatformProjectGeneratorCollection PlatformProjectGenerators = new PlatformProjectGeneratorCollection();
			foreach (Type CheckType in Assembly.GetExecutingAssembly().GetTypes())
			{
				if (CheckType.IsClass && !CheckType.IsAbstract && CheckType.IsSubclassOf(typeof(PlatformProjectGenerator)))
				{
					PlatformProjectGenerator Generator = (PlatformProjectGenerator)Activator.CreateInstance(CheckType, Arguments, Logger)!;
					foreach (UnrealTargetPlatform Platform in Generator.GetPlatforms())
					{
						if (DisablePlatformProjectGenerators == null || !DisablePlatformProjectGenerators.Any(x => x.Equals(Platform.ToString(), StringComparison.OrdinalIgnoreCase)))
						{
							Logger.LogDebug("Registering project generator {CheckType} for {Platform}", CheckType, Platform);
							PlatformProjectGenerators.RegisterPlatformProjectGenerator(Platform, Generator, Logger);
						}
					}
				}
			}

			// print out any errors to the log
			List<string> BadPlatformNames = new List<string>();
			Logger.LogDebug("\n---   SDK INFO START   ---");
			foreach (string PlatformName in UnrealTargetPlatform.GetValidPlatformNames())
			{
				UEBuildPlatformSDK? SDK = UEBuildPlatformSDK.GetSDKForPlatform(PlatformName);
				if (SDK != null && SDK.bIsSdkAllowedOnHost)
				{
					// print out the info to the log, and if it's invalid, remember it
					SDKStatus Validity = SDK.PrintSDKInfoAndReturnValidity(LogEventType.Verbose, LogFormatOptions.NoConsoleOutput, LogEventType.Warning, LogFormatOptions.NoConsoleOutput);
					if (Validity == SDKStatus.Invalid)
					{
						BadPlatformNames.Add(PlatformName);
					}
				}
			}
			if (BadPlatformNames.Count > 0)
			{
				Log.TraceInformationOnce("\nSome Platforms were skipped due to invalid SDK setup: {0}.\nSee the log file for detailed information\n\n", String.Join(", ", BadPlatformNames));
				Logger.LogInformation("");
			}
			Logger.LogDebug("---   SDK INFO END   ---");
			Logger.LogDebug("");

			// look for a single target name param
			string? SingleTargetName = null;
			if (Arguments.HasValue("-SingleTarget="))
			{
				SingleTargetName = Arguments.GetString("-SingleTarget=");
			}

			CSharpProjectCache CSharpProjCache = new(Logger);

			// Create each project generator and run it
			Dictionary<ProjectFileFormat, ProjectFileGenerator> Generators = new();
			foreach (ProjectFileFormat ProjectFileFormat in ProjectFileFormats.Distinct())
			{
				ProjectFileGenerator Generator;
				switch (ProjectFileFormat)
				{
					case ProjectFileFormat.Make:
						Generator = new MakefileGenerator(ProjectFile, CSharpProjCache);
						break;
					case ProjectFileFormat.CMake:
						Generator = new CMakefileGenerator(ProjectFile, CSharpProjCache);
						break;
					case ProjectFileFormat.QMake:
						Generator = new QMakefileGenerator(ProjectFile, CSharpProjCache);
						break;
					case ProjectFileFormat.KDevelop:
						Generator = new KDevelopGenerator(ProjectFile, CSharpProjCache);
						break;
					case ProjectFileFormat.CodeLite:
						Generator = new CodeLiteGenerator(ProjectFile, CSharpProjCache, Arguments);
						break;
					case ProjectFileFormat.VisualStudio:
						Generator = new VCProjectFileGenerator(ProjectFile, CSharpProjCache, VCProjectFileFormat.Default, Arguments);
						break;
					case ProjectFileFormat.VisualStudio2022:
						Generator = new VCProjectFileGenerator(ProjectFile, CSharpProjCache, VCProjectFileFormat.VisualStudio2022, Arguments);
						break;
					case ProjectFileFormat.VisualStudio2026:
						Generator = new VCProjectFileGenerator(ProjectFile, CSharpProjCache, VCProjectFileFormat.VisualStudio2026, Arguments);
						break;
					case ProjectFileFormat.XCode:
						Generator = new XcodeProjectFileGenerator(ProjectFile, CSharpProjCache, Arguments);
						break;
					case ProjectFileFormat.Eddie:
						Generator = new EddieProjectFileGenerator(ProjectFile, CSharpProjCache);
						break;
					case ProjectFileFormat.VisualStudioCode:
						Generator = new VSCodeProjectFileGenerator(ProjectFile, CSharpProjCache);
						break;
					case ProjectFileFormat.VisualStudioWorkspace:
						Generator = new VSWorkspaceProjectFileGenerator(ProjectFile, CSharpProjCache, Arguments);
						break;
					case ProjectFileFormat.CLion:
						Generator = new CLionGenerator(ProjectFile, CSharpProjCache);
						break;
					case ProjectFileFormat.Rider:
						Generator = new RiderProjectFileGenerator(ProjectFile, CSharpProjCache, Arguments);
						break;
					case ProjectFileFormat.AndroidStudio:
						Generator = new AndroidStudioFileGenerator(ProjectFile, CSharpProjCache);
						break;
#if __VPROJECT_AVAILABLE__
					case ProjectFileFormat.VProject:
						Generator = new VProjectFileGenerator(ProjectFile, CSharpProjCache);
						break;
#endif
					default:
						throw new BuildException("Unhandled project file type '{0}'", ProjectFileFormat);
				}
				// remember if we only wanted a single target (similar to -game -project, except usable with progarms without uprojects)
				Generator.SingleTargetName = SingleTargetName;

				Generators[ProjectFileFormat] = Generator;
			}

			// Check there are no superfluous command line arguments
			// TODO (still pass raw arguments below)
			// Arguments.CheckAllArgumentsUsed();

			// Now generate project files
			ProjectFileGenerator.bGenerateProjectFiles = true;
			bool bGenerateSuccess = true;
			foreach (KeyValuePair<ProjectFileFormat, ProjectFileGenerator> Pair in Generators)
			{
				using ITimelineEvent ExecuteScope = Timeline.ScopeEvent($"Generating {Pair.Key} project files");
				Logger.LogInformation("");
				Logger.LogInformation("Generating {ProjectFormat} project files:", Pair.Key);

				ProjectFileGenerator.Current = Pair.Value;
				Arguments.ApplyTo(Pair.Value);
#pragma warning disable 0618 // Type or member is obsolete
				bGenerateSuccess = Pair.Value.GenerateProjectFiles(PlatformProjectGenerators, Arguments.GetRawArray(), false, Logger);
#pragma warning restore 0618 // Type or member is obsolete
				ProjectFileGenerator.Current = null;

				if (!bGenerateSuccess)
				{
					break;
				}
			}
			if (bGenerateSuccess && Generators.Count > 0)
			{
				Generators.First().Value.GenerateQueryTargetsDataForEditor(Arguments.GetRawArray(), Logger);
			}
			return bGenerateSuccess ? Task.FromResult((int)CompilationResult.Succeeded) : Task.FromResult((int)CompilationResult.OtherCompilationError);
		}

		private void ValidateProjectFileFormats(ILogger Logger)
		{
			if (ProjectFileFormats.Contains(ProjectFileFormat.VisualStudio2022) && ProjectFileFormats.Contains(ProjectFileFormat.VisualStudio2026))
			{
				Logger.LogWarning("Cannot generate projects for both Visual Studio 2022 and Visual Studio S2026 - please check your settings " +
					"(BuildConfiguration.xml and Editor Preferences). Generating for Visual Studio 2026.");
				ProjectFileFormats.Remove(ProjectFileFormat.VisualStudio2022);
			}
		}
	}
}
