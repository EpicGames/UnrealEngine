// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace UnrealBuildTool
{
	/// <summary>
	/// Systems that need to be configured to execute a tool mode
	/// </summary>
	[Flags]
	internal enum ToolModeOptions
	{
		/// <summary>
		/// Do not initialize anything
		/// </summary>
		None = 0,

		/// <summary>
		/// Start prefetching metadata for the engine folder as early as possible
		/// </summary>
		StartPrefetchingEngine = 1,

		/// <summary>
		/// Initializes the XmlConfig system
		/// </summary>
		XmlConfig = 2,

		/// <summary>
		/// Registers build platforms
		/// </summary>
		BuildPlatforms = 4,

		/// <summary>
		/// Registers build platforms
		/// </summary>
		BuildPlatformsHostOnly = 8,

		/// <summary>
		/// Registers build platforms for validation
		/// </summary>
		BuildPlatformsForValidation = 16,

		/// <summary>
		/// Only allow a single instance running in the branch at once
		/// </summary>
		SingleInstance = 32,

		/// <summary>
		/// Print out the total time taken to execute
		/// </summary>
		ShowExecutionTime = 64,

		/// <summary>
		/// Capture logs as early as possible in a StartupTraceListener object
		/// </summary>
		UseStartupTraceListener = 128,
	}

	internal interface IToolMode
	{
		/// <summary>
		/// Entry point for this command.
		/// </summary>
		/// <param name="Arguments">List of command line arguments</param>
		/// <param name="Logger"></param>
		/// <returns>Exit code for the process</returns>
		abstract Task<int> ExecuteAsync(CommandLineArguments Arguments, ILogger Logger);
	}

	/// <summary>
	/// Base interface for standalone UBT modes. Different modes can be invoked using the -Mode=[Name] argument on the command line.
	/// The log system will be initialized before calling the mode, but little else.
	/// </summary>
	internal interface IToolMode<TToolMode> : IToolMode where TToolMode : IToolMode<TToolMode>, new()
	{

		/// <summary>
		/// The name of this tool mode.
		/// </summary>
		static abstract string Name { get; }

		/// <summary>
		/// The options for this tool mode.
		/// </summary>
		static abstract ToolModeOptions Options { get; }
	}
}
