// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Text;
using System.Text.Json;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.ObjectPool;
using UnrealBuildBase;

// IWYUMode is a mode that can be used to clean up includes in source code. It uses the clang based tool include-what-you-use (IWYU) to figure out what is needed in each .h/.cpp file
// and then cleans up accordingly. This mode can be used to clean up unreal as well as plugins and projects on top of unreal.
// Note, IWYU is not perfect. There are still c++ features not supported so even though it will do a good job cleaning up it might require a little bit of hands on but not much.
// When iwyu for some reason removes includes you want to keep, there are ways to anotate the code. Look at the pragma link below.
// 
// IWYU Github:  https://github.com/include-what-you-use/include-what-you-use 
// IWYU Pragmas: https://github.com/include-what-you-use/include-what-you-use/blob/master/docs/IWYUPragmas.md
// 
// Here are examples of how a commandline could look (using ushell)
// 
//  1. Will build lyra editor target using include-what-you-use.exe instead of clang.exe, and preview how iwyu would update module LyraGame and all the modules depending on it
//  
//   .build target LyraEditor win64 development -- -Mode=IWYU -ModuleToUpdate=LyraGame -UpdateDependents
//   
//  2. Same as 1 but will check out files from p4 and modify them
//  
//   .build target LyraEditor win64 development -- -Mode=IWYU -ModuleToUpdate=LyraGame -UpdateDependents -Write
//   
//  3. Will build and then update all modules that has Lyra/Plugins in its path. Note it will only include modules that are part of LyraEditor target.
// 
//   .build target LyraEditor win64 development -- -Mode=IWYU -PathToUpdate=Lyra/Plugins -Write
//   
//  4. Update Niagara plugins private code (not public api) with preview
// 
//   .build target UnrealEditor win64 development -- -Mode=IWYU -PathToUpdate=Engine/Plugins/FX/Niagara -UpdateOnlyPrivate

namespace UnrealBuildTool
{
	/// <summary>
	/// Representing an #include inside a code file.
	/// </summary>
	class IWYUIncludeEntry
	{
		/// <summary>
		/// Full path using forward paths, eg: d:/dev/folder/file.h
		/// </summary>
		public string Full { get; set; } = "";

		/// <summary>
		/// Printable path for actual include. Does not include quotes or angle brackets. eg: folder/file.h
		/// </summary>
		public string Printable { get; set; } = "";

		/// <summary>
		/// Reason include exists
		/// </summary>
		public string Reason { get; set; } = "";

		/// <summary>
		/// Reference to file info. Not populated with .iwyu file
		/// </summary>
		public IWYUInfo? Resolved;

		/// <summary>
		/// Decides if printable should be #include quote or angle bracket
		/// </summary>
		public bool System { get; set; }

		/// <summary>
		/// If true #include was found inside a declaration such as a namespace or class/struct/function
		/// </summary>
		public bool InsideDecl { get; set; }

		public bool InPath(string path)
		{
			return Full.StartsWith(path, StringComparison.OrdinalIgnoreCase);
		}
	}

	/// <summary>
	/// Comparer for include entries using full path to compare
	/// </summary>
	class IWYUIncludeEntryFullPathComparer : IEqualityComparer<IWYUIncludeEntry>
	{
		public bool Equals(IWYUIncludeEntry? x, IWYUIncludeEntry? y)
		{
			return x!.Full == y!.Full;
		}

		public int GetHashCode(IWYUIncludeEntry x)
		{
			return x.Full.GetHashCode(StringComparison.Ordinal);
		}
	}

	/// <summary>
	/// Representing a forward declaration inside a code file. eg: class MyClass;
	/// </summary>
	struct IWYUForwardEntry
	{
		/// <summary>
		/// The string to add to the file
		/// </summary>
		public string Printable { get; set; }

		/// <summary>
		/// True if forward declaration has already been seen in the file
		/// </summary>
		public bool Present { get; set; }
	}

	/// <summary>
	/// Representing an include that the code file needs that is missing in the include list
	/// Note, in cpp files it is listing the includes that are in the matching h file
	/// </summary>
	struct IWYUMissingInclude
	{
		/// <summary>
		/// Full path using forward paths, eg: d:/dev/folder/file.h
		/// </summary>
		public string Full { get; set; }
	}

	/// <summary>
	/// Representing a file (header or source). Each .iwyu file has a list of these (most of the time only one)
	/// </summary>
	class IWYUInfo
	{
		/// <summary>
		/// Full path of file using forward paths, eg: d:/dev/folder/file.h
		/// </summary>
		public string File { get; set; } = "";

		/// <summary>
		/// Includes that this file needs. This is what iwyu have decided is used.
		/// </summary>
		public List<IWYUIncludeEntry> Includes { get; set; } = [];

		/// <summary>
		/// Forward declarations that this file needs. This is what iwyu have decided is used.
		/// </summary>
		public List<IWYUForwardEntry> ForwardDeclarations { get; set; } = [];

		/// <summary>
		/// Includes seen in the file. This is how it looked like on disk.
		/// </summary>
		public List<IWYUIncludeEntry> IncludesSeenInFile { get; set; } = [];

		/// <summary>
		/// Includes that didnt end up in the Includes list but is needed by the code file
		/// </summary>
		public List<IWYUMissingInclude> MissingIncludes { get; set; } = [];

		/// <summary>
		/// Transitive includes. This is all the includes that someone gets for free when including this file
		/// Note, this is based on what iwyu believes should be in all includes. So it will not look at "seen includes"
		/// and only use its generated list.
		/// </summary>
		public Dictionary<string, string> TransitiveIncludes = [];

		/// <summary>
		/// Transitive forward declarations. Same as transitive includes
		/// </summary>
		public Dictionary<string, string> TransitiveForwardDeclarations = [];

		/// <summary>
		/// Which .iwyu file that produced this info. Note, it might be null for some special cases (like .generated.h files)
		/// </summary>
		public IWYUFile? Source;

		/// <summary>
		/// Module that this file belongs to
		/// </summary>
		public UEBuildModule? Module;

		/// <summary>
		/// Is true if this file was included inside a declaration such as a namespace or class/struct/function
		/// </summary>
		public bool IncludedInsideDecl;

		/// <summary>
		/// Is true if this file ends with .cpp
		/// </summary>
		public bool IsCpp;
	}

	/// <summary>
	/// This is what include-what-you-use.exe produces for each build. a .iwyu file that is parsed into these instances
	/// </summary>
	class IWYUFile
	{
		/// <summary>
		/// The different headers and source files that was covered by this execution of iwyu.
		/// </summary>
		public List<IWYUInfo> Files { get; set; } = new();

		/// <summary>
		/// Name of .iwyu file
		/// </summary>
		public string? Name;
	}

	/// <summary>
	/// Exception when formatting of file not supported
	/// </summary>
	public class IWYUUnsupportedFormattingException : Exception
	{
		internal IWYUUnsupportedFormattingException(string? message, Exception? exception = null) : base(message, exception)
		{
		}
	}

	/// <summary>
	/// Profiles different unity sizes and prints out the different size and its timings
	/// </summary>
	internal sealed class IWYUMode : IToolMode<IWYUMode>
	{
		public static string Name => "IWYU";
		public static ToolModeOptions Options => ToolModeOptions.XmlConfig | ToolModeOptions.BuildPlatforms | ToolModeOptions.SingleInstance | ToolModeOptions.StartPrefetchingEngine | ToolModeOptions.ShowExecutionTime;

		/// <summary>
		/// Specifies the file to use for logging.
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		public string? IWYUBaseLogFileName;

		/// <summary>
		/// Will check out files from p4 and write to disk
		/// </summary>
		[CommandLine("-Write")]
		public bool bWrite = false;

		/// <summary>
		/// Update only includes in cpp files and don't touch headers
		/// </summary>
		[CommandLine("-UpdateOnlyPrivate")]
		public bool bUpdateOnlyPrivate = false;

		/// <summary>
		/// Which module to run IWYU on. Will also include all modules depending on this module
		/// </summary>
		[CommandLine("-ModuleToUpdate")]
		public string ModuleToUpdate = "";

		/// <summary>
		/// Which directory to run IWYU on. Will search for modules that has module directory matching this string
		/// If no module is found in -PathToUpdate it handles this for individual files instead.
		/// Note this can be combined with -ModuleToUpdate to double filter
		/// PathToUpdate supports multiple paths using semi colon separation
		/// </summary>
		[CommandLine("-PathToUpdate")]
		public string PathToUpdate = "";

		/// <summary>
		/// Same as PathToUpdate but provide the list of paths in a text file instead.
		/// Add all the desired paths on a separate line
		/// </summary>
		[CommandLine("-PathToUpdateFile")]
		public string PathToUpdateFile = "";

		/// <summary>
		/// Also update modules that are depending on the module we are updating
		/// </summary>
		[CommandLine("-UpdateDependents")]
		public bool bUpdateDependents = false;

		/// <summary>
		/// Will check out files from p4 and write to disk
		/// </summary>
		[CommandLine("-NoP4")]
		public bool bNoP4 = false;

		/// <summary>
		/// Allow files to not add includes if they are transitively included by other includes
		/// </summary>
		[CommandLine("-NoTransitive")]
		public bool bNoTransitiveIncludes = false;

		/// <summary>
		/// Will remove headers that are needed but redundant because they are included through other needed includes
		/// </summary>
		[CommandLine("-RemoveRedundant")]
		public bool bRemoveRedundantIncludes = false;

		/// <summary>
		/// Will skip compiling before updating. Handle with care, this is dangerous since files might not match .iwyu files
		/// </summary>
		[CommandLine("-NoCompile")]
		public bool bNoCompile = false;

		/// <summary>
		/// Will ignore update check that .iwyu is newer than source files.
		/// </summary>
		[CommandLine("-IgnoreUpToDateCheck")]
		public bool bIgnoreUpToDateCheck = false;

		/// <summary>
		/// If set, this will keep removed includes in #if/#endif scope at the end of updated file.
		/// Applied to non-private headers that are part of the Engine folder.
		/// </summary>
		[CommandLine("-DeprecateTag")]
		private string? HeaderDeprecationTagOverride;

		public string HeaderDeprecationTag
		{
			get
			{
				if (!String.IsNullOrEmpty(HeaderDeprecationTagOverride))
				{
					return HeaderDeprecationTagOverride;
				}
				return EngineIncludeOrderHelper.GetLatestDeprecationDefine();
			}
			set => HeaderDeprecationTagOverride = value;
		}

		/// <summary>
		/// Compare current include structure with how it would look like if iwyu was applied on all files
		/// </summary>
		[CommandLine("-Compare")]
		public bool bCompare = false;

		/// <summary>
		/// For Development only - Will write a toc referencing all .iwyu files. Toc can be used by -ReadToc
		/// </summary>
		[CommandLine("-WriteToc")]
		public bool bWriteToc = false;

		/// <summary>
		/// For Development only - Will read a toc to find all .iwyu files instead of building the code.
		/// </summary>
		[CommandLine("-ReadToc")]
		public bool bReadToc = false;

		/// <summary>
		/// Will print out info about the file in question. Transitive includes etc
		/// </summary>
		[CommandLine("-PrintInfo")]
		public string PrintInfo = "";

		/// <summary>
		/// Export resolved IWYU metadata to a JSON file for external tools.
		/// </summary>
		[CommandLine("-ExportMetadata=")]
		public string? ExportMetadataPath;

		private string? GetModuleToUpdateName(TargetDescriptor Descriptor)
		{
			if (!String.IsNullOrEmpty(ModuleToUpdate))
			{
				return ModuleToUpdate;
			}

			if (Descriptor.OnlyModuleNames.Count > 0)
			{
				return Descriptor.OnlyModuleNames.First();
			}

			return null;
		}

		private static string AdjustModulePathForMatching(string ModulePath)
		{
			string NewModulePath = ModulePath.Replace('\\', '/');
			if (!NewModulePath.EndsWith('/'))
			{
				NewModulePath += '/';
			}
			return NewModulePath;
		}

		/// <summary>
		/// Execute the command
		/// </summary>
		/// <param name="Arguments">Command line arguments</param>
		/// <returns>Exit code</returns>
		/// <param name="Logger"></param>
		public async Task<int> ExecuteAsync(CommandLineArguments Arguments, ILogger Logger)
		{
			Arguments.ApplyTo(this);

			Logger.LogInformation($"====================================================");
			Logger.LogInformation("Running IWYU. {PreviewMode}", bWrite ? "" : "(Preview mode. Add -Write to write modifications to files)");
			Logger.LogInformation($"====================================================");

			// Fixup the log path if it wasn't overridden by a config file
			IWYUBaseLogFileName ??= FileReference.Combine(Unreal.EngineProgramSavedDirectory, "UnrealBuildTool", "IWYULog.txt").FullName;

			// Create the log file, and flush the startup listener to it
			if (!Arguments.HasOption("-NoLog") && !Log.HasFileWriter())
			{
				Log.AddFileWriter("DefaultLogTraceListener", new(IWYUBaseLogFileName));
			}
			else
			{
				Log.RemoveStartupTraceListener();
			}

			// Create the build configuration object, and read the settings
			CommandLineArguments BuildArguments = Arguments.Append(new[] { "-IWYU" }); // Add in case it is not added (it is needed for iwyu toolchain)
			BuildConfiguration BuildConfiguration = new BuildConfiguration();
			XmlConfig.ApplyTo(BuildConfiguration);
			BuildArguments.ApplyTo(BuildConfiguration);
			BuildConfiguration.MaxNestedPathLength = 220; // For now since the path is slightly longer

			// Parse all the target descriptors
			List<TargetDescriptor> TargetDescriptors = TargetDescriptor.ParseCommandLine(BuildArguments, BuildConfiguration, Logger);

			if (TargetDescriptors.Count != 1)
			{
				Logger.LogError("IWYUMode can only handle command lines that produce one target (Cmdline: {CommandLine})", Arguments);
				return -1;
			}

			TargetDescriptors[0].IntermediateEnvironment = UnrealIntermediateEnvironment.IWYU;

			string? ModuleToUpdateName = GetModuleToUpdateName(TargetDescriptors[0]);

			if (!String.IsNullOrEmpty(ModuleToUpdateName))
			{
				foreach (string OnlyModuleName in TargetDescriptors[0].OnlyModuleNames)
				{
					if (!String.Equals(OnlyModuleName, ModuleToUpdateName, StringComparison.OrdinalIgnoreCase))
					{
						Logger.LogError("ModuleToUpdate '{ModuleToUpdate}' was not in list of specified modules: {Modules}", ModuleToUpdateName, String.Join(", ", TargetDescriptors[0].OnlyModuleNames));
						return -1;
					}
				}
			}

			string[] PathToUpdateList = Array.Empty<string>();
			if (!String.IsNullOrEmpty(PathToUpdate))
			{
				PathToUpdateList = PathToUpdate.Split(";");
			}
			else if (!String.IsNullOrEmpty(PathToUpdateFile))
			{
				PathToUpdateList = await File.ReadAllLinesAsync(PathToUpdateFile);
			}

			if (PathToUpdateList.Length > 0)
			{
				for (int I = 0; I != PathToUpdateList.Length; ++I)
				{
					PathToUpdateList[I] = PathToUpdateList[I].Replace('\\', '/');
				}
			}

			string TargetName = $"{TargetDescriptors[0].Name}_{TargetDescriptors[0].Configuration}_{TargetDescriptors[0].Platform}";

			// Calculate file paths filter to figure out which files that IWYU will run on
			HashSet<string> ValidPaths = [];
			Dictionary<string, UEBuildModule> PathToModule = [];
			Dictionary<string, UEBuildModule> NameToModule = [];
			List<string> systemPaths = [];
			{
				using ITimelineEvent _ = Timeline.ScopeEvent("CreatingTargets");
				// Create target to be able to traverse all existing modules
				Logger.LogInformation("Creating BuildTarget for {Target}...", TargetName);
				UEBuildTarget Target = UEBuildTarget.Create(TargetDescriptors[0], BuildConfiguration, Logger);

				if (TargetDescriptors[0].Platform.IsInGroup(UnrealPlatformGroup.Microsoft))
				{
					VCEnvironment env = Target.Rules.WindowsPlatform.Environment!;
					systemPaths.Add(env.ToolChainDir.FullName.Replace('\\', '/'));
					systemPaths.Add(env.WindowsSdkDir.FullName.Replace('\\', '/'));
				}

				Logger.LogInformation($"Calculating file filter for IWYU...");

				UEBuildModule? UEModuleToUpdate = null;

				// Turn provided module string into UEBuildModule reference
				if (!String.IsNullOrEmpty(ModuleToUpdateName))
				{
					UEModuleToUpdate = Target.GetModuleByName(ModuleToUpdateName);
					if (UEModuleToUpdate == null)
					{
						Logger.LogError("Can't find module with name {ModuleToUpdate}", ModuleToUpdateName);
						return -1;
					}
				}

				if (PathToUpdateList.Length == 0 && UEModuleToUpdate == null)
				{
					if (String.IsNullOrEmpty(ExportMetadataPath))
					{
						Logger.LogError($"Need to provide -ModuleToUpdate=<modulename> or -PathToUpdate=<partofpath> to run IWYU.");
						return -1;
					}

					// When exporting metadata without a module/path filter, include all modules
					Logger.LogInformation("No -ModuleToUpdate or -PathToUpdate specified. Will analyze all modules for metadata export.");
				}

				int ModulesToUpdateCount = 0;
				int ModulesSkippedCount = 0;

				// Traverse all modules to figure out which ones should be updated. This is based on -ModuleToUpdate and -PathToUpdate
				List<UEBuildModule> ReferencedModules = new();
				HashSet<UEBuildModule> IgnoreReferencedModules = new();
				foreach (UEBuildBinary Binary in Target.Binaries)
				{
					foreach (UEBuildModule Module in Binary.Modules)
					{
						if (IgnoreReferencedModules.Add(Module))
						{
							ReferencedModules.Add(Module);
							Module.GetAllDependencyModules(ReferencedModules, IgnoreReferencedModules, true, false, false);
						}
					}
				}
				foreach (UEBuildModule Module in ReferencedModules)
				{
					NameToModule.TryAdd(Module.Name, Module);
					foreach (UEBuildModule Module2 in Module.GetDependencies(true, true))
					{
						NameToModule.TryAdd(Module2.Name, Module2);
					}
				}

				Queue<UEBuildModule> toUpdateQueue = [];
				HashSet<UEBuildModule> toUpdateHandled = [];
				if (UEModuleToUpdate != null)
				{
					toUpdateQueue.Enqueue(UEModuleToUpdate);
					toUpdateHandled.Add(UEModuleToUpdate);
				}

				foreach (UEBuildModule Module in NameToModule.Values)
				{
					string ModuleDir = Module.ModuleDirectory.FullName.Replace('\\', '/');
					if (!PathToModule.TryAdd(ModuleDir, Module))
					{
						continue;
					}
					foreach (DirectoryItem AdditionalDir in Module.ModuleDirectories)
					{
						if (AdditionalDir.Location != Module.ModuleDirectory)
						{
							PathToModule.TryAdd(AdditionalDir.FullName.Replace('\\', '/'), Module);
						}
					}

					if (Module.Rules.Type != ModuleRules.ModuleType.CPlusPlus)
					{
						continue;
					}

					if (PathToUpdateList.Length > 0)
					{
						bool Match = false;
						for (int I = 0; I != PathToUpdateList.Length; ++I)
						{
							Match = Match || ModuleDir.Contains(PathToUpdateList[I], StringComparison.OrdinalIgnoreCase);
						}

						if (Match)
						{
							if (toUpdateHandled.Add(Module))
							{
								toUpdateQueue.Enqueue(Module);
							}
						}
					}
				}

				void AddModuleDirs(UEBuildModule module)
				{
					if (module.Rules.IWYUSupport == IWYUSupport.None)
					{
						++ModulesSkippedCount;
						return;
					}

					++ModulesToUpdateCount;

					string ModuleDir = module.ModuleDirectory.FullName.Replace('\\', '/');

					// When adding to ValidPaths, make sure the Module directory ends in a / so we match the exact folder later
					ValidPaths.Add(AdjustModulePathForMatching(ModuleDir));

					foreach (DirectoryItem AdditionalDir in module.ModuleDirectories)
					{
						if (AdditionalDir.Location != module.ModuleDirectory)
						{
							ValidPaths.Add(AdjustModulePathForMatching(AdditionalDir.FullName));
						}
					}
				}

				while (toUpdateQueue.Count != 0)
				{
					UEBuildModule module = toUpdateQueue.Dequeue();
					AddModuleDirs(module);

					if (bUpdateDependents)
					{
						foreach (UEBuildModule refModule in module.ReferenceStackParentModules!)
						{
							if (toUpdateHandled.Add(refModule))
							{
								toUpdateQueue.Enqueue(refModule);
							}
						}
					}
				}

				if (ValidPaths.Count == 0 && PathToUpdateList.Length > 0)
				{
					foreach (string Path in PathToUpdateList)
					{
						ValidPaths.Add(Path);
					}
					Logger.LogInformation("Will update files matching {PathToUpdate}. Note, path is case sensitive", PathToUpdate);
				}
				else
				{
					Logger.LogInformation(
						"Will update {ModulesToUpdateCount} module(s) using IWYU. ({ModulesSkippedCount} skipped because of bEnforceIWYU=false)...",
						ModulesToUpdateCount,
						ModulesSkippedCount);
				}
			}

			Dictionary<string, IWYUInfo> Infos = new();
			ConcurrentBag<(IWYUInfo, List<IWYUIncludeEntry>)> GeneratedCppInfos = [];

			using (ISourceFileWorkingSet WorkingSet = new EmptySourceFileWorkingSet())
			{
				List<string>? TocContent = null;
				int ReadSuccess = 1;

				if (bReadToc)
				{
					string TocName = TargetName + ".txt";
					FileReference TocReference = FileReference.Combine(Unreal.EngineDirectory, "Intermediate", "IWYU", TocName);
					Logger.LogInformation("Reading TOC from {TOCPath}...", TocReference.FullName);
					if (FileReference.Exists(TocReference))
					{
						TocContent = [.. await FileReference.ReadAllLinesAsync(TocReference)];
					}
				}

				if (TocContent == null)
				{
					TargetDescriptor Descriptor = TargetDescriptors.First();

					// Create the make file that contains all the actions we will use to find .iwyu files.
					Logger.LogInformation($"Creating MakeFile for target...");
					TargetMakefile Makefile;
					try
					{
						Makefile = await BuildMode.CreateMakefileAsync(BuildConfiguration, Descriptor, WorkingSet, Logger);
					}
					finally
					{
						SourceFileMetadataCache.SaveAll();
					}

					// Use the make file to build unless -NoCompile is set.
					if (!bNoCompile)
					{
						CppDependencyCache dependencyCache = new();
						try
						{
							await BuildMode.BuildAsync([(Descriptor, Makefile)], BuildConfiguration, BuildOptions.None, null, dependencyCache, Logger);
						}
						finally
						{
							dependencyCache.SaveAll();
						}
					}

					HashSet<FileItem> OutputItems = new HashSet<FileItem>();
					if (Descriptor.OnlyModuleNames.Count > 0)
					{
						foreach (string OnlyModuleName in Descriptor.OnlyModuleNames)
						{
							FileItem[]? OutputItemsForModule;
							if (!Makefile.ModuleNameToOutputItems.TryGetValue(OnlyModuleName, out OutputItemsForModule))
							{
								throw new BuildException("Unable to find output items for module '{0}'", OnlyModuleName);
							}
							OutputItems.UnionWith(OutputItemsForModule);
						}
					}
					else
					{
						// Use all the output items from the target
						OutputItems.UnionWith(Makefile.OutputItems);
					}

					TocContent = new();

					foreach (FileItem OutputItem in OutputItems)
					{
						if (OutputItem.Name.EndsWith(".iwyu", StringComparison.Ordinal))
						{
							TocContent.Add(OutputItem.AbsolutePath);
						}
					}
				}

				// First parse all iwyu files and add main entry to lookup (that is the last entry in each file)
				Logger.LogInformation("Parsing {IWYUFileCount} .iwyu files...", TocContent.Count);
				using ITimelineEvent parsingIwyu = Timeline.ScopeEvent("ParsingIwyu");
				Parallel2.ForEach(TocContent, (IWYUFilePath) =>
				{
					string? JsonContent = File.ReadAllText(IWYUFilePath);
					if (JsonContent == null)
					{
						Logger.LogError("Failed to read file {IWYUFilePath}", IWYUFilePath);
						Interlocked.Exchange(ref ReadSuccess, 0);
						return;
					}

					try
					{
						IWYUFile? IWYUFile = JsonSerializer.Deserialize<IWYUFile>(JsonContent);
						if (IWYUFile == null)
						{
							Logger.LogError("Failed to deserialize file {IWYUFilePath}", IWYUFilePath);
							Interlocked.Exchange(ref ReadSuccess, 0);
							return;
						}

						if (IWYUFile.Files.Count == 0)
						{
							Logger.LogError("No files found in {IWYUFilePath}", IWYUFilePath);
							Interlocked.Exchange(ref ReadSuccess, 0);
							return;
						}

						IWYUFile.Name = IWYUFilePath;

						IWYUInfo mainInfo = IWYUFile.Files.Last();
						mainInfo.Source = IWYUFile;

						lock (Infos)
						{
							if (!Infos.TryAdd(mainInfo.File, mainInfo))
							{
								Logger.LogError("Found {IWYUFilePath} multiple times", mainInfo.File);
								Interlocked.Exchange(ref ReadSuccess, 0);
								return;
							}
						}
					}
					catch (Exception e)
					{
						Logger.LogError(e, "Failed to parse json {IWYUFilePath}: {Message} - File will be deleted", IWYUFilePath, e.Message);
						//File.Delete(IWYUFilePath);
						Interlocked.Exchange(ref ReadSuccess, 0);
						return;
					}
				});
				parsingIwyu.Finish();

				ConcurrentBag<(IWYUInfo mainInfo, List<IWYUInfo> generatedHeaders)> generatedHeadersToFixup = new();

				// Now traverse main infos and find additional infos and fix them up
				Logger.LogInformation($"Adding additional iwyu infos...");
				IWYUInfo[] mainInfos = Infos.Values.ToArray();
				Parallel2.ForEach(mainInfos, (mainInfo) =>
				{
					IWYUFile IWYUFile = mainInfo.Source!;

					IWYUInfo? headerAsMain = null;
					IWYUInfo? headerInCpp = null;
					List<IWYUInfo>? generatedHeaders = null;
					List<IWYUInfo>? generatedSourceFiles = null;

					// Go through all infos in iwyu
					foreach (IWYUInfo info in IWYUFile.Files)
					{
						bool isCpp = info.File.EndsWith(".cpp", StringComparison.Ordinal);
						info.IsCpp = isCpp;
						info.Module = GetModule(info.File, PathToModule);

						foreach (IWYUIncludeEntry entry in info.IncludesSeenInFile)
						{
							if (entry.Printable[0] == '<')
							{
								entry.Printable = entry.Printable.Substring(1, entry.Printable.Length - 2);
								entry.System = true;
							}
						}

						if (info == mainInfo)
						{
							break;
						}

						bool isGeneratedCpp = isCpp && info.File.EndsWith(".gen.cpp", StringComparison.Ordinal);

						if (!isGeneratedCpp && info.File.EndsWith(".ipp", StringComparison.Ordinal))
						{
							isGeneratedCpp = true;
						}

						if (isGeneratedCpp)
						{
							info.Source = IWYUFile;
							generatedSourceFiles ??= new();
							generatedSourceFiles.Add(info);
							continue;
						}

						bool isHeader = !isCpp && info.File.EndsWith(".h", StringComparison.Ordinal);
						bool isGeneratedHeader = isHeader && (info.File.EndsWith(".generated.h", StringComparison.Ordinal) || info.File.EndsWith(".gen.h", StringComparison.Ordinal));

						// Set Source on regular headers so they pass the Source != null check in IsValidForUpdate
						// and get processed by UpdateFile. Generated headers intentionally keep Source null.
						if (isHeader && !isGeneratedHeader)
						{
							info.Source = IWYUFile;
						}

						lock (Infos)
						{
							if (!Infos.TryAdd(info.File, info))
							{
								if (isHeader && !isGeneratedHeader)
								{
									headerInCpp = info;
									headerAsMain = Infos[info.File];
									continue;
								}
								// TODO: Lots of .inl files here
								//Logger.LogError($"Tried to add file {info.File}");
								//Interlocked.Exchange(ref ReadSuccess, 0);
							}
						}

						if (isGeneratedHeader)
						{
							generatedHeaders ??= new();

							generatedHeaders.Add(info);
						}
					}

					// Fix up main file

					if (!mainInfo.IsCpp && generatedHeaders != null)
					{
						generatedHeadersToFixup.Add((mainInfo!, generatedHeaders!));
					}

					// We want to resolve reference to generated source already here to make sure transitive includes end up correctly
					// .. so we will swap Includes and IncludesSeenInFile
					if (generatedSourceFiles != null)
					{
						foreach (IWYUInfo generatedFile in generatedSourceFiles)
						{
							GeneratedCppInfos.Add((generatedFile, generatedFile.Includes));
							generatedFile.Includes = generatedFile.IncludesSeenInFile;
						}
						if (mainInfo.IsCpp)
						{
							List<IWYUIncludeEntry> generatedIncludes = [];
							foreach (IWYUIncludeEntry include in mainInfo.IncludesSeenInFile)
							{
								foreach (IWYUInfo generatedFile in generatedSourceFiles)
								{
									if (include.Full == generatedFile.File)
									{
										include.Resolved = generatedFile;
										generatedIncludes.Add(include);
										break;
									}
								}
							}

							foreach (IWYUIncludeEntry generatedInclude in generatedIncludes)
							{
								bool included = false;
								foreach (IWYUIncludeEntry include in mainInfo.Includes)
								{
									if (include.Full == generatedInclude.Full)
									{
										include.Resolved = generatedInclude.Resolved;
										included = true;
										break;
									}
								}
								if (!included)
								{
									mainInfo.Includes.Add(generatedInclude);
								}
							}
						}
					}

					// This is annoying. But for some reason list of files needed in header can be different
					// based on if it builds standalone or with the cpp file.
					// This should not happen if header is first in include list.
					// What we need to do is to take all includes that are in
					if (mainInfo.IsCpp && headerInCpp != null)
					{
						if (headerInCpp.Includes.Count != headerAsMain!.Includes.Count)
						{
							HashSet<IWYUIncludeEntry> exists = new HashSet<IWYUIncludeEntry>(mainInfo.Includes, new IWYUIncludeEntryFullPathComparer());
							exists.UnionWith(headerAsMain!.Includes);
							foreach (IWYUIncludeEntry i in headerInCpp.Includes)
							{
								if (!exists.Contains(i))
								{
									// TODO: Note that h and cpp might have different printable include paths since h and cpp are often not in same folder
									mainInfo.Includes.Add(i);
								}
							}
						}
					}
				});

				// Generated headers will never be modified and will keep the includes seen in file.
				// We also add all the includes the generated files have into the owning file
				Parallel2.ForEach(generatedHeadersToFixup, (entry) =>
				{
					IWYUInfo mainInfo = entry.mainInfo;
					List<IWYUInfo> generatedHeaders = entry.generatedHeaders;
					HashSet<IWYUIncludeEntry> exists = new HashSet<IWYUIncludeEntry>(mainInfo.Includes, new IWYUIncludeEntryFullPathComparer());
					foreach (IWYUInfo generatedHeader in generatedHeaders)
					{
						exists.UnionWith(generatedHeader.IncludesSeenInFile);
					}

					foreach (IWYUInfo generatedHeader in generatedHeaders)
					{
						foreach (IWYUIncludeEntry i in generatedHeader.Includes)
						{
							if (!exists.Contains(i))
							{
								mainInfo.Includes.Add(i);
							}
						}

						generatedHeader.Includes = generatedHeader.IncludesSeenInFile;
					}
				});

				// Something went wrong parsing iwyu files.
				if (ReadSuccess == 0)
				{
					return -1;
				}

				if (bWriteToc)
				{
					Logger.LogInformation($"Writing TOC that references all .iwyu files ");
					DirectoryReference TocDir = DirectoryReference.Combine(Unreal.EngineDirectory, "Intermediate", "IWYU");
					DirectoryReference.CreateDirectory(TocDir);
					string TocName = TargetName + ".txt";
					FileReference TocReference = FileReference.Combine(TocDir, TocName);
					await FileReference.WriteAllLinesAsync(TocReference, TocContent);
				}
			}

			KeyValuePair<string, IWYUIncludeEntry?> GetInfo(string Include, string Path)
			{
				string FullPath = DirectoryReference.Combine(Unreal.EngineDirectory, Path).FullName.Replace('\\', '/');
				IWYUInfo? Out = null;
				if (!Infos.TryGetValue(FullPath, out Out))
				{
					return new(Include, null);
				}

				IWYUIncludeEntry Entry = new();
				Entry.Printable = Include;
				Entry.Full = FullPath;
				Entry.Resolved = Out;
				return new(Include, Entry);
			}

			// Always keep
			// Windows/WindowsHWrapper.h
			// Windows/AllowWindowsPlatformTypes.h
			// Windows/HideWindowsPlatformTypes.h

			Dictionary<string, IWYUIncludeEntry?> SpecialIncludes = new Dictionary<string, IWYUIncludeEntry?>(
				new[]
				{
					GetInfo("UObject/ObjectMacros.h", "Source/Runtime/CoreUObject/Public/UObject/ObjectMacros.h"),
					GetInfo("UObject/ScriptMacros.h", "Source/Runtime/CoreUObject/Public/UObject/ScriptMacros.h"),
					GetInfo("VerseInteropUtils.h", "Restricted/NotForLicensees/Plugins/Solaris/Source/VerseNative/Public/VerseInteropTypes.h"),
					GetInfo("Containers/ContainersFwd.h", "Source/Runtime/Core/Public/Containers/ContainersFwd.h"),
					GetInfo("Misc/OptionalFwd.h", "Source/Runtime/Core/Public/Misc/OptionalFwd.h"),
					GetInfo("Templates/SharedPointerFwd.h", "Source/Runtime/Core/Public/Templates/SharedPointerFwd.h"),
					GetInfo("Misc/ExpressionParserTypesFwd.h", "Source/Runtime/Core/Public/Misc/ExpressionParserTypesFwd.h"),
					GetInfo("Misc/NotNull.h", "Source/Runtime/Core/Public/Misc/NotNull.h"),
					GetInfo("UObject/StrongObjectPtrTemplatesFwd.h", "Source/Runtime/Core/Public/UObject/StrongObjectPtrTemplatesFwd.h"),
					GetInfo("Delegates/Delegate.h", "Source/Runtime/Core/Public/Delegates/Delegate.h"),
				}
			);

			Dictionary<string, IWYUIncludeEntry?> ForwardingHeaders = new Dictionary<string, IWYUIncludeEntry?>()
			{
				{ "TMap", SpecialIncludes["Containers/ContainersFwd.h"] },
				{ "TSet", SpecialIncludes["Containers/ContainersFwd.h"] },
				{ "TArray", SpecialIncludes["Containers/ContainersFwd.h"] },
				{ "TArrayView", SpecialIncludes["Containers/ContainersFwd.h"] },
				{ "TOptional", SpecialIncludes["Misc/OptionalFwd.h"] },
				{ "TSharedPtr", SpecialIncludes["Templates/SharedPointerFwd.h"] },
				{ "TSharedRef", SpecialIncludes["Templates/SharedPointerFwd.h"] },
				{ "TWeakPtr", SpecialIncludes["Templates/SharedPointerFwd.h"] },
				{ "TSharedFromThis", SpecialIncludes["Templates/SharedPointerFwd.h"] },
				{ "TCompiledToken", SpecialIncludes["Misc/ExpressionParserTypesFwd.h"] },
				{ "TExpressionToken", SpecialIncludes["Misc/ExpressionParserTypesFwd.h"] },
				{ "TStrongObjectPtr", SpecialIncludes["UObject/StrongObjectPtrTemplatesFwd.h"] },
				{ "TNotNull", SpecialIncludes["Misc/NotNull.h"] },
				{ "TMulticastDelegateRegistration", SpecialIncludes["Delegates/Delegate.h"] },
			};
			/*
			// Add all .generated.h files as entries in the lookup and explicitly add the includes they have which will never be removed
			Logger.LogInformation($"Generating infos for .generated.h files...");
			if (GeneratedHeaderInfos.Count > 0)
			{
				IWYUIncludeEntry? ObjectMacrosInclude = SpecialIncludes["UObject/ObjectMacros.h"];
				IWYUIncludeEntry? ScriptMacrosInclude = SpecialIncludes["UObject/ScriptMacros.h"];
				IWYUIncludeEntry? VerseInteropUtilsInclude = SpecialIncludes["VerseInteropUtils.h"];

				foreach (string Gen in GeneratedHeaderInfos)
				{
					IWYUInfo GenInfo = new();
					GenInfo.File = Gen;
					if (Gen.EndsWith(".generated.h"))
					{
						if (ObjectMacrosInclude != null)
						{
							GenInfo.IncludesSeenInFile.Add(ObjectMacrosInclude);
							GenInfo.Includes.Add(ObjectMacrosInclude);
						}
						if (ScriptMacrosInclude != null)
						{
							GenInfo.IncludesSeenInFile.Add(ScriptMacrosInclude);
							GenInfo.Includes.Add(ScriptMacrosInclude);
						}
					}
					else
					{
						if (VerseInteropUtilsInclude != null)
						{
							GenInfo.IncludesSeenInFile.Add(VerseInteropUtilsInclude);
							GenInfo.Includes.Add(VerseInteropUtilsInclude);
						}
					}
					Infos.Add(Gen, GenInfo);
				}
			}
			*/
			Logger.LogInformation("Found {IWYUEntryCount} IWYU entries...", Infos.Count);

			Logger.LogInformation($"Resolving Includes...");
			using ITimelineEvent resolvingIncludes = Timeline.ScopeEvent("ResolvingIncludes");

			ConcurrentDictionary<IWYUInfo, IWYUInfo> IncludedInsideDecl = new();
			Dictionary<string, IWYUInfo> SkippedHeaders = new();

			Parallel2.ForEach(Infos.Values, Info =>
			{
				// If KeepAsIs we transfer all "seen includes" into the include list.
				if (Info.Module != null && Info.Module.Rules.IWYUSupport == IWYUSupport.KeepAsIs)
				{
					foreach (IWYUIncludeEntry Entry in Info.IncludesSeenInFile)
					{
						// Special hack, we don't want CoreMinimal to be the reason we remove includes transitively
						if (!Entry.Printable.Contains("CoreMinimal.h", StringComparison.Ordinal))
						{
							Info.Includes.Add(Entry);
						}
					}
				}

				if (!Info.IsCpp)
				{
					// See if we need to replace forward declarations with includes
					Info.ForwardDeclarations.RemoveAll(Entry =>
					{
						int LastSpace = Entry.Printable.LastIndexOf(' ');
						if (LastSpace == -1)
						{
							throw new BuildException($"{Info.Source?.Name} contains ForwardDeclaration that is empty");
						}
						string TypeName = Entry.Printable.Substring(LastSpace + 1, Entry.Printable.Length - LastSpace - 2);
						IWYUIncludeEntry? IncludeEntry;
						ForwardingHeaders.TryGetValue(TypeName, out IncludeEntry);
						if (IncludeEntry == null)
						{
							return false;
						}
						Info.Includes.Add(IncludeEntry);
						return true;
					});
				}

				// We don't want to mess around with third party includes.. since we can't see the hierarchy we just assumes that they are optimally included
				Info.Includes.RemoveAll(Entry =>
				{
					return Entry.Full.Contains("/ThirdParty/", StringComparison.Ordinal) || systemPaths.Any(systemPath => Entry.InPath(systemPath));
				});

				foreach (IWYUIncludeEntry Entry in Info.IncludesSeenInFile)
				{
					if (Entry.Full.Contains("/ThirdParty/", StringComparison.Ordinal) || systemPaths.Any(systemPath => Entry.InPath(systemPath)))
					{
						Info.Includes.Add(Entry);
					}

					if (Entry.InsideDecl && Infos.TryGetValue(Entry.Full, out Entry.Resolved))
					{
						IncludedInsideDecl.TryAdd(Entry.Resolved, Info);
						Info.Includes.Add(Entry);
					}

					// We don't want to remove these because they enable includes of windows headers.
					if (Entry.Printable.StartsWith("Windows/AllowWindows", StringComparison.Ordinal) ||
						Entry.Printable.StartsWith("Windows/HideWindows", StringComparison.Ordinal) ||
						Entry.Printable.StartsWith("Windows/WindowsHWrapper.h", StringComparison.Ordinal))
					{
						Info.Includes.Add(Entry);
					}
				}

				// This is workaround for iwyu bug. If you include STableRow.h you also need SListViewEntry.h
				// Need to fix iwyu bug but also I want the dependency in the code to go away
				IWYUIncludeEntry? STableRowEntry = null;
				IWYUIncludeEntry? SListViewEntry = null;

				foreach (IWYUIncludeEntry Include in Info.Includes)
				{
					if (Include.Resolved == null)
					{
						if (!Infos.TryGetValue(Include.Full, out Include.Resolved))
						{
							if (!Include.Full.Contains(".gen.cpp", StringComparison.Ordinal) &&
								!Include.Full.Contains("/ThirdParty/", StringComparison.Ordinal) &&
								!Include.Full.Contains("/AutoSDK/", StringComparison.Ordinal) &&
								!Include.Full.Contains("/VNI/", StringComparison.Ordinal))
							{
								lock (SkippedHeaders)
								{
									if (!SkippedHeaders.TryGetValue(Include.Full, out Include.Resolved))
									{
										IWYUInfo NewInfo = new();
										Include.Resolved = NewInfo;
										NewInfo.File = Include.Full;
										SkippedHeaders.Add(Include.Full, NewInfo);
									}
								}
							}
						}

						STableRowEntry ??= Include.Printable == "Widgets/Views/STableRow.h" ? Include : null;
						SListViewEntry ??= Include.Printable == "Widgets/Views/SListView.h" ? Include : null;
					}
				}

				if (STableRowEntry != null && SListViewEntry == null)
				{
					SListViewEntry = new IWYUIncludeEntry() { Full = STableRowEntry.Full.Replace("STableRow.h", "SListView.h", StringComparison.Ordinal), Printable = "Widgets/Views/SListView.h" };
					Info.Includes.Add(SListViewEntry);
					Infos.TryGetValue(SListViewEntry.Full, out SListViewEntry.Resolved);
				}

				foreach (IWYUMissingInclude MissingInclude in Info.MissingIncludes)
				{
					if (Info.TransitiveIncludes.ContainsKey(MissingInclude.Full))
					{
						continue;
					}
					if (Infos.TryGetValue(MissingInclude.Full, out IWYUInfo? value) && value.Module != null)
					{
						string fixedPath = value.File.Replace('/', '\\');
						foreach (DirectoryReference? Dir in value.Module.PublicIncludePaths.Union(value.Module.PrivateIncludePaths).Union(value.Module.PublicSystemIncludePaths))
						{
							if (fixedPath.StartsWith(Dir.FullName, StringComparison.Ordinal))
							{
								string printable = value.File.Substring(Dir.FullName.Length + 1);
								IWYUIncludeEntry entry = new IWYUIncludeEntry() { Full = MissingInclude.Full, Printable = printable, System = false, Resolved = value, };
								Info.Includes.Add(entry);
								break;
							}
						}
					}
				}
			});

			// Resolving includes inside generated files.
			Parallel2.ForEach(GeneratedCppInfos, entry =>
			{
				foreach (IWYUIncludeEntry include in entry.Item1.Includes)
				{
					Infos.TryGetValue(include.Full, out include.Resolved);
				}
			});
			resolvingIncludes.Finish();

			using ITimelineEvent parsingSkipped = Timeline.ScopeEvent("ParsingSkippedHeaders");
			int InfosCount = Infos.Count;
			Logger.LogInformation($"Parsing included headers not supporting being compiled...");
			ParseSkippedHeaders(SkippedHeaders, Infos, PathToModule, NameToModule);
			Logger.LogInformation("Added {IWYUEntryCount} more read-only IWYU entries...", Infos.Count - InfosCount);

			foreach ((IWYUInfo Info, IWYUInfo From) in IncludedInsideDecl)
			{
				if (Info.IncludesSeenInFile.Count != 0) // Remove include in ScriptSerialization.h and remove this check
				{
					Logger.LogWarning("{IWYUFilePath} - Included inside declaration in {FromIWYUFilePath} but has includes itself.", Info.File, From.File);
				}
				Info.Includes.Clear();
				Info.ForwardDeclarations.Clear();
			}
			parsingSkipped.Finish();

			Logger.LogInformation($"Generating transitive include lists and forward declaration lists...");
			using ITimelineEvent generateTransitive = Timeline.ScopeEvent("GenerateTransitiveIncludeLists");
			ObjectPool<StringBuilder> stacks = ObjectPool.Create<StringBuilder>();
			Parallel2.ForEach(Infos.Values, Info =>
			{
				StringBuilder stack = stacks.Get();
				Info.TransitiveIncludes.EnsureCapacity(300);
				Info.TransitiveForwardDeclarations.EnsureCapacity(300);
				CalculateTransitive(Info, Info, stack, Info.TransitiveIncludes, Info.TransitiveForwardDeclarations, false);
				stacks.Return(stack);
			});
			generateTransitive.Finish();

			// If we have built .gen.cpp it means that it is not inlined in another cpp file
			// And we need to promote the includes needed to the header since we can never modify the .gen.cpp file
			Logger.LogInformation($"Transferring needed includes from .gen.cpp to owning file...");

			// Safe to parallelize: each .gen.cpp resolves to a unique IncluderInfo
			Parallel2.ForEach(GeneratedCppInfos, entry =>
			{
				IWYUInfo GeneratedCpp = entry.Item1;

				// Calculate transitive includes for .gen.cpp/ipp files because they are not in Infos.Values list
				StringBuilder stack = stacks.Get();
				GeneratedCpp.TransitiveIncludes.EnsureCapacity(10);
				GeneratedCpp.TransitiveForwardDeclarations.EnsureCapacity(10);
				CalculateTransitive(GeneratedCpp, GeneratedCpp, stack, GeneratedCpp.TransitiveIncludes, GeneratedCpp.TransitiveForwardDeclarations, false);
				stacks.Return(stack);

				IWYUInfo? IncluderInfo = null;

				// If there is only one file in .iwyu it means that .gen.cpp was compiled separately
				// And we need to make sure IncluderInfo has all the includes the generated cpp file needs
				if (GeneratedCpp.Source!.Files.Count == 1)
				{
					int NameStart = GeneratedCpp.File.LastIndexOf('/');
					string HeaderName = GeneratedCpp.File.Substring(NameStart + 1, GeneratedCpp.File.Length - NameStart - ".gen.cpp".Length) + "h";
					foreach (IWYUIncludeEntry Include in GeneratedCpp.Includes)
					{
						NameStart = Include.Full.LastIndexOf('/');
						string IncludeName = Include.Full.Substring(NameStart + 1);
						if (HeaderName == IncludeName)
						{
							Infos.TryGetValue(Include.Full, out IncluderInfo);
							break;
						}
					}

					if (IncluderInfo == null)
					{
						return;
					}
				}
				else // this .gen.cpp is inlined in a cpp..
				{
					IncluderInfo = GeneratedCpp.Source.Files.FirstOrDefault(I => I.File.Contains(".cpp", StringComparison.Ordinal) && !I.File.Contains(".gen", StringComparison.Ordinal));

					if (IncluderInfo == null)
					{
						return;
					}
				}

				foreach (IWYUIncludeEntry Include in entry.Item2)
				{
					//if (neededIncludes.Contains(Include.Full))
					//{
					//	continue;
					//}

					// TODO: Remove UObject check once we've added "IWYU pragma: keep" around the includes in ScriptMacros and ObjectMacros

					//if (Include.Full.Contains(".generated.h") || Include.Printable.StartsWith("UObject/"))
					//{
					//	continue;
					//}

					// If file included by .gen.cpp (or file .gen.cpp is inlined into) does not see needed include, then we need to add it
					if (IncluderInfo!.TransitiveIncludes.ContainsKey(Include.Full))
					{
						continue;
					}

					bool isExcluded = Include.Full.Contains("/ThirdParty/", StringComparison.Ordinal) || systemPaths.Any(systemPath => Include.InPath(systemPath));
					if (!isExcluded)
					{
						if (Include.Resolved == null)
						{
							Infos.TryGetValue(Include.Full, out Include.Resolved);
						}
						IncluderInfo.Includes.Add(Include);
					}
				}
			});

			if (!String.IsNullOrEmpty(PrintInfo))
			{
				PrintInfo = PrintInfo.Replace('\\', '/');
				bool found = false;
				foreach (IWYUInfo info in Infos.Values)
				{
					if (info.File.Contains(PrintInfo, StringComparison.OrdinalIgnoreCase))
					{
						Logger.LogInformation("--- {IWYUFilePath} ---", info.File);
						Logger.LogInformation($"  Transitive includes");
						foreach (KeyValuePair<string, string> i in info.TransitiveIncludes)
						{
							Logger.LogInformation("    {TransitiveInclude}", i.Key);
						}
						found = true;
					}
				}
				if (found)
				{
					return 0;
				}

				Logger.LogError("PrintInfo file not found ({PrintInfoPath})", PrintInfo);
				return 1;
			}

			if (!String.IsNullOrEmpty(ExportMetadataPath))
			{
				using ITimelineEvent exportEvent = Timeline.ScopeEvent("ExportMetadata");
				ExportMetadata(ExportMetadataPath, Infos, NameToModule, systemPaths, TargetDescriptors[0], HeaderDeprecationTag);
				Logger.LogInformation("Exported IWYU metadata to {Path}", ExportMetadataPath);

				// If only exporting metadata (no -Write or -Compare), skip the update/compare step
				if (!bWrite && !bCompare)
				{
					return 0;
				}
			}

			if (bCompare)
			{
				using ITimelineEvent compareFiles = Timeline.ScopeEvent("CompareFiles");
				return CompareFiles(Infos, ValidPaths, PathToModule, NameToModule, Logger);
			}
			else
			{
				using ITimelineEvent updateFiles = Timeline.ScopeEvent("UpdateFiles");
				return UpdateFiles(Infos, ValidPaths, Logger);
			}
		}

		static UEBuildModule? GetModule(string File, Dictionary<string, UEBuildModule> PathToModule)
		{
			string FilePath = File;
			while (true)
			{
				int LastIndexOfSlash = FilePath.LastIndexOf('/');
				if (LastIndexOfSlash == -1)
				{
					break;
				}

				FilePath = FilePath.Substring(0, LastIndexOfSlash);
				UEBuildModule? Module;
				if (PathToModule.TryGetValue(FilePath, out Module))
				{
					return Module;
				}
			}
			return null;
		}

		static string? GetFullName(string Include, string From, UEBuildModule? Module, Dictionary<string, UEBuildModule> NameToModule, HashSet<UEBuildModule>? Visited = null)
		{
			if (Module == null)
			{
				return null;
			}
			foreach (DirectoryReference? Dir in Module.PublicIncludePaths.Union(Module.PrivateIncludePaths).Union(Module.PublicSystemIncludePaths))
			{
				FileReference FileReference = FileReference.Combine(Dir, Include);
				FileItem FileItem = FileItem.GetItemByFileReference(FileReference);
				if (FileItem.Exists)
				{
					return FileItem.Location.FullName.Replace('\\', '/');
				}
			}

			bool Nested = Visited != null;

			if (!Nested)
			{
				FileReference FileReference = FileReference.Combine(new FileReference(From).Directory, Include);
				FileItem FileItem = FileItem.GetItemByFileReference(FileReference);
				if (FileItem.Exists)
				{
					return FileItem.Location.FullName.Replace('\\', '/');
				}
			}

			foreach (string? PublicModule in Module.Rules.PublicDependencyModuleNames.Union(Module.Rules.PublicIncludePathModuleNames).Union(Module.Rules.PrivateIncludePathModuleNames).Union(Module.Rules.PrivateDependencyModuleNames))
			{
				UEBuildModule? DependencyModule;
				if (NameToModule.TryGetValue(PublicModule, out DependencyModule))
				{
					if (Visited == null)
					{
						Visited = new();
						Visited.Add(Module);
						Visited.Add(DependencyModule);
					}
					else if (!Visited.Add(DependencyModule))
					{
						continue;
					}

					string? Str = GetFullName(Include, From, DependencyModule, NameToModule, Visited);
					if (Str != null)
					{
						return Str;
					}
				}
			}

			if (!Nested)
			{
				// This should only show system includes..
				//Console.WriteLine($"Can't resolve {Include} from module {Module.Name}");
			}

			return null;
		}

		static void ParseSkippedHeaders(Dictionary<string, IWYUInfo> SkippedHeaders, Dictionary<string, IWYUInfo> Infos, Dictionary<string, UEBuildModule> PathToModule, Dictionary<string, UEBuildModule> NameToModule)
		{
			foreach (KeyValuePair<string, IWYUInfo> Kvp in SkippedHeaders)
			{
				Infos.Add(Kvp.Key, Kvp.Value);
			}
			while (true)
			{
				Dictionary<string, IWYUInfo> NewSkippedHeaders = new();
				Parallel2.ForEach(SkippedHeaders.Values, Info =>
				{
					Info.Module = GetModule(Info.File, PathToModule);

					bool HasIncludeGuard = false;
					int IfCount = 0;
					int SkipDepth = 0; // > 0 means we're inside a known-false #if block
					string[] Lines = File.ReadAllLines(Info.File);
					foreach (string Line in Lines)
					{
						ReadOnlySpan<char> LineSpan = Line.AsSpan().TrimStart();
						if (LineSpan.Length == 0 || LineSpan[0] != '#')
						{
							continue;
						}
						LineSpan = LineSpan.Slice(1).TrimStart();
						if (LineSpan.StartsWith("if"))
						{
							// Include guards are special
							if (!HasIncludeGuard)
							{
								if (LineSpan.StartsWith("ifndef ") && LineSpan.EndsWith("_H"))
								{
									HasIncludeGuard = true;
									continue;
								}
							}

							if (SkipDepth > 0)
							{
								// Already skipping — just track nesting depth
								SkipDepth++;
							}
							else
							{
								// Check if this is a known-false condition
								ReadOnlySpan<char> Condition = LineSpan.StartsWith("if ") ? LineSpan.Slice(3).TrimStart()
									: LineSpan.StartsWith("ifdef ") ? LineSpan.Slice(6).TrimStart()
									: ReadOnlySpan<char>.Empty;

								if (Condition.Length > 0 &&
									(Condition.SequenceEqual("0") ||
									 Condition.StartsWith("UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_")))
								{
									SkipDepth = 1;
								}
							}
							++IfCount;
						}
						else if (LineSpan.StartsWith("endif"))
						{
							if (SkipDepth > 0)
							{
								SkipDepth--;
							}
							--IfCount;
						}
						else if (LineSpan.StartsWith("include"))
						{
							if (SkipDepth > 0)
							{
								continue; // Inside a known-false #if block — skip this include
							}
							ReadOnlySpan<char> IncludeSpan = LineSpan.Slice("include".Length).TrimStart();
							char LeadingIncludeChar = IncludeSpan[0];
							if (LeadingIncludeChar == '"' || LeadingIncludeChar == '<')
							{
								ReadOnlySpan<char> Start = IncludeSpan.Slice(1);
								int EndIndex = Start.IndexOf(LeadingIncludeChar == '"' ? '"' : '>');
								ReadOnlySpan<char> FileSpan = Start.Slice(0, EndIndex);
								string Include = FileSpan.ToString();
								string? File = GetFullName(Include, Info.File, Info.Module, NameToModule);
								if (File != null)
								{
									IWYUInfo? IncludeInfo;
									lock (Infos)
									{
										if (!Infos.TryGetValue(File, out IncludeInfo))
										{
											if (!SkippedHeaders.TryGetValue(File, out IncludeInfo))
											{
												IncludeInfo = new();
												IncludeInfo.File = File;
												NewSkippedHeaders.Add(File, IncludeInfo);
												Infos.Add(File, IncludeInfo);
											}
										}
									}
									if (IncludeInfo != null)
									{
										IWYUIncludeEntry Entry = new();
										Entry.Full = File;
										Entry.Printable = Include;
										Entry.System = LeadingIncludeChar == '<';
										Entry.Resolved = IncludeInfo;
										Info.IncludesSeenInFile.Add(Entry);
										Info.Includes.Add(Entry);
									}
								}
							}
						}
					}
				});
				if (NewSkippedHeaders.Count == 0)
				{
					return;
				}
				SkippedHeaders = NewSkippedHeaders;
			}
		}

		static bool IsValidForUpdate(IWYUInfo Info, HashSet<string> ValidPaths, bool ObeyModuleRules)
		{
			if (Info.Source == null) // .generated.h is also in this list, ignore them
			{
				return false;
			}

			if (ObeyModuleRules && Info.Module?.Rules.IWYUSupport != IWYUSupport.Full)
			{
				return false;
			}

			// There are some codegen files with this name
			if (Info.File.Contains(".gen.h", StringComparison.Ordinal))
			{
				return false;
			}

			// Skip ThirdParty files if there is a thirdparty sub folder inside a module
			if (Info.File.Contains("/ThirdParty/", StringComparison.Ordinal))
			{
				return false;
			}

			// Filter out files
			foreach (string ValidPath in ValidPaths)
			{
				if (Info.File.Contains(ValidPath, StringComparison.Ordinal))
				{
					return true;
				}
			}
			return false;
		}

		int UpdateFiles(Dictionary<string, IWYUInfo> Infos, HashSet<string> ValidPaths, ILogger Logger)
		{
			SortedDictionary<string, List<string>>? sortedLog = null;
			if (!bWrite)
			{
				sortedLog = [];
			}

			List<ValueTuple<IWYUInfo, List<string>>> UpdatedFiles = new(); // <FilePath>, <NewLines>
			HashSet<IWYUFile> SkippedFiles = new();
			int SkippedCount = 0;
			int OutOfDateCount = 0;

			Logger.LogInformation($"Updating code files (in memory)...");

			uint FilesParseCount = 0;
			int ProcessSuccess = 1;

			void UpdateFile(IWYUInfo Info)
			{
				if (!IsValidForUpdate(Info, ValidPaths, true))
				{
					return;
				}

				bool IsCpp = Info.IsCpp;
				bool IsPrivate = IsCpp || Info.File.Contains("/Private/", StringComparison.Ordinal);
				bool IsInternal = Info.File.Contains("/Internal/", StringComparison.Ordinal);

				// If we only want to update private files we early out for non-private headers
				if (!IsPrivate && (bUpdateOnlyPrivate || (Info.Module?.Rules.IWYUSupport == IWYUSupport.KeepPublicAsIsForNow)))
				{
					return;
				}

				Interlocked.Increment(ref FilesParseCount);

				string MatchingH = "";
				if (IsCpp)
				{
					int LastSlash = Info.File.LastIndexOf('/') + 1;
					MatchingH = Info.File.Substring(LastSlash, Info.File.Length - LastSlash - 4) + ".h";
				}

				Dictionary<string, string> TransitivelyIncluded = [];
				SortedSet<string> CleanedupIncludes = [];
				SortedSet<string> ForwardDeclarationsToAdd = [];

				foreach (IWYUIncludeEntry Include in Info.Includes)
				{
					// We never remove header with name matching cpp
					string NameWithoutPath = Include.Printable;
					int LastSlash = NameWithoutPath.LastIndexOf('/');
					if (LastSlash != -1)
					{
						NameWithoutPath = NameWithoutPath.Substring(LastSlash + 1);
					}

					if (IsCpp && Include.Full.EndsWith(".generated.h", StringComparison.Ordinal))
					{
						continue;
					}

					string QuotedPrintable = Include.System ? $"<{Include.Printable}>" : $"\"{Include.Printable}\"";

					bool Keep = true;
					if (Info.File == Include.Full) // Sometimes IWYU outputs include to the same file if .gen.cpp is inlined and includes file with slightly different path. just skip those
					{
						Keep = false;
					}
					else if (IsCpp && MatchingH == NameWithoutPath)
					{
						Keep = true;
					}
					else if (!bNoTransitiveIncludes)
					{
						foreach (IWYUIncludeEntry Include2 in Info.Includes)
						{
							if (Include2.Resolved != null && Include != Include2)
							{
								string Key = Include.Full;
								string? TransitivePath;
								if (Include2.Resolved.TransitiveIncludes!.TryGetValue(Key, out TransitivePath))
								{
									if (sortedLog != null)
									{
										TransitivelyIncluded.TryAdd(QuotedPrintable, String.Join(" -> ", Include2.Printable, TransitivePath));
									}

									Keep = false;
									break;
								}
							}
						}
					}
					if (Keep)
					{
						CleanedupIncludes.Add(QuotedPrintable);
					}
				}

				// We don't remove seen includes that are redundant because they are included through other includes that are needed.
				if (!bRemoveRedundantIncludes)
				{
					List<string> ToReadd = new();
					foreach (IWYUIncludeEntry Seen in Info.IncludesSeenInFile)
					{
						string QuotedPrintable = Seen.System ? $"<{Seen.Printable}>" : $"\"{Seen.Printable}\"";
						if (!CleanedupIncludes.Contains(QuotedPrintable) && Info.TransitiveIncludes.ContainsKey(Seen.Full))
						{
							CleanedupIncludes.Add(QuotedPrintable);
						}
					}
				}

				// Ignore forward declarations for cpp files. They are very rarely needed and we let the user add them manually instead
				if (!IsCpp)
				{
					foreach (IWYUForwardEntry ForwardDeclaration in Info.ForwardDeclarations)
					{
						bool Add = ForwardDeclaration.Present == false;
						if (!bNoTransitiveIncludes)
						{
							foreach (IWYUIncludeEntry Include2 in Info.Includes)
							{
								if (Include2.Resolved != null && Include2.Resolved.TransitiveForwardDeclarations!.ContainsKey(ForwardDeclaration.Printable))
								{
									Add = false;
									break;
								}
							}
						}
						if (Add)
						{
							ForwardDeclarationsToAdd.Add(ForwardDeclaration.Printable);
						}
					}
				}

				// Read all lines of the header/source file
				string[] ExistingLines = File.ReadAllLines(Info.File);

				if (ExistingLines.Length > 1 && ExistingLines[0].Contains("automatically generated", StringComparison.Ordinal))
				{
					return;
				}

				SortedDictionary<string, string> LinesToRemove = new();
				SortedSet<string> IncludesToAdd = new(CleanedupIncludes);

				bool HasIncludes = false;
				string? FirstForwardDeclareLine = null;

				HashSet<string> SeenIncludes = new();
				foreach (IWYUIncludeEntry SeenInclude in Info.IncludesSeenInFile)
				{
					if (SeenInclude.Printable.EndsWith(".ipp", StringComparison.Ordinal)) // Abuse this to make sure we never remove .ipp includes
					{
						IncludesToAdd.Remove($"\"{SeenInclude.Printable}\"");
						continue;
					}

					if (SeenInclude.System)
					{
						SeenIncludes.Add($"<{SeenInclude.Printable}>");
					}
					else
					{
						SeenIncludes.Add($"\"{SeenInclude.Printable}\"");
					}
				}

				bool ForceKeepScope = false;
				bool ErrorOnMoreIncludes = false;
				int LineIndex = -1;

				// This makes sure that we have at least HAL/Platform.h included if the file contains XXX_API
				bool Contains_API = false;
				bool LookFor_API = false;
				if (Info.Includes.Count == 0)
				{
					foreach (IWYUMissingInclude Missing in Info.MissingIncludes)
					{
						LookFor_API = LookFor_API || Missing.Full.Contains("Definitions.h", StringComparison.Ordinal);
					}
				}

				int includeCounter = 0;
				// Traverse all lines in file and figure out which includes that should be added or removed
				foreach (string Line in ExistingLines)
				{
					++LineIndex;

					ReadOnlySpan<char> LineSpan = Line.AsSpan().Trim();
					bool StartsWithHash = LineSpan.Length > 0 && LineSpan[0] == '#';
					if (StartsWithHash)
					{
						LineSpan = LineSpan.Slice(1).TrimStart();
					}

					if (!StartsWithHash || !LineSpan.StartsWith("include"))
					{
						// Might be forward declaration.. 
						if (ForwardDeclarationsToAdd.Remove(Line)) // Skip adding the ones that already exists
						{
							FirstForwardDeclareLine ??= Line;
						}

						if (Line.Contains("IWYU pragma: ", StringComparison.Ordinal))
						{
							if (Line.Contains(": begin_keep", StringComparison.Ordinal))
							{
								ForceKeepScope = true;
							}
							else if (Line.Contains(": end_keep", StringComparison.Ordinal))
							{
								ForceKeepScope = false;
							}
						}

						// File is autogenerated by some tool, don't mess with it
						if (Line.Contains("AUTO GENERATED CONTENT, DO NOT MODIFY", StringComparison.Ordinal))
						{
							return;
						}

						if (LookFor_API && !Contains_API && Line.Contains("_API", StringComparison.Ordinal))
						{
							Contains_API = true;
						}

						continue;
					}

					if (ErrorOnMoreIncludes)
					{
						Interlocked.Exchange(ref ProcessSuccess, 0);
						Logger.LogError("{IWYUFilePath} - Found special include using macro and did not expect more includes in this file", Info.File);
						return;
					}

					++includeCounter;

					HasIncludes = true;

					bool ForceKeep = false;

					ReadOnlySpan<char> IncludeSpan = LineSpan.Slice("include".Length).TrimStart();
					char LeadingIncludeChar = IncludeSpan[0];
					if (LeadingIncludeChar != '"' && LeadingIncludeChar != '<')
					{
						if (IncludeSpan.IndexOf("UE_INLINE_GENERATED_CPP_BY_NAME") != -1)
						{
							int Open = IncludeSpan.IndexOf('(') + 1;
							int Close = IncludeSpan.IndexOf(')');
							string ActualInclude = $"\"{IncludeSpan.Slice(Open, Close - Open).ToString()}.gen.cpp\"";
							IncludesToAdd.Remove(ActualInclude);
						}
						else if (IncludeSpan.IndexOf("COMPILED_PLATFORM_HEADER") != -1)
						{
							int Open = IncludeSpan.IndexOf('(') + 1;
							int Close = IncludeSpan.IndexOf(')');
							string ActualInclude = $"\"Windows/Windows{IncludeSpan.Slice(Open, Close - Open).ToString()}\"";
							IncludesToAdd.Remove(ActualInclude);
						}
						else
						{
							// TODO: These are includes made through defines. IWYU should probably report these in their original shape
							// .. so a #include MY_SPECIAL_INCLUDE is actually reported as MY_SPECIAL_INCLUDE from IWYU instead of what the define expands to
							// For now, let's assume that if there is one line in the file that is an include 
							if (IncludesToAdd.Count == 1)
							{
								//IncludesToAdd.Clear();
								//ErrorOnMoreIncludes = true;
							}
						}
						continue;
					}
					else
					{
						int Index = IncludeSpan.Slice(1).IndexOf(LeadingIncludeChar == '"' ? '"' : '>');
						if (Index != -1)
						{
							IncludeSpan = IncludeSpan.Slice(0, Index + 2);
						}

						if (Line.Contains("IWYU pragma: ", StringComparison.Ordinal))
						{
							ForceKeep = true;
						}
					}

					string Include = IncludeSpan.ToString();

					// If Include is not seen it means that it is probably inside a #if/#endif with condition false. These includes we can't touch
					if (!SeenIncludes.Contains(Include))
					{
						continue;
					}

					if (!ForceKeep && !ForceKeepScope && !CleanedupIncludes.Contains(Include))
					{
						LinesToRemove.TryAdd(Line, Include);
					}
					else
					{
						if (!IncludesToAdd.Remove(Include))
						{
							// Include can be relative path of cpp file
							if (Info.IsCpp && includeCounter == 1) // First include can have a different path
							{
							}
						}
					}
				}

				if (Contains_API && LookFor_API)
				{
					//IncludesToAdd.Add(new IWYUIncludeEntry() { Full = PlatformInfo!.File, Printable = "HAL/Platform.h", Resolved = PlatformInfo });
					if (!LinesToRemove.Remove("#include \"HAL/Platform.h\""))
					{
						IncludesToAdd.Add("\"HAL/Platform.h\"");
					}
				}

				// Nothing has changed! early out of this file
				if (IncludesToAdd.Count == 0 && LinesToRemove.Count == 0 && ForwardDeclarationsToAdd.Count == 0)
				{
					return;
				}

				// If code file last write time is newer than IWYU file this means that iwyu is not up to date and needs to compile before we can apply anything
				if (!bIgnoreUpToDateCheck)
				{
					FileInfo IwyuFileInfo = new FileInfo(Info.Source!.Name!);
					FileInfo CodeFileInfo = new FileInfo(Info.File!);
					if (CodeFileInfo.LastWriteTime > IwyuFileInfo.LastWriteTime)
					{
						Interlocked.Increment(ref OutOfDateCount);
						return;
					}
				}

				SortedSet<string> LinesToAdd = new();

				foreach (string IncludeToAdd in IncludesToAdd)
				{
					LinesToAdd.Add("#include " + IncludeToAdd);
				}

				if (sortedLog != null)
				{
					List<string> LogLines = new();
					{
						foreach (string I in LinesToAdd)
						{
							LogLines.Add("  +" + I);
						}
						foreach (KeyValuePair<string, string> Pair in LinesToRemove)
						{
							string Line = "  -" + Pair.Key;
							string? Reason;
							if (TransitivelyIncluded.TryGetValue(Pair.Value, out Reason))
							{
								Line += "  (Transitively included from " + Reason + ")";
							}
							LogLines.Add(Line);
						}
						foreach (string I in ForwardDeclarationsToAdd)
						{
							LogLines.Add("  +" + I);
						}
					}

					lock (sortedLog)
					{
						sortedLog.Add(Info.File, LogLines);
					}
				}

				List<string> NewLines = new(ExistingLines.Length);
				SortedSet<string> LinesRemoved = new();

				if (!HasIncludes)
				{
					LineIndex = 0;
					foreach (string OldLine in ExistingLines)
					{
						NewLines.Add(OldLine);
						++LineIndex;

						if (!OldLine.TrimStart().StartsWith("#pragma", StringComparison.Ordinal))
						{
							continue;
						}

						NewLines.Add("");
						foreach (string Line in LinesToAdd)
						{
							NewLines.Add(Line);
						}

						NewLines.AddRange(ExistingLines.Skip(LineIndex));
						break;
					}
				}
				else
				{
					// This is a bit of a tricky problem to solve in a generic ways since there are lots of exceptions and hard to make assumptions
					// Right now we make the assumption that if there are 
					// That will be the place where we add/remove our includes.

					int ContiguousNonIncludeLineCount = 0;
					bool IsInFirstIncludeBlock = true;
					int LastSeenIncludeBeforeCode = -1;
					int FirstForwardDeclareLineIndex = -1;

					foreach (string OldLine in ExistingLines)
					{
						ReadOnlySpan<char> OldLineSpan = OldLine.AsSpan().TrimStart();
						bool StartsWithHash = OldLineSpan.Length > 0 && OldLineSpan[0] == '#';
						if (StartsWithHash)
						{
							OldLineSpan = OldLineSpan.Slice(1).TrimStart();
						}

						bool IsInclude = StartsWithHash && OldLineSpan.StartsWith("include");

						string OldLineTrimmedStart = OldLine.TrimStart();

						if (!IsInclude)
						{
							if (IsInFirstIncludeBlock)
							{
								if (!String.IsNullOrEmpty(OldLineTrimmedStart))
								{
									++ContiguousNonIncludeLineCount;
								}
								if (LastSeenIncludeBeforeCode != -1 && ContiguousNonIncludeLineCount > 10)
								{
									IsInFirstIncludeBlock = false;
								}
								if (OldLineTrimmedStart.StartsWith("#if", StringComparison.Ordinal))
								{
									// This logic is a bit shaky but handle the situations where file starts with #if and ends with #endif
									if (LastSeenIncludeBeforeCode != -1)
									{
										IsInFirstIncludeBlock = false;
									}
								}

								// This need to be inside "IsInFirstIncludeBlock" check because some files have forward declares far down in the file
								if (FirstForwardDeclareLine == OldLine)
								{
									FirstForwardDeclareLineIndex = NewLines.Count;
								}
							}

							NewLines.Add(OldLine);
							continue;
						}

						ContiguousNonIncludeLineCount = 0;

						if (!LinesToRemove.ContainsKey(OldLine))
						{
							// If we find #include SOME_DEFINE we assume that should be last and "end" the include block with that
							if (!OldLineTrimmedStart.Contains('\"', StringComparison.Ordinal) && !OldLineTrimmedStart.Contains('<', StringComparison.Ordinal))
							{
								IsInFirstIncludeBlock = false;
							}
							else if (LinesToAdd.Count != 0 && (!IsCpp || LastSeenIncludeBeforeCode != -1))
							{
								string LineToAdd = LinesToAdd.First();
								if (String.Compare(LineToAdd, OldLine, StringComparison.Ordinal) < 0)
								{
									NewLines.Add(LineToAdd);
									LinesToAdd.Remove(LineToAdd);
								}
							}

							NewLines.Add(OldLine);
						}
						else
						{
							FirstForwardDeclareLineIndex = -1; // This should never happen, but just in case, reset since lines have changed
							LinesRemoved.Add(OldLine);
						}

						if (IsInFirstIncludeBlock)
						{
							LastSeenIncludeBeforeCode = NewLines.Count - 1;
						}
					}

					if (LinesToAdd.Count > 0)
					{
						if (LastSeenIncludeBeforeCode == -1)
						{
							throw new IWYUUnsupportedFormattingException($"Code structure to complicated for IWYU to update {Info.File}. Definitions before includes?");
						}
						int InsertPos = LastSeenIncludeBeforeCode + 1;
						if (NewLines[LastSeenIncludeBeforeCode].Contains(".generated.h", StringComparison.Ordinal) || NewLines[LastSeenIncludeBeforeCode].Contains(".gen.h", StringComparison.Ordinal))
						{
							--InsertPos;
						}

						NewLines.InsertRange(InsertPos, LinesToAdd);
						LastSeenIncludeBeforeCode += LinesToAdd.Count;

						if (FirstForwardDeclareLineIndex != -1 && FirstForwardDeclareLineIndex > LastSeenIncludeBeforeCode)
						{
							FirstForwardDeclareLineIndex += LinesToAdd.Count;
						}
					}

					if (ForwardDeclarationsToAdd.Count > 0)
					{
						// TODO: Need to make sure we are not in #if/#endif

						int InsertPos;
						if (FirstForwardDeclareLineIndex == -1)
						{
							InsertPos = LastSeenIncludeBeforeCode + 1;
							if (!String.IsNullOrEmpty(NewLines[InsertPos]))
							{
								NewLines.Insert(InsertPos++, "");
							}
							NewLines.Insert(InsertPos + 1, "");
						}
						else
						{
							InsertPos = FirstForwardDeclareLineIndex;
						}
						NewLines.InsertRange(InsertPos + 1, ForwardDeclarationsToAdd);
					}
				}

				// If file is public, in engine and we have a deprecation tag set we will 
				// add a deprecated include scope at the end of the file (unless scope already exists, then we'll add it inside that)
				string EngineDir = Unreal.EngineDirectory.FullName.Replace('\\', '/');
				if (!(IsPrivate || IsInternal) && Info.File.StartsWith(EngineDir, StringComparison.Ordinal) && !String.IsNullOrEmpty(HeaderDeprecationTag))
				{
					Dictionary<string, string> PrintableToFull = new();
					foreach (IWYUIncludeEntry Seen in Info.IncludesSeenInFile)
					{
						PrintableToFull.TryAdd(Seen.Printable, Seen.Full);
					}

					// Remove the includes in LinesRemoved
					LinesRemoved.RemoveWhere(Line =>
					{
						ReadOnlySpan<char> IncludeSpan = Line.AsSpan(8).TrimStart();
						char LeadingIncludeChar = IncludeSpan[0];
						if (LeadingIncludeChar != '"' && LeadingIncludeChar != '<')
						{
							return false;
						}
						int Index = IncludeSpan.Slice(1).IndexOf(LeadingIncludeChar == '"' ? '"' : '>');
						if (Index == -1)
						{
							return false;
						}
						IncludeSpan = IncludeSpan.Slice(1, Index);
						string? Full;
						if (!PrintableToFull.TryGetValue(IncludeSpan.ToString(), out Full))
						{
							return false;
						}

						return Info.TransitiveIncludes.ContainsKey(Full);
					});

					if (LinesRemoved.Count > 0)
					{
						int IndexOfDeprecateScope = -1;
						string Match = "#if " + HeaderDeprecationTag;
						for (int I = NewLines.Count - 1; I != 0; --I)
						{
							if (NewLines[I] == Match)
							{
								IndexOfDeprecateScope = I + 1;
								break;
							}
						}

						if (IndexOfDeprecateScope == -1)
						{
							NewLines.Add("");
							NewLines.Add(Match);
							IndexOfDeprecateScope = NewLines.Count;
							NewLines.Add("#endif");
						}
						else
						{
							// Scan the already added includes to prevent additional adds.
							int EndIndex = IndexOfDeprecateScope;
							while (EndIndex < NewLines.Count && NewLines[EndIndex] != "#endif")
							{
								LinesRemoved.Remove(NewLines[EndIndex]);
								EndIndex++;
							}
						}

						NewLines.InsertRange(IndexOfDeprecateScope, LinesRemoved);
					}
				}
				lock (UpdatedFiles)
				{
					UpdatedFiles.Add(new(Info, NewLines));
				}
			}

			Parallel2.ForEach(Infos.Values, Info =>
			{
				try
				{
					UpdateFile(Info);
				}
				catch (IWYUUnsupportedFormattingException ex)
				{
					Logger.LogError(ex, "{Message}", ex.Message);
					Interlocked.Exchange(ref ProcessSuccess, 0);
				}
				catch (Exception ex)
				{
					Logger.LogError(ex, "Exception while processing {File} {Exception}", Info.File, ex.ToString());
					Interlocked.Exchange(ref ProcessSuccess, 0);
				}
			});

			// Something went wrong processing code files.
			if (ProcessSuccess == 0)
			{
				return -1;
			}

			Logger.LogInformation(
				"Parsed {FilesParseCount} and updated {UpdatedFilesCount} files (Found {OutOfDateCount} .iwyu files out of date)",
				FilesParseCount,
				UpdatedFiles.Count,
				OutOfDateCount);

			// Wooohoo, all files are up-to-date
			if (UpdatedFiles.Count == 0)
			{
				Logger.LogInformation($"All files are up to date!");
				return 0;
			}

			// If we have been logging we can exit now since we don't want to write any files to disk
			if (sortedLog != null)
			{
				foreach (KeyValuePair<string, List<string>> kv in sortedLog)
				{
					Console.WriteLine(kv.Key);
					foreach (string line in kv.Value)
					{
						Console.WriteLine(line);
					}

					Console.WriteLine();
				}
				return 0;
			}

			List<Process> P4Processes = new();

			void AddP4Process(string Arguments)
			{
				Process Process = new Process();
				Process.StartInfo.WindowStyle = ProcessWindowStyle.Hidden;
				Process.StartInfo.CreateNoWindow = true;
				Process.StartInfo.FileName = "p4.exe";
				Process.StartInfo.Arguments = Arguments;
				Process.Start();
				P4Processes.Add(Process);
			}

			bool WaitForP4()
			{
				bool P4Success = true;
				foreach (Process P4Process in P4Processes)
				{
					P4Process.WaitForExit();
					if (P4Process.ExitCode != 0)
					{
						P4Success = false;
						Logger.LogError("p4 edit failed - {P4Arguments}", P4Process.StartInfo.Arguments);
					}
					P4Process.Close();
				}
				P4Processes.Clear();
				return P4Success;
			}

			if (!bNoP4)
			{
				List<IWYUInfo> ReadOnlyFileInfos = new();
				foreach ((IWYUInfo Info, List<string> NewLines) in UpdatedFiles)
				{
					if (new FileInfo(Info.File).IsReadOnly)
					{
						ReadOnlyFileInfos.Add(Info);
					}
				}

				if (ReadOnlyFileInfos.Count > 0)
				{
					// Check out files in batches. This can go quite crazy if there are lots of files.
					// Should probably revisit this code to prevent 100s of p4 processes to start at once
					Logger.LogInformation("Opening {ReadOnlyCount} files for edit in P4... ({SkippedCount} files skipped)", ReadOnlyFileInfos.Count, SkippedCount);
					int ShowCount = 8;
					foreach (IWYUInfo Info in ReadOnlyFileInfos)
					{
						Logger.LogInformation("   edit {IWYUFilePath}", Info.File);
						if (--ShowCount == 0)
						{
							break;
						}
					}
					if (ReadOnlyFileInfos.Count > 5)
					{
						Logger.LogInformation("   ... and {RemainingReadOnlyCount} more.", ReadOnlyFileInfos.Count - 5);
					}

					StringBuilder P4Arguments = new();
					int BatchSize = 10;
					int BatchCount = 0;
					int Index = 0;
					foreach (IWYUInfo Info in ReadOnlyFileInfos)
					{
						if (!SkippedFiles.Contains(Info.Source!))
						{
							P4Arguments.Append(" \"").Append(Info.File).Append('\"');
							++BatchCount;
						}
						++Index;
						if (BatchCount == BatchSize || Index == ReadOnlyFileInfos.Count)
						{
							AddP4Process($"edit{P4Arguments}");
							P4Arguments.Clear();
							BatchCount = 0;
						}
					}

					// Waiting for edit
					if (!WaitForP4())
					{
						return -1;
					}
				}
			}

			bool WriteSuccess = true;
			Logger.LogInformation("Writing {WriteCount} files to disk...", UpdatedFiles.Count - SkippedCount);

			foreach ((IWYUInfo Info, List<string> NewLines) in UpdatedFiles)
			{
				if (SkippedFiles.Contains(Info.Source!))
				{
					continue;
				}

				try
				{
					File.WriteAllLines(Info.File, NewLines);
				}
				catch (Exception e)
				{
					Logger.LogError("Failed to write {IWYUFilePath}: {Message} - File will be reverted", Info.File, e.Message);
					SkippedFiles.Add(Info.Source!); // In case other entries from same file is queued

					AddP4Process($"revert {String.Join(' ', Info.Source!.Files.Select(f => f.File))}");
					WriteSuccess = false;
				}
			}

			// Waiting for reverts
			if (!WaitForP4())
			{
				return -1;
			}

			if (!WriteSuccess)
			{
				return -1;
			}

			Logger.LogInformation($"Done!");
			return 0;
		}

		/// <summary>
		/// Calculate all indirect transitive includes for a file. This list contains does not contain itself and will handle circular dependencies
		/// </summary>
		static void CalculateTransitive(IWYUInfo Root, IWYUInfo Info, StringBuilder Stack, Dictionary<string, string> TransitiveIncludes, Dictionary<string, string> TransitiveForwardDeclarations, bool UseSeenIncludes)
		{
			List<IWYUIncludeEntry> Includes = UseSeenIncludes ? Info.IncludesSeenInFile : Info.Includes;
			foreach (IWYUIncludeEntry Include in Includes)
			{
				string Key = Include.Full;

				if (TransitiveIncludes.ContainsKey(Key) || Include.Resolved == Root)
				{
					continue;
				}

				int length = Stack.Length;
				if (length > 0)
				{
					Stack.Append(" <- ");
				}
				Stack.Append(Include.Printable);
				TransitiveIncludes.Add(Key, Stack.ToString());
				if (Include.Resolved != null)
				{
					CalculateTransitive(Root, Include.Resolved, Stack, TransitiveIncludes, TransitiveForwardDeclarations, UseSeenIncludes);
				}
				Stack.Length = length;
			}

			foreach (IWYUForwardEntry ForwardDeclaration in Info.ForwardDeclarations)
			{
				string Key = ForwardDeclaration.Printable;
				TransitiveForwardDeclarations.TryAdd(Key, Info.File);
			}
		}

		private static int CompareFiles(Dictionary<string, IWYUInfo> Infos, HashSet<string> ValidPaths, Dictionary<string, UEBuildModule> PathToModule, Dictionary<string, UEBuildModule> NameToModule, ILogger Logger)
		{
			Logger.LogInformation($"Comparing code files...");

			Dictionary<string, IWYUInfo> SkippedHeaders = new();

			Logger.LogInformation($"Parsing seen headers not supporting being compiled...");
			Parallel2.ForEach(Infos.Values, Info =>
			{
				foreach (IWYUIncludeEntry Include in Info.IncludesSeenInFile)
				{
					if (Include.Resolved == null)
					{
						if (!Infos.TryGetValue(Include.Full, out Include.Resolved))
						{
							lock (SkippedHeaders)
							{
								if (!SkippedHeaders.TryGetValue(Include.Full, out Include.Resolved))
								{
									IWYUInfo NewInfo = new();
									Include.Resolved = NewInfo;
									NewInfo.File = Include.Full;
									SkippedHeaders.Add(Include.Full, NewInfo);
								}
							}
						}
					}
				}
			});
			ParseSkippedHeaders(SkippedHeaders, Infos, PathToModule, NameToModule);

			//List<ValueTuple<float, IWYUInfo>> RatioList = new();

			ConcurrentDictionary<string, long> AllSeenIncludes = new();

			List<ValueTuple<IWYUInfo, Dictionary<string, string>>> InfosToCompare = new();

			Logger.LogInformation($"Generating transitive seen include lists...");
			Parallel2.ForEach(Infos.Values, Info =>
			{
				if (!IsValidForUpdate(Info, ValidPaths, false) || Info.File.EndsWith(".h", StringComparison.Ordinal))
				{
					return;
				}

				Dictionary<string, string> SeenTransitiveIncludes = new();
				StringBuilder stack = new();
				CalculateTransitive(Info, Info, stack, SeenTransitiveIncludes, new Dictionary<string, string>(), true);

				lock (InfosToCompare)
				{
					InfosToCompare.Add(new(Info, SeenTransitiveIncludes));
				}

				foreach (KeyValuePair<string, string> Include in SeenTransitiveIncludes.Union(Info.TransitiveIncludes))
				{
					AllSeenIncludes.TryAdd(Include.Key, 0);
				}
			});

			Dictionary<string, long> FileToSize = new Dictionary<string, long>(AllSeenIncludes);

			Logger.LogInformation($"Reading file sizes...");
			Parallel2.ForEach(AllSeenIncludes, Info =>
			{
				FileToSize[Info.Key] = (new FileInfo(Info.Key)).Length;
			});

			Logger.LogInformation($"Calculate total amount of bytes included per file...");
			List<ValueTuple<long, IWYUInfo>> ByteSizeDiffList = new();
			Parallel2.ForEach(InfosToCompare, Kvp =>
			{
				long OptimizedSize = 0;
				long SeenSize = 0;
				foreach (string Include in Kvp.Item1.TransitiveIncludes.Keys)
				{
					OptimizedSize += FileToSize[Include];
				}

				foreach (string? Include in Kvp.Item2.Keys)
				{
					SeenSize += FileToSize[Include];
				}

				long Diff = SeenSize - OptimizedSize;
				lock (ByteSizeDiffList)
				{
					ByteSizeDiffList.Add(new(Diff, Kvp.Item1));
				}
			});

			ByteSizeDiffList.Sort((a, b) =>
			{
				if (a.Item1 == b.Item1)
				{
					return 0;
				}

				if (a.Item1 < b.Item1)
				{
					return 1;
				}

				return -1;
			});

			string PrettySize(long Size)
			{
				if (Size > 1024 * 1024)
				{
					return (((float)Size) / (1024 * 1024)).ToString("0.00") + "mb";
				}

				if (Size > 1024)
				{
					return (((float)Size) / (1024)).ToString("0.00") + "kb";
				}

				return Size + "b";
			}

			int PrintCount = 20;

			Logger.LogInformation("");
			Logger.LogInformation("Top {PrintCount} most reduction in bytes parsed by compiler frontend", PrintCount);
			Logger.LogInformation("----------------------------------------------------------");
			int End = Math.Min(20, ByteSizeDiffList.Count);

			int LongestName = 0;
			for (int I = 0; I != End; ++I)
			{
				IWYUInfo File = ByteSizeDiffList[I].Item2;
				LongestName = Math.Max(LongestName, Path.GetFileName(File.File).Length);
			}

			for (int I = 0; I != End; ++I)
			{
				long Saved = ByteSizeDiffList[I].Item1;
				IWYUInfo File = ByteSizeDiffList[I].Item2;
				long OptimizedSize = 0;
				foreach (string Include in File.TransitiveIncludes.Keys)
				{
					OptimizedSize += FileToSize[Include];
				}

				float Percent = 100.0f - (((float)OptimizedSize) / (OptimizedSize + Saved) * 100.0f);
				string Name = Path.GetFileName(File.File);
				Logger.LogInformation(
					"{PaddedIWYUFilePath}   {OriginalSize} -> {OptimizedSize}  (Saved {SavedBytes} or {Percent:0.0}%)",
					Name.PadRight(LongestName),
					PrettySize(OptimizedSize + Saved).PadRight(8),
					PrettySize(OptimizedSize).PadRight(8),
					PrettySize(Saved).PadRight(7),
					Percent);
			}
			Logger.LogInformation("");

			DirectoryReference iwyuDir = DirectoryReference.Combine(Unreal.EngineDirectory, "Intermediate", "IWYU");
			Directory.CreateDirectory(iwyuDir.FullName);
			string resultFile = DirectoryReference.Combine(iwyuDir, "CompareToplist.txt").FullName;
			File.WriteAllLines(resultFile, ByteSizeDiffList.Take(200).Select(Item => Item.Item2.File));
			Logger.LogInformation("Sorted top list written to {ResultPath} (Can be used as input to -PathToUpdateFile=<file>)", resultFile);
			return 0;
		}

		static void ExportMetadata(string OutputPath, Dictionary<string, IWYUInfo> Infos, Dictionary<string, UEBuildModule> NameToModule, List<string> SystemPaths, TargetDescriptor Target, string DeprecationTag)
		{
			using FileStream Stream = new FileStream(OutputPath, FileMode.Create);
			using Utf8JsonWriter Writer = new Utf8JsonWriter(Stream, new JsonWriterOptions { Indented = false });

			Writer.WriteStartObject();

			// Header
			Writer.WriteNumber("Version", 1);
			Writer.WriteString("TargetName", Target.Name);
			Writer.WriteString("Platform", Target.Platform.ToString());
			Writer.WriteString("Configuration", Target.Configuration.ToString());
			Writer.WriteString("EngineDir", Unreal.EngineDirectory.FullName);
			Writer.WriteString("Timestamp", DateTime.UtcNow.ToString("o"));
			Writer.WriteString("DeprecationDefine", DeprecationTag);

			// Modules
			Writer.WriteStartObject("Modules");
			foreach (KeyValuePair<string, UEBuildModule> Kvp in NameToModule)
			{
				UEBuildModule Module = Kvp.Value;
				Writer.WriteStartObject(Kvp.Key);
				Writer.WriteString(nameof(Module.Name), Module.Name);
				Writer.WriteString("Directory", Module.ModuleDirectory.FullName);
				Writer.WriteString("IWYUSupport", Module.Rules.IWYUSupport.ToString());

				Writer.WriteStartArray("PublicDependencies");
				foreach (string Dep in Module.Rules.PublicDependencyModuleNames)
				{
					Writer.WriteStringValue(Dep);
				}
				Writer.WriteEndArray();

				Writer.WriteStartArray("PrivateDependencies");
				foreach (string Dep in Module.Rules.PrivateDependencyModuleNames)
				{
					Writer.WriteStringValue(Dep);
				}
				Writer.WriteEndArray();

				Writer.WriteEndObject();
			}
			Writer.WriteEndObject();

			// Files
			Writer.WriteStartObject("Files");
			foreach (KeyValuePair<string, IWYUInfo> Kvp in Infos)
			{
				IWYUInfo Info = Kvp.Value;
				Writer.WriteStartObject(Kvp.Key);
				Writer.WriteString("File", Info.File);
				Writer.WriteString("Module", Info.Module?.Name ?? "");
				Writer.WriteBoolean("IsCpp", Info.IsCpp);

				// Includes (IWYU recommended)
				Writer.WriteStartArray("Includes");
				foreach (IWYUIncludeEntry Include in Info.Includes)
				{
					Writer.WriteStartObject();
					Writer.WriteString("Full", Include.Full);
					Writer.WriteString("Printable", Include.Printable);
					Writer.WriteString("Reason", Include.Reason);
					Writer.WriteBoolean("System", Include.System);
					Writer.WriteBoolean("InsideDecl", Include.InsideDecl);
					Writer.WriteString("ResolvedModule", Include.Resolved?.Module?.Name ?? "");
					Writer.WriteEndObject();
				}
				Writer.WriteEndArray();

				// Includes seen in file (on disk)
				Writer.WriteStartArray("IncludesSeenInFile");
				foreach (IWYUIncludeEntry Seen in Info.IncludesSeenInFile)
				{
					Writer.WriteStartObject();
					Writer.WriteString("Full", Seen.Full);
					Writer.WriteString("Printable", Seen.Printable);
					Writer.WriteBoolean("System", Seen.System);
					Writer.WriteEndObject();
				}
				Writer.WriteEndArray();

				// Forward declarations
				Writer.WriteStartArray("ForwardDeclarations");
				foreach (IWYUForwardEntry Fwd in Info.ForwardDeclarations)
				{
					Writer.WriteStartObject();
					Writer.WriteString("Printable", Fwd.Printable);
					Writer.WriteBoolean("Present", Fwd.Present);
					Writer.WriteEndObject();
				}
				Writer.WriteEndArray();

				// Missing includes
				Writer.WriteStartArray("MissingIncludes");
				foreach (IWYUMissingInclude Missing in Info.MissingIncludes)
				{
					Writer.WriteStartObject();
					Writer.WriteString("Full", Missing.Full);
					Writer.WriteEndObject();
				}
				Writer.WriteEndArray();

				Writer.WriteEndObject();
			}
			Writer.WriteEndObject();

			// System paths
			Writer.WriteStartArray("SystemPaths");
			foreach (string Path in SystemPaths)
			{
				Writer.WriteStringValue(Path);
			}
			Writer.WriteEndArray();

			Writer.WriteEndObject();
		}
	}
}
