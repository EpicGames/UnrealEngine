// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text.RegularExpressions;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// Generate a clang compile_commands file for a target
	/// </summary>
	internal sealed class InlineGeneratedCpps : IToolMode<InlineGeneratedCpps>
	{
		public static string Name => "InlineGeneratedCpps";
		public static ToolModeOptions Options => ToolModeOptions.XmlConfig | ToolModeOptions.BuildPlatforms | ToolModeOptions.SingleInstance | ToolModeOptions.StartPrefetchingEngine | ToolModeOptions.ShowExecutionTime;

		/// <summary>
		/// Regex that matches #include statements.
		/// </summary>
		static readonly Regex IncludeRegex = new Regex("^[ \t]*#[ \t]*include[ \t]*[\"](?<HeaderFile>[^\"]*)[\"]", RegexOptions.Compiled | RegexOptions.Singleline | RegexOptions.ExplicitCapture);

		/// <summary>
		/// Regex that matches #include generated header statements.
		/// </summary>
		static readonly Regex IncludeGenHeaderRegex = new Regex("^[ \t]*#[ \t]*include[ \t]*[\"](?<HeaderFile>[^\"]*)\\.generated\\.h[\"]", RegexOptions.Compiled | RegexOptions.Multiline | RegexOptions.ExplicitCapture);

		/// <summary>
		/// Regex that matches inline macro.
		/// </summary>
		static readonly Regex InlineReflectionMarkupRegex = new Regex(@"UE_INLINE_GENERATED_CPP_BY_NAME\((.+)\)", RegexOptions.Compiled | RegexOptions.Multiline);

		static readonly Regex EnumRegex = new Regex(@"\s*UENUM.*(\r\n?|\n)\s*enum\s+\w+\s*$", RegexOptions.Compiled | RegexOptions.Multiline);
		static readonly Regex SameLineCommentRegex = new Regex(@"/\*.*\*/", RegexOptions.Compiled | RegexOptions.Singleline);
		static readonly Regex BeginningCommentRegex = new Regex(@"/\*.*", RegexOptions.Compiled | RegexOptions.Singleline);
		static readonly Regex EndingCommentRegex = new Regex(@".*\*/", RegexOptions.Compiled | RegexOptions.Singleline);
		static readonly Regex IfRegex = new Regex(@"^[ \t]*#[ \t]*if.*", RegexOptions.Compiled | RegexOptions.Singleline);
		static readonly Regex EndIfRegex = new Regex(@"^[ \t]*#[ \t]*endif.*", RegexOptions.Compiled | RegexOptions.Singleline);

		[CommandLine("-Filter=", Description = "Set of filters for files to include in the database. Relative to the root directory, or to the project file.")]
		public List<string> FilterRules = new List<string>();

		[CommandLine("-CheckoutWithP4", Description = "Flags that this task should use p4 to check out the file before updating it.")]
		public bool bCheckoutWithP4 = false;

		[CommandLine("-NoOutput", Description = "Flags that the updated files shouldn't be saved.")]
		public bool bNoOutput = false;

		/// <summary>
		/// Execute the command
		/// </summary>
		/// <param name="Arguments">Command line arguments</param>
		/// <returns>Exit code</returns>
		/// <param name="Logger"></param>
		public async Task<int> ExecuteAsync(CommandLineArguments Arguments, ILogger Logger)
		{
			Arguments.ApplyTo(this);

			// Create the build configuration object, and read the settings
			BuildConfiguration BuildConfiguration = new BuildConfiguration();
			XmlConfig.ApplyTo(BuildConfiguration);
			Arguments.ApplyTo(BuildConfiguration);

			// Parse the filter argument
			FileFilter? FileFilter = null;
			if (FilterRules.Count > 0)
			{
				FileFilter = new FileFilter(FileFilterType.Exclude);
				foreach (string FilterRule in FilterRules)
				{
					FileFilter.AddRules(FilterRule.Split(';'));
				}
			}

			// Force C++ modules to always include their generated code directories
			UEBuildModuleCPP.bForceAddGeneratedCodeIncludePath = true;

			// Parse all the target descriptors
			List<TargetDescriptor> TargetDescriptors = TargetDescriptor.ParseCommandLine(Arguments, BuildConfiguration, Logger);

			// Generate the compile DB for each target
			using (ISourceFileWorkingSet WorkingSet = new EmptySourceFileWorkingSet())
			{
				// Find the compile commands for each file in the target
				Dictionary<FileReference, string> FileToCommand = new Dictionary<FileReference, string>();
				foreach (TargetDescriptor TargetDescriptor in TargetDescriptors)
				{
					// Create a makefile for the target
					UEBuildTarget Target = UEBuildTarget.Create(TargetDescriptor, BuildConfiguration, Logger);

					// Create all the binaries and modules
					CppCompileEnvironment GlobalCompileEnvironment = Target.CreateCompileEnvironmentForProjectFiles(Logger);
					foreach (UEBuildBinary Binary in Target.Binaries)
					{
						CppCompileEnvironment BinaryCompileEnvironment = Binary.CreateBinaryCompileEnvironment(GlobalCompileEnvironment);

						foreach (UEBuildModuleCPP Module in Binary.Modules.OfType<UEBuildModuleCPP>())
						{
							UEBuildModuleCPP.InputFileCollection InputFileCollection = Module.FindInputFiles(Target.Platform, new Dictionary<DirectoryItem, FileItem[]>(), Logger);

							List<FileItem> FileList = new List<FileItem>();
							foreach (FileItem InputFile in InputFileCollection.CPPFiles)
							{
								if (FileFilter == null || FileFilter.Matches(InputFile.Location.MakeRelativeTo(Unreal.RootDirectory)))
								{
									FileList.Add(InputFile);
								}
							}

							if (FileList.Any())
							{
								CppCompileEnvironment env = Module.CreateCompileEnvironmentForIntellisense(Target.Rules, BinaryCompileEnvironment, Logger);

								foreach (FileItem InputFile in FileList)
								{
									if (BinaryCompileEnvironment.MetadataCache.GetListOfInlinedGeneratedCppFiles(InputFile).Any())
									{
										continue;
									}

									// Search for a header that matches this source file
									string InputFilenameWithoutCpp = InputFile.Location.GetFileName().Replace(".cpp", String.Empty, StringComparison.Ordinal);
									FileItem? FoundHeader = InputFileCollection.HeaderFiles.FirstOrDefault(hf => hf.Name.Replace(".h", String.Empty, StringComparison.Ordinal) == InputFilenameWithoutCpp);
									if (FoundHeader == null)
									{
										continue;
									}
									else
									{
										string HeaderText = await FileReference.ReadAllTextAsync(FoundHeader.Location);
										if (!IncludeGenHeaderRegex.IsMatch(HeaderText))
										{
											continue;
										}
										else
										{
											// We can't inline the gen.cpp when the header is using an enum
											if (EnumRegex.IsMatch(HeaderText))
											{
												continue;
											}
										}
									}

									List<string> TextLines = [.. await FileReference.ReadAllLinesAsync(InputFile.Location)];
									if (TextLines.Any(Text => InlineReflectionMarkupRegex.IsMatch(Text)))
									{
										continue;
									}

									List<string> CleanTextLines = new List<string>(TextLines.Count);

									bool bInComment = false;
									for (int i = 0; i < TextLines.Count; i++)
									{
										string Line = TextLines[i];

										if (Line.Contains("//", StringComparison.InvariantCultureIgnoreCase))
										{
											Line = Line.Substring(0, Line.IndexOf("//", StringComparison.Ordinal));
										}

										if (SameLineCommentRegex.IsMatch(Line))
										{
											Line = SameLineCommentRegex.Replace(Line, String.Empty);
										}

										if (BeginningCommentRegex.IsMatch(Line))
										{
											Line = BeginningCommentRegex.Replace(Line, String.Empty);
											bInComment = true;
										}

										if (EndingCommentRegex.IsMatch(Line))
										{
											Line = EndingCommentRegex.Replace(Line, String.Empty);
											bInComment = false;
										}

										if (bInComment)
										{
											Line = String.Empty;
										}

										CleanTextLines.Add(Line);
									}

									Stack<int> IfEndBlocks = new();
									int LastLineOfInclude = -1;
									for (int i = 0; i < CleanTextLines.Count; i++)
									{
										string Line = CleanTextLines[i];

										if (IfEndBlocks.Count == 0 && IncludeRegex.IsMatch(Line))
										{
											LastLineOfInclude = i;
											continue;
										}

										if (IfRegex.IsMatch(Line))
										{
											IfEndBlocks.Push(i);
										}
										else if (EndIfRegex.IsMatch(Line))
										{
											IfEndBlocks.Pop();
										}
									}

									try
									{
										// add the include
										{
											if (LastLineOfInclude == -1)
											{
												LastLineOfInclude = TextLines.Count - 1;
												TextLines.Add(String.Empty);
											}
											else if (LastLineOfInclude + 1 == TextLines.Count || !String.IsNullOrEmpty(TextLines[LastLineOfInclude + 1]))
											{
												TextLines.Insert(LastLineOfInclude + 1, String.Empty);
											}

											TextLines.Insert(LastLineOfInclude + 1, String.Format("#include UE_INLINE_GENERATED_CPP_BY_NAME({0})", FoundHeader.Name.Replace(".h", String.Empty, StringComparison.Ordinal)));
											TextLines.Insert(LastLineOfInclude + 1, String.Empty);
										}

										Logger.LogInformation("Updating {IncludePath}", InputFile.FullName);

										if (!bNoOutput)
										{
											if (bCheckoutWithP4)
											{
												System.Diagnostics.Process Process = new System.Diagnostics.Process();
												System.Diagnostics.ProcessStartInfo StartInfo = new System.Diagnostics.ProcessStartInfo();
												Process.StartInfo.WindowStyle = System.Diagnostics.ProcessWindowStyle.Hidden;
												Process.StartInfo.FileName = "p4.exe";
												Process.StartInfo.Arguments = $"edit {InputFile.FullName}";
												Process.Start();
												await Process.WaitForExitAsync();
											}
											await System.IO.File.WriteAllLinesAsync(InputFile.FullName, TextLines);
										}
									}
									catch (Exception ex)
									{
										Logger.LogWarning("Failed to write to file: {Exception}", ex);
									}
								}
							}
						}
					}

					SourceFileMetadataCache.SaveAll();
				}
			}

			return 0;
		}
	}
}
