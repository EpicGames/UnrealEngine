// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Linq;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// Exports a target as a JSON file
	/// </summary>
	internal sealed class JsonExportMode : IToolMode<JsonExportMode>
	{
		public static string Name => "JsonExport";
		public static ToolModeOptions Options => ToolModeOptions.XmlConfig | ToolModeOptions.BuildPlatforms | ToolModeOptions.SingleInstance;

		/// <summary>
		/// Execute any actions which result in code generation (eg. ISPC compilation)
		/// </summary>
		[CommandLine("-ExecCodeGenActions")]
		public bool bExecCodeGenActions = false;

		/// <summary>
		/// Execute this command
		/// </summary>
		/// <param name="Arguments">Command line arguments</param>
		/// <returns>Exit code (always zero)</returns>
		/// <param name="Logger"></param>
		public async Task<int> ExecuteAsync(CommandLineArguments Arguments, ILogger Logger)
		{
			Arguments.ApplyTo(this);

			// Create the build configuration object, and read the settings
			BuildConfiguration BuildConfiguration = new BuildConfiguration();
			XmlConfig.ApplyTo(BuildConfiguration);
			Arguments.ApplyTo(BuildConfiguration);

			List<TargetDescriptor> TargetDescriptors = TargetDescriptor.ParseCommandLine(Arguments, BuildConfiguration, Logger);
			foreach (TargetDescriptor TargetDescriptor in TargetDescriptors)
			{
				// Create the target
				UEBuildTarget Target = UEBuildTarget.Create(TargetDescriptor, BuildConfiguration, Logger);

				// Get the output file
				FileReference? OutputFile = TargetDescriptor.AdditionalArguments.GetFileReferenceOrDefault("-OutputFile=", null);
				if (OutputFile == null)
				{
					OutputFile = Target.ReceiptFileName.ChangeExtension(".json");
				}

				// Execute code generation actions
				if (bExecCodeGenActions)
				{
					using (ISourceFileWorkingSet WorkingSet = new EmptySourceFileWorkingSet())
					{
						// Create the makefile
						TargetMakefile Makefile = await Target.BuildAsync(BuildConfiguration, WorkingSet, TargetDescriptor, Logger);
						List<LinkedAction> Actions = Makefile.Actions.ConvertAll(x => new LinkedAction(x, TargetDescriptor));
						ActionGraph.Link(Actions, Logger);

						// Filter all the actions to execute
						HashSet<FileItem> PrerequisiteItems = new HashSet<FileItem>(Makefile.Actions.SelectMany(x => x.ProducedItems).Where(x => x.HasExtension(".h") || x.HasExtension(".cpp")));
						List<LinkedAction> PrerequisiteActions = ActionGraph.GatherPrerequisiteActions(Actions, PrerequisiteItems);

						// Execute these actions
						if (PrerequisiteActions.Count > 0)
						{
							Logger.LogInformation("Executing actions that produce source files...");
							ActionGraph.CreateDirectoriesForProducedItems(PrerequisiteActions);
							await ActionGraph.ExecuteActionsAsync(BuildConfiguration, PrerequisiteActions, new List<TargetDescriptor> { TargetDescriptor }, Logger);
						}
					}
				}

				// Write the output file
				Logger.LogInformation("Writing {OutputFile}...", OutputFile);
				Target.ExportJson(OutputFile);
			}
			return 0;
		}
	}
}
