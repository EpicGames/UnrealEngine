// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Text.RegularExpressions;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool.Modes
{
	/// <summary>
	/// Identifies plugins with python requirements and attempts to install all dependencies using pip.
	/// </summary>
	internal sealed class PipInstallMode : IToolMode<PipInstallMode>
	{
		public static string Name => "PipInstall";
		public static ToolModeOptions Options => ToolModeOptions.XmlConfig | ToolModeOptions.BuildPlatformsHostOnly | ToolModeOptions.SingleInstance | ToolModeOptions.StartPrefetchingEngine | ToolModeOptions.ShowExecutionTime;

		private enum ActionBits : byte
		{
			NoOp = 0,
			GenReqs = 1,
			SetupPip = 2,
			ParseReqs = 4,
			InstallReqs = 8,
			ViewLicenses = 16,
		}

		public enum PipAction : byte
		{
			OnlySetupParse = ActionBits.SetupPip | ActionBits.ParseReqs,
			OnlyInstall = ActionBits.InstallReqs,

			GenRequirements = ActionBits.GenReqs,
			Setup = GenRequirements | ActionBits.SetupPip,
			Parse = Setup | ActionBits.ParseReqs,
			Install = Parse | ActionBits.InstallReqs,
			ViewLicenses = Install | ActionBits.ViewLicenses,
		}

		/// <summary>
		/// Full path to python interpreter engine was built with (if unspecified use value in PythonSDKRoot.txt)
		/// </summary>
		[CommandLine("-PythonInterpreter=", Description = "Full path to python interpreter to use in case the engine is built against an external python SDK")]
		public FileReference? PythonInterpreter = null;

		/// <summary>
		/// The action the pip install tool should implement (GenRequirements, Setup, Parse, Install, ViewLicenses, default: Install)
		/// </summary>
		[CommandLine("-PipAction=", Description = "Pip action: [GenRequirements, Setup, Parse, Install, ViewLicenses, default: Install]")]
		public PipAction? Action = null;

		/// <summary>
		/// Disable requiring hashes in pip install requirements (NOTE: this is insecure and may simplify supply-chain attacks)
		/// </summary>
		[CommandLine("-IgnoreHashes", Description = "Do not require package hashes (WARNING: Enabling this flag is security risk)")]
		public bool bIgnoreHashes = false;

		/// <summary>
		/// Allow overriding the index url (this will also disable extra-urls)
		/// </summary>
		[CommandLine("-OverrideIndexUrl", Description = "Use the specified index-url (WARNING: Should not be combined with IgnoreHashes)")]
		public string? OverrideIndexUrl = null;

		/// <summary>
		/// Run pip installer on all plugins in project and engine
		/// </summary>
		[CommandLine("-AllPlugins", Description = "Run pip installer on all plugins in project and engine")]
		public bool bAllPlugins = false;

		/// <summary>
		/// Execute the command
		/// </summary>
		/// <param name="Arguments">Command line arguments</param>
		/// <param name="Logger"></param>
		/// <returns>Exit code</returns>
		public Task<int> ExecuteAsync(CommandLineArguments Arguments, ILogger Logger)
		{
			// HACK: This env var must be cleared or it carries into python subprocesses and python sys.executable detection breaking venvs
			Environment.SetEnvironmentVariable("PYTHONEXECUTABLE", null);

			Arguments.ApplyTo(this);

			// Create the build configuration object, and read the settings
			BuildConfiguration BuildConfiguration = new BuildConfiguration();
			XmlConfig.ApplyTo(BuildConfiguration);
			Arguments.ApplyTo(BuildConfiguration);

			// Parse all the target descriptors
			List<TargetDescriptor> TargetDescriptors = TargetDescriptor.ParseCommandLine(Arguments, BuildConfiguration.bUsePrecompiled, BuildConfiguration.bSkipRulesCompile, BuildConfiguration.bForceRulesCompile, Logger);
			foreach (TargetDescriptor TargetDescriptor in TargetDescriptors)
			{
				//Logger.LogInformation("Pip Installer: {TargetDescriptor}", TargetDescriptor);
				int retcode = PipInstallProjectDependencies(TargetDescriptor, BuildConfiguration, Logger);
				if (retcode != 0)
				{
					return Task.FromResult(retcode);
				}
			}

			return Task.FromResult(0);
		}

		private int PipInstallProjectDependencies(TargetDescriptor TargetDescriptor, BuildConfiguration BuildConfiguration, ILogger Logger)
		{
			if (TargetDescriptor.ProjectFile == null)
			{
				Logger.LogError("No valid project file for Target: {}", TargetDescriptor.ToString());
				return 1;
			}

			UEBuildTarget Target = UEBuildTarget.Create(TargetDescriptor, BuildConfiguration, Logger);
			if (Target.TargetType != TargetType.Editor)
			{
				Logger.LogWarning("PipInstall unsupported for non-editor target: {TargetName} (Skipping)", TargetDescriptor.Name);
				return 0;
			}

			Action ??= PipAction.Install;

			DirectoryReference ProjectDir = DirectoryReference.FromFile(TargetDescriptor.ProjectFile);
			DirectoryReference InstallDir = DirectoryReference.Combine(ProjectDir, "Intermediate", "PipInstall");

			// Let env variable override pip install path
			DirectoryReference? EnvInstallPath = DirectoryReference.FromString(Environment.GetEnvironmentVariable("UE_PIPINSTALL_PATH"));
			if (EnvInstallPath != null)
			{
				InstallDir = EnvInstallPath;
			}

			PipEnv Pip = new(InstallDir, TargetDescriptor, Logger, ProgressWriter.bWriteMarkup);
			if ((Action & (PipAction)ActionBits.GenReqs) != 0)
			{
				// Make sure the virtual environment used for installs is compatible with python interpreter version
				Pip.RemoveInvalidVenv(PythonInterpreter);

				Pip.WritePluginsListing(Target, Logger, bAllPlugins);
				if (!Pip.WritePluginDependencies())
				{
					return 1;
				}
			}

			if ((Action & (PipAction)ActionBits.SetupPip) != 0)
			{
				if (!Pip.SetupPipEnv(PythonInterpreter))
				{
					return 1;
				}
			}

			if ((Action & (PipAction)ActionBits.ParseReqs) != 0)
			{

				if (!Pip.ParsePluginDependencies(!bIgnoreHashes))
				{
					return 1;
				}
			}

			if ((Action & (PipAction)ActionBits.InstallReqs) != 0)
			{
				if (!Pip.InstallPluginDependencies(!bIgnoreHashes, false, OverrideIndexUrl))
				{
					return 1;
				}
			}

			if ((Action & (PipAction)ActionBits.ViewLicenses) != 0)
			{
				if (!Pip.ViewInstalledLicenses())
				{
					return 1;
				}
			}

			return 0;
		}
	}

	/// <summary>
	/// PipEnv helper class for setting up a self-contained pip environment and running pip commands (particularly pip install) within that env.
	/// </summary>
	class PipEnv
	{
		// Don't bother to re-install pip install tools if this version is already installed
		// NOTE: This version must also be changed in PipInstall.cpp in order to support editor startup process
		private const string PipInstallUtilsVer = "0.1.5";

		// Generated from enabled plugins list
		private const string PluginsListingFilename = "pyreqs_plugins.list";
		// List full-paths to all enabled plugins site-package dirs (general/current platform)
		private const string PluginsSitePackageFilename = "plugin_site_package.pth";
		// This is the unparsed merged requirements file
		private const string MergedReqsInFilename = "merged_requirements.in";
		// These files are used as input for the pip installer
		private const string ExtraUrlsFilename = "extra_urls.txt";
		private const string MergedRequirementsFilename = "merged_requirements.txt";

		private readonly UnrealArch TargetArch;
		private readonly UnrealTargetPlatform TargetPlatform;
		private readonly DirectoryReference InstallDir;
		private readonly FileReference PythonVenvExe;
		private readonly string? PythonVenvVer;

		private readonly ILogger Logger;
		private readonly IBaseProgressLogFactory LoggerFactory;

		public PipEnv(DirectoryReference InInstallDir, TargetDescriptor InTargetDesc, ILogger InLogger, bool UseProgressWriter = false)
		{
			if (InTargetDesc.Architectures.bIsMultiArch)
			{
				InLogger.LogWarning("PipInstall expects single-arch target(host): {Arch}", InTargetDesc.Architectures.ToString());
			}
			
			Logger = InLogger;
			InstallDir = InInstallDir;
			TargetPlatform = InTargetDesc.Platform;
			TargetArch = InTargetDesc.Architectures.SingleArchitecture;

			LoggerFactory = (UseProgressWriter) ? new PipProgressLogCreator(Logger) : new SimpleCmdLogCreator(Logger);

			PythonVenvExe = GetVenvInterpreter(InstallDir, TargetPlatform);
			PythonVenvVer = ParseVenvVersion(InstallDir);
		}

		public void RemoveInvalidVenv(FileReference? EnginePython)
		{
			// Invalid or non-existent virtual env
			if (PythonVenvVer == null)
			{
				// Only delete virtual environment if version mismatch
				FileReference VenvConfig = FileReference.Combine(InstallDir, "pyvenv.cfg");
				if (FileReference.Exists(VenvConfig))
				{
					CleanVenvDir();
				}

				return;
			}

			FileReference EnginePythonInterp = GetEnginePythonInterpreter(EnginePython);
			if (!FileReference.Exists(EnginePythonInterp))
			{
				Logger.LogError("PipInstall: Invalid path to UE python interpreter: {Interp}", EnginePythonInterp.ToString());
				return;
			}

			using (IBaseCmdProgressLogger SimpleLogger = new SimpleCmdLogger(Logger))
			{
				const string PyInterpVerCheckCmd = "import sys; exit(0) if f'{sys.version_info[0]}.{sys.version_info[1]}.{sys.version_info[2]}' == sys.argv[1] else exit(1)";
				if (RunPythonCmd(EnginePythonInterp, $"-c \"{PyInterpVerCheckCmd}\" \"{PythonVenvVer}\"", SimpleLogger) != 0)
				{
					Logger.LogWarning("PipInstall: Found Incompatible virtual environment ({VenvVer}), removing...", PythonVenvVer);
					CleanVenvDir();
				}
			}
		}

		public void WritePluginsListing(UEBuildTarget Target, ILogger Logger, bool bAllPlugins = false)
		{
			// TODO: The path listing file won't match the .pth generated in-engine.
			//		 In particular the additional paths setting

			FileReference PluginsListingFile = FileReference.Combine(InstallDir, PluginsListingFilename);
			DirectoryReference PipSitePackagesPath = DirectoryReference.Combine(InstallDir, "Lib", "site-packages");
			FileReference PyPluginsSitePackageFile = FileReference.Combine(PipSitePackagesPath, PluginsSitePackageFilename);

			if (FileReference.Exists(PluginsListingFile))
			{
				FileReference.Delete(PluginsListingFile);
			}

			List<PluginInfo> CheckPlugins = new List<PluginInfo>();
			if (bAllPlugins)
			{
				CheckPlugins.AddAll(Plugins.ReadEnginePlugins(Unreal.EngineDirectory).ToArray());
				CheckPlugins.AddAll(Plugins.ReadProjectPlugins(Target.ProjectDirectory).ToArray());
			}
			else if (Target.EnabledPlugins != null)
			{
				foreach (UEBuildPlugin Plugin in Target.EnabledPlugins)
				{
					CheckPlugins.Add(Plugin.Info);
				}
			}
			else
			{
				return;
			}

			// May need to handle this differently
			string ArchStr = MakeArchString(TargetArch);

			List<string> PluginsSitePackages = new List<string>();
			List<string> PluginsList = new List<string>();

			string[] SitePkgSubdirs = { 
				Path.Combine("Lib", Target.Platform.ToString(), ArchStr, "site-packages"),
				Path.Combine("Lib", Target.Platform.ToString(), "site-packages"),
				Path.Combine("Lib", "site-packages")
			};
			
			foreach (PluginInfo Plugin in CheckPlugins)
			{
				// Add all existing plugin site-packages subdirs to be written to pth file listing
				DirectoryReference PythonContentPath = DirectoryReference.Combine(Plugin.Directory, "Content", "Python");
				foreach (string CheckSubdir in SitePkgSubdirs)
				{
					DirectoryReference CheckPluginPath = DirectoryReference.Combine(PythonContentPath, CheckSubdir);
					if (DirectoryReference.Exists(CheckPluginPath))
					{
						PluginsSitePackages.Add(CheckPluginPath.ToString());
					}
				}

				if (!JsonObject.TryRead(Plugin.File, out JsonObject? PluginJson))
				{
					Logger.LogWarning("Unable to parse {PluginFile}", Plugin.File.ToString());
					continue;
				}

				foreach (JsonObject _ in PipEnv.CompatibleRequirements(PluginJson, TargetPlatform, TargetArch))
				{
					PluginsList.Add(Plugin.File.ToString());
					break;
				}
			}

			// Add UE_PYTHONPATH to .pth file
			string? UEPythonPaths = Environment.GetEnvironmentVariable("UE_PYTHONPATH");
			if (UEPythonPaths != null)
			{
				string[] EnvPaths = UEPythonPaths.Split(Path.PathSeparator, StringSplitOptions.RemoveEmptyEntries | StringSplitOptions.TrimEntries);
				foreach (string EnvPath in EnvPaths)
				{
					PluginsSitePackages.Add(EnvPath);
				}
			}

			if (!DirectoryReference.Exists(PipSitePackagesPath))
			{
				DirectoryReference.CreateDirectory(PipSitePackagesPath);
			}

			FileReference.WriteAllLines(PluginsListingFile, PluginsList);
			FileReference.WriteAllLines(PyPluginsSitePackageFile, PluginsSitePackages);
		}

		public bool WritePluginDependencies()
		{
			FileReference PluginsListingFile = FileReference.Combine(InstallDir, PluginsListingFilename);
			FileReference MergedReqsInFile = FileReference.Combine(InstallDir, MergedReqsInFilename);
			FileReference ExtraUrlsFile = FileReference.Combine(InstallDir, ExtraUrlsFilename);

			// Make merged requirements input file
			if (!MergeRequirements(PluginsListingFile, out List<string>? MergedRequirements, out List<string>? ExtraIndexUrls))
			{
				return false;
			}

			FileReference.WriteAllLines(MergedReqsInFile, MergedRequirements!);
			FileReference.WriteAllLines(ExtraUrlsFile, ExtraIndexUrls!);

			return true;
		}

		public bool SetupPipEnv(FileReference? EnginePython, bool ForceRebuild = false)
		{
			using (IBaseCmdProgressLogger CmdLogger = LoggerFactory.Create("Creating pip installer virtual environment", 5))
			{
				if (!ForceRebuild && FileReference.Exists(PythonVenvExe))
				{
					return SetupPipInstallUtils(CmdLogger);
				}

				if (ForceRebuild && DirectoryReference.Exists(InstallDir))
				{
					DirectoryReference.Delete(InstallDir, true);
				}

				FileReference EnginePythonInterp = GetEnginePythonInterpreter(EnginePython);
				if (!FileReference.Exists(EnginePythonInterp))
				{
					Logger.LogError("PipInstall: Invalid path to UE python interpreter: {Interp}", EnginePythonInterp.ToString());
					return false;
				}

				int result = RunPythonCmd(EnginePythonInterp, $"-m venv \"{InstallDir}\"", CmdLogger);
				if (result != 0 || !FileReference.Exists(PythonVenvExe))
				{
					return false;
				}

				return SetupPipInstallUtils(CmdLogger);
			}
		}

		public bool ParsePluginDependencies(bool bPipStrictHashCheck = true)
		{
			FileReference MergedReqsInFile = FileReference.Combine(InstallDir, MergedReqsInFilename);
			FileReference MergedRequirmentsFile = FileReference.Combine(InstallDir, MergedRequirementsFilename);

			// NOTE: Hashes are all-or-nothing so if we are ignoring, just remove them all with the parser
			string DisableHashing = "";
			if (!bPipStrictHashCheck)
			{
				DisableHashing = "--disable-hashes";
			}

			using (IBaseCmdProgressLogger CmdLogger = new PythonCmdLogger(Logger))
			{
				return (RunPythonVenv($"-m ue_parse_plugin_reqs {DisableHashing} -vv \"{MergedReqsInFile}\" \"{MergedRequirmentsFile}\"", CmdLogger) == 0);
			}
		}

		public bool InstallPluginDependencies(bool bPipStrictHashCheck = true, bool OfflineOnly = false, string? ForceIndexUrl = null)
		{
			FileReference MergedRequirementsFile = FileReference.Combine(InstallDir, MergedRequirementsFilename);
			FileReference ExtraUrlsFile = FileReference.Combine(InstallDir, ExtraUrlsFilename);

			if (!FileReference.Exists(MergedRequirementsFile))
			{
				return true;
			}

			string[]? ExtraUrls = null;
			if (FileReference.Exists(ExtraUrlsFile))
			{
				ExtraUrls = FileReference.ReadAllLines(ExtraUrlsFile);
			}

			// Pip install from merged requirments
			// TODO: Support pip-tools compile/sync system
			return PipInstall(MergedRequirementsFile, ExtraUrls, bPipStrictHashCheck, OfflineOnly, ForceIndexUrl);
		}

		public bool ViewInstalledLicenses()
		{
			// TODO: Check that install is up to date and reverse-map package to plugin requirements
			//DirectoryReference WorkDir = DirectoryReference.FromFile(PluginsListingFile);

			using (IBaseCmdProgressLogger CmdLogger = new PythonCmdLogger(Logger))
			{
				return (RunPythonVenv($"-m ue_py_license_check -vv", CmdLogger) == 0);
			}
		}

		public static IEnumerable<JsonObject> CompatibleRequirements(JsonObject PluginJson, UnrealTargetPlatform Platform, UnrealArch Arch)
		{
			// Check for python requirements field
			if (!PluginJson.TryGetObjectArrayField("PythonRequirements", out JsonObject[]? RequirementsJson))
			{
				yield break;
			}

			foreach (JsonObject PlatformReqs in RequirementsJson)
			{
				PlatformReqs.TryGetStringField("Platform", out string? PlatformField);
				if (!CompatiblePlatform(PlatformField, Platform))
				{
					continue;
				}
				
				PlatformReqs.TryGetStringField("Architecture", out string? ArchField);
				if (!CompatibleArch(ArchField, Arch))
				{
					continue;
				}

				yield return PlatformReqs;
			}
		}

		private static bool CompatiblePlatform(string? PlatformField, UnrealTargetPlatform Platform)
		{
			return (PlatformField == null)
				|| String.Equals(PlatformField, Platform.ToString(), StringComparison.OrdinalIgnoreCase)
				|| String.Equals(PlatformField, "All", StringComparison.OrdinalIgnoreCase);
		}

		private static bool CompatibleArch(string? ArchField, UnrealArch Arch)
		{
			return (ArchField == null)
				|| String.Equals(ArchField, MakeArchString(Arch), StringComparison.OrdinalIgnoreCase)
				|| String.Equals(ArchField, "All", StringComparison.OrdinalIgnoreCase);
		}

		private static string MakeArchString(UnrealArch Arch)
		{
			if (Arch == UnrealArch.X64)
			{
				return "X64";
			}

			if (Arch == UnrealArch.Arm64 || Arch == UnrealArch.Arm64ec)
			{
				return "Arm64";
			}

			return "Unknown";
		}
		
		private string? ParseVenvVersion(DirectoryReference VenvDir)
		{
			FileReference VenvConfig = FileReference.Combine(VenvDir, "pyvenv.cfg");
			if (!FileReference.Exists(VenvConfig))
			{
				return null;
			}

			string ConfigInfo = FileReference.ReadAllText(VenvConfig);
			Match m = Regex.Match(ConfigInfo, @"version\s*=\s*(\d+\.\d+\.\d+)", RegexOptions.IgnoreCase);
			if (!m.Success)
			{
				Logger.LogWarning("PipInstall: Unable to match venv version config: {VenvFile}", ConfigInfo);
				return null;
			}

			return m.Groups[1].Value;
		}

		private void CleanVenvDir()
		{
			if (!DirectoryReference.Exists(InstallDir))
			{
				DirectoryReference.CreateDirectory(InstallDir);
				return;
			}

			// HACK: On windows these script files are set read-only and can't be deleted
			foreach (FileReference File in DirectoryReference.EnumerateFiles(DirectoryReference.Combine(InstallDir, "Scripts")))
			{
				FileReference.SetAttributes(File, FileAttributes.Normal);
			}

			DirectoryReference.Delete(InstallDir, true);
			DirectoryReference.CreateDirectory(InstallDir);
		}

		private bool PipInstall(FileReference RequirementsFile, string[]? ExtraUrls, bool bPipStrictHashCheck, bool OfflineOnly, string? ForceIndexUrl)
		{
			string[] Reqs = FileReference.ReadAllLines(RequirementsFile);
			int RequirementsCount = Reqs.Length;

			if (RequirementsCount == 0)
			{
				return true;
			}

			string Args = "-m pip install --disable-pip-version-check --only-binary=:all:";
			if (bPipStrictHashCheck)
			{
				Args += " --require-hashes";
			}

			if (OfflineOnly)
			{
				Args += " --no-index";
			}
			else if (ForceIndexUrl != null)
			{
				Args += "--index-url " + ForceIndexUrl;
			}
			else if (ExtraUrls != null)
			{
				foreach (string Url in ExtraUrls)
				{
					Args += " --extra-index-url " + Url;
				}
			}

			Args += " -r \"" + RequirementsFile.ToString() + "\"";

			using (IBaseCmdProgressLogger StatusLogger = LoggerFactory.Create("Installing Python Dependencies...", RequirementsCount))
			{
				int Result = RunPythonVenv(Args, StatusLogger);
				return (Result == 0);
			}
		}

		private bool MergeRequirements(FileReference PluginsListingFile, out List<string>? MergedRequirements, out List<string>? ExtraIndexUrls)
		{
			ExtraIndexUrls = null;
			MergedRequirements = null;

			if (!FileReference.Exists(PluginsListingFile))
			{
				return false;
			}

			// TODO: Support --index-url in per-plugin manner (split pip calls when necessary)
			List<string> RequirementsList = new();
			List<string> ExtraUrlsList = new();

			string[] PythonPlugins = FileReference.ReadAllLines(PluginsListingFile);
			foreach (string PluginPath in PythonPlugins)
			{
				FileReference? PluginFile = FileReference.FromString(PluginPath);
				if (PluginFile is null || !FileReference.Exists(PluginFile))
				{
					continue;
				}

				JsonObject PluginJson = JsonObject.Read(PluginFile);
				// This shouldn't happen here as we pre-filter, but check in case that changes
				if (!PluginJson.TryGetObjectArrayField("PythonRequirements", out JsonObject[]? RequirementsJson))
				{
					Logger.LogDebug("PipInstall: Warning: Plugin has no requirements (but in lising): {Plugin}", PluginFile.ToString());
					continue;
				}

				foreach (JsonObject PlatformReqs in CompatibleRequirements(PluginJson, TargetPlatform, TargetArch))
				{
					if (!PlatformReqs.TryGetStringArrayField("Requirements", out string[]? Reqs))
					{
						Logger.LogError("PythonRequirements missing 'Requirements' field in {Plugin} (Skipping)", PluginsListingFile.ToString());
						continue;
					}

					if (PlatformReqs.TryGetStringArrayField("ExtraIndexUrls", out string[]? ExtraUrls))
					{
						ExtraUrlsList.AddRange(ExtraUrls);
					}

					foreach (string Req in Reqs)
					{
						RequirementsList.Add(Req + " # " + PluginFile.GetFileName());
					}
				}
			}

			RequirementsList.Sort();

			ExtraIndexUrls = ExtraUrlsList;
			MergedRequirements = RequirementsList;

			return true;
		}

		public bool CheckPipInstallUtils()
		{
			// Verify that correct version of pip install utils is already available
			string Args = $"-c \"import pkg_resources;dist=pkg_resources.working_set.find(pkg_resources.Requirement.parse('ue-pipinstall-utils'));exit(dist.version!='{PipInstallUtilsVer}' if dist is not None else 1)\"";
			using IBaseCmdProgressLogger CmdLogger = new PythonCmdLogger(Logger);
			return (RunPythonVenv(Args, CmdLogger) == 0);
		}

		private bool SetupPipInstallUtils(IBaseCmdProgressLogger StatusLogger)
		{
			if (CheckPipInstallUtils())
			{
				return true;
			}

			Logger.LogInformation("PipInstall: Updating UE PipInstall Utilities");
			FileReference? PythonScriptPlugin = GetPythonScriptPlugin();
			if (PythonScriptPlugin == null)
			{
				Logger.LogError("PipInstall: Unable to locate engine PythonScriptPlugin");
				return false;
			}
			DirectoryReference PythonScriptDir = DirectoryReference.FromFile(PythonScriptPlugin);
			DirectoryReference PipInstallUtilsDir = DirectoryReference.Combine(PythonScriptDir, "Content", "Python", "PipInstallUtils");
			DirectoryReference PipWheelsDir = DirectoryReference.Combine(PythonScriptDir, "Content", "Python", "Lib", "wheels");
			FileReference RequirementsFile = FileReference.Combine(PipInstallUtilsDir, "requirements.txt");
			string Args = $"-m pip install --upgrade --no-index --find-links \"{PipWheelsDir}\" -r \"{RequirementsFile}\" ue-pipinstall-utils=={PipInstallUtilsVer}";
			int Result = RunPythonVenv(Args, StatusLogger);
			return (Result == 0);
		}

		private int RunPythonVenv(string Args, IBaseCmdProgressLogger InCmdLogger)
		{
			return RunPythonCmd(PythonVenvExe, Args, InCmdLogger);
		}

		private int RunPythonCmd(FileReference PythonBin, string Args, IBaseCmdProgressLogger InCmdLogger)
		{
			Logger.LogDebug("PythonCmd: {PythonBin} {Args}", PythonBin.ToString(), Args);

			Process Process = new Process();
			Process.StartInfo.WindowStyle = ProcessWindowStyle.Hidden;
			Process.StartInfo.CreateNoWindow = true;
			Process.StartInfo.UseShellExecute = false;
			Process.StartInfo.RedirectStandardInput = true;
			Process.StartInfo.RedirectStandardOutput = true;
			Process.StartInfo.RedirectStandardError = true;
			Process.StartInfo.FileName = PythonBin.ToString();
			Process.StartInfo.Arguments = Args;

			Process.OutputDataReceived += (object o, DataReceivedEventArgs args) => { InCmdLogger.OutputData(args); };
			Process.ErrorDataReceived += (object o, DataReceivedEventArgs args) => { InCmdLogger.ErrorData(args); };

			Process.Start();

			Process.BeginOutputReadLine();
			Process.BeginErrorReadLine();

			Process.WaitForExit();

			return Process.ExitCode;
		}

		private FileReference GetEnginePythonInterpreter(FileReference? EnginePython)
		{
			if (EnginePython != null)
			{
				return EnginePython;
			}

			DirectoryReference EngineDir = Unreal.EngineDirectory;
			DirectoryReference PythonSDKRoot = DirectoryReference.Combine(EngineDir, "Binaries", "ThirdParty", "Python3", TargetPlatform.ToString());

			FileReference PythonSDKFile = FileReference.Combine(PythonSDKRoot, "PythonSDKRoot.txt");
			if (FileReference.Exists(PythonSDKFile))
			{
				string SDKRootText = FileReference.ReadAllText(PythonSDKFile).Trim();
				PythonSDKRoot = new DirectoryReference(SDKRootText.Replace("{ENGINE_DIR}", EngineDir.ToString(), StringComparison.Ordinal));
			}
			else
			{
				Logger.LogWarning("PipInstall: PythonSDKRoot.txt does not exist, assming engine was built using internal python.");
			}

			return GetBaseInterpreter(PythonSDKRoot, TargetPlatform);
		}

		private static FileReference? GetPythonScriptPlugin()
		{
			List<PluginInfo> AllPlugins = Plugins.ReadAvailablePlugins(Unreal.EngineDirectory, null, null);
			PluginInfo? ScriptPlugin = AllPlugins.Find(x => x.Name == "PythonScriptPlugin");
			return ScriptPlugin?.File;
		}

		static FileReference GetVenvInterpreter(DirectoryReference VenvDir, UnrealTargetPlatform InPlatform)
		{
			if (InPlatform.IsInGroup(UnrealPlatformGroup.Windows))
			{
				return FileReference.Combine(VenvDir, "Scripts", "python.exe");
			}
			else
			{
				return FileReference.Combine(VenvDir, "bin", "python3");
			}
		}

		static FileReference GetBaseInterpreter(DirectoryReference PyDir, UnrealTargetPlatform InPlatform)
		{
			if (InPlatform.IsInGroup(UnrealPlatformGroup.Windows))
			{
				return FileReference.Combine(PyDir, "python.exe");
			}
			else
			{
				return FileReference.Combine(PyDir, "bin", "python3");
			}
		}
	}

	/// <summary>
	/// Simple interface for logging command stdout/stderr with progress tags if supported
	/// </summary>
	interface IBaseCmdProgressLogger : IDisposable
	{
		public void OutputData(DataReceivedEventArgs DataLine);
		public void ErrorData(DataReceivedEventArgs ErrorLine);
		public void FinishProgress();
	}

	interface IBaseProgressLogFactory
	{
		public IBaseCmdProgressLogger Create(string Message, int GuessSteps);
	}

	/// <summary>
	/// Simple factory types so that users don't need to implement the factory
	/// </summary>
	class SimpleCmdLogCreator : IBaseProgressLogFactory
	{
		private readonly ILogger Logger;

		public SimpleCmdLogCreator(ILogger InLogger) { Logger = InLogger; }
		public IBaseCmdProgressLogger Create(string Message, int GuessSteps)
		{
			return new SimpleCmdLogger(Logger);
		}
	}

	class PipProgressLogCreator : IBaseProgressLogFactory
	{
		private readonly ILogger Logger;

		public PipProgressLogCreator(ILogger InLogger) { Logger = InLogger; }
		public IBaseCmdProgressLogger Create(string Message, int GuessSteps)
		{
			return new PipProgressLogger(Logger, Message, GuessSteps);
		}
	}

	/// <summary>
	/// Basic command logger which just echos output/errors to Logger (indent stdout data by 2 spaces)
	/// </summary>
	class SimpleCmdLogger : IBaseCmdProgressLogger
	{
		private readonly ILogger Logger;
		public SimpleCmdLogger(ILogger InLogger)
		{
			Logger = InLogger;
		}
		public void OutputData(DataReceivedEventArgs DataLine)
		{
			if (String.IsNullOrEmpty(DataLine.Data))
			{
				return;
			}

			Logger.LogInformation("  {Data}", DataLine.Data);
		}
		public void ErrorData(DataReceivedEventArgs ErrorLine)
		{
			if (String.IsNullOrEmpty(ErrorLine.Data))
			{
				return;
			}

			Logger.LogError("{ErrorData}", ErrorLine.Data);
		}

		public void Dispose() { }

		public void FinishProgress() { }
	}

	/// <summary>
	/// Pyuthon command wraps python logging facility outputs to appropriate log channel
	/// </summary>
	class PythonCmdLogger : IBaseCmdProgressLogger
	{
		private readonly ILogger Logger;
		public PythonCmdLogger(ILogger InLogger)
		{
			Logger = InLogger;
		}
		public void OutputData(DataReceivedEventArgs DataLine)
		{
			// NOTE: By default python's logging functionality writes to stderr (at least on windows)
			//       but we run this code for both stdout/stderr anyway
			if (String.IsNullOrEmpty(DataLine.Data))
			{
				return;
			}

			HandleLogging(DataLine.Data);
		}
		public void ErrorData(DataReceivedEventArgs ErrorLine)
		{
			// NOTE: By default python's logging functionality writes to stderr (at least on windows)
			//       but we run this code for both stdout/stderr anyway
			if (String.IsNullOrEmpty(ErrorLine.Data))
			{
				return;
			}

			HandleLogging(ErrorLine.Data);
		}

		public void Dispose() { }

		public void FinishProgress() { }

		private void HandleLogging(string LogLine)
		{
			SplitOn(LogLine, ':', out string CheckTag, out string? LineTag);
			if (LineTag == null)
			{
				Logger.LogInformation("{Data}", LogLine);
				return;
			}

			switch (CheckTag)
			{
				case "DEBUG": Logger.LogDebug("  {Line}", SplitRight(LineTag, ':').Trim()); break;
				case "INFO": Logger.LogInformation("  {Line}", SplitRight(LineTag, ':').Trim()); break;
				case "WARNING": Logger.LogWarning("Warning: {Line}", SplitRight(LineTag, ':').Trim()); break;
				case "ERROR": Logger.LogError("Error: {Line}", SplitRight(LineTag, ':').Trim()); break;
				case "CRITICAL": Logger.LogCritical("Error: {Line}", SplitRight(LineTag, ':').Trim()); break;
				default: Logger.LogInformation("  {Line}", LogLine); break;
			}
		}

		private static void SplitOn(string InStr, char SplitChar, out string Left, out string? Right)
		{
			Left = InStr;
			Right = null;

			int SplitIdx = InStr.IndexOf(SplitChar, StringComparison.Ordinal);
			if (SplitIdx < 0)
			{
				return;
			}

			Left = InStr.Substring(0, SplitIdx);
			Right = InStr.Substring(SplitIdx + 1);
		}

		private static string SplitRight(string InStr, char SplitChar)
		{
			SplitOn(InStr, SplitChar, out string Left, out string? Right);

			// Return the string if the split works
			if (Right == null)
			{
				return Left;
			}

			return Right;
		}
	}

	/// <summary>
	/// Basic command writer that redirects stdout to file, and writes stderr normally (used to output pip freeze to file)
	/// </summary>
	class FileCmdLogger : IBaseCmdProgressLogger
	{
		private readonly ILogger Logger;
		private readonly TextWriter TextWriter;
		public FileCmdLogger(FileReference InTargetFile, ILogger InLogger)
		{
			Logger = InLogger;
			TextWriter = new StreamWriter(InTargetFile.ToString());
		}
		public void OutputData(DataReceivedEventArgs DataLine)
		{
			if (String.IsNullOrEmpty(DataLine.Data))
			{
				return;
			}

			TextWriter.WriteLine(DataLine.Data.Trim());
		}
		public void ErrorData(DataReceivedEventArgs ErrorLine)
		{
			if (String.IsNullOrEmpty(ErrorLine.Data))
			{
				return;
			}

			Logger.LogError("{ErrorData}", ErrorLine.Data);
		}

		public void Dispose()
		{
			TextWriter.Close();
		}

		public void FinishProgress() { }
	}

	/// <summary>
	/// Simple interface for logging command stdout/stderr with progress tags if supported
	/// </summary>
	class PipProgressLogger : IBaseCmdProgressLogger
	{
		private readonly ILogger Logger;
		private ProgressWriter Writer;

		private int StepsDone;
		private int TotalSteps;

		// Start strings to use 
		private static readonly string[] MatchStrs = { "Requirement", "Collecting", "Installing" };
		private readonly Dictionary<string, string> LogReplaceStrs = new();

		public PipProgressLogger(ILogger InLogger, string message, int GuessSteps)
		{
			Logger = InLogger;
			StepsDone = 0;
			TotalSteps = Math.Max(GuessSteps, 1);

			Writer = new ProgressWriter(message, true, Logger);

			LogReplaceStrs["Installing collected packages:"] = "Installing collected python package dependencies:";
		}

		static bool CheckUpdateStr(string CheckStr)
		{
			foreach (string ProgressMatch in MatchStrs)
			{
				if (CheckStr.StartsWith(ProgressMatch, StringComparison.InvariantCultureIgnoreCase))
				{
					return true;
				}
			}

			return false;
		}

		string ReplaceMatchStr(string CheckStr)
		{
			foreach (KeyValuePair<string, string> ChkPair in LogReplaceStrs)
			{
				if (CheckStr.Contains(ChkPair.Key, StringComparison.Ordinal))
				{
					return CheckStr.Replace(ChkPair.Key, ChkPair.Value, StringComparison.Ordinal);
				}
			}

			return CheckStr;
		}

		public void OutputData(DataReceivedEventArgs DataLine)
		{
			// Currently we assume only one pip command so it should be finished on (null)
			if (String.IsNullOrEmpty(DataLine.Data))
			{
				return;
			}

			string CheckStr = DataLine.Data.Trim();
			bool ShouldUpdate = CheckUpdateStr(CheckStr);
			if (ShouldUpdate)
			{
				CheckStr = ReplaceMatchStr(CheckStr);

				Writer = new ProgressWriter(CheckStr, true, Logger);
				Writer.Write(StepsDone, TotalSteps);

				StepsDone += 1;
				TotalSteps = Math.Max(TotalSteps, StepsDone + 1);
			}
			else
			{
				Logger.LogInformation("{CheckStr}", CheckStr);
			}
		}

		public void Dispose()
		{
			FinishProgress();
			Writer.Dispose();
		}

		public void FinishProgress()
		{
			StepsDone = TotalSteps;
			Writer.Write(StepsDone, TotalSteps);
		}

		public void ErrorData(DataReceivedEventArgs ErrorLine)
		{
			if (String.IsNullOrEmpty(ErrorLine.Data))
			{
				return;
			}

			Logger.LogError("PipInstall: {ErrorData}", ErrorLine.Data);
		}
	}
}
