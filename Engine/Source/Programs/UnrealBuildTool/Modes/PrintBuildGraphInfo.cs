// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace UnrealBuildTool
{
	/// <summary>
	/// Generate a clang compile_commands file for a target
	/// </summary>
	internal sealed class PrintBuildGraphInfo : IToolMode<PrintBuildGraphInfo>
	{
		public static string Name => "PrintBuildGraphInfo";
		public static ToolModeOptions Options => ToolModeOptions.XmlConfig | ToolModeOptions.BuildPlatforms | ToolModeOptions.SingleInstance | ToolModeOptions.StartPrefetchingEngine | ToolModeOptions.ShowExecutionTime;

		/// <summary>
		/// Execute the command
		/// </summary>
		/// <param name="Arguments">Command line arguments</param>
		/// <returns>Exit code</returns>
		/// <param name="Logger"></param>
		public Task<int> ExecuteAsync(CommandLineArguments Arguments, ILogger Logger)
		{
			Arguments.ApplyTo(this);

			// Create the build configuration object, and read the settings
			BuildConfiguration BuildConfiguration = new BuildConfiguration();
			XmlConfig.ApplyTo(BuildConfiguration);
			Arguments.ApplyTo(BuildConfiguration);

			// Parse all the target descriptors
			List<TargetDescriptor> TargetDescriptors = TargetDescriptor.ParseCommandLine(Arguments, BuildConfiguration, Logger);

			using (ISourceFileWorkingSet WorkingSet = new EmptySourceFileWorkingSet())
			{
				// Find the compile commands for each file in the target
				Dictionary<FileReference, string> FileToCommand = new();
				foreach (TargetDescriptor TargetDescriptor in TargetDescriptors)
				{
					Logger.LogInformation("------------------------");
					Logger.LogInformation("Target: {TargetName}", TargetDescriptor.Name);
					Logger.LogInformation("------------------------");

					// Create a makefile for the target
					UEBuildTarget Target = UEBuildTarget.Create(TargetDescriptor, BuildConfiguration, Logger);

					List<UEBuildModule> NoUnityModules = new();
					List<UEBuildModule> NotOptimizedModules = new();
					List<UEBuildModule> NoPCHModules = new();
					List<UEBuildModule> PrivatePCHModules = new();
					List<UEBuildModule> SharedPCHWithPrivateHeaderModules = new();

					// Figure out all the modules referenced by this target. This includes all the modules that are referenced, not just the ones compiled into binaries.
					CppCompileEnvironment GlobalCompileEnvironment = Target.CreateCompileEnvironmentForProjectFiles(Logger);
					HashSet<UEBuildModule> Modules = new HashSet<UEBuildModule>();
					foreach (UEBuildBinary Binary in Target.Binaries)
					{
						CppCompileEnvironment BinaryCompileEnvironment = Binary.CreateBinaryCompileEnvironment(GlobalCompileEnvironment);
						foreach (UEBuildModule Module in UEBuildModule.StableTopologicalSort(Binary.Modules))
						{
							if (Module is UEBuildModuleCPP ModuleCPP)
							{
								ModuleCPP.CreateCompileEnvironmentForIntellisense(Target.Rules, BinaryCompileEnvironment, Logger);
							}

							Modules.Add(Module);
						}
					}

					// Gather module info
					foreach (UEBuildModule Module in Modules)
					{
						if (Module.Rules.PCHUsage == ModuleRules.PCHUsageMode.NoPCHs)
						{
							NoPCHModules.Add(Module);
						}
						else if (Module.Rules.PrivatePCHHeaderFile != null && (Module.Rules.PCHUsage == ModuleRules.PCHUsageMode.NoSharedPCHs || Module.Rules.PCHUsage == ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs))
						{
							PrivatePCHModules.Add(Module);
						}
						else if (Module.Rules.PrivatePCHHeaderFile != null && Module.Rules.PCHUsage == ModuleRules.PCHUsageMode.UseSharedPCHs)
						{
							SharedPCHWithPrivateHeaderModules.Add(Module);
						}

						if (!Module.Rules.bUseUnity)
						{
							NoUnityModules.Add(Module);
						}

						if (Module.Rules.OptimizeCode == ModuleRules.CodeOptimization.Never)
						{
							NotOptimizedModules.Add(Module);
						}
					}

					if (NoUnityModules.Any())
					{
						Logger.LogInformation("Modules not using unity files:");
						NoUnityModules.SortBy(Module => Module.Name);
						foreach (UEBuildModule Module in NoUnityModules)
						{
							Logger.LogInformation(" {ModuleName}", Module.Name);
						}
					}

					if (NotOptimizedModules.Any())
					{
						Logger.LogInformation("Modules not optimized:");
						NotOptimizedModules.SortBy(Module => Module.Name);
						foreach (UEBuildModule Module in NotOptimizedModules)
						{
							Logger.LogInformation(" {ModuleName}", Module.Name);
						}
					}

					if (SharedPCHWithPrivateHeaderModules.Any())
					{
						Logger.LogInformation("Modules using a shared PCH and private PCH header:");
						Logger.LogInformation(" Note these might not compile properly when PCHs are disabled.");
						SharedPCHWithPrivateHeaderModules.SortBy(Module => Module.Name);
						foreach (UEBuildModule Module in SharedPCHWithPrivateHeaderModules)
						{
							Logger.LogInformation("  {ModuleName}", Module.Name);
						}
					}

					Logger.LogInformation("PCH Usage:");

					if (NoPCHModules.Any())
					{
						Logger.LogInformation(" Modules not using PCHs:");
						NoPCHModules.SortBy(Module => Module.Name);
						foreach (UEBuildModule Module in NoPCHModules)
						{
							Logger.LogInformation("  {ModuleName}", Module.Name);
						}
					}

					if (PrivatePCHModules.Any())
					{
						Logger.LogInformation(" Modules using private PCHs:");
						PrivatePCHModules.SortBy(Module => Module.Name);
						foreach (UEBuildModule Module in PrivatePCHModules)
						{
							Logger.LogInformation("  {ModuleName}", Module.Name);
						}
					}

					Dictionary<UEBuildModule, PrecompiledHeaderTemplate> ModulesUsingSharedPCHs = new();
					if (GlobalCompileEnvironment.SharedPCHs.Any())
					{
						Logger.LogInformation(" Shared PCHs:");
						List<PrecompiledHeaderTemplate> SortedSharedPCHs = new(GlobalCompileEnvironment.SharedPCHs);
						SortedSharedPCHs.SortBy(SharedPCH => SharedPCH.Module.Name);
						foreach (PrecompiledHeaderTemplate Template in SortedSharedPCHs)
						{
							Logger.LogInformation("  Module {ModuleName} - {TimesUsed} Permutations:", Template.Module.Name, Template.Instances.Count);
							List<PrecompiledHeaderInstance> SortedInstances = new(Template.Instances);
							SortedInstances.SortBy(Instance => Instance.HeaderFile.Name);
							foreach (PrecompiledHeaderInstance Instance in SortedInstances)
							{
								Logger.LogInformation("   {InstanceName} - Used by {TimesUsed} modules:", Instance.HeaderFile, Instance.Modules.Count);
								if (Instance.ParentPCHInstance != null)
								{
									Logger.LogInformation("    ParentPCH: {ParentInstanceName}", Instance.ParentPCHInstance.HeaderFile);
								}

								List<UEBuildModuleCPP> SortedModules = new(Instance.Modules);
								SortedModules.SortBy(Module => Module.Name);
								foreach (UEBuildModuleCPP Module in SortedModules)
								{
									Logger.LogInformation("    {ModuleName}", Module.Name);
									ModulesUsingSharedPCHs.Add(Module, Template);
								}
							}
						}
					}

					Logger.LogInformation("Circular Dependencies that are not declared.");
					Logger.LogInformation("NOTE: Anything prefixed with a '!!!' means that the circular dependency could possibly create linker errors.");
					foreach (UEBuildModule Module in Modules)
					{
						List<Stack<UEBuildModule>> ModuleDepStacks = new List<Stack<UEBuildModule>>();
						FindCircularDependencyModules(Module, Module, new HashSet<UEBuildModule>(), true, new Stack<UEBuildModule>(), ModuleDepStacks);
						if (ModuleDepStacks.Any())
						{
							bool UsesSharedPCH = ModulesUsingSharedPCHs.ContainsKey(Module);
							Logger.LogInformation(" {ModuleName} - {DepCount}", Module.Name, ModuleDepStacks.Count);
							foreach (Stack<UEBuildModule> DepModuleStack in ModuleDepStacks)
							{
								StringBuilder Dependencies = new StringBuilder();

								if (UsesSharedPCH && DepModuleStack.Contains(ModulesUsingSharedPCHs[Module].Module))
								{
									Dependencies.Append("!!! ");
								}

								UEBuildModule[] DepModuleArray = DepModuleStack.Reverse().ToArray();
								for (int DepModuleIndex = 0; DepModuleIndex < DepModuleArray.Length; DepModuleIndex++)
								{
									Dependencies.Append(DepModuleArray[DepModuleIndex].Name);
									if (DepModuleIndex != DepModuleArray.Length - 1)
									{
										Dependencies.Append(" -> ");
									}
								}
								Logger.LogInformation("   {Dependencies}", Dependencies);
							}
						}
					}
				}
			}

			return Task.FromResult(0);
		}

		private static void FindCircularDependencyModules(UEBuildModule SearchForBuildModule, UEBuildModule CurrentBuildModule, HashSet<UEBuildModule> IgnoreReferencedModules, bool bIncludePrivateDependencyModules, Stack<UEBuildModule> CurrentStack, List<Stack<UEBuildModule>> ModuleDepStacks)
		{
			CurrentStack.Push(CurrentBuildModule);

			if (SearchForBuildModule == CurrentBuildModule && CurrentStack.Count > 1)
			{
				ModuleDepStacks.Add(new Stack<UEBuildModule>(CurrentStack.Reverse()));
			}

			List<UEBuildModule> AllDependencyModules = new List<UEBuildModule>(
				((bIncludePrivateDependencyModules && CurrentBuildModule.PrivateDependencyModules != null) ? CurrentBuildModule.PrivateDependencyModules.Count : 0) +
				(CurrentBuildModule.PublicDependencyModules != null ? CurrentBuildModule.PublicDependencyModules.Count : 0) +
				(CurrentBuildModule.PublicIncludePathModules != null ? CurrentBuildModule.PublicIncludePathModules!.Count : 0)
				);
			if (bIncludePrivateDependencyModules && CurrentBuildModule.PrivateDependencyModules != null)
			{
				AllDependencyModules.AddRange(CurrentBuildModule.PrivateDependencyModules);
			}
			if (CurrentBuildModule.PublicDependencyModules != null)
			{
				AllDependencyModules.AddRange(CurrentBuildModule.PublicDependencyModules!);
			}
			if (CurrentBuildModule.PublicIncludePathModules != null)
			{
				AllDependencyModules.AddRange(CurrentBuildModule.PublicIncludePathModules);
			}

			foreach (UEBuildModule DependencyModule in AllDependencyModules)
			{
				// Don't follow circular back-references!
				if (!CurrentBuildModule.HasCircularDependencyOn(DependencyModule))
				{
					if (IgnoreReferencedModules.Add(DependencyModule))
					{
						// Recurse into dependent modules
						FindCircularDependencyModules(SearchForBuildModule, DependencyModule, new HashSet<UEBuildModule>(IgnoreReferencedModules), false, CurrentStack, ModuleDepStacks);
					}
				}
			}
			CurrentStack.Pop();
		}
	}
}
