// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text.RegularExpressions;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// Profiles different unity sizes and prints out the different size and its timings
	/// </summary>
	internal sealed class ProfileUnitySizesMode : IToolMode<ProfileUnitySizesMode>
	{
		public static string Name => "ProfileUnitySizes";
		public static ToolModeOptions Options => ToolModeOptions.XmlConfig | ToolModeOptions.BuildPlatforms | ToolModeOptions.SingleInstance | ToolModeOptions.StartPrefetchingEngine | ToolModeOptions.ShowExecutionTime;

#pragma warning disable IDE0044 // Make field readonly - these private static fields are set by command-line parsing.
		/// <summary>
		/// Set of filters for files to include in the database. Relative to the root directory, or to the project file.
		/// </summary>
		[CommandLine("-Filter=")]
		List<string> FilterRules = new List<string>();
#pragma warning restore IDE0044

		class TimingData
		{
			public double ExecutorTiming = 0;
			public double CPUTiming = 0;
			public int UnitySize = 0;
			public int NumFiles = 0;

			public bool IsValid()
			{
				return ExecutorTiming != 0;
			}
		}

		class TimingLogger : ILogger
		{
			static readonly Regex ExecutorTimingRegex = new Regex(@"executor.*\s(\d+\.*\d*)\sseconds");
			static readonly Regex NumFilesRegex = new Regex(@"\[\d+/(\d+)\]");
			static readonly Regex CPUTimingRegex = new Regex(@"CPU Time:\s(\d+\.*\d*)");
			private readonly ILogger Inner;

			public TimingData TimingData = new();

			public TimingLogger(ILogger inner)
			{
				Inner = inner;
				// TODO: Assigning to a global static inside this ctor seems dangerous, is this needed?
				EpicGames.Core.Log.EventParser.Logger = this;
			}

			/// <inheritdoc/>
			public IDisposable? BeginScope<TState>(TState State) where TState : notnull
			{
				return Inner.BeginScope(State);
			}

			/// <inheritdoc/>
			public bool IsEnabled(LogLevel LogLevel)
			{
				return Inner.IsEnabled(LogLevel);
			}

			/// <inheritdoc/>
			public void Log<TState>(LogLevel LogLevel, EventId EventId, TState State, Exception? Exception, Func<TState, Exception?, string> Formatter)
			{
				if (State != null)
				{
					string? LogText = State.ToString();
					if (!String.IsNullOrEmpty(LogText))
					{
						// Console.WriteLine(LogText);
						Match ExecutorTimingMatch = ExecutorTimingRegex.Match(LogText);
						if (ExecutorTimingMatch.Success)
						{
							if (!Double.TryParse(ExecutorTimingMatch.Groups[1].Value, out TimingData.ExecutorTiming))
							{
								Console.WriteLine($"Failed to parse '{LogText}'");
							}
						}

						Match CPUTimingMatch = CPUTimingRegex.Match(LogText);
						if (CPUTimingMatch.Success)
						{
							if (!Double.TryParse(CPUTimingMatch.Groups[1].Value, out TimingData.CPUTiming))
							{
								Console.WriteLine($"Failed to parse '{LogText}'");
							}
						}

						Match NumFilesMatch = NumFilesRegex.Match(LogText);
						if (NumFilesMatch.Success)
						{
							if (!Int32.TryParse(NumFilesMatch.Groups[1].Value, out TimingData.NumFiles))
							{
								Console.WriteLine($"Failed to parse '{LogText}'");
							}
						}
					}
				}
			}
		}

		/// <summary>
		/// Execute the command
		/// </summary>
		/// <param name="Arguments">Command line arguments</param>
		/// <returns>Exit code</returns>
		/// <param name="Logger"></param>
		public async Task<int> ExecuteAsync(CommandLineArguments Arguments, ILogger Logger)
		{
			Arguments.ApplyTo(this);

			// Create the build configuration object, and read the settings
			BuildConfiguration BuildConfiguration = new BuildConfiguration();
			XmlConfig.ApplyTo(BuildConfiguration);
			Arguments.ApplyTo(BuildConfiguration);

			// Parse the filter argument
			FileFilter? FileFilter = null;
			if (FilterRules.Count > 0)
			{
				FileFilter = new FileFilter(FileFilterType.Exclude);
				foreach (string FilterRule in FilterRules)
				{
					FileFilter.AddRules(FilterRule.Split(';'));
				}
			}

			// Force C++ modules to always include their generated code directories
			UEBuildModuleCPP.bForceAddGeneratedCodeIncludePath = true;

			// Parse all the target descriptors
			List<TargetDescriptor> TargetDescriptors = TargetDescriptor.ParseCommandLine(Arguments, BuildConfiguration, Logger);

			foreach (TargetDescriptor TargetDescriptor in TargetDescriptors)
			{
				List<UEBuildModule> ModuleList = new();

				TargetDescriptor.AdditionalArguments = TargetDescriptor.AdditionalArguments.Append(new string[] { "-NoSNDBS", "-NoXGE" });

				// Create a makefile for the target
				TimingLogger TimingLogger = new(Logger);
				UEBuildTarget Target = UEBuildTarget.Create(TargetDescriptor, BuildConfiguration, TimingLogger);
				UEToolChain TargetToolChain = Target.CreateToolchain(Target.Platform, TimingLogger);

				CppCompileEnvironment GlobalCompileEnvironment = Target.CreateCompileEnvironmentForProjectFiles(TimingLogger);
				foreach (UEBuildBinary Binary in Target.Binaries)
				{
					CppCompileEnvironment BinaryCompileEnvironment = Binary.CreateBinaryCompileEnvironment(GlobalCompileEnvironment);
					foreach (UEBuildModule Module in Binary.Modules)
					{
						if (FileFilter == null || FileFilter.Matches(Module.RulesFile.MakeRelativeTo(Unreal.RootDirectory)))
						{
							if (Module.Rules.Type != ModuleRules.ModuleType.External &&
								Module.Rules.bUseUnity)
							{
								ModuleList.Add(Module);
							}
						}
					}
				}

				// build each Module
				ModuleList.SortBy(module => module.Name);
				foreach (UEBuildModule Module in ModuleList)
				{
					await CompileModuleAsync(BuildConfiguration, TargetDescriptor, Target, Module, Logger);
				}
			}

			return 0;
		}

		/// <summary>
		/// Compile the module multiple times looking for the best unity size 
		/// </summary>
		private static async Task CompileModuleAsync(BuildConfiguration BuildConfiguration, TargetDescriptor TargetDescriptor, UEBuildTarget Target, UEBuildModule Module, ILogger Logger)
		{
			TargetDescriptor.OnlyModuleNames.Clear();
			TargetDescriptor.OnlyModuleNames.Add(Module.Name);

			const int UnitySizeDivision = 8;
			const int TotalBuilds = UnitySizeDivision + 3;

			Logger.LogInformation("{Module}:", Module.Name);

			int CurrentModuleUnitySize = Module.Rules.GetNumIncludedBytesPerUnityCPP();
			int TargetUnitySize = Target.Rules.NumIncludedBytesPerUnityCPP;

			int BuildNum = 1;
			await CompileModuleAsync($"  [{BuildNum++}/{TotalBuilds}] ", BuildConfiguration, TargetDescriptor, Module, Logger, CurrentModuleUnitySize, true, false);

			TimingData CurrentCompileTime = await GetBestCompileModuleTimeAsync($"  [{BuildNum++}/{TotalBuilds}] ", BuildConfiguration, TargetDescriptor, Module, Logger, CurrentModuleUnitySize, false, false);
			if (!CurrentCompileTime.IsValid())
			{
				Logger.LogInformation("Skipping module because it doesn't compile with current settings.");
				return;
			}

			TimingData DisableUnityCompileTime = await GetBestCompileModuleTimeAsync($"  [{BuildNum++}/{TotalBuilds}] ", BuildConfiguration, TargetDescriptor, Module, Logger, TargetUnitySize, false, true);

			int MaxUnitySize = TargetUnitySize * 2;
			List<TimingData> Timings = new();
			int UnitySizeInc = MaxUnitySize / UnitySizeDivision;
			int CurrentUnitySize = UnitySizeInc;
			for (int UnitySizeIndex = 0; UnitySizeIndex < UnitySizeDivision; UnitySizeIndex++)
			{
				TimingData NewTiming = await GetBestCompileModuleTimeAsync($"  [{BuildNum++}/{TotalBuilds}] ", BuildConfiguration, TargetDescriptor, Module, Logger, CurrentUnitySize, false, false);
				Timings.Add(NewTiming);
				CurrentUnitySize += UnitySizeInc;

				if (NewTiming.NumFiles == 1)
				{
					break;
				}
			}

			Logger.LogInformation("{Module} Timings CPUTiming(secs) | ExecutorTiming(secs) | NumFiles:", Module.Name);
			PrintUnityInfo($"Current({CurrentModuleUnitySize})", CurrentCompileTime, Logger);
			PrintUnityInfo("Disabled", DisableUnityCompileTime, Logger);
			CurrentUnitySize = UnitySizeInc;
			TimingData BestTiming = CurrentCompileTime;
			foreach (TimingData Timing in Timings)
			{
				PrintUnityInfo(CurrentUnitySize.ToString(), Timing, Logger);
				CurrentUnitySize += UnitySizeInc;

				if (Timing.IsValid() &&
					BestTiming.NumFiles != Timing.NumFiles &&
					BestTiming.ExecutorTiming > Timing.ExecutorTiming &&
					BestTiming.CPUTiming > Timing.CPUTiming)
				{
					BestTiming = Timing;
				}
			}

			if (BestTiming != CurrentCompileTime)
			{
				Logger.LogInformation("Better unity size than current: {BestTimingUnitySize}", BestTiming.UnitySize);
			}
		}

		/// <summary>
		/// Print the timing data
		/// </summary>
		private static void PrintUnityInfo(string TimingPrefix, TimingData TimingData, ILogger Logger)
		{
			const string FirstColWidth = "15";
			const string ColWidth = "10";
			if (TimingData.IsValid())
			{
				const string Format = $"  {{TimingPrefix,-{FirstColWidth}}}: {{CPUTime,-{ColWidth}}} | {{ExecutorTime,-{ColWidth}}} | {{FileCount,-{ColWidth}}}";
				Logger.LogInformation(Format, TimingPrefix, TimingData.CPUTiming, TimingData.ExecutorTiming, TimingData.NumFiles);
			}
			else
			{
				const string Format = $"  {{TimingPrefix,-{FirstColWidth}}}: Failed";
				Logger.LogInformation(Format, TimingPrefix);
			}
		}

		/// <summary>
		/// Returns best compile timings after building the module times
		/// </summary>
		private static async Task<TimingData> GetBestCompileModuleTimeAsync(string LogPrefix, BuildConfiguration BuildConfiguration, TargetDescriptor TargetDescriptor, UEBuildModule Module, ILogger Logger, int UnitySize, bool bPriming, bool bDisableUnity)
		{
			const int CompileCount = 3;
			List<TimingData> AllTimingData = new();
			for (int CompileIndex = 0; CompileIndex < CompileCount; CompileIndex++)
			{
				TimingData NewTimingData = await CompileModuleAsync(LogPrefix, BuildConfiguration, TargetDescriptor, Module, Logger, UnitySize, bPriming, bDisableUnity);
				if (!NewTimingData.IsValid())
				{
					return NewTimingData;
				}
				AllTimingData.Add(NewTimingData);
			}

			TimingData? BestTimingData = AllTimingData.MinBy(TimingData => TimingData.ExecutorTiming);
			BestTimingData ??= AllTimingData[0];
			return BestTimingData;
		}

		/// <summary>
		/// Compiles the module and returns the timing information
		/// </summary>
		private static async Task<TimingData> CompileModuleAsync(string LogPrefix, BuildConfiguration BuildConfiguration, TargetDescriptor TargetDescriptor, UEBuildModule Module, ILogger Logger, int UnitySize, bool bPriming, bool bDisableUnity)
		{
			// Store the old arguments
			string[] OldArgs = TargetDescriptor.AdditionalArguments.GetRawArray();
			TimingData NewTimingData = new TimingData();

			try
			{
				if (!bPriming)
				{
					// Clear the output directory
					Logger.LogInformation("{LogPrefix}Deleting intermediate directory...", LogPrefix);
					try
					{
						DirectoryItem IntermDir = DirectoryItem.GetItemByDirectoryReference(Module.IntermediateDirectory);
						IntermDir.CacheFiles();
						IntermDir.ResetCachedInfo();
						DirectoryReference.Delete(Module.IntermediateDirectory, true);
					}
					catch (Exception ex)
					{
						Logger.LogError(ex, "{LogPrefix}Failed to delete {Module}'s intermediate directory.", LogPrefix, Module.Name);
					}

					// Add the module name to the cmdline
					TargetDescriptor.AdditionalArguments = TargetDescriptor.AdditionalArguments.Append(new string[] { $"-BytesPerUnityCPP={UnitySize}", "-DisableModuleNumIncludedBytesPerUnityCPPOverride" });
					TargetDescriptor.bUseUnityBuild = !bDisableUnity;
				}

				using (ISourceFileWorkingSet WorkingSet = new EmptySourceFileWorkingSet())
				{
					if (bPriming)
					{
						Logger.LogInformation("{LogPrefix}Priming module...", LogPrefix);
					}
					else if (bDisableUnity)
					{
						Logger.LogInformation("{LogPrefix}Compiling with no unity files...", LogPrefix);
					}
					else
					{
						Logger.LogInformation("{LogPrefix}Compiling with unity size '{UnitySize}'...", LogPrefix, UnitySize);
					}

					TimingLogger NewTimingLogger = new(Logger);
					await BuildMode.BuildAsync(new List<TargetDescriptor>() { TargetDescriptor }, BuildConfiguration, WorkingSet, BuildOptions.None, null, new(), NewTimingLogger);
					NewTimingData = NewTimingLogger.TimingData;
				}
			}
			catch
			{
				Logger.LogInformation("{LogPrefix}Compile Failed", LogPrefix);
			}

			NewTimingData.UnitySize = UnitySize;
			EpicGames.Core.Log.EventParser.Flush(); // we need flush here to get all the logging info for this build
			if (!bPriming)
			{
				if (NewTimingData.IsValid())
				{
					Logger.LogInformation(
						"{LogPrefix}Finished: CPUTime:{CPUTime}s | ExecutorTime:{ExecutorTime}s | NumFiles:{FileCount}",
						LogPrefix,
						NewTimingData.CPUTiming,
						NewTimingData.ExecutorTiming,
						NewTimingData.NumFiles);
				}
			}

			// Restore the old arguments
			TargetDescriptor.AdditionalArguments = new CommandLineArguments(OldArgs);

			return NewTimingData;
		}
	}
}
