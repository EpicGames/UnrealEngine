// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Collections.Immutable;
using System.IO;
using System.Linq;
using System.Text.Json;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Logging.Abstractions;
using OpenTracing.Util;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	// TODO: Namespacing?
	enum QueryType
	{
		Capabilities,
		AvailableTargets,
		TargetDetails,
		TargetModuleReferences,
	}

	internal class TargetIntellisenseInfo
	{
		internal class CompileSettings
		{
			public List<string> IncludePaths { get; set; } = [];
			public List<string> Defines { get; set; } = [];
			public string? Standard { get; set; }
			public List<string> ForcedIncludes { get; set; } = [];
			public string? CompilerPath { get; set; }
			public List<string> CompilerArgs { get; set; } = [];
			public Dictionary<string, List<string>> CompilerAdditionalArgs { get; set; } = [];
			public string? WindowsSdkVersion { get; set; }
		}

		public ConcurrentDictionary<UEBuildModule, CompileSettings> ModuleToCompileSettings = [];
		public ConcurrentDictionary<DirectoryReference, UEBuildModule> DirToModule = [];

		public UEBuildModule? FindModuleForFile(FileReference File)
		{
			DirectoryReference? Dir = File.Directory;
			while (Dir != null)
			{
				if (DirToModule.TryGetValue(Dir, out UEBuildModule? Module))
				{
					return Module;
				}
				Dir = Dir.ParentDirectory;
			}
			return null;
		}

		public UEBuildModule? FindModuleForDirectory(DirectoryReference Directory)
		{
			DirectoryReference? Dir = Directory;
			while (Dir != null)
			{
				if (DirToModule.TryGetValue(Dir, out UEBuildModule? Module))
				{
					return Module;
				}
				Dir = Dir.ParentDirectory;
			}
			return null;
		}
	}

	internal class LaunchSettings
	{
		public string? Description { get; set; }
		public string? BinaryPath { get; set; }
		public List<string> Arguments { get; set; } = [];
	}

	internal class TargetConfigs
	{
		public string TargetPath { get; set; } = String.Empty;
		public string ProjectPath { get; set; } = String.Empty;
		public string TargetType { get; set; } = String.Empty;
		public List<string> Platforms { get; set; } = [];
		public List<string> Configurations { get; set; } = [];
	}

	internal sealed class QueryMode : IToolMode<QueryMode>
	{
		public static string Name => "Query";
		public static ToolModeOptions Options => ToolModeOptions.BuildPlatforms | ToolModeOptions.XmlConfig | ToolModeOptions.UseStartupTraceListener;

		[CommandLine("-LogDirectory=")]
		public DirectoryReference? LogDirectory = null;

		[CommandLine("-OutputPath=")]
		public FileReference? OutputPath = null;

		[CommandLine("-Query=")]
		public QueryType? Query = null;

		[CommandLine("-LoadTargets")]
		public bool bLoadTargets = false;

		[CommandLine("-IncludeEngineSource=")]
		public bool bIncludeEngineSource = true;

		[CommandLine("-Target=")]
		public string? TargetName;

		[CommandLine("-Configuration=")]
		public string? TargetConfiguration;

		[CommandLine("-Platform=")]
		public string? TargetPlatform;

		[CommandLine("-Indented")]
		public bool bIndented;

		[CommandLine("-NoIntellisense")]
		public bool bNoIntellisense = false;

		private readonly BuildConfiguration BuildConfiguration = new();

		public Task<int> ExecuteAsync(CommandLineArguments Arguments, ILogger Logger)
		{
			Arguments.ApplyTo(this);

			if (LogDirectory == null)
			{
				LogDirectory = DirectoryReference.Combine(Unreal.EngineProgramSavedDirectory, "UnrealBuildTool", "QueryMode");
			}

			DirectoryReference.CreateDirectory(LogDirectory);

			string LogFileName;
			switch (Query)
			{
				case QueryType.TargetDetails:
					LogFileName = $"Log_{TargetName}_{TargetConfiguration}_{TargetPlatform}.txt";
					break;
				default:
					LogFileName = $"Log_{Query}.txt";
					break;

			}

			FileReference LogFile = FileReference.Combine(LogDirectory, LogFileName);
			Log.AddFileWriter("DefaultLogTraceListener", LogFile);

			XmlConfig.ApplyTo(BuildConfiguration);
			Arguments.ApplyTo(BuildConfiguration);

			// TODO: Document this hack 
			ProjectFileGenerator.bGenerateProjectFiles = true;

			if (Query == null)
			{
				return Task.FromResult(0);
			}

			JsonSerializerOptions ResponseOptions = new JsonSerializerOptions
			{
				PropertyNamingPolicy = JsonNamingPolicy.CamelCase,
				WriteIndented = bIndented,
			};
			switch (Query)
			{
				case QueryType.Capabilities:
					Logger.LogInformation("QueryCapabilities");
					return QueryCapabilities(Logger, ResponseOptions);
				case QueryType.AvailableTargets:
					Logger.LogInformation("QueryAvailableTargets");
					return QueryAvailableTargetsAsync(Arguments, Logger, ResponseOptions);
				case QueryType.TargetDetails:
					Logger.LogInformation("QueryTargetDetails");
					return QueryTargetDetails(Arguments, Logger, ResponseOptions);
				case QueryType.TargetModuleReferences:
					Logger.LogInformation("QueryTargetModuleReferences");
					return QueryTargetModuleReferences(Arguments, Logger, ResponseOptions);
			}

			return Task.FromResult(0);
		}

		private async Task WriteResultsAsync(object? Value, JsonSerializerOptions JsonOptions, ILogger Logger)
		{
			if (OutputPath == null)
			{
				Console.WriteLine(JsonSerializer.Serialize(Value, JsonOptions));
				return;
			}
			using FileStream Stream = new FileStream(OutputPath.FullName, FileMode.Create, FileAccess.Write);
			Logger.LogInformation("Writing {File}...", OutputPath.FullName);
			await JsonSerializer.SerializeAsync(Stream, Value, JsonOptions);
		}

		private async Task<int> QueryCapabilities(ILogger Logger, JsonSerializerOptions JsonOptions)
		{
			var Reply = new
			{
				Queries = new List<string> { QueryType.Capabilities.ToString(), QueryType.AvailableTargets.ToString(), QueryType.TargetDetails.ToString(), QueryType.TargetModuleReferences.ToString() }
			};
			await WriteResultsAsync(Reply, JsonOptions, Logger);
			return 0;
		}

		private async Task<int> QueryAvailableTargetsAsync(CommandLineArguments Arguments, ILogger Logger, JsonSerializerOptions JsonOptions)
		{
			try
			{
				Utils.TryParseProjectFileArgument(Arguments, Logger, out FileReference? ProjectFileArg);

				List<UnrealTargetPlatform> Platforms = [];
				foreach (UnrealTargetPlatform Platform in UnrealTargetPlatform.GetValidPlatforms())
				{
					// If there is a build platform present, add it to the SupportedPlatforms list
					UEBuildPlatform? BuildPlatform;
					if (UEBuildPlatform.TryGetBuildPlatform(Platform, out BuildPlatform))
					{
						if (InstalledPlatformInfo.IsValidPlatform(Platform, EProjectType.Code))
						{
							Platforms.Add(Platform);
						}
					}
				}

				List<UnrealTargetConfiguration> AllowedTargetConfigurations = [];
				AllowedTargetConfigurations = [.. Enum.GetValues(typeof(UnrealTargetConfiguration)).Cast<UnrealTargetConfiguration>()];

				List<UnrealTargetConfiguration> Configurations = [];
				foreach (UnrealTargetConfiguration CurConfiguration in AllowedTargetConfigurations)
				{
					if (CurConfiguration != UnrealTargetConfiguration.Unknown)
					{
						if (InstalledPlatformInfo.IsValidConfiguration(CurConfiguration, EProjectType.Code))
						{
							Configurations.Add(CurConfiguration);
						}
					}
				}

				List<FileReference> Projects = ProjectFileArg != null ? [ProjectFileArg] : [.. NativeProjects.EnumerateProjectFiles(Logger)];
				List<FileReference> AllTargetFiles = ProjectFileGenerator.DiscoverTargets(Projects, Logger, null, Platforms, bIncludeEngineSource: bIncludeEngineSource, bIncludeTempTargets: false);

				Dictionary<string, TargetConfigs> Targets = [];
				string? DefaultTarget = null;
				foreach (FileReference TargetFilePath in AllTargetFiles)
				{
					try
					{
						FileReference? ProjectPath = Projects.FirstOrDefault(p => TargetFilePath.IsUnderDirectory(p.Directory));
						string TargetType = String.Empty;
						List<UnrealTargetConfiguration> SupportedConfigurations = [.. Configurations];
						List<UnrealTargetPlatform> SupportedPlatforms = [.. Platforms];
						if (bLoadTargets)
						{
							List<string> RawArgs = [TargetFilePath.GetFileNameWithoutAnyExtensions(), UnrealTargetConfiguration.Development.ToString(), UnrealTargetPlatform.Win64.ToString()];
							if (ProjectPath != null)
							{
								RawArgs.Add($"-Project={ProjectPath.FullName}");
							}
							CommandLineArguments Args = new CommandLineArguments([.. RawArgs]);

							List<TargetDescriptor> TargetDescriptors = [];
							TargetDescriptor.ParseSingleCommandLine(Args, false, false, false, TargetDescriptors, NullLogger.Instance);

							// Ensure the intermediate environment does not conflict with normal builds
							TargetDescriptors[0].IntermediateEnvironment = UnrealIntermediateEnvironment.Query;

							UEBuildTarget CurrentTarget;
							using (GlobalTracer.Instance.BuildSpan("UEBuildTarget.Create()").StartActive())
							{
								bool bUsePrecompiled = false;

								// Prevent multiple conflicting processes building TargetRules at the same time
								string MutexName = GlobalSingleInstanceMutex.GetUniqueMutexForPath("UnrealBuildTool_QueryMode_UEBuildTarget-Create", Unreal.RootDirectory);
								using (await SingleInstanceMutex.AcquireAsync(MutexName))
								{
									CurrentTarget = UEBuildTarget.Create(TargetDescriptors[0], false, false, bUsePrecompiled, TargetDescriptors[0].IntermediateEnvironment, NullLogger.Instance);
								}
							}
							TargetType = CurrentTarget.Rules.Type.ToString();
							SupportedConfigurations = [.. CurrentTarget.Rules.SupportedConfigurations];
							SupportedPlatforms = [.. CurrentTarget.Rules.SupportedPlatforms];
						}

						string TargetName = TargetFilePath.GetFileNameWithoutAnyExtensions();
						Targets.Add(TargetName, new TargetConfigs()
						{
							TargetPath = TargetFilePath.FullName,
							ProjectPath = ProjectPath?.ToString() ?? String.Empty,
							TargetType = TargetType,
							Configurations = [.. SupportedConfigurations.Select(x => x.ToString())],
							Platforms = [.. SupportedPlatforms.Select(x => x.ToString())]
						});
						if (DefaultTarget == null || TargetName == "UnrealEditor")
						{
							DefaultTarget = TargetName;
						}
					}
					catch (Exception)
					{
						continue;
					}
				}

				var Reply = new
				{
					Targets,
					DefaultTarget,
					DefaultPlatform = Platforms[0].ToString(),
					DefaultConfiguration = UnrealTargetConfiguration.Development.ToString(),
				};

				await WriteResultsAsync(Reply, JsonOptions, Logger);
				return 0;
			}
			catch (Exception Ex)
			{
				Logger.LogError("Failed to query available targets: {Error}", Ex.Message);
				Logger.LogDebug(Ex, "Unhandled exception: {Ex}", ExceptionUtils.FormatExceptionDetails(Ex));
				return 1;
			}
		}

		private async Task<int> QueryTargetDetails(CommandLineArguments Arguments, ILogger Logger, JsonSerializerOptions JsonOptions)
		{
			if (TargetName == null)
			{
				Logger.LogError("Missing argument Target");
				return 1;
			}
			else if (TargetConfiguration == null)
			{
				Logger.LogError("Missing argument Configuration");
				return 1;
			}
			else if (TargetPlatform == null)
			{
				Logger.LogError("Missing argument Platform");
				return 1;
			}

			Utils.TryParseProjectFileArgument(Arguments, Logger, out FileReference? ProjectFileArg);
			List<string> RawArgs = [TargetName, TargetConfiguration, TargetPlatform];
			if (ProjectFileArg != null)
			{
				RawArgs.Add(ProjectFileArg.ToString());
			}
			RawArgs.AddRange(Arguments.GetUnusedArguments());
			RawArgs.Add("-NoPCHChain"); // Currently unsupported
			CommandLineArguments Args = new CommandLineArguments([.. RawArgs]);
			List<TargetDescriptor> TargetDescriptors = [];

			TargetDescriptor.ParseSingleCommandLine(Args, false, false, false, TargetDescriptors, Logger);
			if (TargetDescriptors.Count == 0)
			{
				// TOOD: Error 
				return 1;
			}

			// Ensure the intermediate environment does not conflict with normal builds
			TargetDescriptors[0].IntermediateEnvironment = UnrealIntermediateEnvironment.Query;

			try
			{
				UEBuildTarget CurrentTarget;
				using (GlobalTracer.Instance.BuildSpan("UEBuildTarget.Create()").StartActive())
				{
					bool bUsePrecompiled = false;

					// Prevent multiple conflicting processes building TargetRules at the same time
					string MutexName = GlobalSingleInstanceMutex.GetUniqueMutexForPath("UnrealBuildTool_QueryMode_UEBuildTarget-Create", Unreal.RootDirectory);
					using (await SingleInstanceMutex.AcquireAsync(MutexName))
					{
						CurrentTarget = UEBuildTarget.Create(TargetDescriptors[0], false, false, bUsePrecompiled, TargetDescriptors[0].IntermediateEnvironment, Logger);
					}
				}

				UEBuildBinary? LaunchBinary = CurrentTarget.Binaries.FirstOrDefault(Binary => Binary.Modules.Any(Module => Module.Name == CurrentTarget.Rules.LaunchModuleName));
				if (LaunchBinary == null)
				{
					throw new BuildException("Unable to find launch binary for target");
				}

				LaunchSettings CurrentLaunchSettings = new LaunchSettings();
				CurrentLaunchSettings.Description = $"{CurrentTarget.TargetName} {CurrentTarget.Configuration} {CurrentTarget.Platform}";
				CurrentLaunchSettings.BinaryPath = LaunchBinary.OutputFilePath.FullName;
				if (CurrentTarget.ProjectFile != null && CurrentTarget.TargetType != TargetType.Program)
				{
					CurrentLaunchSettings.Arguments.Add(CurrentTarget.ProjectFile.FullName);
				}

				TargetIntellisenseInfo? CurrentTargetIntellisenseInfo = null;
				if (!bNoIntellisense)
				{
					// Create the makefile for the target and export the module information
					{
						using ISourceFileWorkingSet WorkingSet = new EmptySourceFileWorkingSet();
						TargetMakefile Makefile;
						try
						{
							Makefile = await BuildMode.CreateMakefileAsync(BuildConfiguration, TargetDescriptors[0], WorkingSet, Logger);
						}
						finally
						{
							SourceFileMetadataCache.SaveAll();
						}
					}

					CurrentTargetIntellisenseInfo = new TargetIntellisenseInfo();
					// Partially duplicated from UEBuildTarget.Build because we just want to get C++ compile actions without running UHT
					// or generating link actions / full dependency graph 
					CppConfiguration CppConfiguration = UEBuildTarget.GetCppConfiguration(CurrentTarget.Configuration);
					SourceFileMetadataCache MetadataCache = SourceFileMetadataCache.CreateHierarchy(null, Logger);
					CppCompileEnvironment GlobalCompileEnvironment = new CppCompileEnvironment(CurrentTarget.Platform, CppConfiguration, CurrentTarget.Architectures, MetadataCache);
					LinkEnvironment GlobalLinkEnvironment = new LinkEnvironment(GlobalCompileEnvironment.Platform, GlobalCompileEnvironment.Configuration, GlobalCompileEnvironment.Architectures);

					UEToolChain TargetToolChain = CurrentTarget.CreateToolchain(CurrentTarget.Platform, Logger);
					TargetToolChain.SetEnvironmentVariables();
					CurrentTarget.SetupGlobalEnvironment(TargetToolChain, GlobalCompileEnvironment, GlobalLinkEnvironment);

					if (CurrentTarget.Rules.bUseSharedPCHs)
					{
						// Find all the shared PCHs.
						CurrentTarget.FindSharedPCHs(CurrentTarget.Binaries, GlobalCompileEnvironment, Logger);

						// Create all the shared PCH instances before processing the modules
						UEBuildTarget.CreateSharedPCHInstances(CurrentTarget.Rules, TargetToolChain, CurrentTarget.Binaries, GlobalCompileEnvironment, new NullActionGraphBuilder(Logger), Logger);
					}

					await Parallel.ForEachAsync(CurrentTarget.Binaries, async (Binary, CancellationToken) =>
					{
						CppCompileEnvironment BinaryCompileEnvironment = Binary.CreateBinaryCompileEnvironment(GlobalCompileEnvironment);

						IEnumerable<UEBuildModuleCPP> Modules = Binary.Modules.OfType<UEBuildModuleCPP>().Where(x => x.Binary == Binary);

						await Parallel.ForEachAsync(Modules, (Module, CancellationToken) =>
						{
							if (CurrentTargetIntellisenseInfo.ModuleToCompileSettings.ContainsKey(Module))
							{
								return ValueTask.CompletedTask;
							}

							foreach (DirectoryItem Dir in Module.ModuleDirectories)
							{
								CurrentTargetIntellisenseInfo.DirToModule.TryAdd(Dir.Location, Module);
							}
							if (Module.GeneratedCodeDirectory != null)
							{
								CurrentTargetIntellisenseInfo.DirToModule.TryAdd(Module.GeneratedCodeDirectory, Module);
							}

							CppCompileEnvironment ModuleCompileEnvironment = Module.CreateCompileEnvironmentForIntellisense(CurrentTarget.Rules, BinaryCompileEnvironment, Logger);
							if (ModuleCompileEnvironment.PrecompiledHeaderAction == PrecompiledHeaderAction.Include)
							{
								FileItem IncludeHeader = FileItem.GetItemByFileReference(ModuleCompileEnvironment.PrecompiledHeaderIncludeFilename!);
								ModuleCompileEnvironment.ForceIncludeFiles.Insert(0, IncludeHeader);
							}

							// Remove include paths and defintiions from an environment used to get the command line args
							CppCompileEnvironment ModuleCompileEnvironmentForArgs = new CppCompileEnvironment(ModuleCompileEnvironment);
							ModuleCompileEnvironmentForArgs.SystemIncludePaths.Clear();
							ModuleCompileEnvironmentForArgs.UserIncludePaths.Clear();
							ModuleCompileEnvironmentForArgs.Definitions.Clear();

							TargetIntellisenseInfo.CompileSettings Settings = new TargetIntellisenseInfo.CompileSettings();
							Settings.IncludePaths.AddRange(ModuleCompileEnvironment.UserIncludePaths.Select(x => x.ToString()));
							Settings.IncludePaths.AddRange(ModuleCompileEnvironment.SystemIncludePaths.Select(x => x.ToString()));
							if (TargetToolChain is VCToolChain TargetVCToolChain)
							{
								Settings.IncludePaths.AddRange(TargetVCToolChain.GetVCIncludePaths().Select(x => x.ToString()));
							}
							Settings.Defines = ModuleCompileEnvironment.Definitions;
							Settings.Standard = ModuleCompileEnvironment.CppStandard.ToString();
							Settings.ForcedIncludes = [.. ModuleCompileEnvironment.ForceIncludeFiles.Select(x => x.ToString())];
							Settings.CompilerPath = TargetToolChain.GetCppCompilerPath()?.ToString();
							Settings.CompilerArgs = [.. TargetToolChain.GetGlobalCommandLineArgs(ModuleCompileEnvironmentForArgs)];
							Settings.CompilerAdditionalArgs.Add("c", [.. TargetToolChain.GetCCommandLineArgs(ModuleCompileEnvironmentForArgs)]);
							Settings.CompilerAdditionalArgs.Add("cpp", [.. TargetToolChain.GetCPPCommandLineArgs(ModuleCompileEnvironmentForArgs)]);
							Settings.WindowsSdkVersion = CurrentTarget.Rules.WindowsPlatform.WindowsSdkVersion;
							CurrentTargetIntellisenseInfo.ModuleToCompileSettings.TryAdd(Module, Settings);

							return ValueTask.CompletedTask;
						});
					});
				}

				var Result = new
				{
					DirToModule = CurrentTargetIntellisenseInfo?.DirToModule.ToImmutableSortedDictionary(x => x.Key.ToString(), x => x.Value.Name),
					ModuleToCompileSettings = CurrentTargetIntellisenseInfo?.ModuleToCompileSettings.ToImmutableSortedDictionary(x => x.Key.Name, x => x.Value),
					LaunchSettings = CurrentLaunchSettings,
					CurrentTarget.Rules.IsTestTarget
				};

				await WriteResultsAsync(Result, JsonOptions, Logger);
				return 0;
			}
			catch (Exception Ex)
			{
				Logger.LogError("Failed to query available targets: {Error}", Ex.Message);
				Logger.LogDebug(Ex, "Unhandled exception: {Ex}", ExceptionUtils.FormatExceptionDetails(Ex));
				return 1;
			}
		}

		private async Task<int> QueryTargetModuleReferences(CommandLineArguments Arguments, ILogger Logger, JsonSerializerOptions JsonOptions)
		{
			if (TargetName == null)
			{
				Logger.LogError("Missing argument Target");
				return 1;
			}
			else if (TargetConfiguration == null)
			{
				Logger.LogError("Missing argument Configuration");
				return 1;
			}
			else if (TargetPlatform == null)
			{
				Logger.LogError("Missing argument Platform");
				return 1;
			}

			Utils.TryParseProjectFileArgument(Arguments, Logger, out FileReference? ProjectFileArg);
			List<string> RawArgs = [TargetName, TargetConfiguration, TargetPlatform];
			if (ProjectFileArg != null)
			{
				RawArgs.Add(ProjectFileArg.ToString());
			}
			RawArgs.AddRange(Arguments.GetUnusedArguments());
			RawArgs.Add("-NoPCHChain"); // Currently unsupported
			CommandLineArguments Args = new CommandLineArguments([.. RawArgs]);
			List<TargetDescriptor> TargetDescriptors = [];

			TargetDescriptor.ParseSingleCommandLine(Args, false, false, false, TargetDescriptors, Logger);
			if (TargetDescriptors.Count == 0)
			{
				// TOOD: Error 
				return 1;
			}

			// Ensure the intermediate environment does not conflict with normal builds
			TargetDescriptors[0].IntermediateEnvironment = UnrealIntermediateEnvironment.Query;

			try
			{
				UEBuildTarget CurrentTarget;
				using (GlobalTracer.Instance.BuildSpan("UEBuildTarget.Create()").StartActive())
				{
					bool bUsePrecompiled = false;

					// Prevent multiple conflicting processes building TargetRules at the same time
					string MutexName = GlobalSingleInstanceMutex.GetUniqueMutexForPath("UnrealBuildTool_QueryMode_UEBuildTarget-Create", Unreal.RootDirectory);
					using (await SingleInstanceMutex.AcquireAsync(MutexName))
					{
						CurrentTarget = UEBuildTarget.Create(TargetDescriptors[0], false, false, bUsePrecompiled, TargetDescriptors[0].IntermediateEnvironment, Logger);
					}
				}

				UEBuildBinary? LaunchBinary = CurrentTarget.Binaries.FirstOrDefault(Binary => Binary.Modules.Any(Module => Module.Name == CurrentTarget.Rules.LaunchModuleName));
				if (LaunchBinary == null)
				{
					throw new BuildException("Unable to find launch binary for target");
				}

				// Create the makefile for the target and export the module information
				{
					using ISourceFileWorkingSet WorkingSet = new EmptySourceFileWorkingSet();
					TargetMakefile Makefile;
					try
					{
						Makefile = await BuildMode.CreateMakefileAsync(BuildConfiguration, TargetDescriptors[0], WorkingSet, Logger);
					}
					finally
					{
						SourceFileMetadataCache.SaveAll();
					}
				}

				ConcurrentDictionary<UEBuildModuleCPP, object> ModuleInfo = [];

				await Parallel.ForEachAsync(CurrentTarget.Binaries, async (Binary, CancellationToken) =>
				{
					IEnumerable<UEBuildModuleCPP> Modules = Binary.Modules.OfType<UEBuildModuleCPP>().Where(x => x.Binary == Binary);

					await Parallel.ForEachAsync(Modules, (Module, CancellationToken) =>
					{
						if (ModuleInfo.ContainsKey(Module))
						{
							return ValueTask.CompletedTask;
						}

						var Info = new
						{
							ShortName = Module.Rules.ShortName,
							ModuleDirectory = Module.ModuleDirectory.FullName,
							// Strip the query suffix from the target name so we output the real path.
							IntermediateDirectory = Module.IntermediateDirectory.FullName.Replace($"{CurrentTarget.TargetName}QRY", CurrentTarget.TargetName, StringComparison.Ordinal),
							Public = new SortedSet<string>(Module.PublicDependencyModules?.Where(x => x is UEBuildModuleCPP).Select(x => x.Name) ?? []),
							Private = new SortedSet<string>(Module.PrivateDependencyModules?.Where(x => x is UEBuildModuleCPP).Select(x => x.Name) ?? []),
							PublicInclude = new SortedSet<string>(Module.PublicIncludePathModules?.Where(x => x is UEBuildModuleCPP).Select(x => x.Name) ?? []),
							PrivateInclude = new SortedSet<string>(Module.PrivateIncludePathModules?.Where(x => x is UEBuildModuleCPP).Select(x => x.Name) ?? []),
							DynamicallyLoaded = new SortedSet<string>(Module.DynamicallyLoadedModules?.Where(x => x is UEBuildModuleCPP).Select(x => x.Name) ?? []),
							Circular = new SortedSet<string>(Module.Rules.CircularlyReferencedDependentModules),
						};

						ModuleInfo.TryAdd(Module, Info);

						return ValueTask.CompletedTask;
					});
				});

				var Result = new
				{
					Modules = ModuleInfo.ToImmutableSortedDictionary(x => x.Key.Name, x => x.Value)
				};

				await WriteResultsAsync(Result, JsonOptions, Logger);
				return 0;
			}
			catch (Exception Ex)
			{
				Logger.LogError("Failed to query available targets: {Error}", Ex.Message);
				Logger.LogDebug(Ex, "Unhandled exception: {Ex}", ExceptionUtils.FormatExceptionDetails(Ex));
				return 1;
			}
		}
	}
}
