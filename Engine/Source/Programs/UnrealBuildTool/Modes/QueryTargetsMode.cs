// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Linq;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// Queries information about the targets supported by a project
	/// </summary>
	internal sealed class QueryTargetsMode : IToolMode<QueryTargetsMode>
	{
		public static string Name => "QueryTargets";
		public static ToolModeOptions Options => ToolModeOptions.XmlConfig | ToolModeOptions.BuildPlatformsHostOnly | ToolModeOptions.SingleInstance;

#pragma warning disable IDE0044 // Make field readonly - these private static fields are set by command-line parsing.
		/// <summary>
		/// Path to the project file to query
		/// </summary>
		[CommandLine("-Project=")]
		FileReference? ProjectFile = null;

		/// <summary>
		/// Path to the output file to receive information about the targets
		/// </summary>
		[CommandLine("-Output=")]
		FileReference? OutputFile = null;

		/// <summary>
		/// Write out program targets, has presedence over IncludeAllTargets.
		/// </summary>
		[CommandLine("-DontIncludeProgramTargets", Value = "false")]
		bool bIncludeProgramTargets = true;

		/// <summary>
		/// Write out all targets, even if a default is specified in the BuildSettings section of the Default*.ini files.
		/// </summary>
		[CommandLine("-IncludeAllTargets")]
		bool bIncludeAllTargets = false;

		/// <summary>
		/// Write all targets from parent assemblies (useful for EngineRules which can be chained)
		/// </summary>
		[CommandLine("-IncludeParentAssembly", Value = "true")]
		[CommandLine("-DontIncludeParentAssembly", Value = "false")]
		bool bIncludeParentAssembly = true;
#pragma warning restore IDE0044

		/// <summary>
		/// Execute the mode
		/// </summary>
		/// <param name="arguments">Command line arguments</param>
		/// <returns></returns>
		/// <param name="logger"></param>
		public async Task<int> ExecuteAsync(CommandLineArguments arguments, ILogger logger)
		{
			arguments.ApplyTo(this);

			// Create the build configuration object, and read the settings
			BuildConfiguration buildConfiguration = new BuildConfiguration();
			XmlConfig.ApplyTo(buildConfiguration);
			arguments.ApplyTo(buildConfiguration);

			// Create the log file, and flush the startup listener to it
			if (!arguments.HasOption("-NoLog") && !Log.HasFileWriter())
			{
				FileReference LogFile = FileReference.Combine(Unreal.EngineProgramSavedDirectory, "UnrealBuildTool", "Log_QueryTargets.txt");
				Log.AddFileWriter("DefaultLogTraceListener", LogFile);
			}
			else
			{
				Log.RemoveStartupTraceListener();
			}

			// Ensure the path to the output file is valid
			OutputFile ??= GetDefaultOutputFile(ProjectFile);

			// Create the rules assembly
			RulesAssembly assembly;
			string mutexName = GlobalSingleInstanceMutex.GetUniqueMutexForPath("UnrealBuildTool_QueryMode_UEBuildTarget-Create", Unreal.RootDirectory);
			using (await SingleInstanceMutex.AcquireAsync(mutexName))
			{
				assembly = ProjectFile != null
					? RulesCompiler.CreateProjectRulesAssembly(ProjectFile, buildConfiguration.bUsePrecompiled, buildConfiguration.bSkipRulesCompile, buildConfiguration.bForceRulesCompile, logger)
					: RulesCompiler.CreateEngineRulesAssembly(buildConfiguration.bUsePrecompiled, buildConfiguration.bSkipRulesCompile, buildConfiguration.bForceRulesCompile, logger);
			}

			// Write information about these targets
			WriteTargetInfo(ProjectFile, assembly, OutputFile, arguments, logger, bIncludeAllTargets, bIncludeParentAssembly, bIncludeProgramTargets);
			logger.LogInformation("Written {OutputFile}", OutputFile);
			return 0;
		}

		/// <summary>
		/// Gets the path to the target info output file
		/// </summary>
		/// <param name="ProjectFile">Project file being queried</param>
		/// <returns>Path to the output file</returns>
		public static FileReference GetDefaultOutputFile(FileReference? ProjectFile)
		{
			return FileReference.Combine(ProjectFile?.Directory ?? Unreal.EngineDirectory, "Intermediate", "TargetInfo.json");
		}

		/// <summary>
		/// Writes information about the targets in an assembly to a file
		/// </summary>
		/// <param name="projectFile">The project file for the targets being built</param>
		/// <param name="assembly">The rules assembly for this target</param>
		/// <param name="outputFile">Output file to write to</param>
		/// <param name="arguments"></param>
		/// <param name="logger">Logger for output</param>
		/// <param name="bIncludeAllTargets">Include all targets even if a default target is specified for a given target type.</param>
		/// <param name="bIncludeParentAssembly">Include all targets from parent assemblies (useful for EngineRules which can be chained)</param>
		/// <param name="bIncludeProgramTargets">Include or exclude all program targets, has presedence over bIncludeAllTargets.</param>
		public static void WriteTargetInfo(FileReference? projectFile, RulesAssembly assembly, FileReference outputFile, CommandLineArguments arguments, ILogger logger,
			bool bIncludeAllTargets = true, bool bIncludeParentAssembly = false, bool bIncludeProgramTargets = true)
		{
			// Construct all the targets in this assembly
			List<string> targetNames = new List<string>();
			assembly.GetAllTargetNames(targetNames, bIncludeParentAssembly);

			ConcurrentDictionary<TargetType, string?> defaultTargetNameCache = [];
			ConcurrentBag<(string name, TargetType type, FileReference? path, bool? isDefaultTarget)> entries = [];
			Parallel2.ForEach(targetNames, (targetName) =>
			{
				// skip target rules that are platform extension or platform group specializations
				string[] targetPathSplit = targetName.Split('_', StringSplitOptions.RemoveEmptyEntries);
				if (targetPathSplit.Length > 1 && (UnrealTargetPlatform.IsValidName(targetPathSplit.Last()) || UnrealPlatformGroup.IsValidName(targetPathSplit.Last())))
				{
					return;
				}

				// Construct the rules object
				TargetRules targetRules;
				try
				{
					UnrealArchitectures architectures = UnrealArchitectureConfig.ForPlatform(BuildHostPlatform.Current.Platform).ActiveArchitectures(projectFile, targetName);
					targetRules = assembly.CreateTargetRules(targetName, BuildHostPlatform.Current.Platform, UnrealTargetConfiguration.Development, architectures, projectFile, arguments, logger, ValidationOptions: TargetRulesValidationOptions.ValidateNothing);
				}
				catch (Exception ex)
				{
					logger.LogWarning("Unable to construct target rules for {TargetName}", targetName);
					logger.LogDebug(ex, "{Ex}", ExceptionUtils.FormatException(ex));
					return;
				}

				bool? bIsDefaultTarget = null;

				// Program targets are controlled by bIncludeProgramTargets which has presedence over bIncludeAllTargets
				if (targetRules.Type == TargetType.Program)
				{
					if (!bIncludeProgramTargets)
					{
						return;
					}
				}
				// Check if default target for Client, Editor, Game and Server targets
				else
				{
					if (projectFile != null)
					{
						string? defaultTargetName = defaultTargetNameCache.GetOrAdd(
							targetRules.Type,
							t => ProjectFileGenerator.GetProjectDefaultTargetNameForType(projectFile.Directory, t));
						if (defaultTargetName != null)
						{
							bIsDefaultTarget = defaultTargetName == targetName;
						}
					}

					// If we don't want all targets, skip over non-defaults.
					if (!bIncludeAllTargets && bIsDefaultTarget.HasValue && !bIsDefaultTarget.Value)
					{
						return;
					}
				}

				// Get the path to the target
				FileReference? path = assembly.GetTargetFileName(targetName);

				entries.Add((targetName, targetRules.Type, path, bIsDefaultTarget));
			});

			// Generate json
			using System.IO.StringWriter stringWriter = new();
			using (JsonWriter writer = new(stringWriter))
			{
				writer.WriteObjectStart();
				writer.WriteArrayStart("Targets");
				foreach ((string name, TargetType type, FileReference? path, bool? isDefaultTarget) in entries.OrderBy(x => x.path))
				{
					// Write the target info
					writer.WriteObjectStart();
					writer.WriteValue("Name", name);
					if (path != null)
					{
						writer.WriteValue("Path", path.MakeRelativeTo(outputFile.Directory));
					}
					writer.WriteValue("Type", type.ToString());

					if (bIncludeAllTargets && isDefaultTarget.HasValue)
					{
						writer.WriteValue("DefaultTarget", isDefaultTarget.Value);
					}
					writer.WriteObjectEnd();
				}
				writer.WriteArrayEnd();
				writer.WriteObjectEnd();
			}

			// Write the output file
			DirectoryReference.CreateDirectory(outputFile.Directory);
			FileReference.WriteAllTextIfDifferent(outputFile, stringWriter.ToString());
			// even if the contents don't change, we need to update the last write time (this is safer to do in parallel, unlike WriteAllText)
			FileReference.SetLastWriteTimeUtc(outputFile, DateTime.UtcNow);
		}
	}
}
