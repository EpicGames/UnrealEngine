// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Collections.Immutable;
using System.IO;
using System.Reflection;
using System.Text.Encodings.Web;
using System.Text.Json;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildTool.Actions;
using UnrealBuildTool.Actions.ResultStore;
using UnrealBuildTool.Storage.Impl;

namespace UnrealBuildTool.Modes;

internal sealed class ReportBuildResultsMode : IToolMode<ReportBuildResultsMode>
{
	public static string Name => "ReportBuildResults";
	public static ToolModeOptions Options => ToolModeOptions.XmlConfig | ToolModeOptions.BuildPlatforms;

	public async Task<int> ExecuteAsync(CommandLineArguments arguments, ILogger logger)
	{
		BuildConfiguration buildConfiguration = new();
		XmlConfig.ApplyTo(buildConfiguration);
		arguments.ApplyTo(buildConfiguration);

		List<TargetDescriptor> targetDescriptors = TargetDescriptor.ParseCommandLine(arguments, buildConfiguration, logger);

		await using ContentAddressableActionResultStore resultStore = new(new FileSystemStorageProvider(), logger);

		JsonSerializerOptions options = new JsonSerializerOptions()
		{
			WriteIndented = true,
			// It's very unlikely that the report would be consumed in a way that would be unsafe by using the below encoding,
			// and the default "safe" behaviour severely hurts readability.
			Encoder = JavaScriptEncoder.UnsafeRelaxedJsonEscaping,
		};

		foreach (TargetDescriptor targetDescriptor in targetDescriptors)
		{
			OriginalTargetDescriptor originalTargetDescriptor = targetDescriptor.Original;
			ReportBuildResultsArgs args = originalTargetDescriptor.AdditionalArguments.ApplyTo<ReportBuildResultsArgs>(logger);

			originalTargetDescriptor = RemoveReportArgsFromOriginalTargetDescriptor(originalTargetDescriptor);

			ActionResultManifest? manifest = await resultStore.LoadManifestAsync(originalTargetDescriptor, CancellationToken.None);
			if (manifest is null)
			{
				logger.LogInformation("Manifest for target '{Target}' is not available", targetDescriptor);
				continue;
			}

			await using Stream stream = args.OutputFile is not null
				? FileReference.Open(new(args.OutputFile), FileMode.Create, FileAccess.Write, FileShare.Read)
				: Console.OpenStandardOutput();

			ActionResultManifest transformedManifest = TransformManifest(manifest, args);

			await JsonSerializer.SerializeAsync(stream, transformedManifest, options);
			Console.WriteLine();
		}

		return 0;
	}

	/// <summary>
	/// Remove the report args as they would affect the hash calculation.
	/// </summary>
	/// <remarks>
	/// Note that this relies on the report args being unique so we don't accidentally remove build args, hence them
	/// all being prefixed with `Report` rather than using shorter, more generic names.
	/// </remarks>
	private static OriginalTargetDescriptor RemoveReportArgsFromOriginalTargetDescriptor(OriginalTargetDescriptor descriptor)
	{
		CommandLineArguments args = descriptor.AdditionalArguments;
		foreach (PropertyInfo propertyInfo in typeof(ReportBuildResultsArgs).GetProperties())
		{
			CommandLineAttribute commandLineAttribute = propertyInfo.GetCustomAttribute<CommandLineAttribute>()!;
			string prefix = commandLineAttribute.Prefix!;
			if (args.HasOption(prefix))
			{
				args = args.Remove(prefix, out _);
			}
		}

		return descriptor with { AdditionalArguments = args };
	}

	private static ActionResultManifest TransformManifest(ActionResultManifest manifest, ReportBuildResultsArgs args)
	{
		// Note that we swap the order of comparison because we want to sort descending to highlight the largest values.
		ImmutableArray<ActionResult> sortedResults = args.OrderBy switch
		{
			OrderByValue.None          => manifest.Results,
			OrderByValue.ExecutionTime => manifest.Results.Sort((a, b) => b.Metrics.ExecutionTime.CompareTo(a.Metrics.ExecutionTime)),
			OrderByValue.ProcessorTime => manifest.Results.Sort((a, b) => b.Metrics.ProcessorTime.CompareTo(a.Metrics.ProcessorTime)),
			OrderByValue.Memory        => manifest.Results.Sort((a, b) => b.Metrics.PeakMemoryBytes.CompareTo(a.Metrics.PeakMemoryBytes)),
			_                         => throw new NotSupportedException(),
		};

		int length = args.Limit == 0 ? sortedResults.Length : Math.Min(sortedResults.Length, args.Limit);
		ImmutableArray<ActionResult> limitedResults = sortedResults.Slice(0, length);

		return new(manifest.TargetDescriptor, limitedResults);
	}

	private enum OrderByValue
	{
		None,
		ExecutionTime,
		ProcessorTime,
		Memory,
	}

	private class ReportBuildResultsArgs
	{
		[CommandLine("-ReportOutputFile=")]
		public string? OutputFile { get; set; }

		[CommandLine("-ReportOrderBy=")]
		public OrderByValue OrderBy { get; set; }

		[CommandLine("-ReportLimit=")]
		public int Limit { get; set; }
	}
}
