// Copyright Epic Games, Inc. All Rights Reserved.

using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace UnrealBuildTool
{
	/// <summary>
	/// Register all platforms (and in the process, configure all autosdks)
	/// </summary>
	internal sealed class SetupPlatforms : IToolMode<SetupPlatforms>
	{
		public static string Name => "SetupPlatforms";
		public static ToolModeOptions Options => ToolModeOptions.XmlConfig | ToolModeOptions.BuildPlatforms | ToolModeOptions.SingleInstance;

		/// <summary>
		/// Execute the tool mode
		/// </summary>
		/// <param name="Arguments">Command line arguments</param>
		/// <returns>Exit code</returns>
		/// <param name="Logger"></param>
		public Task<int> ExecuteAsync(CommandLineArguments Arguments, ILogger Logger)
		{
			return Task.FromResult(0);
		}
	}
}
