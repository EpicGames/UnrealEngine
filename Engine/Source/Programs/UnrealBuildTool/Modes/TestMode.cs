// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Reflection;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// Builds low level tests on one or more targets.
	/// </summary>
	internal sealed class TestMode : IToolMode<TestMode>
	{
		public static string Name => "Test";
		public static ToolModeOptions Options => ToolModeOptions.XmlConfig | ToolModeOptions.BuildPlatforms | ToolModeOptions.SingleInstance | ToolModeOptions.ShowExecutionTime;

		private readonly string ENGINE_MODULE = "Engine";
		private readonly string APPLICATION_CORE_MODULE = "ApplicationCore";
		private readonly string COREUOBJECT_MODULE = "CoreUObject";
		private readonly string EDITOR_MODULE = "UnrealEd";

#pragma warning disable IDE0044 // Make field readonly - these private static fields are set by command-line parsing.
		/// <summary>
		/// Whether we're building implicit tests (default is explicit)
		/// </summary>
		[CommandLine("-Implicit")]
		bool bImplicit = false;

		/// <summary>
		/// Whether we're cleaning tests instead of just building
		/// </summary>
		[CommandLine("-CleanTests")]
		bool bCleanTests = false;

		/// <summary>
		/// Whether we're rebuilding tests instead of just building
		/// </summary>
		[CommandLine("-RebuildTests")]
		bool bRebuildTests = false;

		/// <summary>
		/// Updates tests' build graph metadata files for explicit tests
		/// </summary>
		[CommandLine("-GenerateMetadata")]
		bool bGenerateMetadata = false;

        /// <summary>
        /// Specific project file to run tests on or update test metadata
        /// </summary>
        [CommandLine("-Project")]
        FileReference? ProjectFile = null;

		/// <summary>
		/// Specific modules to be selected as the only test bearers, as a comma separated string
		/// </summary>
		[CommandLine("-SelectModules")]
		string SelectModules = String.Empty;
#pragma warning restore IDE0044

		/// <summary>
		/// Main entry point
		/// </summary>
		/// <param name="Arguments">Command-line arguments</param>
		/// <param name="Logger"></param>
		public async Task<int> ExecuteAsync(CommandLineArguments Arguments, ILogger Logger)
		{
			Arguments.ApplyTo(this);

			// Create the build configuration object, and read the settings
			BuildConfiguration BuildConfiguration = new BuildConfiguration();
			Arguments.ApplyTo(BuildConfiguration);
			XmlConfig.ApplyTo(BuildConfiguration);

			if (bGenerateMetadata)
			{
				Logger.LogInformation("Retrieving rules...");

                RulesAssembly RulesAssembly;

                if (ProjectFile != null)
                {
                    RulesAssembly = RulesCompiler.CreateProjectRulesAssembly(ProjectFile, BuildConfiguration.bUsePrecompiled, BuildConfiguration.bSkipRulesCompile, BuildConfiguration.bForceRulesCompile, Logger);
                }
                else
                {
                    RulesAssembly = RulesCompiler.CreateEngineRulesAssembly(BuildConfiguration.bUsePrecompiled, BuildConfiguration.bSkipRulesCompile, BuildConfiguration.bForceRulesCompile, Logger);
                }
				GenerateMetadataForTestsInAssembly(RulesAssembly, Logger);
				if (ProjectFile == null && RulesAssembly.Parent != null)
				{
					// Non-program rules (includes plugins)
					GenerateMetadataForTestsInAssembly(RulesAssembly.Parent, Logger);
				}

				return 0;
			}

			// Parse all the targets being built
			List<TargetDescriptor> TargetDescriptors = TargetDescriptor.ParseCommandLine(Arguments, BuildConfiguration, Logger);

			if (TargetDescriptors.Count > 1)
			{
				throw new BuildException($"Only one target must be specified in test mode but we got {String.Join(", ", TargetDescriptors.Select(T => T.Name))} instead.");
			}

			TargetDescriptor TestTargetDescriptor = TargetDescriptors.First();

			// Analyzing dependency graph before any build commands to automatically set compilation flags.
			AnalyzeDependencyGraph(TestTargetDescriptor, BuildConfiguration, Logger);

			// Reset arguments and add a dummy argument that will invalidate the makefile if a build had been done outside of test mode
			List<string> AdditionalArgs = [.. TestTargetDescriptor.AdditionalArguments.AsEnumerable(), "-TestMode"];
			if (!String.IsNullOrWhiteSpace(SelectModules))
			{
				AdditionalArgs.Add($"-SelectModules={SelectModules}");
			}
			TestTargetDescriptor.AdditionalArguments = new(AdditionalArgs.ToArray());
			

			if (bImplicit)
			{
				TestTargetDescriptor.Name += "Tests";
				TestTargetDescriptor.IsTestsTarget = true;
			}

			using (ISourceFileWorkingSet WorkingSet = SourceFileWorkingSet.Create(Unreal.RootDirectory, new HashSet<DirectoryReference>(), Logger))
			{
				if (bCleanTests)
				{
					new CleanMode().Clean(TargetDescriptors, BuildConfiguration, Logger);
				}
				else if (bRebuildTests)
				{
					new CleanMode().Clean(TargetDescriptors, BuildConfiguration, Logger);
					await BuildMode.BuildAsync(TargetDescriptors, BuildConfiguration, WorkingSet, BuildOptions.None, null, new(), Logger);
				}
				else
				{
					await BuildMode.BuildAsync(TargetDescriptors, BuildConfiguration, WorkingSet, BuildOptions.None, null, new(), Logger);
				}
			}
			return 0;
		}

		private static void GenerateMetadataForTestsInAssembly(RulesAssembly Assembly, ILogger Logger)
		{
			Type ModuleRulesType = typeof(TestModuleRules);
			Dictionary<Type, HashSet<Type>> InheritedGroups = new Dictionary<Type, HashSet<Type>>();
			HashSet<Type> AllTestModules = Assembly.GetTypes()
				.Where(t => t.IsAssignableTo(ModuleRulesType))
				.ToHashSet();
			foreach(Type Type in AllTestModules)
			{
				// Find root class that's not TestModuleRules, the while loop always stops because AllTestModules contains only types that inherit from TestModuleRules
				Type? RootBaseType = Type.BaseType;
				while(true)
				{
					if (RootBaseType != null)
					{
						if (RootBaseType.Name == ModuleRulesType.Name)
						{
							break;
						}
						else if (RootBaseType.BaseType != null && RootBaseType.BaseType.Name != ModuleRulesType.Name)
						{
							RootBaseType = RootBaseType.BaseType;
						}
						else
						{
							break;
						}
					}
					else
					{
						break;
					}
				}
				// Add to inheritance group if it's a derived class
				if (RootBaseType != null && RootBaseType.Name != Type.Name && AllTestModules.Contains(RootBaseType))
				{
					if (!InheritedGroups.ContainsKey(RootBaseType))
					{
						InheritedGroups[RootBaseType] = new HashSet<Type>();
					}
					InheritedGroups[RootBaseType].Add(Type);
					AllTestModules.Remove(Type); // Removed so that we use InheritedGroups to call static constructors of inherited types to completely initialize the TestMetadata field
				}
			}
			foreach (Type TestModuleType in AllTestModules)
			{
				MethodInfo? UpdateBuildGraphMetadata = FindMethodUpwards(TestModuleType, "UpdateBuildGraphMetadata", BindingFlags.Static | BindingFlags.NonPublic); 

				// Call static constructor if exists, sets TestMetadata properties (can be set from derived module rules classes, including for NDA platforms
				if (UpdateBuildGraphMetadata != null && TestModuleType.TypeInitializer != null)
				{
					Logger.LogInformation("Generating test metadata for module {TestModule}", TestModuleType.FullName);

					// Initialize TestMetadata in base class using static constructor
					TestModuleType.TypeInitializer.Invoke(null, null);
					if (InheritedGroups.ContainsKey(TestModuleType))
					{
						foreach (Type InheritedTestModuleType in InheritedGroups[TestModuleType])
						{
							if (InheritedTestModuleType.TypeInitializer != null)
							{
								// Update TestMetadata on inherited types through static constructor.
								// UpdateBuildGraphMetadata will have a complete TestMetadata to act on.
								Logger.LogInformation("Subclass found, update test metadata: {InheritedTestModule}", InheritedTestModuleType.FullName);
								InheritedTestModuleType.TypeInitializer.Invoke(null, null);
							}
						}
					}
					FieldInfo? TestMetadataField = FindFieldUpwards(TestModuleType, "TestMetadata", BindingFlags.Static | BindingFlags.NonPublic);
					if (TestMetadataField != null)
					{
						TestModuleRules.Metadata MetadataValue = (TestModuleRules.Metadata)TestMetadataField.GetValue(null)!;
						if (MetadataValue != null && !String.IsNullOrEmpty(MetadataValue.TestName))
						{
							UpdateBuildGraphMetadata.Invoke(null,
                                new object[] {
                                    MetadataValue,
                                    Path.GetDirectoryName(RulesCompiler.GetFileNameFromType(TestModuleType)) ?? "",
                                    TestModuleType.FullName ?? "",
									Logger
								});
						}
					}
				}
			}
		}

		private static MethodInfo? FindMethodUpwards(Type TestType, string MethodName, BindingFlags Flags)
		{
			Type CurrentLevelType = TestType;
			MethodInfo? FindMethod;
			while (CurrentLevelType != null &&  CurrentLevelType != typeof(ModuleRules))
			{
				FindMethod = CurrentLevelType.GetMethod(MethodName, Flags);
				if (FindMethod == null)
				{
					CurrentLevelType = CurrentLevelType.BaseType!;
				}
				else
				{
					return FindMethod;
				}
			}
			return null;
		}

		private static FieldInfo? FindFieldUpwards(Type TestType, string FieldName, BindingFlags Flags)
		{
			Type CurrentLevelType = TestType;
			FieldInfo? FindField;
			while (CurrentLevelType != null && CurrentLevelType != typeof(ModuleRules))
			{
				FindField = CurrentLevelType.GetField(FieldName, Flags);
				if (FindField == null)
				{
					CurrentLevelType = CurrentLevelType.BaseType!;
				}
				else
				{
					return FindField;
				}
			}
			return null;
		}

		private void AnalyzeDependencyGraph(TargetDescriptor TargetDescriptor, BuildConfiguration BuildConfiguration, ILogger Logger)
		{
			UEBuildTarget target = UEBuildTarget.Create(TargetDescriptor, BuildConfiguration, Logger);
			DirectoryReference.CreateDirectory(target.ReceiptFileName.Directory);

			HashSet<string> moduleNames = [.. target.Binaries.SelectMany(x => x.Modules).Select(x => x.Name)];

			bool updated = false;
			if (moduleNames.Contains(ENGINE_MODULE) && TestTargetRules.bTestsRequireEngine == false)
			{
				TestTargetRules.bTestsRequireEngine = true;
				Logger.LogInformation("{Module} module found in dependency graph, will compile against it.", ENGINE_MODULE);
				updated = true;
			}
			else if (moduleNames.Contains(APPLICATION_CORE_MODULE) && !TestTargetRules.bTestsRequireApplicationCore)
			{
				TestTargetRules.bTestsRequireApplicationCore = true;
				Logger.LogInformation("{Module} module found in dependency graph, will compile against it.", APPLICATION_CORE_MODULE);
				updated = true;
			}
			else if (moduleNames.Contains(COREUOBJECT_MODULE) && !TestTargetRules.bTestsRequireCoreUObject)
			{
				TestTargetRules.bTestsRequireCoreUObject = true;
				Logger.LogInformation("{Module} module found in dependency graph, will compile against it.", COREUOBJECT_MODULE);
				updated = true;
			}
			else if (moduleNames.Contains(EDITOR_MODULE) && !TestTargetRules.bTestsRequireEditor)
			{
				TestTargetRules.bTestsRequireEditor = true;
				Logger.LogInformation("{Module} module found in dependency graph, will compile against it.", EDITOR_MODULE);
				updated = true;
			}
			// If a property changes scan again for additional dependencies
			if (updated)
			{
				AnalyzeDependencyGraph(TargetDescriptor, BuildConfiguration, Logger);
			}
		}
	}
}

