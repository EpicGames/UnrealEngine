// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace UnrealBuildTool
{
	/// <summary>
	/// Validates the various platforms to determine if they are ready for building
	/// </summary>
	internal sealed class ValidatePlatformsMode : IToolMode<ValidatePlatformsMode>
	{
		public static string Name => "ValidatePlatforms";
		public static ToolModeOptions Options => ToolModeOptions.XmlConfig | ToolModeOptions.BuildPlatformsForValidation | ToolModeOptions.SingleInstance;

#pragma warning disable IDE0044 // Make field readonly - these private static fields are set by command-line parsing.
		/// <summary>
		/// Platforms to validate
		/// </summary>
		[CommandLine("-Platforms=", ListSeparator = '+')]
		HashSet<UnrealTargetPlatform> Platforms = new HashSet<UnrealTargetPlatform>();

		/// <summary>
		/// Whether to validate all platforms
		/// </summary>
		[CommandLine("-AllPlatforms")]
		bool bAllPlatforms = false;

		/// <summary>
		/// Whether to output SDK versions.
		/// </summary>
		[CommandLine("-OutputSDKs")]
		bool bOutputSDKs = false;
#pragma warning restore IDE0044

		/// <summary>
		/// Executes the tool with the given arguments
		/// </summary>
		/// <param name="Arguments">Command line arguments</param>
		/// <returns>Exit code</returns>
		/// <param name="Logger"></param>
		public Task<int> ExecuteAsync(CommandLineArguments Arguments, ILogger Logger)
		{
			// Output a message if there are any arguments that are still unused
			Arguments.ApplyTo(this);
			Arguments.CheckAllArgumentsUsed();

			// If the -AllPlatforms argument is specified, add all the known platforms into the list
			if (bAllPlatforms)
			{
				Platforms.UnionWith(UnrealTargetPlatform.GetValidPlatforms());
			}

			// Output a line for each registered platform
			List<string> ExceptionMessages = new List<string>();
			foreach (UnrealTargetPlatform Platform in Platforms)
			{
				UEBuildPlatform.TryGetBuildPlatform(Platform, out UEBuildPlatform? BuildPlatform);
				string PlatformSDKString = "";
				if (bOutputSDKs)
				{
					PlatformSDKString = "<UNKNOWN>";
					if (BuildPlatform != null)
					{
						try
						{
							PlatformSDKString = UEBuildPlatform.GetSDK(Platform)!.GetMainVersion();
						}
						catch (Exception Ex)
						{
							ExceptionMessages.Add(Ex.Message.TrimEnd());
						}
					}
				}

				if (BuildPlatform != null && BuildPlatform.HasRequiredSDKsInstalled() == SDKStatus.Valid)
				{
					Logger.LogInformation("##PlatformValidate: {Platform} VALID {PlatformSdkString}", Platform.ToString(), PlatformSDKString);
				}
				else
				{
					Logger.LogInformation("##PlatformValidate: {Platform} INVALID {PlatformSdkString}", Platform.ToString(), PlatformSDKString);
				}
			}
			foreach (string Message in ExceptionMessages)
			{
				Logger.LogInformation("{Message}", Message);
			}
			return Task.FromResult(0);
		}
	}
}
