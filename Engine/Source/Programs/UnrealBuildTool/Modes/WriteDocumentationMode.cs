// Copyright Epic Games, Inc. All Rights Reserved.

using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace UnrealBuildTool
{
	/// <summary>
	/// Generates documentation from reflection data
	/// </summary>
	internal sealed class WriteDocumentationMode : IToolMode<WriteDocumentationMode>
	{
		public static string Name => "WriteDocumentation";
		public static ToolModeOptions Options => ToolModeOptions.None;

		/// <summary>
		/// Enum for the type of documentation to generate
		/// </summary>
		enum DocumentationType
		{
			BuildConfiguration,
			ModuleRules,
			TargetRules,
		}

#pragma warning disable IDE0044 // Make field readonly - these private static fields are set by command-line parsing.
		/// <summary>
		/// Type of documentation to generate
		/// </summary>
		[CommandLine(Required = true)]
		DocumentationType Type = DocumentationType.BuildConfiguration;

		/// <summary>
		/// The HTML file to write to
		/// </summary>
		[CommandLine(Required = true)]
		FileReference OutputFile = null!;
#pragma warning restore IDE0044

		/// <summary>
		/// Entry point for this command
		/// </summary>
		/// <returns></returns>
		public Task<int> ExecuteAsync(CommandLineArguments Arguments, ILogger Logger)
		{
			Arguments.ApplyTo(this);
			Arguments.CheckAllArgumentsUsed();

			switch (Type)
			{
				case DocumentationType.BuildConfiguration:
					XmlConfig.WriteDocumentation(OutputFile, Logger);
					break;
				case DocumentationType.ModuleRules:
					RulesDocumentation.WriteDocumentation(typeof(ModuleRules), OutputFile, Logger);
					break;
				case DocumentationType.TargetRules:
					RulesDocumentation.WriteDocumentation(typeof(TargetRules), OutputFile, Logger);
					break;
				default:
					throw new BuildException("Invalid documentation type: {0}", Type);
			}
			return Task.FromResult(0);
		}
	}
}
