// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using System.Text.Json;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// Parameters for the WriteMetadata mode
	/// </summary>
	/// <param name="ProjectFile">The project file</param>
	/// <param name="VersionFile">Output location for the version file</param>
	/// <param name="Version">The new version to write. This should only be set on the engine step</param>
	/// <param name="ReceiptFile">Output location for the target file</param>
	/// <param name="Receipt">The new receipt to write. This should only be set on the target step</param>
	/// <param name="FileToManifest">Map of module manifest filenames to their location on disk</param>
	/// <param name="FileToLoadOrderManifest">Map of load order manifest filenames to their locations on disk (generally, at most one load oder manifest is expected)</param>
	/// <param name="MergeModules">If this target uses merged modules. When true module manifests will not be merged with on disk version when buildids match</param>
	[Serializable]
	record WriteMetadataTargetInfo(FileReference? ProjectFile, FileReference? VersionFile, BuildVersion? Version, FileReference? ReceiptFile, TargetReceipt? Receipt, Dictionary<FileReference, ModuleManifest> FileToManifest, Dictionary<FileReference, LoadOrderManifest>? FileToLoadOrderManifest, bool MergeModules)
	{
		/// <summary>
		/// Map of load order manifest filenames to their locations on disk (generally, at most one load order manifest is expected).
		/// </summary>
		public Dictionary<FileReference, LoadOrderManifest> FileToLoadOrderManifest { get; init; } = FileToLoadOrderManifest ?? [];
	}

	/// <summary>
	/// Writes all metadata files at the end of a build (receipts, version files, etc...). This is implemented as a separate mode to allow it to be done as part of the action graph.
	/// </summary>
	internal sealed class WriteMetadataMode : IToolMode<WriteMetadataMode>
	{
		public static string Name => "WriteMetadata";
		public static ToolModeOptions Options => ToolModeOptions.None;

		/// <summary>
		/// Version number for output files. This is not used directly, but can be appended to command-line invocations of the tool to ensure that actions to generate metadata are updated if the output format changes. 
		/// The action graph is regenerated whenever UBT is rebuilt, so this should always match.
		/// </summary>
		public const int CurrentVersionNumber = 2;

		/// <summary>
		/// Serializer options for serializing <see cref="WriteMetadataTargetInfo"/> instances.
		/// </summary>
		internal static readonly JsonSerializerOptions SerializerOptions = new()
		{
			IncludeFields = true,
		};

		/// <summary>
		/// Execute the command
		/// </summary>
		/// <param name="Arguments">Command line arguments</param>
		/// <returns>Exit code</returns>
		/// <param name="Logger"></param>
		public async Task<int> ExecuteAsync(CommandLineArguments Arguments, ILogger Logger)
		{
			// Acquire a different mutex to the regular UBT instance, since this mode will be called as part of a build. We need the mutex to ensure that building two modular configurations 
			// in parallel don't clash over writing shared *.modules files (eg. DebugGame and Development editors).
			string MutexName = GlobalSingleInstanceMutex.GetUniqueMutexForPath("UnrealBuildTool_WriteMetadata", Unreal.RootDirectory);
			using (await SingleInstanceMutex.AcquireAsync(MutexName))
			{
				return await ExecuteInternal(Arguments, Logger);
			}
		}

		/// <summary>
		/// Execute the command, having obtained the appropriate mutex
		/// </summary>
		/// <param name="Arguments">Command line arguments</param>
		/// <param name="Logger">Logger for output</param>
		/// <returns>Exit code</returns>
		private static async Task<int> ExecuteInternal(CommandLineArguments Arguments, ILogger Logger)
		{
			// Read the target info
			FileReference FilePath = Arguments.GetFileReference("-Input=");
			WriteMetadataTargetInfo TargetInfo;
			using (FileStream FileStream = FileReference.Open(FilePath, FileMode.Open, FileAccess.Read, FileShare.Read))
			{
				WriteMetadataTargetInfo? MaybeTargetInfo = await JsonSerializer.DeserializeAsync<WriteMetadataTargetInfo>(FileStream, SerializerOptions);
				TargetInfo = MaybeTargetInfo ?? throw new BuildLogEventException("Target info file '{FilePath}' was invalid", FilePath);
			}

			bool bNoManifestChanges = Arguments.HasOption("-NoManifestChanges");
			int VersionNumber = Arguments.GetInteger("-Version=");
			Arguments.CheckAllArgumentsUsed();

			// Make sure the version number is correct
			if (VersionNumber != CurrentVersionNumber)
			{
				throw new BuildException("Version number to WriteMetadataMode is incorrect (expected {0}, got {1})", CurrentVersionNumber, VersionNumber);
			}

			// Get the build id to use
			string? BuildId;
			if (TargetInfo.Version != null && !String.IsNullOrEmpty(TargetInfo.Version.BuildId))
			{
				BuildId = TargetInfo.Version.BuildId;
			}
			else if (TargetInfo.Receipt != null && !String.IsNullOrEmpty(TargetInfo.Receipt.Version.BuildId))
			{
				BuildId = TargetInfo.Receipt.Version.BuildId;
			}
			else if (TargetInfo.VersionFile != null && BuildVersion.TryRead(TargetInfo.VersionFile, out BuildVersion? PrevVersion) && CanRecycleBuildId(PrevVersion.BuildId, TargetInfo.FileToManifest, Logger))
			{
				BuildId = PrevVersion.BuildId;
			}
			else
			{
				BuildId = Guid.NewGuid().ToString();
			}

			// Read all the existing manifests and merge them into the new ones if they have the same build id and not merging modules
			if (!TargetInfo.MergeModules)
			{
				foreach (KeyValuePair<FileReference, ModuleManifest> Pair in TargetInfo.FileToManifest)
				{
					if (TryReadManifest(Pair.Key, Logger, out ModuleManifest? SourceManifest) && SourceManifest.BuildId == BuildId)
					{
						MergeManifests(SourceManifest, Pair.Value);
					}
				}
			}

			// Update the build id in all the manifests, and write them out
			foreach (KeyValuePair<FileReference, ModuleManifest> Pair in TargetInfo.FileToManifest)
			{
				FileReference ManifestFile = Pair.Key;
				if (!Unreal.IsFileInstalled(ManifestFile))
				{
					ModuleManifest Manifest = Pair.Value;
					Manifest.BuildId = BuildId ?? String.Empty;

					if (!FileReference.Exists(ManifestFile))
					{
						// If the file doesn't already exist, just write it out
						DirectoryReference.CreateDirectory(ManifestFile.Directory);
						Manifest.Write(ManifestFile);
					}
					else
					{
						// Otherwise write it to a buffer first
						string OutputText;
						using (StringWriter Writer = new StringWriter())
						{
							Manifest.Write(Writer);
							OutputText = Writer.ToString();
						}

						// Check if the manifest has changed. Note that if a manifest is out of date, we should have generated a new build id causing the contents to differ.
						if (bNoManifestChanges)
						{
							string CurrentText = await FileReference.ReadAllTextAsync(ManifestFile);
							if (CurrentText != OutputText)
							{
								Logger.LogError(
									"Build modifies {File}. This is not permitted. Before:\n    {OldFile}\nAfter:\n    {NewFile}",
									ManifestFile,
									CurrentText.Replace("\n", "\n    ", StringComparison.Ordinal),
									OutputText.Replace("\n", "\n    ", StringComparison.Ordinal));
							}
						}

						// Write it to disk
						await FileReference.WriteAllTextAsync(ManifestFile, OutputText);
					}
				}
			}

			// Write load order manifests out.
			foreach (KeyValuePair<FileReference, LoadOrderManifest> Pair in TargetInfo.FileToLoadOrderManifest)
			{
				FileReference ManifestFile = Pair.Key;
				if (!Unreal.IsFileInstalled(ManifestFile))
				{
					LoadOrderManifest Manifest = Pair.Value;

					if (!FileReference.Exists(ManifestFile))
					{
						// If the file doesn't already exist, just write it out
						DirectoryReference.CreateDirectory(ManifestFile.Directory);
						Manifest.Write(ManifestFile);
					}
					else
					{
						// Otherwise write it to a buffer first
						string OutputText;
						using (StringWriter Writer = new StringWriter())
						{
							Manifest.Write(Writer);
							OutputText = Writer.ToString();
						}

						// Check if the manifest has changed. Note that if a manifest is out of date, we should have generated a new build id causing the contents to differ.
						if (bNoManifestChanges)
						{
							string CurrentText = await FileReference.ReadAllTextAsync(ManifestFile);
							if (CurrentText != OutputText)
							{
								Logger.LogError(
									"Build modifies {File}. This is not permitted. Before:\n    {OldFile}\nAfter:\n    {NewFile}",
									ManifestFile,
									CurrentText.Replace("\n", "\n    ", StringComparison.Ordinal),
									OutputText.Replace("\n", "\n    ", StringComparison.Ordinal));
							}
						}

						// Write it to disk
						await FileReference.WriteAllTextAsync(ManifestFile, OutputText);
					}
				}
			}

			// Write out the version file
			if (TargetInfo.Version != null && TargetInfo.VersionFile != null)
			{
				DirectoryReference.CreateDirectory(TargetInfo.VersionFile.Directory);
				TargetInfo.Version.BuildId = BuildId;
				TargetInfo.Version.Write(TargetInfo.VersionFile);
			}

			// Write out the receipt
			if (TargetInfo.Receipt != null && TargetInfo.ReceiptFile != null)
			{
				DirectoryReference.CreateDirectory(TargetInfo.ReceiptFile.Directory);
				TargetInfo.Receipt.Version.BuildId = BuildId;
				TargetInfo.Receipt.Write(TargetInfo.ReceiptFile);
			}

			return 0;
		}

		/// <summary>
		/// Checks if this 
		/// </summary>
		/// <param name="BuildId"></param>
		/// <param name="FileToManifest"></param>
		/// <param name="Logger"></param>
		/// <returns></returns>
		private static bool CanRecycleBuildId(string? BuildId, Dictionary<FileReference, ModuleManifest> FileToManifest, ILogger Logger)
		{
			foreach (FileReference ManifestFileName in FileToManifest.Keys)
			{
				ModuleManifest? Manifest;
				if (ManifestFileName.IsUnderDirectory(Unreal.EngineDirectory) && TryReadManifest(ManifestFileName, Logger, out Manifest) && Manifest.BuildId == BuildId)
				{
					DateTime ManifestTime = FileReference.GetLastWriteTimeUtc(ManifestFileName);
					foreach (string FileName in Manifest.ModuleNameToFileName.Values)
					{
						FileInfo ModuleInfo = new FileInfo(FileReference.Combine(ManifestFileName.Directory, FileName).FullName);
						if (!ModuleInfo.Exists || ModuleInfo.LastWriteTimeUtc > ManifestTime)
						{
							return false;
						}
					}
				}
			}
			return true;
		}

		/// <summary>
		/// Attempts to read a manifest from the given location
		/// </summary>
		/// <param name="ManifestFileName">Path to the manifest</param>
		/// <param name="Logger"></param>
		/// <param name="Manifest">If successful, receives the manifest that was read</param>
		/// <returns>True if the manifest was read correctly, false otherwise</returns>
		public static bool TryReadManifest(FileReference ManifestFileName, ILogger Logger, [NotNullWhen(true)] out ModuleManifest? Manifest)
		{
			if (FileReference.Exists(ManifestFileName))
			{
				try
				{
					Manifest = ModuleManifest.Read(ManifestFileName);
					return true;
				}
				catch (Exception Ex)
				{
					Logger.LogWarning("Unable to read '{ManifestFileName}'; ignoring.", ManifestFileName);
					Logger.LogInformation("{Ex}", ExceptionUtils.FormatExceptionDetails(Ex));
				}
			}

			Manifest = null;
			return false;
		}

		/// <summary>
		/// Merge a manifest into another manifest
		/// </summary>
		/// <param name="SourceManifest">The source manifest</param>
		/// <param name="TargetManifest">The target manifest to merge into</param>
		static void MergeManifests(ModuleManifest SourceManifest, ModuleManifest TargetManifest)
		{
			foreach (KeyValuePair<string, string> ModulePair in SourceManifest.ModuleNameToFileName)
			{
				if (!TargetManifest.ModuleNameToFileName.ContainsKey(ModulePair.Key))
				{
					TargetManifest.ModuleNameToFileName.Add(ModulePair.Key, ModulePair.Value);
				}
			}
		}
	}
}
