// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Xml.Linq;
using Microsoft.Extensions.Logging;

namespace UnrealBuildTool
{
	class AndroidAARHandler
	{
		public class AndroidAAREntry
		{
			public string BaseName;
			public string Version;
			public string Filename;
			public List<string> Dependencies;

			public AndroidAAREntry(string InBaseName, string InVersion, string InFilename)
			{
				BaseName = InBaseName;
				Version = InVersion;
				Filename = InFilename;
				Dependencies = new List<string>();
			}

			public void AddDependency(string InBaseName)
			{
				Dependencies.Add(InBaseName);  // + "-" + InVersion);  will replace version with latest
			}

			public void ClearDependencies()
			{
				Dependencies.Clear();
			}
		}

		public readonly List<string> Repositories;
		public readonly List<AndroidAAREntry> AARList;
		private readonly List<AndroidAAREntry> JARList;

		/// <summary>
		/// Handler for AAR and JAR dependency determination and staging
		/// </summary>
		public AndroidAARHandler()
		{
			Repositories = new List<string>();
			AARList = new List<AndroidAAREntry>();
			JARList = new List<AndroidAAREntry>();
		}

		/// <summary>
		/// Add a new respository path to search for AAR and JAR files
		/// </summary>
		/// <param name="RepositoryPath">Directory containing the repository</param>
		/// <param name="Logger">Logger for output</param>
		public void AddRepository(string RepositoryPath, ILogger Logger)
		{
			if (Directory.Exists(RepositoryPath))
			{
				Logger.LogInformation("Added repository: {RepositoryPath}", RepositoryPath);
				Repositories!.Add(RepositoryPath);
			}
			else
			{
				Logger.LogWarning("AddRepository: Directory {RepositoryPath} not found!", RepositoryPath);
			}
		}

		/// <summary>
		/// Add new respository paths to search for AAR and JAR files (recursive)
		/// </summary>
		/// <param name="RepositoryPath">Root directory containing the repository</param>
		/// <param name="SearchPattern">Search pattern to match</param>
		/// <param name="Logger">Logger for output</param>
		public void AddRepositories(string RepositoryPath, string SearchPattern, ILogger Logger)
		{
			if (Directory.Exists(RepositoryPath))
			{
				List<string> ToCheck = new List<string>();
				ToCheck.Add(RepositoryPath);
				while (ToCheck.Count > 0)
				{
					int LastIndex = ToCheck.Count - 1;
					string CurrentDir = ToCheck[LastIndex];
					ToCheck.RemoveAt(LastIndex);
					foreach (string SearchPath in Directory.GetDirectories(CurrentDir))
					{
						if (SearchPath.Contains(SearchPattern, StringComparison.Ordinal))
						{
							Logger.LogInformation("Added repository: {SearchPath}", SearchPath);
							Repositories!.Add(SearchPath);
						}
						else
						{
							ToCheck.Add(SearchPath);
						}
					}
				}
			}
			else
			{
				Logger.LogInformation("AddRepositories: Directory {RepositoryPath} not found; ignored", RepositoryPath);
			}
		}

		private static string GetElementValue(XElement SourceElement, XName ElementName, string DefaultValue)
		{
			XElement? Element = SourceElement.Element(ElementName);
			return (Element != null) ? Element.Value : DefaultValue;
		}

		private string? FindPackageFile(string PackageName, string BaseName, string Version)
		{
			string[] Sections = PackageName.Split('.');
			string PackagePath = Path.Combine(Sections);

			foreach (string Repository in Repositories!)
			{
				string PackageDirectory = Path.Combine(Repository, PackagePath, BaseName, Version);

				if (Directory.Exists(PackageDirectory))
				{
					return PackageDirectory;
				}
			}

			return null;
		}
		private static bool HasAnyVersionCharacters(string InValue)
		{
			for (int Index = 0; Index < InValue.Length; Index++)
			{
				char c = InValue[Index];
				if (c == '.' || (c >= '0' && c <= '9'))
				{
					return true;
				}
			}
			return false;
		}

		private static bool HasOnlyVersionCharacters(string InValue)
		{
			for (int Index = 0; Index < InValue.Length; Index++)
			{
				char c = InValue[Index];
				if (c != '.' && !(c >= '0' && c <= '9'))
				{
					return false;
				}
			}
			return true;
		}

		private static uint GetVersionValue(string VersionString)
		{
			// read up to 4 sections (ie. 20.0.3.5), first section most significant
			// each section assumed to be 0 to 255 range
			uint Value = 0;
			try
			{
				string[] Sections = VersionString.Split(".".ToCharArray());
				Value |= (Sections.Length > 0) ? (UInt32.Parse(Sections[0]) << 24) : 0;
				Value |= (Sections.Length > 1) ? (UInt32.Parse(Sections[1]) << 16) : 0;
				Value |= (Sections.Length > 2) ? (UInt32.Parse(Sections[2]) << 8) : 0;
				Value |= (Sections.Length > 3) ? UInt32.Parse(Sections[3]) : 0;
			}
			catch (Exception)
			{
				// ignore poorly formed version
			}
			return Value;
		}

		// clean up the version (Maven version info here: https://docs.oracle.com/middleware/1212/core/MAVEN/maven_version.htm)
		// only going to handle a few cases, not proper ranges (keeps the rightmost valid version which should be highest)
		// will still return something but will include an error in log, but don't want to throw an exception
		private static string CleanupVersion(string Filename, string InVersion, ILogger Logger)
		{
			string WorkVersion = InVersion;

			// if has commas, keep the rightmost part with actual numbers
			if (WorkVersion.Contains(',', StringComparison.Ordinal))
			{
				string[] CommaParts = WorkVersion.Split(',');
				WorkVersion = "";
				for (int Index = CommaParts.Length - 1; Index >= 0; Index--)
				{
					if (HasAnyVersionCharacters(CommaParts[Index]))
					{
						WorkVersion = CommaParts[Index];
						break;
					}
				}
			}

			// if not left with a possibly valid number, stop
			if (!HasAnyVersionCharacters(WorkVersion))
			{
				Logger.LogError("AAR Dependency file {Filename} version unknown! {InVersion}", Filename, InVersion);
				return InVersion;
			}

			// just remove any parens or brackets left
			WorkVersion = WorkVersion
				.Replace("(", "", StringComparison.Ordinal)
				.Replace(")", "", StringComparison.Ordinal)
				.Replace("[", "", StringComparison.Ordinal)
				.Replace("]", "", StringComparison.Ordinal);

			// just return it if now looks value
			if (HasOnlyVersionCharacters(WorkVersion))
			{
				return WorkVersion;
			}

			// give an error, not likely going to work, though
			Logger.LogError("AAR Dependency file {Filename} version unknown! {InVersion}", Filename, InVersion);
			return InVersion;
		}

		/// <summary>
		/// Adds a new required JAR file and resolves dependencies
		/// </summary>
		/// <param name="PackageName">Name of the package the JAR belongs to in repository</param>
		/// <param name="BaseName">Directory in repository containing the JAR</param>
		/// <param name="Version">Version of the AAR to use</param>
		/// <param name="Logger">Logger instance</param>
		public void AddNewJAR(string PackageName, string BaseName, string Version, ILogger Logger)
		{
			string? BasePath = FindPackageFile(PackageName, BaseName, Version);
			if (BasePath == null)
			{
				Logger.LogError("AAR: Unable to find package {Package}!", PackageName + "/" + BaseName);
				return;
			}
			string BaseFilename = Path.Combine(BasePath, BaseName + "-" + Version);

			// Check if already added
			uint NewVersionValue = GetVersionValue(Version);
			for (int JARIndex = 0; JARIndex < JARList!.Count; JARIndex++)
			{
				if (JARList[JARIndex].BaseName == BaseName)
				{
					// Is it the same version or older?  ignore if so
					uint EntryVersionValue = GetVersionValue(JARList[JARIndex].Version);
					if (NewVersionValue <= EntryVersionValue)
					{
						return;
					}

					Logger.LogInformation("AAR: {BaseName}: {Version1} newer than {Version2}", JARList[JARIndex].BaseName, Version, JARList[JARIndex].Version);

					// This is a newer version; remove old one
					JARList.RemoveAt(JARIndex);
					break;
				}
			}

			//Logger.LogInformation("JAR: {BaseName}", BaseName);
			AndroidAAREntry AAREntry = new AndroidAAREntry(BaseName, Version, BaseFilename);
			JARList.Add(AAREntry);

			// Check for dependencies
			XDocument DependsXML;
			string DependencyFilename = BaseFilename + ".pom";
			if (File.Exists(DependencyFilename))
			{
				try
				{
					DependsXML = XDocument.Load(DependencyFilename);
				}
				catch (Exception e)
				{
					Logger.LogError("AAR Dependency file {File} parsing error! {Ex}", DependencyFilename, e);
					return;
				}
			}
			else
			{
				Logger.LogError("AAR: Dependency file {DependencyFilename} missing!", DependencyFilename);
				return;
			}

			string NameSpace = DependsXML.Root!.Name.NamespaceName;
			XName DependencyName = XName.Get("dependency", NameSpace);
			XName GroupIdName = XName.Get("groupId", NameSpace);
			XName ArtifactIdName = XName.Get("artifactId", NameSpace);
			XName VersionName = XName.Get("version", NameSpace);
			XName ScopeName = XName.Get("scope", NameSpace);
			XName TypeName = XName.Get("type", NameSpace);

			foreach (XElement DependNode in DependsXML.Descendants(DependencyName))
			{
				string DepGroupId = GetElementValue(DependNode, GroupIdName, "");
				string DepArtifactId = GetElementValue(DependNode, ArtifactIdName, "");
				string DepVersion = CleanupVersion(DependencyFilename + "." + DepGroupId + "." + DepArtifactId, GetElementValue(DependNode, VersionName, ""), Logger);
				string DepScope = GetElementValue(DependNode, ScopeName, "compile");
				string DepType = GetElementValue(DependNode, TypeName, "jar");

				//Logger.LogInformation("Dependency: {DepGroupId} {DepArtifactId} {DepVersion} {DepScope} {DepType}", DepGroupId, DepArtifactId, DepVersion, DepScope, DepType);

				// ignore test scope
				if (DepScope == "test")
				{
					continue;
				}

				if (DepType == "aar")
				{
					AddNewAAR(DepGroupId, DepArtifactId, DepVersion, Logger);
				}
				else if (DepType == "jar")
				{
					AddNewJAR(DepGroupId, DepArtifactId, DepVersion, Logger);
				}
			}
		}

		/// <summary>
		/// Adds a new required AAR file and resolves dependencies
		/// </summary>
		/// <param name="PackageName">Name of the package the AAR belongs to in repository</param>
		/// <param name="BaseName">Directory in repository containing the AAR</param>
		/// <param name="Version">Version of the AAR to use</param>
		/// <param name="Logger">Logger for output</param>
		/// <param name="HandleDependencies">Optionally process POM file for dependencies (default)</param>
		public void AddNewAAR(string PackageName, string BaseName, string Version, ILogger Logger, bool HandleDependencies = true)
		{
			if (!HandleDependencies)
			{
				AndroidAAREntry NewAAREntry = new AndroidAAREntry(BaseName, Version, PackageName);
				AARList!.Add(NewAAREntry);
				return;
			}

			string? BasePath = FindPackageFile(PackageName, BaseName, Version);
			if (BasePath == null)
			{
				Logger.LogError("AAR: Unable to find package {Package}!", PackageName + "/" + BaseName);
				return;
			}
			string BaseFilename = Path.Combine(BasePath, BaseName + "-" + Version);

			// Check if already added
			uint NewVersionValue = GetVersionValue(Version);
			for (int AARIndex = 0; AARIndex < AARList!.Count; AARIndex++)
			{
				if (AARList[AARIndex].BaseName == BaseName)
				{
					// Is it the same version or older?  ignore if so
					uint EntryVersionValue = GetVersionValue(AARList[AARIndex].Version);
					if (NewVersionValue <= EntryVersionValue)
					{
						return;
					}

					Logger.LogInformation("AAR: {BaseName}: {Version1} newer than {Version2}", AARList[AARIndex].BaseName, Version, AARList[AARIndex].Version);

					// This is a newer version; remove old one
					// @TODO: be smarter about dependency cleanup (newer AAR might not need older dependencies)
					AARList.RemoveAt(AARIndex);
					break;
				}
			}

			//Logger.LogInformation("AAR: {BaseName}", BaseName);
			AndroidAAREntry AAREntry = new AndroidAAREntry(BaseName, Version, BaseFilename);
			AARList.Add(AAREntry);

			if (!HandleDependencies)
			{
				return;
			}

			// Check for dependencies
			XDocument DependsXML;
			string DependencyFilename = BaseFilename + ".pom";
			if (File.Exists(DependencyFilename))
			{
				try
				{
					DependsXML = XDocument.Load(DependencyFilename);
				}
				catch (Exception e)
				{
					Logger.LogError("AAR Dependency file {File} parsing error! {Ex}", DependencyFilename, e);
					return;
				}
			}
			else
			{
				Logger.LogError("AAR: Dependency file {DependencyFilename} missing!", DependencyFilename);
				return;
			}

			string NameSpace = DependsXML.Root!.Name.NamespaceName;
			XName DependencyName = XName.Get("dependency", NameSpace);
			XName GroupIdName = XName.Get("groupId", NameSpace);
			XName ArtifactIdName = XName.Get("artifactId", NameSpace);
			XName VersionName = XName.Get("version", NameSpace);
			XName ScopeName = XName.Get("scope", NameSpace);
			XName TypeName = XName.Get("type", NameSpace);

			foreach (XElement DependNode in DependsXML.Descendants(DependencyName))
			{
				string DepGroupId = GetElementValue(DependNode, GroupIdName, "");
				string DepArtifactId = GetElementValue(DependNode, ArtifactIdName, "");
				string DepVersion = CleanupVersion(DependencyFilename + "." + DepGroupId + "." + DepArtifactId, GetElementValue(DependNode, VersionName, ""), Logger);
				string DepScope = GetElementValue(DependNode, ScopeName, "compile");
				string DepType = GetElementValue(DependNode, TypeName, "jar");

				//Logger.LogInformation("Dependency: {DepGroupId} {DepArtifactId} {DepVersion} {DepScope} {DepType}", DepGroupId, DepArtifactId, DepVersion, DepScope, DepType);

				// ignore test scope
				if (DepScope == "test")
				{
					continue;
				}

				if (DepType == "aar")
				{
					// Add dependency
					AAREntry.AddDependency(DepArtifactId);

					AddNewAAR(DepGroupId, DepArtifactId, DepVersion, Logger);
				}
				else
				if (DepType == "jar")
				{
					AddNewJAR(DepGroupId, DepArtifactId, DepVersion, Logger);
				}
			}
		}
	}
}
