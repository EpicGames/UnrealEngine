// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Text.RegularExpressions;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

///////////////////////////////////////////////////////////////////
// If you are looking for supported version numbers, look in the
// AndroidPlatformSDK.Versions.cs file next to this file
///////////////////////////////////////////////////////////////////

namespace UnrealBuildTool
{
	partial class AndroidPlatformSDK : UEBuildPlatformSDK
	{
		public AndroidPlatformSDK(ILogger Logger)
			: base(Logger)
		{
		}

		private static void CheckEnvironmentVariables(bool bUseConfig, ILogger Logger)
		{
			Dictionary<string, string> EnvVarNames = new Dictionary<string, string> {
													 {"ANDROID_HOME", "SDKPath"},
													 {"NDKROOT", "NDKPath"},
													 {"JAVA_HOME", "JavaPath"}
													 };
			Dictionary<string, string> AndroidEnv = new Dictionary<string, string>();

			// first check for configuration overrides
			if (bUseConfig)
			{
				ConfigHierarchy configCacheIni = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, null, BuildHostPlatform.Current.Platform);
				string path;
				foreach (KeyValuePair<string, string> kvp in EnvVarNames)
				{
					if (GetPath(configCacheIni, "/Script/AndroidPlatformEditor.AndroidSDKSettings", kvp.Value, out path) && !String.IsNullOrEmpty(path))
					{
						AndroidEnv.Add(kvp.Key, path);
					}
				}
			}

			// fill in from environment variables not already set
			foreach (KeyValuePair<string, string> kvp in EnvVarNames)
			{
				if (!AndroidEnv.ContainsKey(kvp.Key))
				{
					string? envValue = Environment.GetEnvironmentVariable(kvp.Key);
					if (!String.IsNullOrEmpty(envValue))
					{
						AndroidEnv.Add(kvp.Key, envValue);
					}
				}
			}
				
			// If we are not running on Windows, and we are still missing a key then go and find it from alternate source
			if (!OperatingSystem.IsWindows() && !EnvVarNames.All(s => AndroidEnv.ContainsKey(s.Key)))
			{
				string UserProfileDir = Environment.GetFolderPath(Environment.SpecialFolder.UserProfile);
				string BashProfilePath;
				if (!OperatingSystem.IsMacOS())
				{
					BashProfilePath = Path.Combine(UserProfileDir, ".bashrc");
				}
				else
				{
					BashProfilePath = Path.Combine(UserProfileDir, ".zshrc");
					if (!File.Exists(BashProfilePath))
					{
						// Fallbacks
						BashProfilePath = Path.Combine(UserProfileDir, ".bashrc");
						if (!File.Exists(BashProfilePath))
						{
							BashProfilePath = Path.Combine(UserProfileDir, ".bash_profile");
							Logger.LogWarning(".zshrc is missing. Falling back to .bash_profile.");
						}
						else
						{
							Logger.LogWarning(".zshrc is missing. Falling back to .bashrc.");
						}
					}
				}
				if (File.Exists(BashProfilePath))
				{
					string[] BashProfileContents = File.ReadAllLines(BashProfilePath);

					// Walk backwards so we keep the last export setting instead of the first
					for (int LineIndex = BashProfileContents.Length - 1; LineIndex >= 0; --LineIndex)
					{
						foreach (KeyValuePair<string, string> kvp in EnvVarNames)
						{
							if (AndroidEnv.ContainsKey(kvp.Key))
							{
								continue;
							}

							if (BashProfileContents[LineIndex].StartsWith("export " + kvp.Key + "=", StringComparison.Ordinal))
							{
								string PathVar = BashProfileContents[LineIndex].Split('=')[1].Replace("\"", "", StringComparison.Ordinal);
								AndroidEnv.Add(kvp.Key, PathVar);
							}
						}
					}
				}
			}

			// Set for the process
			foreach (KeyValuePair<string, string> kvp in AndroidEnv)
			{
				Environment.SetEnvironmentVariable(kvp.Key, kvp.Value);
			}
		}

		public static string? GetNDKRoot(ILogger Logger)
		{
			string? NDKPath = Environment.GetEnvironmentVariable("NDKROOT");

			// don't register if we don't have an NDKROOT specified
			if (!String.IsNullOrEmpty(NDKPath))
			{
				return NDKPath.Replace("\"", "", StringComparison.Ordinal);
			}

			// set environment variables (don't check config)
			CheckEnvironmentVariables(false, Logger);

			// try NDKROOT again
			NDKPath = Environment.GetEnvironmentVariable("NDKROOT");
			if (!String.IsNullOrEmpty(NDKPath))
			{
				return NDKPath.Replace("\"", "", StringComparison.Ordinal);
			}
			return null;
		}

		protected override string? GetInstalledSDKVersion()
		{
			string? NDKPath = GetNDKRoot(Logger);

			// don't register if we don't have an NDKROOT specified
			if (String.IsNullOrEmpty(NDKPath))
			{
				return null;
			}

			return GetNdkNameFromDirectory(NDKPath);
		}

		static string? GetNdkNameFromDirectory(string Directory)
		{
			// figure out the NDK version
			string? NDKToolchainVersion = null;
			string SourcePropFilename = Path.Combine(Directory, "source.properties");
			if (File.Exists(SourcePropFilename))
			{
				string RevisionString = "";
				string[] PropertyContents = File.ReadAllLines(SourcePropFilename);
				foreach (string PropertyLine in PropertyContents)
				{
					if (PropertyLine.StartsWith("Pkg.Revision", StringComparison.Ordinal))
					{
						RevisionString = PropertyLine;
						break;
					}
				}

				int EqualsIndex = RevisionString.IndexOf('=', StringComparison.Ordinal);
				if (EqualsIndex > 0)
				{
					string[] RevisionParts = RevisionString.Substring(EqualsIndex + 1).Trim().Split('.');
					int RevisionMinor = Int32.Parse(RevisionParts.Length > 1 ? RevisionParts[1] : "0");
					char RevisionLetter = Convert.ToChar('a' + RevisionMinor);
					NDKToolchainVersion = "r" + RevisionParts[0] + (RevisionMinor > 0 ? Char.ToString(RevisionLetter) : "");
				}
			}
			else
			{
				string ReleaseFilename = Path.Combine(Directory, "RELEASE.TXT");
				if (File.Exists(ReleaseFilename))
				{
					string[] PropertyContents = File.ReadAllLines(SourcePropFilename);
					NDKToolchainVersion = PropertyContents[0];
				}
			}

			// return something like r10e, or null if anything went wrong above
			return NDKToolchainVersion;
		}

		static string GetStudioDir()
		{
			if (OperatingSystem.IsLinux())
			{
				return Path.Combine(Environment.GetEnvironmentVariable("HOME")!, "android-studio");
			}
			else if (OperatingSystem.IsMacOS())
			{
				string Directory = Path.Combine(Environment.GetEnvironmentVariable("HOME")!, "Applications/Android Studio.app/Contents");
				return System.IO.Directory.Exists(Directory) ? Directory : "/Applications/Android Studio.app/Contents";
			}
			else
			{
				Debug.Assert(OperatingSystem.IsWindows());

				string? Directory = Microsoft.Win32.Registry.GetValue(@"HKEY_LOCAL_MACHINE\SOFTWARE\Android Studio", "Path", null) as string;
				return !String.IsNullOrEmpty(Directory) ? Directory : Path.Combine(Environment.GetFolderPath(Environment.SpecialFolder.LocalApplicationData), @"Programs\Android Studio");
			}
		}
		
		static string GetStudioSdkDir()
		{
			if (OperatingSystem.IsLinux())
			{
				return Path.Combine(Environment.GetEnvironmentVariable("HOME")!, "Android/Sdk");
			}
			else if (OperatingSystem.IsMacOS())
			{
				return Path.Combine(Environment.GetEnvironmentVariable("HOME")!, "Library/Android/sdk");
			}
			else
			{
				Debug.Assert(OperatingSystem.IsWindows());

				string? Directory = Microsoft.Win32.Registry.GetValue(@"HKEY_LOCAL_MACHINE\SOFTWARE\Android Studio", "SdkPath", null) as string;
				return !String.IsNullOrEmpty(Directory) ? Directory : Path.Combine(Environment.GetFolderPath(Environment.SpecialFolder.LocalApplicationData), @"Android\Sdk");
			}
		}

		public override string[] GetAllInstalledSDKVersions()
		{
			string StudioSdkDir = GetStudioSdkDir();
			
			if (Directory.Exists(Path.Combine(StudioSdkDir, "platforms", GetPlatformSpecificVersion("platforms"))))
			{
				string NDKPath = Path.Combine(StudioSdkDir, "ndk");
				if (Directory.Exists(NDKPath))
				{
					return Directory.EnumerateDirectories(NDKPath).Select(GetNdkNameFromDirectory).Where(Directory => Directory != null).Distinct().ToArray()!;
				}
			}
			
			return [];
		}

		protected override bool TryGetEnvironmentForManualSDK(string SelectedSDKVersion, out Dictionary<string, string>? EnvVarValues, out List<string>? PathAdds, out List<string>? PathRemoves)
		{
			PathAdds = null;
			PathRemoves = null;
			
			string? CurrentSdkDir = Environment.GetEnvironmentVariable("ANDROID_HOME");
			
			if (String.IsNullOrEmpty(CurrentSdkDir) || !Directory.Exists(Path.Combine(CurrentSdkDir, "platforms", GetPlatformSpecificVersion("platforms"))) || GetInstalledSDKVersion() != SelectedSDKVersion)
			{
				try
				{
					string StudioSdkDir = GetStudioSdkDir();
					string NdkDir = Directory.EnumerateDirectories(Path.Combine(StudioSdkDir, "ndk")).First(Directory => GetNdkNameFromDirectory(Directory) == SelectedSDKVersion);

					EnvVarValues = new()
					{
						["ANDROID_HOME"] = StudioSdkDir,
						["ANDROID_SDK_HOME"] = "",
						["ANDROID_NDK_ROOT"] = NdkDir,
						["NDK_ROOT"] = NdkDir,
						["NDKROOT"] = NdkDir
					};

					string StudioDir = GetStudioDir();
					string JavaDir = Path.Combine(StudioDir, "jbr");

					if (!Directory.Exists(JavaDir))
					{
						JavaDir = Path.Combine(StudioDir, "jre");

						if (!Directory.Exists(JavaDir))
						{
							return true;
						}
					}

					if (OperatingSystem.IsMacOS())
					{
						JavaDir = Path.Combine(JavaDir, "Contents", "Home");
					}

					EnvVarValues["JAVA_HOME"] = JavaDir;
					return true;
				}
				catch (DirectoryNotFoundException)
				{
				}
				catch (InvalidOperationException)
				{
				}
			}

			EnvVarValues = null;
			return false;
		}

		public override bool TryConvertVersionToInt(string? StringValue, out ulong OutValue, string? Hint)
		{
			// convert r<num>[letter] to hex
			if (!String.IsNullOrEmpty(StringValue))
			{
				if (String.Equals(Hint, "Software", StringComparison.OrdinalIgnoreCase))
				{
					// returning a version with a format similar to the ndk version.
					string VersionString = String.Format("{0}{1:00}{2:00}", StringValue, 0, 0);
					if (UInt64.TryParse(VersionString, out OutValue))
					{
						return true;
					}
				}
				
				Match Result = Regex.Match(StringValue, @"^r(\d*)([a-z])?");

				if (Result.Success)
				{
					// 8 for number, 8 for letter, 8 for unused patch
					int RevisionNumber = 0;
					if (Result.Groups[2].Success)
					{
						RevisionNumber = (Result.Groups[2].Value[0] - 'a') + 1;
					}
					string VersionString = String.Format("{0}{1:00}{2:00}", Result.Groups[1], RevisionNumber, 0);
					return UInt64.TryParse(VersionString, out OutValue);
				}
			}

			OutValue = 0;
			return false;
		}

		//public override SDKStatus PrintSDKInfoAndReturnValidity(LogEventType Verbosity, LogFormatOptions Options, LogEventType ErrorVerbosity, LogFormatOptions ErrorOptions)
		//{
		//	SDKStatus Validity = base.PrintSDKInfoAndReturnValidity(Verbosity, Options, ErrorVerbosity, ErrorOptions);

		//	if (GetInstalledVersion() != GetMainVersion())
		//	{
		//		Log.WriteLine(Verbosity, Options, "Note: Android toolchain NDK {0} recommended ('{1}' was found)", GetMainVersion(), GetInstalledVersion());
		//	}

		//	return Validity;
		//}

		protected override bool PlatformSupportsAutoSDKs()
		{
			return true;
		}

		// prefer auto sdk on android as correct 'manual' sdk detection isn't great at the moment.
		protected override bool PreferAutoSDK()
		{
			return true;
		}

		private static bool ExtractPath(string Source, out string Path)
		{
			int start = Source.IndexOf('"', StringComparison.Ordinal);
			int end = Source.LastIndexOf('"');
			if (start != 1 && end != -1 && start < end)
			{
				++start;
				Path = Source.Substring(start, end - start);
				return true;
			}
			else
			{
				Path = "";
			}

			return false;
		}

		public static bool GetPath(ConfigHierarchy Ini, string SectionName, string Key, out string Value)
		{
			string? temp;
			if (Ini.TryGetValue(SectionName, Key, out temp))
			{
				return ExtractPath(temp, out Value);
			}
			else
			{
				Value = "";
			}

			return false;
		}

		/// <summary>
		/// checks if the sdk is installed or has been synced
		/// </summary>
		/// <returns></returns>
		protected virtual bool HasAnySDK()
		{
			// The Android SDK is not required to build AndroidTargetPlatform. So for installed builds where its expected another machine will have
			// the SDK setup we can force this to be on to build AndroidTargetPlatform.
			string? ForceAndroidSDK = Environment.GetEnvironmentVariable("FORCE_ANDROID_SDK_ENABLED");

			// ANDROID_SDK_HOME defined messes up newer Android Gradle plugin finding of .android so clear it
			Environment.SetEnvironmentVariable("ANDROID_SDK_HOME", null);

			if (!String.IsNullOrEmpty(ForceAndroidSDK))
			{
				return true;
			}

			// Set environment variables as needed (allow config overrides)
			CheckEnvironmentVariables(true, Logger);
			
			string? NDKPath = Environment.GetEnvironmentVariable("NDKROOT");
			string? JavaPath = Environment.GetEnvironmentVariable("JAVA_HOME");

			// we don't have an NDKROOT specified
			if (String.IsNullOrEmpty(NDKPath))
			{
				Logger.LogInformation("NDKROOT not set");
				return false;
			}

			NDKPath = NDKPath.Replace("\"", "", StringComparison.Ordinal);

			// need a supported llvm
			if (!Directory.Exists(Path.Combine(NDKPath, @"toolchains/llvm")))
			{
				Logger.LogInformation("NDKROOT llvm missing");
				return false;
			}

			// check JDK is valid
			if (String.IsNullOrEmpty(JavaPath))
			{
				Logger.LogInformation("JAVA_HOME not set");
				return false;
			}
			JavaPath = JavaPath.Replace("\"", "", StringComparison.Ordinal);
			string JavaReleaseFile = Path.Combine(JavaPath, "release");
			if (!File.Exists(JavaReleaseFile))
			{
				Logger.LogInformation("JAVA_HOME/release not found: {JavaReleaseFile}", JavaReleaseFile);
				return false;
			}

			// check Java version
			int JavaVersion = 0;
			string[] JavaReleaseLines = File.ReadAllLines(JavaReleaseFile);
			foreach (string LineContents in JavaReleaseLines)
			{
				if (LineContents.StartsWith("JAVA_VERSION=", StringComparison.Ordinal))
				{
					// JAVA_VERSION="17.0.6"
					int VersionStartIndex = LineContents.IndexOf('\"', StringComparison.Ordinal);
					int VersionStopIndex = LineContents.IndexOf('.', StringComparison.Ordinal);
					_ = Int32.TryParse(LineContents.Substring(VersionStartIndex, VersionStopIndex - VersionStartIndex), out JavaVersion);
					break;
				}
			}
			if (JavaVersion < 17)
			{
				Logger.LogInformation("JAVA_HOME release too old {JavaVersion}", JavaVersion);
				return false;
			}

			return true;
		}
	}
}
