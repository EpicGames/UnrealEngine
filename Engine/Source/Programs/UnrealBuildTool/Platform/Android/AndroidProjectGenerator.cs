// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// Base class for platform-specific project generators
	/// </summary>
	class AndroidProjectGenerator : PlatformProjectGenerator
	{
		/// <summary>
		/// Whether Android Game Development Extension is installed in the system. See https://developer.android.com/games/agde for more details.
		/// May be disabled by using -noagde on commandline
		/// </summary>
		private bool AGDEInstalled => AGDEInstalledTask.Result;

		private readonly Task<bool> AGDEInstalledTask;

		public AndroidProjectGenerator(CommandLineArguments Arguments, ILogger Logger)
			: base(Logger)
		{
			AGDEInstalledTask = Task.Run(() =>
			{
				if (OperatingSystem.IsWindows() && !Arguments.HasOption("-noagde"))
				{
					if (Microsoft.Win32.Registry.LocalMachine.OpenSubKey(@"SOFTWARE\WOW6432Node\Google\AndroidGameDevelopmentExtension")?.ValueCount > 0)
					{
						return true;
					}

					try
					{
						string? programFiles86 = Environment.GetEnvironmentVariable("ProgramFiles(x86)");
						if (programFiles86 != null)
						{
							string vswhereExe = Path.Join(programFiles86, @"Microsoft Visual Studio\Installer\vswhere.exe");
							if (File.Exists(vswhereExe))
							{
								using (Process p = new Process())
								{
									ProcessStartInfo info = new ProcessStartInfo
									{
										FileName = vswhereExe,
										Arguments = @"-find Common7\IDE\Extensions\*\Google.VisualStudio.Android.dll",
										RedirectStandardOutput = true,
										UseShellExecute = false
									};
									p.StartInfo = info;
									p.Start();
									return p.StandardOutput.ReadToEnd().Contains("Google.VisualStudio.Android.dll", StringComparison.Ordinal);
								}
							}
						}
					}
					catch (Exception ex)
					{
						Logger.LogInformation("Failed to identify AGDE installation status: {Message}", ex.Message);
					}
				}
				return false;
			});
		}

		/// <summary>
		/// Enumerate all the platforms that this generator supports
		/// </summary>
		public override IEnumerable<UnrealTargetPlatform> GetPlatforms()
		{
			yield return UnrealTargetPlatform.Android;
		}

		/// <inheritdoc/>
		public override bool HasVisualStudioSupport(VSSettings InVSSettings)
		{
			// Debugging, etc. are dependent on the TADP being installed
			return AGDEInstalled;
		}

		/// <inheritdoc/>
		public override string GetVisualStudioPlatformName(VSSettings InVSSettings)
		{
			string PlatformName = InVSSettings.Platform.ToString();

			if (InVSSettings.Platform == UnrealTargetPlatform.Android && AGDEInstalled)
			{
				string longAbi = GetLongAbi(InVSSettings);
				PlatformName = $"Android-{longAbi}";
			}

			return PlatformName;
		}

		/// <inheritdoc/>
		public override void GetAdditionalVisualStudioPropertyGroups(VSSettings InVSSettings, StringBuilder ProjectFileBuilder)
		{
			if (AGDEInstalled)
			{
				base.GetAdditionalVisualStudioPropertyGroups(InVSSettings, ProjectFileBuilder);
			}
		}

		private static string GetShortAbi(VSSettings InVSSettings)
		{
			if (InVSSettings.Architecture == null)
			{
				throw new BuildException("Architecture cannot be null");
			}
			else if (InVSSettings.Architecture == UnrealArch.Arm64)
			{
				return "arm64";
			}
			else if (InVSSettings.Architecture == UnrealArch.X64)
			{
				return "x64";
			}
			else
			{
				throw new BuildException($"Unexpected architecture: {InVSSettings.Architecture}");
			}
		}

		private static string GetLongAbi(VSSettings InVSSettings)
		{
			if (InVSSettings.Architecture == null)
			{
				throw new BuildException("Architecture cannot be null");
			}
			else if (InVSSettings.Architecture == UnrealArch.Arm64)
			{
				return "arm64-v8a";
			}
			else if (InVSSettings.Architecture == UnrealArch.X64)
			{
				return "x86_64";
			}
			else
			{
				throw new BuildException($"Unexpected architecture: {InVSSettings.Architecture}");
			}
		}

		/// <inheritdoc/>
		public override void GetVisualStudioPathsEntries(VSSettings InVSSettings, TargetType TargetType, FileReference TargetRulesPath, FileReference ProjectFilePath, FileReference NMakeOutputPath, StringBuilder ProjectFileBuilder)
		{
			if (AGDEInstalled)
			{
				string shortAbi = GetShortAbi(InVSSettings);
				string longAbi = GetLongAbi(InVSSettings);

				string apkLocation = Path.Combine(
					Path.GetDirectoryName(NMakeOutputPath.FullName)!,
					Path.GetFileNameWithoutExtension(NMakeOutputPath.FullName) + $"-{shortAbi}.apk");

				ProjectFileBuilder.AppendLine($"    <AndroidApkLocation>{apkLocation}</AndroidApkLocation>");
				string intermediateRootPath = Path.GetFullPath(Path.GetDirectoryName(NMakeOutputPath.FullName) + @"\..\..\Intermediate\Android\");
				List<string> symbolLocations = new List<string>
				{
					Path.Combine(intermediateRootPath, shortAbi, "jni", longAbi),
					Path.Combine(intermediateRootPath, shortAbi, "libs", longAbi),
					Path.Combine(intermediateRootPath, "LLDBSymbolsLibs", shortAbi) // support bDontBundleLibrariesInAPK
				};
				ProjectFileBuilder.AppendLine($"    <AndroidSymbolDirectories>{String.Join(";", symbolLocations)}</AndroidSymbolDirectories>");
			}
			else
			{
				base.GetVisualStudioPathsEntries(InVSSettings, TargetType, TargetRulesPath, ProjectFilePath, NMakeOutputPath, ProjectFileBuilder);
			}
		}

		public override string GetExtraBuildArguments(VSSettings InVSSettings)
		{
			if (AGDEInstalled)
			{
				return $" -ForceAPKGeneration -FastIterate" + base.GetExtraBuildArguments(InVSSettings);
			}
			else
			{
				return base.GetExtraBuildArguments(InVSSettings);
			}
		}

		public override string GetVisualStudioUserFileStrings(VisualStudioUserFileSettings VCUserFileSettings, VSSettings InVSSettings, string InConditionString, TargetRules InTargetRules, FileReference TargetRulesPath, FileReference ProjectFilePath, FileReference? NMakeOutputPath, string ProjectName, string UProjectPath, string? ForeignUProjectPath)
		{
			if (AGDEInstalled
				&& (InVSSettings.Platform == UnrealTargetPlatform.Android)
				&& ((InTargetRules.Type == TargetRules.TargetType.Client) || (InTargetRules.Type == TargetRules.TargetType.Game)))
			{
				StringBuilder Out = new StringBuilder();
				Out.AppendLine("  <PropertyGroup " + InConditionString + ">");

				Dictionary<DirectoryReference, DirectoryReference> SourceFileMap = ProjectFileGenerator.GenerateSourceMap(InTargetRules, InTargetRules.ProjectFile);
				Out.Append("    <AndroidLldbStartupCommands>");
				Out.Append($"command script import \"{Path.Combine(Unreal.EngineDirectory.FullName, "Extras", "LLDBDataFormatters", "UEDataFormatters_2ByteChars.py")}\";");
				Out.AppendJoin(';', SourceFileMap.Select(from_to => $"setting append target.source-map {from_to.Key} {from_to.Value}"));
				Out.Append(SourceFileMap.Any() ? ";" : "");
				Out.AppendLine($"$(AndroidLldbStartupCommands)</AndroidLldbStartupCommands>");
				VCUserFileSettings.PatchProperty("AndroidLldbStartupCommands");

				if (NMakeOutputPath != null)
				{
					// It's critical to have AndroidDebugTarget here and for it to be before properties that use it (e.g. AndroidPostApkInstallCommands), otherwise MSBuild will evaluate it to empty string.
					Out.AppendLine("    <AndroidDebugTarget></AndroidDebugTarget>");

					string UatPath = Path.Combine(Unreal.RootDirectory.FullName, "Engine", "Build", "BatchFiles", "RunUAT.bat");
					string ShortAbi = GetShortAbi(InVSSettings);

					// AGDE specifies current debug target in AndroidDebugTarget property in a form of "model:serial:arch".
					// AndroidDebugTarget is a special property and needs to be evaluated in-line. And the push script needs the device serial as first argument to push to the correct device. 
					// MSBuild Property Functions allow to invoke limited C# expression from with-in MSBuild, see https://learn.microsoft.com/en-us/visualstudio/msbuild/property-functions?view=vs-2022
					// They lack conditional statements, so this expression makes a branch-less variant by always appending "::" to AndroidDebugTarget,
					// so regardless of what value it has, Split(':')[1] will never throw an exception.
					if (UProjectPath.Length > 0)
					{
						string ClientParam = InTargetRules.Type == TargetRules.TargetType.Client ? "-client" : "";
						string GetTargetDeviceSerial = "$([System.String]::Concat($(AndroidDebugTarget), \"::\").Split(':')[1])";
						Out.AppendLine(
							$"    <AndroidPostApkInstallCommands>CALL \"{UatPath}\" BuildCookRun -project=\"{UProjectPath.Replace("\"", "", StringComparison.Ordinal)}\" -FastIterate -FromMsBuild {ClientParam} -skipbuild -skipcook -skipstage -deploy -config={InVSSettings.Configuration} -platform={InVSSettings.Platform} -clientarchitecture={ShortAbi} -device=\"{GetTargetDeviceSerial}\";$(AndroidPostApkInstallCommands)</AndroidPostApkInstallCommands>");
					}
					else
					{
						// UAT deploy can only be called with a project file. Fallback to Push_..._so.bat script
						string PushSOScript = Path.Combine(
							Path.GetDirectoryName(NMakeOutputPath.FullName)!,
							"Push_" + Path.GetFileNameWithoutExtension(NMakeOutputPath.FullName) + $"-{ShortAbi}_so.bat");
						string GetTargetDeviceSerial = "$([System.String]::Concat($(AndroidDebugTarget), \"::\").Split(':')[1])";
						Out.AppendLine(
							$"    <AndroidPostApkInstallCommands>IF EXIST {PushSOScript} {PushSOScript} {GetTargetDeviceSerial};$(AndroidPostApkInstallCommands)</AndroidPostApkInstallCommands>");
					}

					// Ensure the following properties are up to date even if .vcxproj.user already exists and are in correct order between each other.
					VCUserFileSettings.PatchProperty("AndroidDebugTarget", true);
					VCUserFileSettings.PatchProperty("AndroidPostApkInstallCommands");
				}

				Out.AppendLine("  </PropertyGroup>");
				return Out.ToString();
			}

			return String.Empty;
		}
		
		public override IList<string> GetSystemIncludePaths(UEBuildTarget InTarget)
		{
			UnrealArch Architecture = InTarget.Architectures.Architectures.Count >= 1 ? InTarget.Architectures.Architectures[0] : UnrealArch.X64;
			string NDKArch = UEDeployAndroid.GetNDKArch(Architecture);
			string NDKTargetTripletName = (NDKArch == "x86_64") ? "x86_64-linux-android" : "aarch64-linux-android";
			string PlatformNDKHostName = UEDeployAndroid.GetPlatformNDKHostName();
			string LLVMRoot = Path.Combine(Environment.ExpandEnvironmentVariables("%NDKROOT%"), "toolchains", "llvm", "prebuilt",
				PlatformNDKHostName);
			string SysRootInclude = Path.Combine(LLVMRoot, "sysroot", "usr", "include");

			List<string> Result = new List<string>();
			Result.Add(SysRootInclude);
			Result.Add(Path.Combine(SysRootInclude, "c++", "v1"));
			Result.Add(Path.Combine(SysRootInclude, NDKTargetTripletName));

			try
			{
				string VersionFileName = Path.Combine(LLVMRoot, "AndroidVersion.txt");
				using StreamReader VersionFile = new StreamReader(VersionFileName);
				string LibsVersion = VersionFile.ReadLine()!;
				int MajorVersionIndex = LibsVersion.IndexOf('.', StringComparison.Ordinal);
				if (MajorVersionIndex != -1)
				{
					LibsVersion = LibsVersion.Substring(0, MajorVersionIndex);
				}
				
				string ClangIncludePath = Path.Combine(LLVMRoot, "lib", "clang", LibsVersion, "include");
				Result.Add(ClangIncludePath);
			}
			catch (Exception e)
			{
				Logger.LogWarning(e, "{Message}", e.Message);
			}
			
			return Result;
		}
	}
}
