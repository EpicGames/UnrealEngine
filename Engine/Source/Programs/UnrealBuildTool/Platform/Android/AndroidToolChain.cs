// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Text.RegularExpressions;
using System.Threading;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	class AndroidToolChain : ClangToolChain, IAndroidToolChain
	{
		// Minimum NDK API level to allow
		public const int MinimumNDKAPILevel = 26;
		public const int MinimumBestVersion = (28 << 24) | (0 << 16) | (3 << 8);
		public const string MinimumBestVersionString = "28.0.3";
		public const string FallbackBestVersionString = "35.0.1";

		// this is architectures with the dash, which we match in filenames that have inlined arch name
		public static readonly Dictionary<UnrealArch, string> AllCpuSuffixes = new()
		{
			{ UnrealArch.Arm64, "-arm64" },
			{ UnrealArch.X64,   "-x64" },
		};

		// short names for the above suffixes
		public static readonly Dictionary<string, string> ShortArchNames = new Dictionary<string, string>()
		{
			{ "", "" },
			{ "-arm64", "a8" },
			{ "-x64", "x6" },
		};

		public enum ClangSanitizer
		{
			None,
			Address,
			HwAddress,
			UndefinedBehavior,
			UndefinedBehaviorMinimal,
			Thread,
		};

		public static string GetCompilerOption(ClangSanitizer Sanitizer)
		{
			switch (Sanitizer)
			{
				case ClangSanitizer.Address: return "address";
				case ClangSanitizer.HwAddress: return "hwaddress";
				case ClangSanitizer.UndefinedBehavior:
				case ClangSanitizer.UndefinedBehaviorMinimal: return "undefined";
				case ClangSanitizer.Thread: return "thread";
				default: return "";
			}
		}

		// Version string from the Android specific build of clang. E.g in Android (6317467 based on r365631c1) clang version 9.0.8
		// this would be 6317467)
		protected static string? AndroidClangBuild;

		// architecture paths to use for filtering include and lib paths
		private static readonly Dictionary<UnrealArch, string[]> AllFilterArchNames = new() {
			{ UnrealArch.Arm64, new string[] { "arm64", "arm64-v8a", "arm64-android" } },
			{ UnrealArch.X64,   new string[] { "x64", "x86_64", "x64-android" } },
			// using Default as a placeholder to remove old folders for arches we no longer support, but licensees may have in their Build rules that we need to strip out
			{ UnrealArch.Deprecated  , new string[] { "armv7", "armeabi-v7a", "arm-android", "x86", "x86-android" } }
		};

		private static readonly Dictionary<UnrealArch, string[]> LibrariesToSkip = new() {
			{ UnrealArch.Arm64, new string[] { "nvToolsExt", "nvToolsExtStub", "vorbisenc", } },
			{ UnrealArch.X64,   new string[] { "nvToolsExt", "nvToolsExtStub", "oculus", "OVRPlugin", "vrapi", "ovrkernel", "systemutils", "openglloader", "ovrplatformloader", "vorbisenc", } }
		};

		private static readonly Dictionary<UnrealArch, string[]> ModulesToSkip = new() {
			{ UnrealArch.Arm64, Array.Empty<string>() },
			{ UnrealArch.X64,   new string[] { "OnlineSubsystemOculus", "OculusHMD", "OculusMR" } }
		};

		private static readonly Dictionary<UnrealArch, string[]> GeneratedModulesToSkip = new() {
			{ UnrealArch.Arm64, Array.Empty<string>() },
			{ UnrealArch.X64,   new string[] { "OculusEntitlementCallbackProxy", "OculusCreateSessionCallbackProxy", "OculusFindSessionsCallbackProxy", "OculusIdentityCallbackProxy", "OculusNetConnection", "OculusNetDriver", "OnlineSubsystemOculus_init" } }
		};

		public static string? NDKToolchainVersion { get; private set; }
		public static ulong NDKVersionInt { get; private set; }

		private static int ClangVersionMajor = -1;
		private static int ClangVersionMinor = -1;
		private static int ClangVersionPatch = -1;

		public static string GetClangVersionString()
		{
			return String.Format("{0}.{1}.{2}", ClangVersionMajor, ClangVersionMinor, ClangVersionPatch);
		}

		public static bool IsNewNDKModel()
		{
			// Google changed NDK structure in r22+
			return NDKVersionInt >= 220000;
		}

		public static bool HasEmbeddedHWASanSupport()
		{
			return NDKVersionInt >= 260000;
		}

		public AndroidToolChain(FileReference? InProjectFile, ILogger InLogger)
			: this(InProjectFile, ClangToolChainOptions.None, InLogger)
		{
		}

		static DirectoryReference? GCCToolchainPath;
		static DirectoryReference? SysrootPath;
		static string ExeExtension = ".exe";

		public AndroidToolChain(FileReference? InProjectFile, ClangToolChainOptions ToolchainOptions, ILogger InLogger)
			: base(ToolchainOptions, InLogger)
		{
			Options = ToolchainOptions;
			ProjectFile = InProjectFile;
			lock (CtorLock)
			{
				StaticInit(Logger);
			}

			// NDK setup (enforce minimum API level)
			int NDKApiLevel64Int = GetNdkApiLevelInt(MinimumNDKAPILevel);

			// toolchain params (note: use ANDROID=1 same as we define it)
			ToolchainLinkParamsArm64 = " --target=aarch64-none-linux-android" + NDKApiLevel64Int + " -DANDROID=1";
			ToolchainLinkParamsx64 = " --target=x86_64-none-linux-android" + NDKApiLevel64Int + " -DANDROID=1";

			ToolchainParamsArm64 = ToolchainLinkParamsArm64;
			ToolchainParamsx64 = ToolchainLinkParamsx64;

			if (!IsNewNDKModel())
			{
				// We need to manually provide -D__ANDROID_API__ for NDK versions prior to r22 only, for newer ones, --target=aarch64-none-linux-android + NDKApiLevel64Int does it for us
				ToolchainParamsArm64 += " -D__ANDROID_API__=" + NDKApiLevel64Int;
				ToolchainParamsx64 += " -D__ANDROID_API__=" + NDKApiLevel64Int;
			}

			ReadElfPath = Path.Combine(NDKPath!, @"toolchains/llvm", ArchitecturePath!, @"bin/llvm-readelf" + ExeExtension);
			ObjDumpPath = Path.Combine(NDKPath!, @"toolchains/llvm", ArchitecturePath!, @"bin/llvm-objdump" + ExeExtension);
		}

		static string? NDKPath;
		static string? ArchitecturePath;

		static void StaticInit(ILogger Logger)
		{
			if (NDKPath != null)
			{
				return;
			}

			NDKPath = AndroidPlatformSDK.GetNDKRoot(Logger);

			// don't register if we don't have an NDKROOT specified
			if (String.IsNullOrEmpty(NDKPath))
			{
				throw new BuildException("NDKROOT is not specified; cannot use Android toolchain.");
			}

			NDKPath = NDKPath.Replace("\"", "", StringComparison.Ordinal);

			string ArchitecturePathWindows32 = @"prebuilt/windows";
			string ArchitecturePathWindows64 = @"prebuilt/windows-x86_64";
			string ArchitecturePathMac = @"prebuilt/darwin-x86_64";
			string ArchitecturePathLinux = @"prebuilt/linux-x86_64";

			if (Directory.Exists(Path.Combine(NDKPath, @"toolchains/llvm", ArchitecturePathWindows64)))
			{
				Logger.LogDebug("        Found Windows 64 bit versions of toolchain");
				ArchitecturePath = ArchitecturePathWindows64;
			}
			else if (Directory.Exists(Path.Combine(NDKPath, @"toolchains/llvm", ArchitecturePathWindows32)))
			{
				Logger.LogDebug("        Found Windows 32 bit versions of toolchain");
				ArchitecturePath = ArchitecturePathWindows32;
			}
			else if (Directory.Exists(Path.Combine(NDKPath, @"toolchains/llvm", ArchitecturePathMac)))
			{
				Logger.LogDebug("        Found Mac versions of toolchain");
				ArchitecturePath = ArchitecturePathMac;
				ExeExtension = "";
			}
			else if (Directory.Exists(Path.Combine(NDKPath, @"toolchains/llvm", ArchitecturePathLinux)))
			{
				Logger.LogDebug("        Found Linux versions of toolchain");
				ArchitecturePath = ArchitecturePathLinux;
				ExeExtension = "";
			}
			else
			{
				throw new BuildException("Couldn't find 32-bit or 64-bit versions of the Android toolchain with NDKROOT: " + NDKPath);
			}

			// get the installed version (in the form r10e and 100500)
			UEBuildPlatformSDK SDK = UEBuildPlatform.GetSDK(UnrealTargetPlatform.Android)!;
			NDKToolchainVersion = SDK.GetInstalledVersion();
			SDK.TryConvertVersionToInt(NDKToolchainVersion, out ulong NDKVersionIntTemp);
			NDKVersionInt = NDKVersionIntTemp;

			// figure out clang version (will live in toolchains/llvm from NDK 21 forward
			if (Directory.Exists(Path.Combine(NDKPath, @"toolchains/llvm")))
			{
				// look for version in AndroidVersion.txt (fail if not found)
				string VersionFilename = Path.Combine(NDKPath, @"toolchains/llvm", ArchitecturePath, "AndroidVersion.txt");
				if (!File.Exists(VersionFilename))
				{
					throw new BuildException("Cannot find supported Android toolchain");
				}
				string[] VersionFile = File.ReadAllLines(VersionFilename);
				string[] VersionParts = VersionFile[0].Split('.');
				ClangVersionMajor = Int32.Parse(VersionParts[0]);
				ClangVersionMinor = (VersionParts.Length > 1) ? Int32.Parse(VersionParts[1]) : 0;
				ClangVersionPatch = (VersionParts.Length > 2) ? Int32.Parse(VersionParts[2]) : 0;
			}
			else
			{
				throw new BuildException("Cannot find supported Android toolchain with NDKPath:" + NDKPath);
			}

			// set up the path to our toolchains
			ClangPath = Utils.CollapseRelativeDirectories(Path.Combine(NDKPath, @"toolchains/llvm", ArchitecturePath, @"bin/clang++" + ExeExtension));

			// Android (6317467 based on r365631c1) clang version 9.0.8 
			string AndroidClangBuildTmp = Utils.RunLocalProcessAndReturnStdOut(ClangPath, "--version", Logger);
			try
			{
				AndroidClangBuild = Regex.Match(AndroidClangBuildTmp, @"(\w+) based on").Groups[1].ToString();
				if (String.IsNullOrEmpty(AndroidClangBuild))
				{
					AndroidClangBuild = Regex.Match(AndroidClangBuildTmp, @"(\w+), based on").Groups[1].ToString();
				}
			}
			catch
			{
				Logger.LogWarning("Failed to retreive build version from {AndroidClangBuild}", AndroidClangBuild);
				AndroidClangBuild = "unknown";
			}

			// use lld for r21+
			ArPathArm64 = Utils.CollapseRelativeDirectories(Path.Combine(NDKPath, @"toolchains/llvm", ArchitecturePath, @"bin/llvm-ar" + ExeExtension));
			ArPathx64 = ArPathArm64;

			GCCToolchainPath = DirectoryReference.FromString(Path.Combine(NDKPath!, @"toolchains/llvm", ArchitecturePath!))!;
			SysrootPath = DirectoryReference.FromString(Path.Combine(NDKPath!, @"toolchains/llvm", ArchitecturePath!, "sysroot").Replace("\\", "/", StringComparison.Ordinal))!;
		}

		public bool Equals(FileReference? InProjectFile, ILogger InLogger)
		{
			return ProjectFile == InProjectFile && Logger == InLogger;
		}

		IEnumerable<string> AdditionalPaths(CppRootPaths Roots)
		{
			return new[]
			{
				$"--gcc-toolchain=\"{NormalizeCommandLinePath(GCCToolchainPath!, Roots)}\"",
				$"--sysroot=\"{NormalizeCommandLinePath(SysrootPath!, Roots)}\"",
			};
		}

		protected override ClangToolChainInfo GetToolChainInfo()
		{
			return new ClangToolChainInfo(DirectoryReference.FromString(AndroidPlatformSDK.GetNDKRoot(Logger)), new(ClangPath!), new(ArPathArm64!), Logger);
		}

		public static string GetGLESVersion(bool bBuildForES31)
		{
			string GLESversion = "0x00030000";

			if (bBuildForES31)
			{
				GLESversion = "0x00030002";
			}

			return GLESversion;
		}

		private bool BuildWithHiddenSymbolVisibility()
		{
			ConfigHierarchy Ini = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, DirectoryReference.FromFile(ProjectFile), UnrealTargetPlatform.Android);
			return Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bBuildWithHiddenSymbolVisibility", out bool bBuild) && bBuild;
		}

		private bool CompressDebugSymbols()
		{
			ConfigHierarchy Ini = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, DirectoryReference.FromFile(ProjectFile), UnrealTargetPlatform.Android);
			return Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bCompressDebugSymbols", out bool bBuild) && bBuild;
		}
		
		private bool DisableFunctionDataSections()
		{
			ConfigHierarchy Ini = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, DirectoryReference.FromFile(ProjectFile), UnrealTargetPlatform.Android);
			return Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bDisableFunctionDataSections", out bool bDisableFunctionDataSections) && bDisableFunctionDataSections;
		}

		private bool EnableAdvancedBinaryCompression()
		{
			ConfigHierarchy Ini = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, DirectoryReference.FromFile(ProjectFile), UnrealTargetPlatform.Android);
			return Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bEnableAdvancedBinaryCompression", out bool bEnableAdvancedBinaryCompression) && bEnableAdvancedBinaryCompression;
		}

		private bool DisableStackProtector()
		{
			ConfigHierarchy Ini = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, DirectoryReference.FromFile(ProjectFile), UnrealTargetPlatform.Android);
			return Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bDisableStackProtector", out bool bDisableStackProtector) && bDisableStackProtector;
		}

		private bool DisableLibCppSharedDependencyValidation()
		{
			ConfigHierarchy Ini = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, DirectoryReference.FromFile(ProjectFile), UnrealTargetPlatform.Android);
			return Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bDisableLibCppSharedDependencyValidation", out bool bDisableLibCppSharedDependencyValidation) && bDisableLibCppSharedDependencyValidation;
		}

		private static FileReference GetVersionScriptFileName(LinkEnvironment LinkEnvironment)
		{
			return FileReference.Combine(LinkEnvironment.IntermediateDirectory!, "ExportSymbols.ldscript");
		}

		public int GetNdkApiLevelInt(int MinNdk = MinimumNDKAPILevel)
		{
			string NDKVersion = GetNdkApiLevel();
			int NDKVersionInt = MinNdk;
			if (NDKVersion.Contains('-', StringComparison.Ordinal))
			{
				int Version;
				if (Int32.TryParse(NDKVersion.Substring(NDKVersion.LastIndexOf('-') + 1), out Version))
				{
					if (Version > NDKVersionInt)
					{
						NDKVersionInt = Version;
					}
				}
			}
			return NDKVersionInt;
		}

		public ulong GetNdkVersionInt() => NDKVersionInt;

		public static int GetClangVersionMajor() => ClangVersionMajor;

		class MinMax
		{
			internal bool PlatformsValid = false;
			internal int MinPlatform = -1;
			internal int MaxPlatform = -1;
		}
		static readonly ConcurrentDictionary<string, Lazy<MinMax>> CachedMinMax = [];

		private static bool ReadMinMaxPlatforms(string PlatformsFilename, out int MinPlatform, out int MaxPlatform)
		{
			MinMax minMax = CachedMinMax.GetOrAdd(PlatformsFilename, new Lazy<MinMax>(() =>
			{
				MinMax minMax = new();

				// try to read it
				try
				{
					JsonObject? PlatformsObj = null;
					if (JsonObject.TryRead(new FileReference(PlatformsFilename), out PlatformsObj))
					{
						minMax.PlatformsValid = PlatformsObj.TryGetIntegerField("min", out minMax.MinPlatform) && PlatformsObj.TryGetIntegerField("max", out minMax.MaxPlatform);
					}
				}
				catch (Exception)
				{
				}
				return minMax;
			})).Value;

			MinPlatform = minMax.MinPlatform;
			MaxPlatform = minMax.MaxPlatform;
			return minMax.PlatformsValid;
		}

		//This doesn't take into account SDK version overrides in packaging
		public int GetMinSdkVersion()
		{
			ConfigHierarchy Ini = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, DirectoryReference.FromFile(ProjectFile), UnrealTargetPlatform.Android);
			// TODO: Should this check the return value and return `MinimumNDKAPILevel` if false?
			Ini.GetInt32("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "MinSDKVersion", out int MinSDKVersion);
			return MinSDKVersion;
		}

		protected virtual bool ValidateNDK(string PlatformsFilename, string ApiString)
		{
			int MinPlatform, MaxPlatform;
			if (!ReadMinMaxPlatforms(PlatformsFilename, out MinPlatform, out MaxPlatform))
			{
				return false;
			}

			if (ApiString.Contains('-', StringComparison.Ordinal))
			{
				int Version;
				if (Int32.TryParse(ApiString.Substring(ApiString.LastIndexOf('-') + 1), out Version))
				{
					return (Version >= MinPlatform && Version <= MaxPlatform);
				}
			}
			return false;
		}

		public virtual string GetNdkApiLevel()
		{
			// ask the .ini system for what version to use
			ConfigHierarchy Ini = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, DirectoryReference.FromFile(ProjectFile), UnrealTargetPlatform.Android);
			string NDKLevel;
			Ini.GetString("/Script/AndroidPlatformEditor.AndroidSDKSettings", "NDKAPILevel", out NDKLevel!);

			// check for project override of NDK API level
			string ProjectNDKLevel;
			Ini.GetString("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "NDKAPILevelOverride", out ProjectNDKLevel!);
			ProjectNDKLevel = ProjectNDKLevel.Trim();
			if (!String.IsNullOrEmpty(ProjectNDKLevel))
			{
				NDKLevel = ProjectNDKLevel;
			}

			string PlatformsFilename = Environment.ExpandEnvironmentVariables("%NDKROOT%/meta/platforms.json");
			FileItem platformsJsonFile = FileItem.GetItemByPath(PlatformsFilename);
			if (!platformsJsonFile.Exists)
			{
				throw new BuildException("No NDK platforms found in {0}", PlatformsFilename);
			}

			if (NDKLevel == "latest")
			{
				if (!ReadMinMaxPlatforms(PlatformsFilename, out _, out int MaxPlatform))
				{
					throw new BuildException("No NDK platforms found in {0}", PlatformsFilename);
				}

				NDKLevel = "android-" + MaxPlatform.ToString();
			}

			// validate the platform NDK is installed
			if (!ValidateNDK(PlatformsFilename, NDKLevel))
			{
				throw new BuildException("The NDK API requested '{0}' not installed in {1}", NDKLevel, PlatformsFilename);
			}

			return NDKLevel;
		}

		public static string GetLargestApiLevel()
		{
			string PlatformsFilename = Environment.ExpandEnvironmentVariables("%NDKROOT%/meta/platforms.json");
			if (!File.Exists(PlatformsFilename))
			{
				throw new BuildException("No NDK platforms found in {0}", PlatformsFilename);
			}

			if (!ReadMinMaxPlatforms(PlatformsFilename, out _, out int MaxPlatform))
			{
				throw new BuildException("No NDK platforms found in {0}", PlatformsFilename);
			}

			return "android-" + MaxPlatform.ToString();
		}

		protected override void GetCompileArguments_IncludePaths(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			// Filter out paths that are not meant for this architecture
			// @todo can remove this when we only add paths properly for architecture
			foreach (DirectoryReference dir in CompileEnvironment.UserIncludePaths)
			{
				if (IsDirectoryForArch(dir.FullName, CompileEnvironment.Architecture))
				{
					Arguments.Add(GetUserIncludePathArgument(dir, CompileEnvironment));
				}
			}

			foreach (DirectoryReference dir in CompileEnvironment.SystemIncludePaths)
			{
				if (IsDirectoryForArch(dir.FullName, CompileEnvironment.Architecture))
				{
					Arguments.Add(GetSystemIncludePathArgument(dir, CompileEnvironment));
				}
			}
		}

		/// <inheritdoc/>
		protected override string EscapePreprocessorDefinition(string Definition)
		{
			return Definition.Contains('"', StringComparison.Ordinal) ? Definition.Replace("\"", "\\\"", StringComparison.Ordinal) : Definition;
		}

		/// <inheritdoc/>
		protected override void GetCompileArguments_WarningsAndErrors(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			base.GetCompileArguments_WarningsAndErrors(CompileEnvironment, Arguments);

			// @todo unlikely all needed
			Arguments.Add("-Wno-unknown-warning-option");   // Clang has been forked for Android - some warnings may not apply.
			Arguments.Add("-Wno-local-type-template-args"); // engine triggers this
			Arguments.Add("-Wno-return-type-c-linkage");    // needed for PhysX
			Arguments.Add("-Wno-reorder");                  // member initialization order
			Arguments.Add("-Wno-logical-op-parentheses");   // needed for external headers we can't change
			Arguments.Add("-Wno-nonportable-include-path"); // not all of these are real
			Arguments.Add("-Wno-extra-qualification");      // metasound DECLARE_METASOUND_xxx sometimes have redundant ::Metasound prefix
			Arguments.Add("-Wno-shorten-64-to-32");         // too many right now, same on other platforms 
		}

		/// <inheritdoc/>
		protected override void GetCompileArguments_Optimizations(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			base.GetCompileArguments_Optimizations(CompileEnvironment, Arguments);

			// optimization level
			if (!CompileEnvironment.bOptimizeCode)
			{
				Arguments.Add("-O0");
			}
			else
			{
				if (CompileEnvironment.OptimizationLevel == OptimizationMode.Size)
				{
					Arguments.Add("-Oz");
				}
				else if (CompileEnvironment.OptimizationLevel == OptimizationMode.SizeAndSpeed)
				{
					Arguments.Add("-Os");
				}
				else
				{
					if (CompileEnvironment.bPGOOptimize || CompileEnvironment.bPGOProfile)
					{
						// -Os tends to be both faster and smaller than -O3 when PGO is enabled
						Arguments.Add("-Os");
					}
					else
					{
						Arguments.Add("-O3");
						Arguments.Add("-mno-outline");  // Explicitly disable functions outlining
					}
				}
			}

			Arguments.Add("-fdebug-types-section");
		}

		/// <inheritdoc/>
		protected override void GetCompileArguments_Architecture(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			base.GetCompileArguments_Architecture(CompileEnvironment, Arguments);

			Arguments.AddRange(AdditionalPaths(CompileEnvironment.RootPaths));

			if (CompileEnvironment.Architecture == UnrealArch.Arm64)
			{
				Arguments.Add(ToolchainParamsArm64);
				Arguments.Add("-D__arm64__");            // for some reason this isn't defined and needed for PhysX
				Arguments.Add("-DPLATFORM_ANDROID_ARM64=1");
				Arguments.Add("-march=armv8-a");
			}
			else if (CompileEnvironment.Architecture == UnrealArch.X64)
			{
				Arguments.Add(ToolchainParamsx64);
				Arguments.Add("-DPLATFORM_ANDROID_X64=1");
				Arguments.Add("-fno-omit-frame-pointer");
				Arguments.Add("-march=atom");
			}
		}

		/// <inheritdoc/>
		protected override void GetCompileArguments_Debugging(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			base.GetCompileArguments_Debugging(CompileEnvironment, Arguments);

			// debug info
			if (CompileEnvironment.bCreateDebugInfo)
			{
				Arguments.Add("-g2");
				Arguments.Add("-gdwarf-4");

				if (CompressDebugSymbols())
				{
					Arguments.Add("-gz");
				}

				if (CompileEnvironment.bDebugLineTablesOnly)
				{
					Arguments.Add("-gline-tables-only");
				}
			}

			if (!DisableStackProtector())
			{
				Arguments.Add("-fstack-protector-strong");  // Emits extra code to check for buffer overflows
			}

			// Add flags for on-device debugging
			if (CompileEnvironment.Configuration == CppConfiguration.Debug)
			{
				Arguments.Add("-fno-omit-frame-pointer");   // Disable removing the save/restore frame pointer for better debugging
				if (CompilerVersionGreaterOrEqual(3, 6, 0))
				{
					Arguments.Add(" -fno-function-sections");    // Improve breakpoint location
				}
			}

			// Some switches interfere with on-device debugging
			if (CompileEnvironment.Configuration != CppConfiguration.Debug && !DisableFunctionDataSections())
			{
				Arguments.Add("-ffunction-sections");   // Places each function in its own section of the output file, linker may be able to perform opts to improve locality of reference
				Arguments.Add("-fdata-sections");       // Places each data item in its own section of the output file, linker may be able to perform opts to improve locality of reference
			}
		}

		/// <inheritdoc/>
		protected override void GetCompilerArguments_Sanitizers(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			// TODO: Reconcile with base
			//base.GetCompilerArguments_Sanitizers(CompileEnvironment, Arguments);

			ClangSanitizer Sanitizer = BuildWithSanitizer();
			if (Sanitizer != ClangSanitizer.None)
			{
				Arguments.Add("-fsanitize=" + GetCompilerOption(Sanitizer));

				if (Sanitizer == ClangSanitizer.Address || Sanitizer == ClangSanitizer.HwAddress)
				{
					Arguments.Add("-fno-omit-frame-pointer");
				}
			}

			//string? SanitizerMode = Environment.GetEnvironmentVariable("ENABLE_ADDRESS_SANITIZER");
			//if ((SanitizerMode != null && SanitizerMode == "YES") || (Options.HasFlag(ClangToolChainOptions.EnableAddressSanitizer)))
			//{
			//	Arguments.Add("-fsanitize=address -fno-omit-frame-pointer");
			//}

			//string? UndefSanitizerMode = Environment.GetEnvironmentVariable("ENABLE_UNDEFINED_BEHAVIOR_SANITIZER");
			//if ((UndefSanitizerMode != null && UndefSanitizerMode == "YES") || (Options.HasFlag(ClangToolChainOptions.EnableUndefinedBehaviorSanitizer)))
			//{
			//	Arguments.Add("-fsanitize=undefined -fno-sanitize=bounds,enum,return,float-divide-by-zero");
			//}

			//if (Options.HasFlag(ClangToolChainOptions.EnableThreadSanitizer))
			//{
			//	Arguments.Add("-fsanitize=thread");
			//}

			//if (Options.HasFlag(ClangToolChainOptions.EnableMemorySanitizer))
			//{
			//	Arguments.Add("-fsanitize=memory");
			//}
		}

		/// <inheritdoc/>
		protected override void GetCompileArguments_Global(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			base.GetCompileArguments_Global(CompileEnvironment, Arguments);

			// NDK setup (enforce minimum API level)
			int NDKApiLevel64Int = GetNdkApiLevelInt(21);       // deliberately use 21 minimum to force NDKApiLevel64Bit to update if below minimum
			string NDKApiLevel64Bit = GetNdkApiLevel();
			if (NDKApiLevel64Int < MinimumNDKAPILevel)
			{
				NDKApiLevel64Int = MinimumNDKAPILevel;
				NDKApiLevel64Bit = "android-" + MinimumNDKAPILevel;
			}

			Log.TraceInformationOnce("Compiling Native 64-bit code with NDK API '{0}'", NDKApiLevel64Bit);

			if (BuildWithHiddenSymbolVisibility())
			{
				Arguments.Add("-fvisibility=hidden");
				Arguments.Add("-fvisibility-inlines-hidden");
				//TODO: add when Android's clang will support this
				//Arguments.Add("-fvisibility-inlines-hidden-static-local-var");
			}

			Arguments.Add(GetRTTIFlag(CompileEnvironment));
			Arguments.Add("-no-canonical-prefixes");
			Arguments.Add("-fno-PIE");
			Arguments.Add("-funwind-tables");           // Just generates any needed static data, affects no code
			Arguments.Add("-fPIC");                     // Generates position-independent code (PIC) suitable for use in a shared library
			Arguments.Add("-fno-strict-aliasing");      // Prevents unwanted or invalid optimizations that could produce incorrect code
			Arguments.Add("-fno-short-enums");          // Do not allocate to an enum type only as many bytes as it needs for the declared range of possible values
			Arguments.Add("-fforce-emit-vtables");      // Helps with devirtualization
			Arguments.Add("-D_FORTIFY_SOURCE=2");       // FORTIFY default
			Arguments.Add($"-DPLATFORM_USED_NDK_VERSION_INTEGER={NDKApiLevel64Int}");       // NDK version
		}

		/// <inheritdoc/>
		protected override FileItem GetCompileArguments_FileType(CppCompileEnvironment CompileEnvironment, FileItem SourceFile, DirectoryReference OutputDir, List<string> Arguments, Action CompileAction, CPPOutput CompileResult)
		{
			FileItem TargetFile = base.GetCompileArguments_FileType(CompileEnvironment, SourceFile, OutputDir, Arguments, CompileAction, CompileResult);

			string Extension = Path.GetExtension(SourceFile.AbsolutePath).ToUpperInvariant();
			if (Extension == ".C")
			{
				if (SourceFile.AbsolutePath.Equals(GetNativeGluePath(), StringComparison.Ordinal))
				{
					// Remove visibility settings for android native glue. Since it doesn't decorate with visibility attributes.
					Arguments.RemoveAll(x => x.StartsWith("-fvisibility", StringComparison.Ordinal));
				}
				// remove any PCH includes - mostly for the force-added .c files in Launch as those will attempt to have the PCH used that was made with .cpp language
				Arguments.RemoveAll(x => x.StartsWith("-include-pch", StringComparison.Ordinal));
			}

			return TargetFile;
		}

		protected override void GetLinkArguments_Sanitizers(LinkEnvironment linkEnvironment, List<string> arguments)
		{
			ClangSanitizer Sanitizer = BuildWithSanitizer();
			if (Sanitizer != ClangSanitizer.None)
			{
				arguments.Add($"-fsanitize={GetCompilerOption(Sanitizer)}");
			}
		}

		protected virtual string GetLinkArguments(LinkEnvironment LinkEnvironment, UnrealArch Architecture, IActionGraphBuilder Graph)
		{
			string Result = String.Join(' ', AdditionalPaths(LinkEnvironment.RootPaths));

			//Result += " -nostdlib";
			Result += " -static-libstdc++";
			Result += " -no-canonical-prefixes";
			Result += " -shared";
			Result += " -Wl,-Bsymbolic";
			Result += " -Wl,--no-undefined";
			if (!DisableFunctionDataSections())
			{
				Result += " -Wl,--gc-sections"; // Enable garbage collection of unused input sections. works best with -ffunction-sections, -fdata-sections
			}

			if (!LinkEnvironment.bCreateDebugInfo)
			{
				Result += " -Wl,--strip-debug";
			}
			else if (CompressDebugSymbols())
			{
				Result += " -gz";
			}

			if (Architecture == UnrealArch.X64)
			{
				Result += ToolchainLinkParamsx64;
				Result += " -march=atom";
			}
			else // if (Architecture == UnrealArch.Arm64)
			{
				Result += ToolchainLinkParamsArm64;
				Result += " -march=armv8-a";
			}

			if (TargetRules?.bIdenticalCodeFolding == true)
			{
				Result += "-Wl,--icf=all";
			}

			if (LinkEnvironment.Configuration == CppConfiguration.Shipping)
			{
				Result += " -Wl,-O3";
			}

			Result += " -Wl,-no-pie";

			// use lld as linker (requires llvm-strip)
			Result += " -fuse-ld=lld";

			// make sure the DT_SONAME field is set properly (or we can a warning toast at startup on new Android)
			Result += " -Wl,-soname,libUnreal.so";

			if (!LinkEnvironment.AdditionalArguments.Contains("--version-script", StringComparison.Ordinal))
			{
				FileReference VersionScriptFileName = GetVersionScriptFileName(LinkEnvironment);
				// Make all symbols hidden (except new/delete operators and ones called from Java)
				Graph.CreateIntermediateTextFile(VersionScriptFileName, "{ global: _Znwm*; _Znam*; _ZdlPv*; _ZdaPv*; Java_*; ANativeActivity_onCreate; JNI_OnLoad; local: *; };");
				Result += " -Wl,--version-script=\"" + VersionScriptFileName + "\"";
			}
			else
			{
				Logger.LogInformation("Linker version script already passed through linker environment additional arguments.");
			}

			Result += " -Wl,--build-id=sha1";               // add build-id to make debugging easier

			// verbose output from the linker
			// Result += " -v";

			if (EnableAdvancedBinaryCompression())
			{
				int MinSDKVersion = GetMinSdkVersion();
				if (MinSDKVersion >= 28)
				{
					//Pack relocations in RELR format and Android APS2 packed format for RELA relocations if they can't be expressed in RELR
					Result += " -Wl,--pack-dyn-relocs=android+relr,--use-android-relr-tags";
				}
				else if (MinSDKVersion >= 23)
				{
					Result += " -Wl,--pack-dyn-relocs=android";
				}

				if (MinSDKVersion >= 23)
				{
					Result += " -Wl,--hash-style=gnu";  // generate GNU style hashes, faster lookup and faster startup. Avoids generating old .hash section. Supported on >= Android M
				}
			}

			// Enable support for non-4k virtual page sizes
			Result += " -z max-page-size=16384";

			{
				List<string> globalArguments = [];
				GetLinkArguments_Global(LinkEnvironment, globalArguments);
				if (globalArguments.Count > 0)
				{
					Result += " ";
					Result += String.Join(' ', globalArguments);
				}
			}

			return Result;
		}

		static string GetArArguments()
		{
			string Result = "";

			Result += " -r";

			return Result;
		}

		public static bool IsDirectoryForArch(string Dir, UnrealArch Arch)
		{
			string dir2 = Dir.Replace('\\', '/');
			while (dir2.Length > 0 && dir2[^1] == '/')
			{
				dir2 = dir2[..^1];
			}

			return IsDirectoryForArchCache.GetOrAdd((dir2, Arch), key => CheckCore(key.Path, Arch));
		}

		private static readonly Dictionary<UnrealArch, HashSet<string>> DisallowedNamesByArch = BuildDisallowed();
		private static readonly ConcurrentDictionary<(string Path, UnrealArch Arch), bool> IsDirectoryForArchCache = new();

		private static bool CheckCore(string normPath, UnrealArch arch)
		{
			HashSet<string> disallowed = DisallowedNamesByArch[arch];

			int start = 0;
			for (int i = 0; i <= normPath.Length; i++)
			{
				if (i == normPath.Length || normPath[i] == '/')
				{
					int len = i - start;
					if (len > 0)
					{
						string segment = normPath.Substring(start, len);

						// Exact directory match with a foreign arch name?
						if (disallowed.Contains(segment))
						{
							return false;
						}

						// Match patterns like "<arch>_API<digits>_NDK<digits>"
						int us = segment.IndexOf('_', StringComparison.Ordinal);
						if (us > 0)
						{
							string baseName = segment.Substring(0, us);
							if (disallowed.Contains(baseName) && LooksLikeApiNdkSuffix(segment.AsSpan(us)))
							{
								return false;
							}
						}
					}
					start = i + 1;
				}
			}
			return true;
		}

		// Expect suffix: "_API<digits>_NDK<digits>" (case-insensitive)
		private static bool LooksLikeApiNdkSuffix(ReadOnlySpan<char> suffix)
		{
			// _API
			if (suffix.Length < 4 || !suffix.Slice(0, 4).Equals("_api".AsSpan(), StringComparison.OrdinalIgnoreCase))
			{
				return false;
			}

			int i = 4, nDigitsApi = 0;
			while (i < suffix.Length && Char.IsDigit(suffix[i]))
			{
				i++;
				nDigitsApi++;
			}

			if (nDigitsApi == 0)
			{
				return false;
			}

			// _NDK
			if (i + 4 > suffix.Length || !suffix.Slice(i, 4).Equals("_ndk".AsSpan(), StringComparison.OrdinalIgnoreCase))
			{
				return false;
			}
			i += 4;

			int nDigitsNdk = 0;
			while (i < suffix.Length && Char.IsDigit(suffix[i]))
			{
				i++;
				nDigitsNdk++;
			}
			return nDigitsNdk > 0 && i == suffix.Length;
		}

		private static Dictionary<UnrealArch, HashSet<string>> BuildDisallowed()
		{
			Dictionary<UnrealArch, HashSet<string>> result = [];
			foreach (UnrealArch arch in AllFilterArchNames.Keys)
			{
				HashSet<string> set = new(StringComparer.OrdinalIgnoreCase);
				foreach (KeyValuePair<UnrealArch, string[]> kv in AllFilterArchNames)
				{
					if (kv.Key != arch)
					{
						foreach (string name in kv.Value)
						{
							set.Add(name);
						}
					}
				}
				result[arch] = set;
			}
			return result;
		}
		
		static bool ShouldSkipModule(string ModuleName, UnrealArch Arch)
		{
			foreach (string ModName in ModulesToSkip[Arch])
			{
				if (ModName == ModuleName)
				{
					return true;
				}
			}

			// if nothing was found, we are okay
			return false;
		}

		private static bool ShouldSkipLib(string FullLib, UnrealArch Arch)
		{
			// strip any absolute path
			string Lib = Path.GetFileNameWithoutExtension(FullLib);
			if (Lib.StartsWith("lib", StringComparison.Ordinal))
			{
				Lib = Lib.Substring(3);
			}

			// reject any libs we outright don't want to link with
			foreach (string LibName in LibrariesToSkip[Arch])
			{
				if (LibName == Lib)
				{
					return true;
				}
			}

			// deal with .so files with wrong architecture
			if (Path.GetExtension(FullLib) == ".so")
			{
				string ParentDirectory = Path.GetDirectoryName(FullLib)!;
				if (!IsDirectoryForArch(ParentDirectory, Arch))
				{
					return true;
				}
			}

			// apply the same directory filtering to libraries as we do to additional library paths
			if (!IsDirectoryForArch(Path.GetDirectoryName(FullLib)!, Arch))
			{
				return true;
			}

			// if another architecture is in the filename, reject it
			foreach (KeyValuePair<UnrealArch, string> ComboName in AllCpuSuffixes)
			{
				if (ComboName.Key != Arch)
				{
					string ArchitectureName = ComboName.Key.ToString();
					if (Lib.EndsWith(ComboName.Value, StringComparison.Ordinal) || Lib.EndsWith(ArchitectureName, StringComparison.Ordinal))
					{
						return true;
					}
				}
			}

			// if nothing was found, we are okay
			return false;
		}

		private static string GetNativeGluePath()
		{
			return Environment.GetEnvironmentVariable("NDKROOT") + "/sources/android/native_app_glue/android_native_app_glue.c";
		}

		private static string GetCpuFeaturesPath()
		{
			return Environment.GetEnvironmentVariable("NDKROOT") + "/sources/android/cpufeatures/cpu-features.c";
		}

		public override CppCompileEnvironment CreateSharedResponseFile(CppCompileEnvironment CompileEnvironment, FileReference OutResponseFile, IActionGraphBuilder Graph)
		{
			// Seems like Android clang toolchain does not handle response files including response files
			return CompileEnvironment;
		}

		private static void GenerateEmptyLinkFunctionsForRemovedModules(List<FileItem> SourceFiles, UnrealArch Arch, string ModuleName, DirectoryReference OutputDirectory, IActionGraphBuilder Graph)
		{
			// Only add to Launch module
			if (!ModuleName.Equals("Launch", StringComparison.Ordinal))
			{
				return;
			}

			string LinkerExceptionsName = "../UELinkerExceptions";
			FileReference LinkerExceptionsCPPFilename = FileReference.Combine(OutputDirectory, LinkerExceptionsName + ".cpp");

			List<string> Result = new List<string>();
			Result.Add("#include \"CoreTypes.h\"");
			Result.Add("");
			if (Arch == UnrealArch.X64)
			{
				Result.Add("#if PLATFORM_ANDROID_X64");
			}
			else
			{
				Result.Add("#if PLATFORM_ANDROID_ARM64");
			}

			foreach (string ModName in ModulesToSkip[Arch])
			{
				Result.Add("  void EmptyLinkFunctionForStaticInitialization" + ModName + "(){}");
			}
			foreach (string ModName in GeneratedModulesToSkip[Arch])
			{
				Result.Add("  void EmptyLinkFunctionForGeneratedCode" + ModName + "(){}");
			}
			Result.Add("#endif");

			Graph.CreateIntermediateTextFile(LinkerExceptionsCPPFilename, Result);

			SourceFiles.Add(FileItem.GetItemByFileReference(LinkerExceptionsCPPFilename));
		}

		// cache the location of NDK tools
		protected static Lock CtorLock = new();
		protected static string? ClangPath;
		protected static string ToolchainParamsArm64 = "";
		protected static string ToolchainParamsx64 = "";
		protected static string ToolchainLinkParamsArm64 = "";
		protected static string ToolchainLinkParamsx64 = "";
		protected static string? ArPathArm64;
		protected static string? ArPathx64;
		protected static string? ReadElfPath;
		protected static string? ObjDumpPath;

		public static string GetStripExecutablePath(UnrealArch UnrealArch)
		{
			string StripPath = ArPathArm64!;
			if (UnrealArch == UnrealArch.X64)
			{
				StripPath = ArPathx64!;
			}
			return StripPath.Replace("-ar", "-strip", StringComparison.Ordinal);
		}

		public static string GetReadElfExecutablePath()
		{
			return ReadElfPath!;
		}

		public static string GetObjDumpExecutablePath()
		{
			return ObjDumpPath!;
		}

		private readonly HashSet<UnrealArch> HasHandledLaunchModule = new();
		private readonly HashSet<UnrealArch> HasHandledCoreModule = new();

		protected override CPPOutput CompileCPPFiles(CppCompileEnvironment CompileEnvironment, IEnumerable<FileItem> InputFiles, DirectoryReference OutputDir, string ModuleName, IActionGraphBuilder Graph)
		{
			if (ShouldSkipCompile(CompileEnvironment) || ShouldSkipModule(ModuleName, CompileEnvironment.Architecture))
			{
				return new CPPOutput();
			}

			List<FileItem> ModifiedInputFiles = new(InputFiles);

			// Deal with Launch module special if first time seen
			if (!HasHandledLaunchModule.Contains(CompileEnvironment.Architecture) && (ModuleName.Equals("Launch", StringComparison.Ordinal) || ModuleName.Equals("AndroidLauncher", StringComparison.Ordinal)))
			{
				// Directly added NDK files for NDK extensions
				ModifiedInputFiles.Add(FileItem.GetItemByPath(GetNativeGluePath()));
				// Deal with dynamic modules removed by architecture
				GenerateEmptyLinkFunctionsForRemovedModules(ModifiedInputFiles, CompileEnvironment.Architecture, ModuleName, OutputDir, Graph);

				HasHandledLaunchModule.Add(CompileEnvironment.Architecture);
			}

			if (!HasHandledCoreModule.Contains(CompileEnvironment.Architecture) && ModuleName.Equals("Core", StringComparison.Ordinal) && (CompileEnvironment.PrecompiledHeaderAction == PrecompiledHeaderAction.None))
			{
				// This is used by Crypto code in Core
				ModifiedInputFiles.Add(FileItem.GetItemByPath(GetCpuFeaturesPath()));
				HasHandledCoreModule.Add(CompileEnvironment.Architecture);

			}

			return base.CompileCPPFiles(CompileEnvironment, ModifiedInputFiles, OutputDir, ModuleName, Graph);
		}

		public static string InlineArchName(string Pathname, UnrealArch Arch, bool bUseShortNames = false)
		{
#pragma warning disable CA1308 // Normalize strings to uppercase - file path may be case-sensitive
			string FinalArch = "-" + Arch.ToString().ToLowerInvariant();
#pragma warning restore CA1308
			if (bUseShortNames)
			{
				FinalArch = ShortArchNames[FinalArch];
			}
			return Path.Combine(Path.GetDirectoryName(Pathname)!, Path.GetFileNameWithoutExtension(Pathname) + FinalArch + Path.GetExtension(Pathname));
		}

		public static string RemoveArchName(string Pathname)
		{
			// remove all architecture names
			foreach (string Arch in AllCpuSuffixes.Values)
			{
				Pathname = Path.Combine(Path.GetDirectoryName(Pathname)!, Path.GetFileName(Pathname).Replace(Arch, "", StringComparison.Ordinal));
			}
			return Pathname;
		}

		public static DirectoryReference InlineArchIncludeFolder(DirectoryReference PathRef, UnrealArch Arch)
		{
			return DirectoryReference.Combine(PathRef, "include", Arch.ToString());
		}

		public override FileItem? LinkFiles(LinkEnvironment LinkEnvironment, bool bBuildImportLibraryOnly, IActionGraphBuilder Graph)
		{
			UnrealArch Arch = LinkEnvironment.Architecture;

			// Create an action that invokes the linker.
			Action LinkAction = Graph.CreateAction(ActionType.Link);
			LinkAction.RootPaths = LinkEnvironment.RootPaths;
			LinkAction.WorkingDirectory = Unreal.EngineSourceDirectory;

			if (LinkEnvironment.bIsBuildingLibrary)
			{
				if (Arch == UnrealArch.Arm64)
				{
					LinkAction.CommandPath = new FileReference(ArPathArm64!);
				}
				else
				{
					LinkAction.CommandPath = new FileReference(ArPathx64!);
				}
			}
			else
			{
				LinkAction.CommandPath = new FileReference(ClangPath!);
			}

			DirectoryReference LinkerPath = LinkAction.WorkingDirectory;

			// Get link arguments.
			LinkAction.CommandArguments = LinkEnvironment.bIsBuildingLibrary ? GetArArguments() : GetLinkArguments(LinkEnvironment, Arch, Graph);

			// Add the output file as a production of the link action.
			FileItem OutputFile;
			OutputFile = FileItem.GetItemByFileReference(LinkEnvironment.OutputFilePaths.Where(x => x.FullName.Contains(LinkEnvironment.Architecture.ToString(), StringComparison.InvariantCultureIgnoreCase)).First());
			LinkAction.ProducedItems.Add(OutputFile);
			LinkAction.StatusDescription = String.Format("{0}", Path.GetFileName(OutputFile.AbsolutePath));
			LinkAction.CommandVersion = AndroidClangBuild!;

			// LinkAction.bPrintDebugInfo = true;

			// Add the output file to the command-line.
			if (LinkEnvironment.bIsBuildingLibrary)
			{
				LinkAction.CommandArguments += String.Format(" \"{0}\"", OutputFile.AbsolutePath);
			}
			else
			{
				LinkAction.CommandArguments += String.Format(" -o \"{0}\"", OutputFile.AbsolutePath);
			}

			if (LinkEnvironment.bAllowLTCG && Options.HasFlag(ClangToolChainOptions.EnableThinLTO))
			{
				// Set the weight to number of logical cores as lld can max out the available cores
				LinkAction.Weight = SystemUtils.GetLogicalProcessorCount();

				// Disallow remote to prevent this long running action from running on an agent if remote linking is enabled
				LinkAction.bCanExecuteRemotely = false;
			}

			// Add the input files to a response file, and pass the response file on the command-line.
			List<string> InputFileNames = new List<string>();
			foreach (FileItem InputFile in LinkEnvironment.InputFiles)
			{
				string InputPath;
				if (InputFile.Location.IsUnderDirectory(LinkEnvironment.IntermediateDirectory!))
				{
					InputPath = NormalizeCommandLinePath(InputFile, LinkEnvironment.RootPaths);
				}
				else
				{
					InputPath = InputFile.Location.FullName;
				}
				InputFileNames.Add(String.Format("\"{0}\"", InputPath.Replace('\\', '/')));

				LinkAction.PrerequisiteItems.Add(InputFile);
			}

			string LinkResponseArguments = "";

			// libs don't link in other libs
			if (!LinkEnvironment.bIsBuildingLibrary)
			{
				// Make a list of library paths to search
				List<string> AdditionalLibraryPaths = new List<string>();
				List<string> AdditionalLibraries = new List<string>();

				// Add the library paths to the additional path list
				foreach (DirectoryReference LibraryPath in LinkEnvironment.SystemLibraryPaths)
				{
					// LinkerPaths could be relative or absolute
					string AbsoluteLibraryPath = Utils.ExpandVariables(LibraryPath.FullName);
					if (IsDirectoryForArch(AbsoluteLibraryPath, Arch))
					{
						// environment variables aren't expanded when using the $( style
						if (Path.IsPathRooted(AbsoluteLibraryPath) == false)
						{
							AbsoluteLibraryPath = Path.Combine(LinkerPath.FullName, AbsoluteLibraryPath);
						}
						AbsoluteLibraryPath = Utils.CollapseRelativeDirectories(AbsoluteLibraryPath);
						if (!AdditionalLibraryPaths.Contains(AbsoluteLibraryPath))
						{
							AdditionalLibraryPaths.Add(AbsoluteLibraryPath);
						}
					}
				}

				// discover additional libraries and their paths
				foreach (string SystemLibrary in LinkEnvironment.SystemLibraries)
				{
					if (!ShouldSkipLib(SystemLibrary, Arch))
					{
						if (String.IsNullOrEmpty(Path.GetDirectoryName(SystemLibrary)))
						{
							if (SystemLibrary.StartsWith("lib", StringComparison.Ordinal))
							{
								AdditionalLibraries.Add(SystemLibrary);
							}
							else
							{
								AdditionalLibraries.Add("lib" + SystemLibrary);
							}
						}
					}
				}

				bool doValidation = DisableLibCppSharedDependencyValidation() && ReadElfPath != null;
				List<FileReference> libsToCheck = [];

				foreach (FileReference Library in LinkEnvironment.Libraries)
				{
					if (ShouldSkipLib(Library.FullName, Arch))
					{
						continue;
					}
					string AbsoluteLibraryPath = Path.GetDirectoryName(Library.FullName)!;
					LinkAction.PrerequisiteItems.Add(FileItem.GetItemByFileReference(Library));

					string Lib = Path.GetFileNameWithoutExtension(Library.FullName);
					if (Lib.StartsWith("lib", StringComparison.Ordinal))
					{
						AdditionalLibraries.Add(Lib);
						if (!AdditionalLibraryPaths.Contains(AbsoluteLibraryPath))
						{
							AdditionalLibraryPaths.Add(AbsoluteLibraryPath);
						}
					}
					else
					{
						AdditionalLibraries.Add(AbsoluteLibraryPath);
					}

					if (doValidation)
					{
						libsToCheck.Add(Library);
					}
				}

				// Do checks in parallel because they are expensive
				Parallel2.ForEach(libsToCheck, Library =>
				{
					string? Output = Utils.RunLocalProcessAndReturnStdOut(ReadElfPath!, "--dynamic \"" + Library.FullName + "\"");
					if (Output != null)
					{
						if (Output.Contains("libc++_shared.so", StringComparison.Ordinal))
						{
							if (IsNewNDKModel())
							{
								throw new BuildException("Lib {0} depends on libc++_shared.so. There are known incompatibility issues when linking libc++_shared.so with Unreal Engine built with NDK22+." +
									" Please rebuild your dependencies with static libc++!", Library.GetFileNameWithoutExtension());
							}
							else
							{
								Logger.LogWarning("Lib {LibName} depends on libc++_shared.so. Unreal Engine is designed to be linked with libs that are built against static libc++ only. Please rebuild your dependencies with static libc++!", Library.GetFileNameWithoutExtension());
							}
						}
					}
				});

				// add the library paths to response
				foreach (string LibaryPath in AdditionalLibraryPaths)
				{
					LinkResponseArguments += String.Format(" -L\"{0}\"", LibaryPath);
				}

				// add libraries in a library group
				LinkResponseArguments += String.Format(" -Wl,--start-group");
				foreach (string AdditionalLibrary in AdditionalLibraries)
				{
					if (AdditionalLibrary.StartsWith("lib", StringComparison.Ordinal))
					{
						LinkResponseArguments += String.Format(" \"-l{0}\"", AdditionalLibrary.Substring(3));
					}
					else
					{
						LinkResponseArguments += String.Format(" \"{0}\"", AdditionalLibrary);
					}
				}
				LinkResponseArguments += String.Format(" -Wl,--end-group");

				// Write the MAP file to the output directory.
				if (LinkEnvironment.bCreateMapFile)
				{
					FileReference MAPFilePath = FileReference.Combine(LinkEnvironment.OutputDirectory!, Path.GetFileNameWithoutExtension(OutputFile.AbsolutePath) + ".map");
					FileItem MAPFile = FileItem.GetItemByFileReference(MAPFilePath);
					LinkResponseArguments += String.Format(" -Wl,--cref -Wl,-Map,\"{0}\"", MAPFilePath);
					LinkAction.ProducedItems.Add(MAPFile);

					// Export a list of object file paths, so we can locate the object files referenced by the map file
					ExportObjectFilePaths(LinkEnvironment, Path.ChangeExtension(MAPFilePath.FullName, ".objpaths"));
				}
			}

			// Add the additional arguments specified by the environment.
			LinkResponseArguments += LinkEnvironment.AdditionalArguments;

			// Write out a response file
			FileReference ResponseFileName = GetResponseFileName(LinkEnvironment, OutputFile);
			InputFileNames.Add(LinkResponseArguments.Replace("\\", "/", StringComparison.Ordinal));

			LinkAction.ResponseFileContents = [.. InputFileNames];
			FileItem ResponseFileItem = Graph.CreateIntermediateTextFile(ResponseFileName, LinkAction.ResponseFileContents);

			LinkAction.CommandArguments += String.Format(" @\"{0}\"", ResponseFileName);
			LinkAction.PrerequisiteItems.Add(ResponseFileItem);

			// Fix up the paths in commandline
			LinkAction.CommandArguments = LinkAction.CommandArguments.Replace("\\", "/", StringComparison.Ordinal);

			// Only execute linking on the local PC.
			LinkAction.bCanExecuteRemotely = false;

			FileReference VersionScriptFileName = GetVersionScriptFileName(LinkEnvironment);
			LinkAction.PrerequisiteItems.Add(FileItem.GetItemByFileReference(VersionScriptFileName));

			Logger.LogInformation("Link: {LinkActionCommandPathFullName} {LinkActionCommandArguments}", LinkAction.CommandPath.FullName, LinkAction.CommandArguments);

			return OutputFile;
		}

		private static void ExportObjectFilePaths(LinkEnvironment LinkEnvironment, string FileName)
		{
			// Write the list of object file directories
			HashSet<DirectoryReference> ObjectFileDirectories = new HashSet<DirectoryReference>();
			foreach (FileItem InputFile in LinkEnvironment.InputFiles)
			{
				ObjectFileDirectories.Add(InputFile.Location.Directory);
			}
			foreach (FileReference Library in LinkEnvironment.Libraries)
			{
				ObjectFileDirectories.Add(Library.Directory);
			}
			foreach (DirectoryReference LibraryPath in LinkEnvironment.SystemLibraryPaths)
			{
				ObjectFileDirectories.Add(LibraryPath);
			}
			foreach (string LibraryPath in (Environment.GetEnvironmentVariable("LIB") ?? "").Split(new char[] { ';' }, StringSplitOptions.RemoveEmptyEntries))
			{
				ObjectFileDirectories.Add(new DirectoryReference(LibraryPath));
			}
			Directory.CreateDirectory(Path.GetDirectoryName(FileName)!);
			File.WriteAllLines(FileName, ObjectFileDirectories.Select(x => x.FullName).OrderBy(x => x).ToArray());
		}

		public override void ModifyBuildProducts(ReadOnlyTargetRules Target, UEBuildBinary Binary, IEnumerable<string> Libraries, IEnumerable<UEBuildBundleResource> BundleResources, Dictionary<FileReference, BuildProductType> BuildProducts)
		{
			// only the .so needs to be in the manifest; we always have to build the apk since its contents depend on the project

			/*
			// the binary will have all of the .so's in the output files, we need to trim down to the shared apk (which is what needs to go into the manifest)
			if (Target.bDeployAfterCompile && Binary.Config.Type != UEBuildBinaryType.StaticLibrary)
			{
				foreach (FileReference BinaryPath in Binary.Config.OutputFilePaths)
				{
					FileReference ApkFile = BinaryPath.ChangeExtension(".apk");
					BuildProducts.Add(ApkFile, BuildProductType.Package);
				}
			}
			*/
		}

		public static void OutputReceivedDataEventHandler(DataReceivedEventArgs Line, ILogger Logger)
		{
			if ((Line != null) && (Line.Data != null))
			{
				Logger.LogInformation("{Output}", Line.Data);
			}
		}

		public static string GetStripPath(FileReference SourceFile)
		{
			string StripExe;
			if (SourceFile.FullName.Contains("-arm64", StringComparison.Ordinal))
			{
				StripExe = ArPathArm64!;
			}
			else
			if (SourceFile.FullName.Contains("-x64", StringComparison.Ordinal))
			{
				StripExe = ArPathx64!;
			}
			else
			{
				throw new BuildException("Couldn't determine Android architecture to strip symbols from {0}", SourceFile.FullName);
			}

			// fix the executable (replace the last -ar with -strip and keep any extension)
			int ArIndex = StripExe.LastIndexOf("-ar", StringComparison.Ordinal);
			StripExe = StripExe.Substring(0, ArIndex) + "-strip" + StripExe.Substring(ArIndex + 3);
			return StripExe;
		}

		public string GetBuildToolsVersion()
		{
			string HomePath = Environment.ExpandEnvironmentVariables("%ANDROID_HOME%");
			if (String.IsNullOrEmpty(HomePath))
			{
				throw new BuildException("Failed to find %ANDROID_HOME%. Ensure environment is configured properly");
			}

			uint BestVersion = 0;
			string? BestVersionString;

			ConfigHierarchy Ini = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, DirectoryReference.FromFile(ProjectFile), UnrealTargetPlatform.Android);
			Ini.GetString("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "BuildToolsOverride", out BestVersionString);

			if (BestVersionString == null || String.IsNullOrEmpty(BestVersionString) || BestVersionString == "latest")
			{
				// get a list of the directories in build-tools.. may be more than one set installed (or none which is bad)
				string[] Subdirs = Directory.GetDirectories(Path.Combine(HomePath, "build-tools"));
				if (Subdirs.Length == 0)
				{
					throw new BuildException("Failed to find %ANDROID_HOME%/build-tools subdirectory. Run SDK manager and install build-tools.");
				}

				// valid directories will have a source.properties with the Pkg.Revision (there is no guarantee we can use the directory name as revision)
				foreach (string CandidateDir in Subdirs)
				{
					string AaptFilename = Path.Combine(CandidateDir, OperatingSystem.IsWindows() ? "aapt.exe" : "aapt");
					string RevisionString = "";
					uint RevisionValue = 0;

					if (File.Exists(AaptFilename))
					{
						string SourcePropFilename = Path.Combine(CandidateDir, "source.properties");
						if (File.Exists(SourcePropFilename))
						{
							string[] PropertyContents = File.ReadAllLines(SourcePropFilename);
							foreach (string PropertyLine in PropertyContents)
							{
								if (PropertyLine.StartsWith("Pkg.Revision=", StringComparison.Ordinal))
								{
									RevisionString = PropertyLine.Substring(13);
									RevisionValue = GetRevisionValue(RevisionString);
									break;
								}
							}
						}
					}

					// remember it if newer version or haven't found one yet
					if (RevisionValue > BestVersion || BestVersionString == null)
					{
						BestVersion = RevisionValue;
						BestVersionString = RevisionString;
					}
				}
			}

			if (BestVersionString == null)
			{
				BestVersionString = FallbackBestVersionString;
				Logger.LogWarning("Failed to find %ANDROID_HOME%/build-tools subdirectory. Will attempt to use {BestVersionString}.", BestVersionString);
			}

			BestVersion = GetRevisionValue(BestVersionString);

			// With Gradle enabled, use at least the minimum version (will be installed by Gradle if missing)
			if (BestVersion < MinimumBestVersion)
			{
				BestVersionString = MinimumBestVersionString;
			}

			return BestVersionString;
		}

		public string GetBuildToolsPath()
		{
			string HomePath = Environment.ExpandEnvironmentVariables("%ANDROID_HOME%");
			return Path.Combine(HomePath, "build-tools", GetBuildToolsVersion());
		}

		public static void StripSymbols(FileReference SourceFile, FileReference TargetFile, UnrealArch UnrealArch, ILogger Logger, bool bStripAll = false)
		{
			string StripPath = GetStripExecutablePath(UnrealArch).Trim('"');
			StripSymbols(StripPath, SourceFile, TargetFile, Logger, bStripAll);
		}

		public static void StripSymbols(FileReference SourceFile, FileReference TargetFile, ILogger Logger, bool bStripAll = false)
		{
			string StripPath = GetStripPath(SourceFile).Trim('"');
			StripSymbols(StripPath, SourceFile, TargetFile, Logger, bStripAll);
		}

		private static void StripSymbols(string StripPath, FileReference SourceFile, FileReference TargetFile, ILogger Logger, bool bStripAll = false)
		{
			string StripCommand = bStripAll ? "--strip-unneeded" : "--strip-debug";

			ProcessStartInfo StartInfo = new ProcessStartInfo();
			StartInfo.FileName = StripPath;
			StartInfo.Arguments = $"{StripCommand} -o \"{TargetFile.FullName}\" \"{SourceFile.FullName}\"";
			StartInfo.UseShellExecute = false;
			StartInfo.CreateNoWindow = true;
			Utils.RunLocalProcessAndLogOutput(StartInfo, Logger);
		}

		public void SignApk(FileReference SourceApk, FileReference TargetApk, bool bZipAlign = false)
		{
			string KeyStore = Path.Combine(Environment.GetFolderPath(Environment.SpecialFolder.UserProfile), ".android", "debug.keystore");
			string KeyStorePassword = "android";
			string KeyAlias = "androiddebugkey";
			string KeyPassword = "android";

			SignApk(SourceApk, TargetApk, KeyStore, KeyStorePassword, KeyAlias, KeyPassword, bZipAlign);
		}

		public void SignApk(FileReference SourceApk, FileReference TargetApk, string KeyStore, string KeyStorePassword, string KeyAlias, string KeyPassword, bool bZipAlign = false)
		{
			Process Proc;
			ProcessStartInfo StartInfo;
			string BuildTools = GetBuildToolsPath();

			if (bZipAlign)
			{
				string ZipAlign = Path.Combine(BuildTools, "zipalign" + (OperatingSystem.IsWindows() ? ".exe" : ""));
				StartInfo = new ProcessStartInfo();
				StartInfo.FileName = ZipAlign;
				StartInfo.UseShellExecute = false;
				StartInfo.WindowStyle = ProcessWindowStyle.Minimized;

				StartInfo.Arguments = $"-P 16 -f -v 4 {SourceApk.FullName} {TargetApk.FullName}";

				Proc = new Process();
				Proc.StartInfo = StartInfo;
				Proc.Start();
				Proc.WaitForExit();
			}

			string ApkSigner = Path.Combine(BuildTools, "apksigner" + (OperatingSystem.IsWindows() ? ".bat" : ""));
			StartInfo = new ProcessStartInfo();
			StartInfo.UseShellExecute = false;
			StartInfo.WindowStyle = ProcessWindowStyle.Minimized;

			if (OperatingSystem.IsWindows())
			{
				StartInfo.FileName = "cmd.exe";
				StartInfo.ArgumentList.Add("/c");
				StartInfo.ArgumentList.Add(ApkSigner);
				StartInfo.ArgumentList.Add("sign");
				StartInfo.ArgumentList.Add("--ks");
				StartInfo.ArgumentList.Add(KeyStore);
				StartInfo.ArgumentList.Add("--ks-pass");
				StartInfo.ArgumentList.Add("pass:" + KeyStorePassword);
				StartInfo.ArgumentList.Add("--ks-key-alias");
				StartInfo.ArgumentList.Add(KeyAlias);
				StartInfo.ArgumentList.Add("--key-pass");
				StartInfo.ArgumentList.Add("pass:" + KeyPassword);
				StartInfo.ArgumentList.Add("--out");
				StartInfo.ArgumentList.Add(TargetApk.FullName);

				if (bZipAlign)
				{
					StartInfo.ArgumentList.Add(TargetApk.FullName);
				}
				else
				{
					StartInfo.ArgumentList.Add(SourceApk.FullName);
				}
			}
			else
			{
				StartInfo.FileName = "/bin/sh";
				StartInfo.ArgumentList.Add("-c");
				StartInfo.ArgumentList.Add("\"" + ApkSigner + "\" sign --ks \"" + KeyStore + "\" --ks-pass pass:" + KeyStorePassword + " --ks-key-alias " + KeyAlias + " --key-pass pass:" + KeyPassword + " --out \"" + TargetApk.FullName + "2\" \"" + SourceApk.FullName + "\"");
			}

			Proc = new Process();
			Proc.StartInfo = StartInfo;
			Proc.Start();
			Proc.WaitForExit();

			if (Proc.ExitCode != 0)
			{
				string Args = "";
				foreach (string Arg in StartInfo.ArgumentList)
				{
					Args += Arg + " ";
				}
				throw new BuildException("{0} failed with args {1}", StartInfo.FileName, Args);
			}
		}

		public ClangSanitizer BuildWithSanitizer()
		{
			if (Options.HasFlag(ClangToolChainOptions.EnableAddressSanitizer))
			{
				return ClangSanitizer.Address;
			}
			else if (Options.HasFlag(ClangToolChainOptions.EnableHWAddressSanitizer))
			{
				return ClangSanitizer.HwAddress;
			}
			else if (Options.HasFlag(ClangToolChainOptions.EnableThreadSanitizer))
			{
				return ClangSanitizer.Thread;
			}
			else if (Options.HasFlag(ClangToolChainOptions.EnableUndefinedBehaviorSanitizer))
			{
				return ClangSanitizer.UndefinedBehavior;
			}
			else if (Options.HasFlag(ClangToolChainOptions.EnableMinimalUndefinedBehaviorSanitizer))
			{
				return ClangSanitizer.UndefinedBehaviorMinimal;
			}

			return ClangSanitizer.None;
		}

		public static uint GetRevisionValue(string VersionString)
		{
			if (VersionString == null)
			{
				return 0;
			}

			// read up to 4 sections (ie. 20.0.3.5), first section most significant
			// each section assumed to be 0 to 255 range
			uint Value = 0;
			try
			{
				string[] Sections = VersionString.Split(".".ToCharArray());
				Value |= (Sections.Length > 0) ? (UInt32.Parse(Sections[0]) << 24) : 0;
				Value |= (Sections.Length > 1) ? (UInt32.Parse(Sections[1]) << 16) : 0;
				Value |= (Sections.Length > 2) ? (UInt32.Parse(Sections[2]) << 8) : 0;
				Value |= (Sections.Length > 3) ? UInt32.Parse(Sections[3]) : 0;
			}
			catch (Exception)
			{
				// ignore poorly formed version
			}
			return Value;
		}
	};
}
