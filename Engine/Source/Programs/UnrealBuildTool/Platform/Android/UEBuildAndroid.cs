// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// Android-specific target settings
	/// </summary>
	public partial class AndroidTargetRules
	{
		/// <summary>
		/// Enables address sanitizer (ASan)
		/// </summary>
		[CommandLine("-EnableASan")]
		public bool bEnableAddressSanitizer = false;

		/// <summary>
		/// Enables HW address sanitizer (HWASan)
		/// </summary>
		[CommandLine("-EnableHWASan")]
		public bool bEnableHWAddressSanitizer = false;

		/// <summary>
		/// Enables thread sanitizer (TSan)
		/// </summary>
		//[CommandLine("-EnableTSan")]
		public bool bEnableThreadSanitizer = false;

		/// <summary>
		/// Enables undefined behavior sanitizer (UBSan)
		/// </summary>
		[CommandLine("-EnableUBSan")]
		public bool bEnableUndefinedBehaviorSanitizer = false;

		/// <summary>
		/// Enables minimal undefined behavior sanitizer (UBSan)
		/// </summary>
		[CommandLine("-EnableMinUBSan")]
		public bool bEnableMinimalUndefinedBehaviorSanitizer = false;

		/// <summary>
		/// Enables runtime ray tracing support.
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/AndroidRuntimeSettings.AndroidRuntimeSettings")]
		public bool bEnableRayTracing = false;

		/// <summary>
		/// Enables the desktop renderer.
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/AndroidRuntimeSettings.AndroidRuntimeSettings")]
		public bool bSupportsVulkanSM5 = false;

		/// <summary>
		/// Enables ASIS plugin and STANDALONE support.
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/AndroidSingleInstanceServiceEditor.AndroidSingleInstanceServiceRuntimeSettings")]
		public bool bEnableASISPlugin = false;

	}

	/// <summary>
	/// Read-only wrapper for Android-specific target settings
	/// </summary>
	public partial class ReadOnlyAndroidTargetRules
	{
		/// <summary>
		/// The private mutable settings object
		/// </summary>
		private readonly AndroidTargetRules Inner;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="Inner">The settings object to wrap</param>
		public ReadOnlyAndroidTargetRules(AndroidTargetRules Inner)
		{
			this.Inner = Inner;
		}

		/// <summary>
		/// Accessors for fields on the inner TargetRules instance
		/// </summary>
		#region Read-only accessor properties 
#pragma warning disable CS1591

		public bool bEnableAddressSanitizer => Inner.bEnableAddressSanitizer;

		public bool bEnableHWAddressSanitizer => Inner.bEnableHWAddressSanitizer;

		public bool bEnableThreadSanitizer => Inner.bEnableThreadSanitizer;

		public bool bEnableUndefinedBehaviorSanitizer => Inner.bEnableUndefinedBehaviorSanitizer;

		public bool bEnableMinimalUndefinedBehaviorSanitizer => Inner.bEnableMinimalUndefinedBehaviorSanitizer;

		public bool bEnableRayTracing => Inner.bEnableRayTracing;

		public bool bSupportsVulkanSM5 => Inner.bSupportsVulkanSM5;

		public bool bEnableASISPlugin => Inner.bEnableASISPlugin;

		public AndroidTargetRules TargetRules => Inner;

#pragma warning restore CS1591
		#endregion
	}

	class AndroidArchitectureConfig : UnrealArchitectureConfig
	{
		public AndroidArchitectureConfig()
			: base(UnrealArchitectureMode.SingleTargetLinkSeparately, new[] { UnrealArch.Arm64, UnrealArch.X64 })
		{

		}

		public override UnrealArchitectures ActiveArchitectures(FileReference? ProjectFile, string? TargetName) => GetProjectArchitectures(ProjectFile);

		public override string GetFolderNameForArchitecture(UnrealArch Architecture)
		{
			return Architecture == UnrealArch.Arm64 ? "a" : "x";
		}

		private static UnrealArchitectures? CachedActiveArchitectures = null;
		private static FileReference? CachedActiveArchesProject = null;
		private static UnrealArchitectures GetProjectArchitectures(FileReference? ProjectFile)
		{
			if (CachedActiveArchitectures == null || ProjectFile != CachedActiveArchesProject)
			{
				List<string> ActiveArches = new();
				CachedActiveArchesProject = ProjectFile;

				// look in ini settings for what platforms to compile for
				ConfigHierarchy Ini = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, DirectoryReference.FromFile(ProjectFile), UnrealTargetPlatform.Android);
				bool bBuild;
				bool bUnsupportedBinaryBuildArch = false;

				if (Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bBuildForArm64", out bBuild) && bBuild)
				{
					ActiveArches.Add("arm64");
				}
				if (Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bBuildForx8664", out bBuild) && bBuild)
				{
					ActiveArches.Add("x64");
				}

				// we expect one to be specified
				if (ActiveArches.Count == 0)
				{
					if (bUnsupportedBinaryBuildArch)
					{
						throw new BuildException("Only architectures unsupported by binary-only engine selected.");
					}
					else
					{
						throw new BuildException("At least one architecture must be specified in Android project settings.");
					}
				}

				CachedActiveArchitectures = new UnrealArchitectures(ActiveArches);
			}
			return CachedActiveArchitectures;
		}
	}

	class AndroidPlatform : UEBuildPlatform
	{
		readonly UEBuildPlatformSDK SDK;

		public AndroidPlatform(UnrealTargetPlatform InTargetPlatform, UEBuildPlatformSDK InSDK, ILogger InLogger)
			: base(InTargetPlatform, InSDK, new AndroidArchitectureConfig(), InLogger)
		{
			SDK = InSDK;
		}

		public AndroidPlatform(AndroidPlatformSDK InSDK, ILogger InLogger) : this(UnrealTargetPlatform.Android, InSDK, InLogger)
		{
		}

		public override void ResetTarget(TargetRules Target)
		{
			ValidateTarget(Target);

			Target.bDeployAfterCompile = true;
		}

		public override void ValidateTarget(TargetRules Target)
		{
			base.ValidateTarget(Target);

			if (!String.IsNullOrWhiteSpace(Environment.GetEnvironmentVariable("CLANG_STATIC_ANALYZER_MODE")))
			{
				Target.StaticAnalyzer = StaticAnalyzer.Default;
				Target.StaticAnalyzerOutputType = (Environment.GetEnvironmentVariable("CLANG_ANALYZER_OUTPUT")?.Contains("html", StringComparison.OrdinalIgnoreCase) == true) ? StaticAnalyzerOutputType.Html : StaticAnalyzerOutputType.Text;
				Target.StaticAnalyzerMode = String.Equals(Environment.GetEnvironmentVariable("CLANG_STATIC_ANALYZER_MODE"), "shallow", StringComparison.OrdinalIgnoreCase) ? StaticAnalyzerMode.Shallow : StaticAnalyzerMode.Deep;
			}
			else if (Target.StaticAnalyzer == StaticAnalyzer.Clang)
			{
				Target.StaticAnalyzer = StaticAnalyzer.Default;
			}

			// Disable linking and ignore build outputs if we're using a static analyzer
			if (Target.StaticAnalyzer == StaticAnalyzer.Default)
			{
				Target.bDisableLinking = true;
				Target.bIgnoreBuildOutputs = true;

				// Clang static analysis requires non unity builds
				Target.bUseUnityBuild = false;
				if (Target.bStaticAnalyzerIncludeGenerated)
				{
					Target.bAlwaysUseUnityForGeneratedFiles = false;
				}
			}

			Target.bCompileRecast = true;
			Target.bCompileISPC = true;

			// disable plugins by architecture (if we are compiling for multiple architectures, we still need to disable the plugin for all architectures)
			if (Target.Architectures.Contains(UnrealArch.Arm64) && Target.Name != "UnrealHeaderTool")
			{
				Target.DisablePlugins.AddRange(Array.Empty<string>());
			}

			if (Target.StaticAllocator == StaticAllocatorType.None)
			{
				Target.StaticAllocator = StaticAllocatorType.Binned3;
			}
			if (Target.AndroidPlatform.bEnableAddressSanitizer || Target.AndroidPlatform.bEnableHWAddressSanitizer)
			{
				Target.StaticAllocator = StaticAllocatorType.Ansi;
			}
		}

		public override bool CanUseXGE()
		{
			// Disable when building on Linux
			return BuildHostPlatform.Current.Platform != UnrealTargetPlatform.Linux;
		}

		/// <inheritdoc/>
		public override VCProjectFileFormat GetRequiredVisualStudioVersion()
		{
			IEnumerable<VisualStudioInstallation> Installations = MicrosoftPlatformSDK.FindVisualStudioInstallations(Logger);

			Version? version = Installations
				.SelectMany(x => x.Packages.Where(x => x.Key.StartsWith("Component.GoogleAndroid", StringComparison.Ordinal)).Select(x => x.Value.Version))
				.Max();

			if (version != null && version < GetVersionRequiredForVisualStudio(VCProjectFileFormat.VisualStudio2026))
			{
				return VCProjectFileFormat.VisualStudio2022;
			}

			return VCProjectFileFormat.Default;
		}

		/// <inheritdoc/>
		public override Version GetVersionRequiredForVisualStudio(VCProjectFileFormat Format)
		{
			return Format >= VCProjectFileFormat.VisualStudio2026
				? new Version(26, 1, 102)
				: new Version();
		}

		public override bool IsBuildProduct(string FileName, string[] NamePrefixes, string[] NameSuffixes)
		{
			return IsBuildProductWithArch(FileName, NamePrefixes, NameSuffixes, ".so")
				|| IsBuildProductWithArch(FileName, NamePrefixes, NameSuffixes, ".apk")
				|| IsBuildProductWithArch(FileName, NamePrefixes, NameSuffixes, ".a");
		}

		static bool IsBuildProductWithArch(string Name, string[] NamePrefixes, string[] NameSuffixes, string Extension)
		{
			// Strip off the extension, then a CPU suffix, before testing whether it matches a build product name.
			if (Name.EndsWith(Extension, StringComparison.InvariantCultureIgnoreCase))
			{
				int ExtensionEndIdx = Name.Length - Extension.Length;
				foreach (string CpuSuffix in AndroidToolChain.AllCpuSuffixes.Values)
				{
					int CpuIdx = ExtensionEndIdx - CpuSuffix.Length;
					if (CpuIdx > 0 && String.Compare(Name, CpuIdx, CpuSuffix, 0, CpuSuffix.Length, StringComparison.OrdinalIgnoreCase) == 0)
					{
						return IsBuildProductName(Name, 0, CpuIdx, NamePrefixes, NameSuffixes);
					}
				}
			}
			return false;
		}

		public override string GetBinaryExtension(UEBuildBinaryType InBinaryType)
		{
			switch (InBinaryType)
			{
				case UEBuildBinaryType.DynamicLinkLibrary:
					return ".so";
				case UEBuildBinaryType.Executable:
					return ".so";
				case UEBuildBinaryType.StaticLibrary:
					return ".a";
			}
			return base.GetBinaryExtension(InBinaryType);
		}

		public override string[] GetDebugInfoExtensions(ReadOnlyTargetRules InTarget, UEBuildBinaryType InBinaryType)
		{
			return Array.Empty<string>();
		}

		public override void FindAdditionalBuildProductsToClean(ReadOnlyTargetRules Target, List<FileReference> FilesToDelete, List<DirectoryReference> DirectoriesToDelete)
		{
			base.FindAdditionalBuildProductsToClean(Target, FilesToDelete, DirectoriesToDelete);

			if (Target.ProjectFile != null)
			{
				DirectoriesToDelete.Add(DirectoryReference.Combine(DirectoryReference.FromFile(Target.ProjectFile), "Intermediate", "Android"));
			}
		}

		public override bool HasDefaultBuildConfig(UnrealTargetPlatform Platform, DirectoryReference ProjectPath)
		{
			string[] BoolKeys = new string[] {
				"bBuildForArm64", "bBuildForX8664",
				"bBuildForES31", "bBuildWithHiddenSymbolVisibility", "bSaveSymbols", "bSupportsVulkan", "bSupportsVulkanSM5"
			};
			string[] StringKeys = new string[] {
				"NDKAPILevelOverride"
			};

			// look up Android specific settings
			if (!DoProjectSettingsMatchDefault(Platform, ProjectPath, "/Script/AndroidRuntimeSettings.AndroidRuntimeSettings",
				BoolKeys, null, StringKeys))
			{
				return false;
			}

			// check the base settings
			return base.HasDefaultBuildConfig(Platform, ProjectPath);
		}

		public override bool ShouldCompileMonolithicBinary(UnrealTargetPlatform InPlatform)
		{
			// This platform currently always compiles monolithic
			return true;
		}

		/// <summary>
		/// Modify the rules for a newly created module, where the target is a different host platform.
		/// This is not required - but allows for hiding details of a particular platform.
		/// </summary>
		/// <param name="ModuleName">The name of the module</param>
		/// <param name="Rules">The module rules</param>
		/// <param name="Target">The target being build</param>
		public override void ModifyModuleRulesForOtherPlatform(string ModuleName, ModuleRules Rules, ReadOnlyTargetRules Target)
		{
			// don't do any target platform stuff if not available for host and opted in
			// do not require SDK to build it since we don't necessarily need it for editor building
			if (!UEBuildPlatform.IsPlatformAvailableForTarget(Platform, Target, bIgnoreSDKCheck: true))
			{
				return;
			}

			if ((Target.Platform == UnrealTargetPlatform.Win64) || (Target.Platform == UnrealTargetPlatform.Mac) || (Target.Platform == UnrealTargetPlatform.Linux))
			{
				bool bBuildShaderFormats = Target.bForceBuildShaderFormats;
				if (!Target.bBuildRequiresCookedData && Target.bBuildDeveloperTools)
				{
					if (ModuleName == "Engine")
					{
						Rules.DynamicallyLoadedModuleNames.Add("AndroidTargetPlatform");
						Rules.DynamicallyLoadedModuleNames.Add("AndroidTargetPlatformSettings");
						Rules.DynamicallyLoadedModuleNames.Add("AndroidTargetPlatformControls");
					}
					else if (ModuleName == "TargetPlatform")
					{
						bBuildShaderFormats = true;
						Rules.DynamicallyLoadedModuleNames.Add("TextureFormatOodle");
						Rules.DynamicallyLoadedModuleNames.Add("TextureFormatASTC");
						Rules.DynamicallyLoadedModuleNames.Add("TextureFormatETC2");  // ETC2 
						//Rules.DynamicallyLoadedModuleNames.Add("AudioFormatADPCM");	//@todo android: android audio
					}
				}

				// allow standalone tools to use targetplatform modules, without needing Engine
				if (ModuleName == "TargetPlatform")
				{
					if (Target.bForceBuildTargetPlatforms)
					{
						Rules.DynamicallyLoadedModuleNames.Add("AndroidTargetPlatform");
						Rules.DynamicallyLoadedModuleNames.Add("AndroidTargetPlatformSettings");
						Rules.DynamicallyLoadedModuleNames.Add("AndroidTargetPlatformControls");
					}

					if (bBuildShaderFormats)
					{
						//Rules.DynamicallyLoadedModuleNames.Add("ShaderFormatAndroid");		//@todo android: ShaderFormatAndroid
					}
				}

				if (ModuleName == "UnrealEd")
				{
					Rules.DynamicallyLoadedModuleNames.Add("AndroidPlatformEditor");
				}
			}
		}

		public override List<FileReference> FinalizeBinaryPaths(FileReference BinaryName, FileReference? ProjectFile, ReadOnlyTargetRules Target)
		{
			// make multiple output binaries
			List<FileReference> AllBinaries = new List<FileReference>();
			foreach (UnrealArch Architecture in Target.Architectures.Architectures)
			{
				string BinaryPath;
				if (Target.bShouldCompileAsDLL)
				{
					BinaryPath = Path.Combine(BinaryName.Directory.FullName, Target.Configuration.ToString(), "libUnreal.so");
				}
				else
				{
					BinaryPath = AndroidToolChain.InlineArchName(BinaryName.FullName, Architecture);
				}

				AllBinaries.Add(new FileReference(BinaryPath));
			}

			return AllBinaries;
		}

		public override void AddExtraModules(ReadOnlyTargetRules Target, List<string> PlatformExtraModules)
		{
			if (Target.Type != TargetType.Program)
			{
				PlatformExtraModules.Add("VulkanRHI");
			}
		}

		/// <summary>
		/// Modify the rules for a newly created module, in a target that's being built for this platform.
		/// This is not required - but allows for hiding details of a particular platform.
		/// </summary>
		/// <param name="ModuleName">The name of the module</param>
		/// <param name="Rules">The module rules</param>
		/// <param name="Target">The target being build</param>
		public override void ModifyModuleRulesForActivePlatform(string ModuleName, ModuleRules Rules, ReadOnlyTargetRules Target)
		{
			if (Target.StaticAnalyzer == StaticAnalyzer.Default)
			{
				Rules.StaticAnalyzerDisabledCheckers.UnionWith(["nullability.NullPassedToNonnull"]);
			}
		}

		public static bool IsAsisEnabled(ReadOnlyTargetRules Target)
		{
			return Target.AndroidPlatform.bEnableASISPlugin;
		}

		public virtual void SetUpSpecificEnvironment(ReadOnlyTargetRules Target, CppCompileEnvironment CompileEnvironment, LinkEnvironment LinkEnvironment, ILogger Logger)
		{
			string NDKPath = Environment.GetEnvironmentVariable("NDKROOT")!;
			NDKPath = NDKPath.Replace("\"", "", StringComparison.Ordinal);

			// There's some conspicuous static initialization that sets the Clang version
			_ = new AndroidToolChain(Target.ProjectFile, Logger);

			// figure out the NDK version
			string? NDKToolchainVersion = SDK.GetInstalledVersion();
			ulong NDKVersionInt;
			SDK.TryConvertVersionToInt(NDKToolchainVersion, out NDKVersionInt);

			// PLATFORM_ANDROID_NDK_VERSION is in the form 150100, where 15 is major version, 01 is the letter (1 is 'a'), 00 indicates beta revision if letter is 00
			CompileEnvironment.Definitions.Add(String.Format("PLATFORM_ANDROID_NDK_VERSION={0}", NDKVersionInt));

			Logger.LogInformation("NDK toolchain: {Version}, NDK version: {NdkVersion}, ClangVersion: {ClangVersion}", NDKToolchainVersion, NDKVersionInt, AndroidToolChain.GetClangVersionString());

			CompileEnvironment.Definitions.Add("PLATFORM_DESKTOP=0");
			CompileEnvironment.Definitions.Add("PLATFORM_CAN_SUPPORT_EDITORONLY_DATA=0");

			CompileEnvironment.Definitions.Add("WITH_OGGVORBIS=1");

			CompileEnvironment.Definitions.Add("UNICODE");
			CompileEnvironment.Definitions.Add("_UNICODE");

			CompileEnvironment.Definitions.Add("PLATFORM_ANDROID=1");
			CompileEnvironment.Definitions.Add("UE_PLATFORM_ANDROID=1");
			CompileEnvironment.Definitions.Add("ANDROID=1");

			CompileEnvironment.Definitions.Add("WITH_EDITOR=0");
			CompileEnvironment.Definitions.Add("USE_NULL_RHI=0");

			DirectoryReference NdkDir = new DirectoryReference(NDKPath);
			//CompileEnvironment.SystemIncludePaths.Add(DirectoryReference.Combine(NdkDir, "sources/cxx-stl/llvm-libc++/include"));

			// the toolchain will actually filter these out
			LinkEnvironment.SystemLibraryPaths.Add(DirectoryReference.Combine(NdkDir, "sources/cxx-stl/llvm-libc++/libs/arm64-v8a"));
			LinkEnvironment.SystemLibraryPaths.Add(DirectoryReference.Combine(NdkDir, "sources/cxx-stl/llvm-libc++/libs/x86_64"));

			CompileEnvironment.SystemIncludePaths.Add(DirectoryReference.Combine(NdkDir, "sources/android/native_app_glue"));
			CompileEnvironment.SystemIncludePaths.Add(DirectoryReference.Combine(NdkDir, "sources/android/cpufeatures"));

			//@TODO: Tegra Gfx Debugger - standardize locations - for now, change the hardcoded paths and force this to return true to test
			if (UseTegraGraphicsDebugger(Target))
			{
				//LinkEnvironment.LibraryPaths.Add("ThirdParty/NVIDIA/TegraGfxDebugger");
				//LinkEnvironment.LibraryPaths.Add("F:/NVPACK/android-kk-egl-t124-a32/stub");
				//LinkEnvironment.AdditionalLibraries.Add("Nvidia_gfx_debugger_stub");
			}

			if (!UseTegraGraphicsDebugger(Target))
			{
				LinkEnvironment.SystemLibraries.Add("GLESv3");
				LinkEnvironment.SystemLibraries.Add("EGL");
			}
			LinkEnvironment.SystemLibraries.Add("android");
			LinkEnvironment.SystemLibraries.Add("OpenSLES");
		}

		public override void SetUpEnvironment(ReadOnlyTargetRules Target, CppCompileEnvironment CompileEnvironment, LinkEnvironment LinkEnvironment)
		{

			CompileEnvironment.Definitions.Add("PLATFORM_DESKTOP=0");
			CompileEnvironment.Definitions.Add("PLATFORM_CAN_SUPPORT_EDITORONLY_DATA=0");

			CompileEnvironment.Definitions.Add("WITH_OGGVORBIS=1");

			CompileEnvironment.Definitions.Add("UNICODE");
			CompileEnvironment.Definitions.Add("_UNICODE");

			CompileEnvironment.Definitions.Add("PLATFORM_ANDROID=1");
			CompileEnvironment.Definitions.Add("UE_PLATFORM_ANDROID=1");
			CompileEnvironment.Definitions.Add("ANDROID=1");

			CompileEnvironment.Definitions.Add("WITH_EDITOR=0");
			CompileEnvironment.Definitions.Add("USE_NULL_RHI=0");

			if (Target.AndroidPlatform.bEnableRayTracing && Target.AndroidPlatform.bSupportsVulkanSM5)
			{
				Logger.LogInformation("Compiling with ray tracing enabled");
				CompileEnvironment.Definitions.Add("RHI_RAYTRACING=1");
			}

			if (Target.AndroidPlatform.bEnableASISPlugin)
			{
				Logger.LogInformation("Compiling with USE_ANDROID_STANDALONE");
				CompileEnvironment.Definitions.Add("USE_ANDROID_STANDALONE=1");

				Logger.LogInformation("Compiling with USE_ANDROID_ALTERNATIVE_SUSPEND");
				CompileEnvironment.Definitions.Add("USE_ANDROID_ALTERNATIVE_SUSPEND=1");
			}

			if (Target.bPGOOptimize || Target.bPGOProfile)
			{
				Logger.LogInformation("PGO {PgoType} build", Target.bPGOOptimize ? "optimize" : "profile");
				if (Target.bPGOOptimize)
				{
					CompileEnvironment.PGODirectory = DirectoryReference.Combine(Target.ProjectFile?.Directory ?? Unreal.WritableEngineDirectory, "Platforms", "Android", "Build", "PGO").FullName;
					CompileEnvironment.PGOFilenamePrefix = String.Format("{0}-Android.profdata", Target.Name);

					LinkEnvironment.PGODirectory = CompileEnvironment.PGODirectory;
					LinkEnvironment.PGOFilenamePrefix = CompileEnvironment.PGOFilenamePrefix;

					Logger.LogInformation("PGO Dir: {PgoDir}", CompileEnvironment.PGODirectory);
					Logger.LogInformation("PGO Prefix: {PgoPrefix}", CompileEnvironment.PGOFilenamePrefix);
				}
			}

			CompileEnvironment.Definitions.Add("INT64_T_TYPES_NOT_LONG_LONG=1");

			SetUpSpecificEnvironment(Target, CompileEnvironment, LinkEnvironment, Logger);

			// deliberately not linking stl or stdc++ here (c++_shared is default)
			LinkEnvironment.SystemLibraries.Add("c");
			LinkEnvironment.SystemLibraries.Add("dl");
			LinkEnvironment.SystemLibraries.Add("log");
			LinkEnvironment.SystemLibraries.Add("m");
			LinkEnvironment.SystemLibraries.Add("z");
			LinkEnvironment.SystemLibraries.Add("atomic");
		}

		private static bool UseTegraGraphicsDebugger(ReadOnlyTargetRules _)
		{
			// Disable for now
			return false;
		}

		public override bool ShouldCreateDebugInfo(ReadOnlyTargetRules Target)
		{
			switch (Target.Configuration)
			{
				case UnrealTargetConfiguration.Development:
				case UnrealTargetConfiguration.Shipping:
				case UnrealTargetConfiguration.Test:
				case UnrealTargetConfiguration.Debug:
				default:
					return true;
			}
		}

		public static ClangToolChainOptions CreateToolChainOptions(AndroidTargetRules TargetRules)
		{
			ClangToolChainOptions Options = ClangToolChainOptions.None;
			if (TargetRules.bEnableAddressSanitizer)
			{
				Options |= ClangToolChainOptions.EnableAddressSanitizer;
			}
			else if (TargetRules.bEnableHWAddressSanitizer)
			{
				Options |= ClangToolChainOptions.EnableHWAddressSanitizer;
			}
			if (TargetRules.bEnableThreadSanitizer)
			{
				Options |= ClangToolChainOptions.EnableThreadSanitizer;
			}
			if (TargetRules.bEnableUndefinedBehaviorSanitizer)
			{
				Options |= ClangToolChainOptions.EnableUndefinedBehaviorSanitizer;
			}
			else if (TargetRules.bEnableMinimalUndefinedBehaviorSanitizer)
			{
				Options |= ClangToolChainOptions.EnableMinimalUndefinedBehaviorSanitizer;
			}

			return Options;
		}

		public override UEToolChain CreateToolChain(ReadOnlyTargetRules Target)
		{
			ClangToolChainOptions Options = CreateToolChainOptions(Target.AndroidPlatform.TargetRules);
			if (Target.bAllowLTCG && Target.bPreferThinLTO)
			{
				Options |= ClangToolChainOptions.EnableThinLTO;
			}
			return new AndroidToolChain(Target.ProjectFile, Options, Logger);
		}
		public virtual UEToolChain CreateTempToolChainForProject(FileReference? ProjectFile)
		{
			AndroidTargetRules TargetRules = new AndroidTargetRules();
			CommandLine.ParseArguments(Environment.GetCommandLineArgs(), TargetRules, Logger);
			ClangToolChainOptions Options = CreateToolChainOptions(TargetRules);
			return new AndroidToolChain(ProjectFile, Options, Logger);
		}

		/// <inheritdoc/>
		public override void Deploy(TargetReceipt Receipt)
		{
			// do not package data if building via UBT.
			FileReference? ProjectFile = Receipt.ProjectFile;
			if (ProjectFile == null)
			{
				Utils.TryParseProjectFileArgument(new CommandLineArguments(Environment.GetCommandLineArgs()), Logger, out ProjectFile, MarkArgumentAsUsed: false);
				if (ProjectFile != null)
				{
					Logger.LogInformation("Deploy: receipt has no ProjectFile; recovered {ProjectFile} from command line.", ProjectFile);
				}
			}

			new UEDeployAndroid(ProjectFile, false, Logger).PrepTargetForDeployment(Receipt);
		}
	}

	class AndroidPlatformFactory : UEBuildPlatformFactory
	{
		public override UnrealTargetPlatform TargetPlatform => UnrealTargetPlatform.Android;

		public override void RegisterBuildPlatforms(ILogger Logger)
		{
			AndroidPlatformSDK SDK = new AndroidPlatformSDK(Logger);

			// Register this build platform
			UEBuildPlatform.RegisterBuildPlatform(new AndroidPlatform(SDK, Logger), Logger);
			UEBuildPlatform.RegisterPlatformWithGroup(UnrealTargetPlatform.Android, UnrealPlatformGroup.Android);
			UEBuildPlatform.RegisterPlatformWithGroup(UnrealTargetPlatform.Android, UnrealPlatformGroup.ThirtyHz);
		}
	}
}