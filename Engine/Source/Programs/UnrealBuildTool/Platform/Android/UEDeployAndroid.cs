// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.Globalization;
using System.IO;
using System.IO.Compression;
using System.Linq;
using System.Security.Cryptography;
using System.Text;
using System.Text.Json;
using System.Text.Json.Serialization;
using System.Xml.Linq;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	class UEDeployAndroid : UEBuildDeploy, IAndroidDeploy
	{
		private const string XML_HEADER = "<?xml version=\"1.0\" encoding=\"utf-8\"?>";

		// filename of current BundleTool
		private const string BUNDLETOOL_JAR = "bundletool-all-1.18.1.jar";

		// classpath of default android build tools gradle plugin
		private const string ANDROID_TOOLS_BUILD_GRADLE_VERSION = "com.android.tools.build:gradle:8.6.1";

		// default NDK version if not set
		private const string DEFAULT_NDK_VERSION = "27.2.12479018";

		// name of the only vulkan validation layer we're interested in 
		private const string ANDROID_VULKAN_VALIDATION_LAYER = "libVkLayer_khronos_validation.so";

		// list of supported aliased namespaces in manifest. First namespace in the list is the default one
		private static readonly List<AliasedXMLNamespace> ManifestXMLNamespaces = new() {
				new AliasedXMLNamespace { Alias = "android", Url = "http://schemas.android.com/apk/res/android" },
				new AliasedXMLNamespace { Alias = "dist", Url = "http://schemas.android.com/apk/distribution" },
				new AliasedXMLNamespace { Alias = "tools", Url = "http://schemas.android.com/tools" } };

		// Minimum Android SDK that must be used for Java compiling
		readonly int MinimumSDKLevel = 34;

		// Minimum SDK version needed for App Bundles
		readonly int MinimumSDKLevelForBundle = 21;

		// Minimum SDK version needed for Gradle based on active plugins
		private int MinimumSDKLevelForGradle = 19;

		// Reserved Java keywords not allowed in package names without modification
		private static readonly string[] JavaReservedKeywords = new string[] {
			"abstract", "assert", "boolean", "break", "byte", "case", "catch", "char", "class", "const", "continue", "default", "do",
			"double", "else", "enum", "extends", "final", "finally", "float", "for", "goto", "if", "implements", "import", "instanceof",
			"int", "interface", "long", "native", "new", "package", "private", "protected", "public", "return", "short", "static",
			"strictfp", "super", "switch", "synchronized", "this", "throw", "throws", "transient", "try", "void", "volatile", "while",
			"false", "null", "true"
		};

		protected FileReference? ProjectFile;

		/// <summary>
		/// Determines whether we package data inside the APK. Based on and  OR of "-ForcePackageData" being
		/// false and bPackageDataInsideApk in /Script/AndroidRuntimeSettings.AndroidRuntimeSettings being true
		/// </summary>
		protected bool bPackageDataInsideApk = false;

		/// <summary>
		/// Ignore AppBundle (AAB) generation setting if "-ForceAPKGeneration" specified
		/// </summary>
		[CommandLine("-ForceAPKGeneration", Value = "true")]
		public bool ForceAPKGeneration = false;

		/// <summary>
		/// Do not use Gradle if previous APK exists and only libUnreal.so changed
		/// </summary>
		[CommandLine("-BypassGradlePackaging", Value = "true")]
		public bool BypassGradlePackaging = false;

		/// <summary>
		/// Whether UBT is invoked from MSBuild.
		/// If false will, disable bDontBundleLibrariesInAPK, unless forced .
		/// </summary>
		[CommandLine(Prefix = "-FromMsBuild", Value = "true")]
		public bool bFromMSBuild = false;
		bool IAndroidDeploy.bFromMSBuild => bFromMSBuild;

		/// <summary>
		/// Forcing bDontBundleLibrariesInAPK to "true" or "false" ignoring any other option.
		/// </summary>
		[CommandLine("-ForceDontBundleLibrariesInAPK=")]
		public bool? ForceDontBundleLibrariesInAPK;

		/// <summary>
		/// Adds LD_PRELOAD'ed .so to capture all malloc/free/etc calls to libc.so and route them to our memory tracing.
		/// </summary>
		[CommandLine("-ScudoMemoryTracing", Value = "true")]
		public bool bEnableScudoMemoryTracing = false;

		/// <summary>
		/// Makes build interation faster 
		/// </summary>
		[CommandLine("-FastIterate", Value = "true")]
		public bool bFastIterate = false;
		
		/// <summary>
		/// Only generates gradle 
		/// </summary>
		[CommandLine("-OnlyGradleGeneration", Value = "true")]
		public bool bOnlyGradleGeneration = false;

		/// <summary>
		/// Won't generate or use OBBs if set
		/// </summary>
		[CommandLine("-NoObbs", Value = "true")]
		public bool bNoObbs = false;
		bool IAndroidDeploy.bNoObbs => bNoObbs;

		public UEDeployAndroid(FileReference? InProjectFile, bool InForcePackageData, ILogger InLogger)
			: base(InLogger)
		{
			ProjectFile = InProjectFile;

			// read the ini value and OR with the command line value
			bool IniValue = ReadPackageDataInsideApkFromIni(null);
			bPackageDataInsideApk = InForcePackageData || IniValue == true;

			CommandLine.ParseArguments(Environment.GetCommandLineArgs(), this, Logger);
		}

		private UnrealPluginLanguage? UPL = null;
		private string ActiveUPLFiles = "";
		private string? UPLHashCode = null;
		private bool ARCorePluginEnabled = false;
		private bool FacebookPluginEnabled = false;
		private bool EOSSDKPluginEnabled = false;
		private UnrealArchitectures? Architectures = null;

		public void SetAndroidPluginData(UnrealArchitectures Architectures, List<string> inPluginExtraData)
		{
			this.Architectures = Architectures;

			List<(string UEArch, string NativeArch)> Archs = Architectures.Architectures.Select(x => (x.ToString(), GetNDKArch(x))).ToList();

			// check if certain plugins are enabled
			ARCorePluginEnabled = false;
			FacebookPluginEnabled = false;
			EOSSDKPluginEnabled = false;
			ActiveUPLFiles = "";
			foreach (string Plugin in inPluginExtraData)
			{
				ActiveUPLFiles += Plugin + "\n";

				// check if the Facebook plugin was enabled
				if (Plugin.Contains("OnlineSubsystemFacebook_UPL", StringComparison.Ordinal))
				{
					FacebookPluginEnabled = true;
					continue;
				}

				// check if the ARCore plugin was enabled
				if (Plugin.Contains("GoogleARCoreBase_APL", StringComparison.Ordinal))
				{
					ARCorePluginEnabled = true;
					continue;
				}

				// check if the EOSShared plugin was enabled
				if (Plugin.Contains("EOSSDK", StringComparison.Ordinal))
				{
					EOSSDKPluginEnabled = true;
					continue;
				}
			}

			UPL = new UnrealPluginLanguage(ProjectFile, inPluginExtraData, Archs, ManifestXMLNamespaces, UnrealTargetPlatform.Android, Logger);
			UPLHashCode = UPL.GetUPLHash();
			//			APL.SetTrace();
		}

		private void SetMinimumSDKLevelForGradle()
		{
			if (FacebookPluginEnabled)
			{
				MinimumSDKLevelForGradle = Math.Max(MinimumSDKLevelForGradle, 15);
			}
			if (ARCorePluginEnabled)
			{
				MinimumSDKLevelForGradle = Math.Max(MinimumSDKLevelForGradle, 19);
			}
			if (EOSSDKPluginEnabled)
			{
				MinimumSDKLevelForGradle = Math.Max(MinimumSDKLevelForGradle, 23);
			}
		}

		private ConfigHierarchy GetConfigCacheIni(ConfigHierarchyType Type)
		{
			return ConfigCache.ReadHierarchy(Type, DirectoryReference.FromFile(ProjectFile), UnrealTargetPlatform.Android);
		}

		private static bool ValidateSDK(string PlatformsDir, string ApiString)
		{
			if (!Directory.Exists(PlatformsDir))
			{
				return false;
			}

			string SDKPlatformDir = Path.Combine(PlatformsDir, ApiString);
			return Directory.Exists(SDKPlatformDir);
		}

		private int GetApiLevelInt(string ApiString)
		{
			int DashIndex = ApiString.LastIndexOf('-');
			if (DashIndex != -1 && Int32.TryParse(ApiString.Substring(DashIndex + 1), out int Version))
			{
				return Version;
			}
			
			Logger.LogWarning("Unable to parse SDK API level '{ApiString}', it is expected to be of the form 'android-NN'", ApiString);
			return 0;
		}

		private string? CachedSDKLevel = null;
		private string GetSdkApiLevel(AndroidToolChain ToolChain)
		{
			if (CachedSDKLevel == null)
			{
				// ask the .ini system for what version to use
				ConfigHierarchy Ini = GetConfigCacheIni(ConfigHierarchyType.Engine);
				string SDKLevel;
				Ini.GetString("/Script/AndroidPlatformEditor.AndroidSDKSettings", "SDKAPILevel", out SDKLevel);

				// check for project override of SDK API level
				string ProjectSDKLevel;
				Ini.GetString("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "SDKAPILevelOverride", out ProjectSDKLevel);
				ProjectSDKLevel = ProjectSDKLevel.Trim();
				if (!String.IsNullOrEmpty(ProjectSDKLevel))
				{
					SDKLevel = ProjectSDKLevel;
				}

				// if we want to use whatever version the ndk uses, then use that
				if (SDKLevel == "matchndk")
				{
					SDKLevel = ToolChain.GetNdkApiLevel();
				}

				// run a command and capture output
				if (SDKLevel == "latest")
				{
					SDKLevel = AndroidToolChain.GetLargestApiLevel();
				}

				// make sure it is at least the minimum needed to compile
				int SDKLevelInt = GetApiLevelInt(SDKLevel);
				if (SDKLevelInt < MinimumSDKLevel)
				{
					Logger.LogInformation("Requires at least SDK API level {MinimumSDKLevel}, currently set to '{SDKLevel}'", MinimumSDKLevel, SDKLevel);
					SDKLevel = AndroidToolChain.GetLargestApiLevel();

					SDKLevelInt = GetApiLevelInt(SDKLevel);
					if (SDKLevelInt < MinimumSDKLevel)
					{
						SDKLevelInt = MinimumSDKLevel;
						SDKLevel = "android-" + MinimumSDKLevel.ToString();
						Logger.LogInformation("Gradle will attempt to download SDK API level {SDKLevelInt}", SDKLevelInt);
					}
				}

				// validate the platform SDK is installed
				string PlatformsDir = Path.Combine(Environment.ExpandEnvironmentVariables("%ANDROID_HOME%"), "platforms");
				if (!ValidateSDK(PlatformsDir, SDKLevel))
				{
					Logger.LogWarning("The SDK API requested '{SdkLevel}' not installed in {PlatformsDir}; Gradle will attempt to download it.", SDKLevel, PlatformsDir);
				}

				Logger.LogInformation("Building Java with SDK API level '{SDKLevel}'", SDKLevel);
				CachedSDKLevel = SDKLevel;
			}

			return CachedSDKLevel;
		}

		private string? CachedBuildToolsVersion = null;
		private string? LastAndroidHomePath = null;

		private string GetBuildToolsVersion(AndroidToolChain ToolChain)
		{
			// return cached path if ANDROID_HOME has not changed
			string HomePath = Environment.ExpandEnvironmentVariables("%ANDROID_HOME%");
			if (CachedBuildToolsVersion != null && LastAndroidHomePath == HomePath)
			{
				return CachedBuildToolsVersion;
			}

			CachedBuildToolsVersion = ToolChain.GetBuildToolsVersion();
			LastAndroidHomePath = HomePath;

			Logger.LogInformation("Building with Build Tools version '{CachedBuildToolsVersion}'", CachedBuildToolsVersion);

			return CachedBuildToolsVersion;
		}

		public static string GetOBBVersionNumber(int PackageVersion)
		{
			string VersionString = PackageVersion.ToString("0");
			return VersionString;
		}

		public bool GetPackageDataInsideApk()
		{
			return bPackageDataInsideApk;
		}

		/// <summary>
		/// Reads the bPackageDataInsideApk from AndroidRuntimeSettings
		/// </summary>
		/// <param name="Ini"></param>
		protected bool ReadPackageDataInsideApkFromIni(ConfigHierarchy? Ini)
		{
			// make a new one if one wasn't passed in
			Ini ??= GetConfigCacheIni(ConfigHierarchyType.Engine);

			// we check this a lot, so make it easy 
			bool bIniPackageDataInsideApk;
			Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bPackageDataInsideApk", out bIniPackageDataInsideApk);

			return bIniPackageDataInsideApk;
		}

		public bool UseExternalFilesDir(bool bDisallowExternalFilesDir, ConfigHierarchy? Ini = null)
		{
			if (bDisallowExternalFilesDir)
			{
				return false;
			}

			// make a new one if one wasn't passed in
			Ini ??= GetConfigCacheIni(ConfigHierarchyType.Engine);

			// we check this a lot, so make it easy 
			bool bUseExternalFilesDir;
			Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bUseExternalFilesDir", out bUseExternalFilesDir);

			return bUseExternalFilesDir;
		}

		public bool IsPackagingForMetaQuest(string CookFlavor, ConfigHierarchy? Ini = null)
		{
			// make a new one if one wasn't passed in
			Ini ??= GetConfigCacheIni(ConfigHierarchyType.Engine);

			bool result = Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bPackageForMetaQuest", out bool bPackageForMetaQuest);

			return (result && bPackageForMetaQuest) || CookFlavor == "_OpenXR";
		}

		public bool DisableVerifyOBBOnStartUp(ConfigHierarchy? Ini = null)
		{
			// make a new one if one wasn't passed in
			Ini ??= GetConfigCacheIni(ConfigHierarchyType.Engine);

			// we check this a lot, so make it easy 
			bool bDisableVerifyOBBOnStartUp;
			Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bDisableVerifyOBBOnStartUp", out bDisableVerifyOBBOnStartUp);

			return bDisableVerifyOBBOnStartUp;
		}

		private static bool SafeDeleteFile(string Filename, bool bCheckExists = true)
		{
			if (!bCheckExists || File.Exists(Filename))
			{
				try
				{
					File.SetAttributes(Filename, FileAttributes.Normal);
					File.Delete(Filename);
					return true;
				}
				catch (System.UnauthorizedAccessException)
				{
					throw new BuildException("File '{0}' is in use; unable to modify it.", Filename);
				}
				catch (System.Exception)
				{
					return false;
				}
			}
			return true;
		}

		public static void CopyFileDirectory(string SourceDir, string DestDir, Dictionary<string, string>? Replacements = null, string[]? Excludes = null)
		{
			if (!Directory.Exists(SourceDir))
			{
				return;
			}

			string[] Files = Directory.GetFiles(SourceDir, "*.*", SearchOption.AllDirectories);
			foreach (string Filename in Files)
			{
				if (Excludes != null)
				{
					// skip files in excluded directories
					string DirectoryName = Path.GetFileName(Path.GetDirectoryName(Filename))!;
					bool bExclude = false;
					foreach (string Exclude in Excludes)
					{
						if (DirectoryName == Exclude)
						{
							bExclude = true;
							break;
						}
					}
					if (bExclude)
					{
						continue;
					}
				}

				// skip template files
				if (Path.GetExtension(Filename) == ".template")
				{
					continue;
				}

				// make the dst filename with the same structure as it was in SourceDir
				string DestFilename = Path.Combine(DestDir, Utils.MakePathRelativeTo(Filename, SourceDir)).Replace('\\', Path.DirectorySeparatorChar).Replace('/', Path.DirectorySeparatorChar);

				// make the subdirectory if needed
				Directory.CreateDirectory(Path.GetDirectoryName(DestFilename)!);

				// some files are handled specially
				string Ext = Path.GetExtension(Filename);
				if (Ext == ".xml" && Replacements != null)
				{
					string Contents = File.ReadAllText(Filename);

					// replace some variables
					foreach (KeyValuePair<string, string> Pair in Replacements)
					{
						Contents = Contents.Replace(Pair.Key, Pair.Value, StringComparison.Ordinal);
					}

					bool bWriteFile = true;
					if (File.Exists(DestFilename))
					{
						string OriginalContents = File.ReadAllText(DestFilename);
						if (Contents == OriginalContents)
						{
							bWriteFile = false;
						}
					}

					// write out file if different
					if (bWriteFile)
					{
						SafeDeleteFile(DestFilename);
						File.WriteAllText(DestFilename, Contents);
					}
				}
				else
				{
					SafeDeleteFile(DestFilename);
					File.Copy(Filename, DestFilename);

					// preserve timestamp and clear read-only flags
					FileInfo DestFileInfo = new FileInfo(DestFilename);
					DestFileInfo.Attributes &= ~FileAttributes.ReadOnly;
					File.SetLastWriteTimeUtc(DestFilename, File.GetLastWriteTimeUtc(Filename));
				}
			}
		}

		public static void DeleteDirectory(string InPath, ILogger Logger, string SubDirectoryToKeep = "")
		{
			// skip the dir we want to
			if (String.Equals(Path.GetFileName(InPath), SubDirectoryToKeep, StringComparison.OrdinalIgnoreCase))
			{
				return;
			}

			// delete all files in here
			string[] Files;
			try
			{
				Files = Directory.GetFiles(InPath);
			}
			catch (Exception)
			{
				// directory doesn't exist so all is good
				return;
			}
			foreach (string Filename in Files)
			{
				try
				{
					// remove any read only flags
					FileInfo FileInfo = new FileInfo(Filename);
					FileInfo.Attributes &= ~FileAttributes.ReadOnly;
					FileInfo.Delete();
				}
				catch (Exception)
				{
					Logger.LogInformation("Failed to delete all files in directory {InPath}. Continuing on...", InPath);
				}
			}

			string[] Dirs = Directory.GetDirectories(InPath, "*.*", SearchOption.TopDirectoryOnly);
			foreach (string Dir in Dirs)
			{
				DeleteDirectory(Dir, Logger, SubDirectoryToKeep);
				// try to delete the directory, but allow it to fail (due to SubDirectoryToKeep still existing)
				try
				{
					Directory.Delete(Dir);
				}
				catch (Exception)
				{
					// do nothing
				}
			}
		}

		private static bool BinaryFileEquals(string SourceFilename, string DestFilename)
		{
			if (!File.Exists(SourceFilename))
			{
				return false;
			}
			if (!File.Exists(DestFilename))
			{
				return false;
			}

			FileInfo SourceInfo = new FileInfo(SourceFilename);
			FileInfo DestInfo = new FileInfo(DestFilename);
			if (SourceInfo.Length != DestInfo.Length)
			{
				return false;
			}

			using (FileStream SourceStream = new FileStream(SourceFilename, FileMode.Open, FileAccess.Read, FileShare.Read))
			using (BinaryReader SourceReader = new BinaryReader(SourceStream))
			using (FileStream DestStream = new FileStream(DestFilename, FileMode.Open, FileAccess.Read, FileShare.Read))
			using (BinaryReader DestReader = new BinaryReader(DestStream))
			{
				while (true)
				{
					byte[] SourceData = SourceReader.ReadBytes(4096);
					byte[] DestData = DestReader.ReadBytes(4096);
					if (SourceData.Length != DestData.Length)
					{
						return false;
					}
					if (SourceData.Length == 0)
					{
						return true;
					}
					if (!SourceData.SequenceEqual(DestData))
					{
						return false;
					}
				}
			}
		}

		private bool CopyIfDifferent(string SourceFilename, string DestFilename, bool bLog, bool bContentCompare)
		{
			if (!File.Exists(SourceFilename))
			{
				return false;
			}

			bool bDestFileAlreadyExists = File.Exists(DestFilename);
			bool bNeedCopy = !bDestFileAlreadyExists;

			if (!bNeedCopy)
			{
				if (bContentCompare)
				{
					bNeedCopy = !BinaryFileEquals(SourceFilename, DestFilename);
				}
				else
				{
					FileInfo SourceInfo = new FileInfo(SourceFilename);
					FileInfo DestInfo = new FileInfo(DestFilename);

					if (SourceInfo.Length != DestInfo.Length)
					{
						bNeedCopy = true;
					}
					// Allow 1s slop in case one of the files is on a FAT volume, and the other is not
					else if (Math.Abs((File.GetLastWriteTimeUtc(SourceFilename) - File.GetLastWriteTimeUtc(DestFilename)).TotalSeconds) > 1)
					{
						// destination file older than source
						bNeedCopy = true;
					}
				}
			}

			if (bNeedCopy)
			{
				if (bLog)
				{
					Logger.LogInformation("Copying {SourceFilename} to {DestFilename}", SourceFilename, DestFilename);
				}

				if (bDestFileAlreadyExists)
				{
					SafeDeleteFile(DestFilename, false);
				}
				File.Copy(SourceFilename, DestFilename);
				File.SetLastWriteTimeUtc(DestFilename, File.GetLastWriteTimeUtc(SourceFilename));

				// did copy
				return true;
			}

			// did not copy
			return false;
		}

		private void CleanCopyDirectory(string SourceDir, string DestDir, string[]? Excludes = null)
		{
			if (!Directory.Exists(SourceDir))
			{
				return;
			}
			if (!Directory.Exists(DestDir))
			{
				CopyFileDirectory(SourceDir, DestDir, null, Excludes);
				return;
			}

			// copy files that are different and make a list of ones to keep
			string[] StartingSourceFiles = Directory.GetFiles(SourceDir, "*.*", SearchOption.AllDirectories);
			List<string> FilesToKeep = new List<string>();
			foreach (string Filename in StartingSourceFiles)
			{
				if (Excludes != null)
				{
					// skip files in excluded directories
					string DirectoryName = Path.GetFileName(Path.GetDirectoryName(Filename))!;
					bool bExclude = false;
					foreach (string Exclude in Excludes)
					{
						if (DirectoryName == Exclude)
						{
							bExclude = true;
							break;
						}
					}
					if (bExclude)
					{
						continue;
					}
				}

				// make the dest filename with the same structure as it was in SourceDir
				string DestFilename = Path.Combine(DestDir, Utils.MakePathRelativeTo(Filename, SourceDir));

				// remember this file to keep
				FilesToKeep.Add(DestFilename);

				// only copy files that are new or different
				if (FilesAreDifferent(Filename, DestFilename))
				{
					if (File.Exists(DestFilename))
					{
						// xml files may have been rewritten but contents still the same so check contents also
						string Ext = Path.GetExtension(Filename);
						if (Ext == ".xml")
						{
							if (File.ReadAllText(Filename) == File.ReadAllText(DestFilename))
							{
								continue;
							}
						}

						// delete it so can copy over it
						SafeDeleteFile(DestFilename);
					}
					else
					{
						// make the subdirectory if needed
						Directory.CreateDirectory(Path.GetDirectoryName(DestFilename)!);
					}

					// copy it
					File.Copy(Filename, DestFilename);

					// preserve timestamp and clear read-only flags
					FileInfo DestFileInfo = new FileInfo(DestFilename);
					DestFileInfo.Attributes &= ~FileAttributes.ReadOnly;
					File.SetLastWriteTimeUtc(DestFilename, File.GetLastWriteTimeUtc(Filename));

					Logger.LogInformation("Copied file {DestFilename}.", DestFilename);
				}
			}

			// delete any files not in the keep list
			string[] StartingDestFiles = Directory.GetFiles(DestDir, "*.*", SearchOption.AllDirectories);
			foreach (string Filename in StartingDestFiles)
			{
				if (!FilesToKeep.Contains(Filename))
				{
					Logger.LogInformation("Deleting unneeded file {Filename}.", Filename);
					SafeDeleteFile(Filename);
				}
			}

			// delete any empty directories
			try
			{
				IEnumerable<string> BaseDirectories = Directory.EnumerateDirectories(DestDir, "*", SearchOption.AllDirectories).OrderByDescending(x => x);
				foreach (string directory in BaseDirectories)
				{
					if (Directory.Exists(directory) && Directory.GetFiles(directory, "*.*", SearchOption.AllDirectories).Length == 0)
					{
						Logger.LogInformation("Cleaning Directory {Directory} as empty.", directory);
						Directory.Delete(directory, true);
					}
				}
			}
			catch (Exception)
			{
				// likely System.IO.DirectoryNotFoundException, ignore it
			}
		}

		public static string GetUnrealBuildFilePath(string EngineDirectory)
		{
			return Path.GetFullPath(Path.Combine(EngineDirectory, "Build/Android/Java"));
		}

		public static string GetUnrealPreBuiltFilePath(string EngineDirectory)
		{
			return Path.GetFullPath(Path.Combine(EngineDirectory, "Build/Android/Prebuilt"));
		}

		public static string GetUnrealJavaSrcPath()
		{
			return Path.Combine("src", "com", "epicgames", "unreal");
		}

		public static string GetUnrealJavaFilePath(string EngineDirectory)
		{
			return Path.GetFullPath(Path.Combine(GetUnrealBuildFilePath(EngineDirectory), GetUnrealJavaSrcPath()));
		}

		public static string GetUnrealJavaBuildSettingsFileName(string EngineDirectory)
		{
			return Path.Combine(GetUnrealJavaFilePath(EngineDirectory), "JavaBuildSettings.java");
		}

		public static string GetUnrealJavaDownloadShimFileName(string Directory)
		{
			return Path.Combine(Directory, "DownloadShim.java");
		}

		public static string GetUnrealTemplateJavaSourceDir(string Directory)
		{
			return Path.Combine(GetUnrealBuildFilePath(Directory), "JavaTemplates");
		}

		public static string GetUnrealTemplateJavaDestination(string Directory, string FileName)
		{
			return Path.Combine(Directory, FileName);
		}

		public static string GetUnrealJavaOBBDataFileName(string Directory)
		{
			return Path.Combine(Directory, "OBBData.java");
		}

		public class TemplateFile
		{
			public string SourceFile;
			public string DestinationFile;

			public TemplateFile(string SourceFile, string DestinationFile)
			{
				this.SourceFile = SourceFile;
				this.DestinationFile = DestinationFile;
			}
		}

		private static void MakeDirectoryIfRequired(string DestFilename)
		{
			string DestSubdir = Path.GetDirectoryName(DestFilename)!;
			if (!Directory.Exists(DestSubdir))
			{
				Directory.CreateDirectory(DestSubdir);
			}
		}

		private int CachedStoreVersion = -1;
		private int CachedStoreVersionOffsetArm64 = 0;
		private int CachedStoreVersionOffsetX8664 = 0;

		public int GetStoreVersion(UnrealArch? Architecture)
		{
			if (CachedStoreVersion < 1)
			{
				ConfigHierarchy Ini = GetConfigCacheIni(ConfigHierarchyType.Engine);
				Ini.GetInt32("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "StoreVersion", out int StoreVersion);
				Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bUseChangeListAsStoreVersion", out bool bUseChangeListAsStoreVersion);

				// override store version with changelist if enabled
				if (bUseChangeListAsStoreVersion)
				{
					// make sure changelist is cached (clear unused warning)
					string EngineVersion = ReadEngineVersion();

					if (EngineVersion == null || !(Int32.TryParse(EngineChangelist, out int Changelist)) || Changelist == 0)
					{
						throw new BuildException("AndroidRuntimeSettings.bUseChangeListAsStoreVersion is enabled but no valid Engine version changelist was found!");
					}

					StoreVersion = Changelist;
				}

				bool IsBuildMachine = Unreal.IsBuildMachine();
				Logger.LogInformation("GotStoreVersion found v{StoreVersion}. (bUseChangeListAsStoreVersion={UseChangeListAsStoreVersion} IsBuildMachine={IsBuildMachine} EngineChangeList={EngineChangeList})", StoreVersion, bUseChangeListAsStoreVersion, IsBuildMachine, EngineChangelist);

				CachedStoreVersion = StoreVersion;

				Ini.GetInt32("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "StoreVersionOffsetArm64", out CachedStoreVersionOffsetArm64);
				Ini.GetInt32("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "StoreVersionOffsetX8664", out CachedStoreVersionOffsetX8664);
			}

			if (Architecture == UnrealArch.Arm64)
			{
				return CachedStoreVersion + CachedStoreVersionOffsetArm64;
			}
			else if (Architecture == UnrealArch.X64)
			{
				return CachedStoreVersion + CachedStoreVersionOffsetX8664;
			}

			return CachedStoreVersion;
		}

		private string? CachedVersionDisplayName = null;

		public string GetVersionDisplayName(bool bIsEmbedded)
		{
			if (String.IsNullOrEmpty(CachedVersionDisplayName))
			{
				ConfigHierarchy Ini = GetConfigCacheIni(ConfigHierarchyType.Engine);
				Ini.GetString("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "VersionDisplayName", out string VersionDisplayName);

				if (Unreal.IsBuildMachine())
				{
					Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bAppendChangeListToVersionDisplayName", out bool bAppendChangeListToVersionDisplayName);
					if (bAppendChangeListToVersionDisplayName)
					{
						VersionDisplayName = String.Format("{0}-{1}", VersionDisplayName, EngineChangelist);
					}

					Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bAppendPlatformToVersionDisplayName", out bool bAppendPlatformToVersionDisplayName);
					if (bAppendPlatformToVersionDisplayName)
					{
						VersionDisplayName = String.Format("{0}-Android", VersionDisplayName);
					}

					// append optional text to version name if embedded build
					if (bIsEmbedded)
					{
						if (Ini.GetString("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "EmbeddedAppendDisplayName", out string EmbeddedAppendDisplayName))
						{
							VersionDisplayName += EmbeddedAppendDisplayName;
						}
					}
				}

				CachedVersionDisplayName = VersionDisplayName;
			}

			return CachedVersionDisplayName;
		}

		public void WriteJavaOBBDataFile(string FileName, string PackageName, List<string> ObbSources, string CookFlavor, bool bPackageDataInsideApk, UnrealArch UnrealArch)
		{
			Logger.LogInformation("\n==== Writing to OBB data file {FileName} ====", FileName);

			// always must write if file does not exist
			bool bFileExists = File.Exists(FileName);
			bool bMustWriteFile = !bFileExists;

			string AppType = "";
			if (CookFlavor.EndsWith("Client", StringComparison.Ordinal))
			{
				//				AppType = ".Client";		// should always be empty now; fix up the name in batch file instead
			}

			int StoreVersion = GetStoreVersion(UnrealArch);

			StringBuilder obbData = new StringBuilder("package " + PackageName + ";\n\n");
			obbData.Append("public class OBBData\n{\n");
			obbData.Append("public static final String AppType = \"" + AppType + "\";\n\n");
			obbData.Append("public static class XAPKFile {\npublic final boolean mIsMain;\npublic final String mFileVersion;\n");
			obbData.Append("public final long mFileSize;\nXAPKFile(boolean isMain, String fileVersion, long fileSize) {\nmIsMain = isMain;\nmFileVersion = fileVersion;\nmFileSize = fileSize;\n");
			obbData.Append("}\n}\n\n");

			// write the data here
			obbData.Append("public static final XAPKFile[] xAPKS = {\n");
			// For each obb file... but we only have one... for now anyway.
			bool first = ObbSources.Count > 1;
			bool AnyOBBExists = false;
			foreach (string ObbSource in ObbSources)
			{
				bool bOBBExists = File.Exists(ObbSource);
				AnyOBBExists |= bOBBExists;

				obbData.Append("new XAPKFile(\n" + (ObbSource.Contains(".patch.", StringComparison.Ordinal) ? "false, // false signifies a patch file\n" : "true, // true signifies a main file\n"));
				obbData.AppendFormat("\"{0}\", // the version of the APK that the file was uploaded against\n", GetOBBVersionNumber(StoreVersion));
				obbData.AppendFormat("{0}L // the length of the file in bytes\n", bOBBExists ? new FileInfo(ObbSource).Length : 0);
				obbData.AppendFormat("){0}\n", first ? "," : "");
				first = false;
			}
			obbData.Append("};\n"); // close off data
			obbData.Append("};\n"); // close class definition off

			// see if we need to replace the file if it exists
			if (!bMustWriteFile && bFileExists)
			{
				string[] obbDataFile = File.ReadAllLines(FileName);

				// Must always write if AppType not defined
				bool bHasAppType = false;
				foreach (string FileLine in obbDataFile)
				{
					if (FileLine.Contains("AppType =", StringComparison.Ordinal))
					{
						bHasAppType = true;
						break;
					}
				}
				if (!bHasAppType)
				{
					bMustWriteFile = true;
				}

				// OBB must exist, contents must be different, and not packaging in APK to require replacing
				if (!bMustWriteFile && AnyOBBExists && !bPackageDataInsideApk && !obbDataFile.SequenceEqual((obbData.ToString()).Split('\n')))
				{
					bMustWriteFile = true;
				}
			}

			if (bMustWriteFile)
			{
				MakeDirectoryIfRequired(FileName);
				using (StreamWriter outputFile = new StreamWriter(FileName, false))
				{
					string[] obbSrc = obbData.ToString().Split('\n');
					foreach (string line in obbSrc)
					{
						outputFile.WriteLine(line);
					}
				}
			}
			else
			{
				Logger.LogInformation("\n==== OBB data file up to date so not writing. ====");
			}
		}

		public void WriteJavaDownloadSupportFiles(string ShimFileName, IEnumerable<TemplateFile> TemplateFiles, Dictionary<string, string> replacements)
		{
			// Deal with the Shim first as that is a known target and is easy to deal with
			// If it exists then read it
			string[]? DestFileContent = File.Exists(ShimFileName) ? File.ReadAllLines(ShimFileName) : null;

			StringBuilder ShimFileContent = new StringBuilder("package com.epicgames.unreal;\n\n");

			ShimFileContent.AppendFormat("import {0}.OBBDownloaderService;\n", replacements["$$PackageName$$"]);
			ShimFileContent.AppendFormat("import {0}.DownloaderActivity;\n", replacements["$$PackageName$$"]);

			// Do OBB file checking without using DownloadActivity to avoid transit to another activity
			ShimFileContent.Append("import android.app.Activity;\n");
			ShimFileContent.Append("import com.google.android.vending.expansion.downloader.Helpers;\n");
			ShimFileContent.AppendFormat("import {0}.OBBData;\n", replacements["$$PackageName$$"]);

			ShimFileContent.Append("\n\npublic class DownloadShim\n{\n");
			ShimFileContent.Append("\tpublic static OBBDownloaderService DownloaderService;\n");
			ShimFileContent.Append("\tpublic static DownloaderActivity DownloadActivity;\n");
			ShimFileContent.Append("\tpublic static Class<DownloaderActivity> GetDownloaderType() { return DownloaderActivity.class; }\n");

			// Do OBB file checking without using DownloadActivity to avoid transit to another activity
			ShimFileContent.Append("\tpublic static boolean expansionFilesDelivered(Activity activity, int version) {\n");
			ShimFileContent.Append("\t\tfor (OBBData.XAPKFile xf : OBBData.xAPKS) {\n");
			ShimFileContent.Append("\t\t\tString fileName = Helpers.getExpansionAPKFileName(activity, xf.mIsMain, Integer.toString(version), OBBData.AppType);\n");
			ShimFileContent.Append("\t\t\tGameActivity.Log.debug(\"Checking for file : \" + fileName);\n");
			ShimFileContent.Append("\t\t\tString fileForNewFile = Helpers.generateSaveFileName(activity, fileName);\n");
			ShimFileContent.Append("\t\t\tString fileForDevFile = Helpers.generateSaveFileNameDevelopment(activity, fileName);\n");
			ShimFileContent.Append("\t\t\tGameActivity.Log.debug(\"which is really being resolved to : \" + fileForNewFile + \"\\n Or : \" + fileForDevFile);\n");
			ShimFileContent.Append("\t\t\tif (Helpers.doesFileExist(activity, fileName, xf.mFileSize, false)) {\n");
			ShimFileContent.Append("\t\t\t\tGameActivity.Log.debug(\"Found OBB here: \" + fileForNewFile);\n");
			ShimFileContent.Append("\t\t\t}\n");
			ShimFileContent.Append("\t\t\telse if (Helpers.doesFileExistDev(activity, fileName, xf.mFileSize, false)) {\n");
			ShimFileContent.Append("\t\t\t\tGameActivity.Log.debug(\"Found OBB here: \" + fileForDevFile);\n");
			ShimFileContent.Append("\t\t\t}\n");
			ShimFileContent.Append("\t\t\telse return false;\n");
			ShimFileContent.Append("\t\t}\n");
			ShimFileContent.Append("\t\treturn true;\n");
			ShimFileContent.Append("\t}\n");

			ShimFileContent.Append("}\n");
			Logger.LogInformation("\n==== Writing to shim file {ShimFileName} ====", ShimFileName);

			// If they aren't the same then dump out the settings
			if (DestFileContent == null || !DestFileContent.SequenceEqual((ShimFileContent.ToString()).Split('\n')))
			{
				MakeDirectoryIfRequired(ShimFileName);
				using (StreamWriter outputFile = new StreamWriter(ShimFileName, false))
				{
					string[] shimSrc = ShimFileContent.ToString().Split('\n');
					foreach (string line in shimSrc)
					{
						outputFile.WriteLine(line);
					}
				}
			}
			else
			{
				Logger.LogInformation("\n==== Shim data file up to date so not writing. ====");
			}

			// Now we move on to the template files
			foreach (TemplateFile template in TemplateFiles)
			{
				string[] templateSrc = File.ReadAllLines(template.SourceFile);
				string[]? templateDest = File.Exists(template.DestinationFile) ? File.ReadAllLines(template.DestinationFile) : null;

				for (int i = 0; i < templateSrc.Length; ++i)
				{
					string srcLine = templateSrc[i];
					bool changed = false;
					foreach (KeyValuePair<string, string> kvp in replacements)
					{
						if (srcLine.Contains(kvp.Key, StringComparison.Ordinal))
						{
							srcLine = srcLine.Replace(kvp.Key, kvp.Value, StringComparison.Ordinal);
							changed = true;
						}
					}
					if (changed)
					{
						templateSrc[i] = srcLine;
					}
				}

				Logger.LogInformation("\n==== Writing to template target file {File} ====", template.DestinationFile);

				if (templateDest == null || templateSrc.Length != templateDest.Length || !templateSrc.SequenceEqual(templateDest))
				{
					MakeDirectoryIfRequired(template.DestinationFile);
					using (StreamWriter outputFile = new StreamWriter(template.DestinationFile, false))
					{
						foreach (string line in templateSrc)
						{
							outputFile.WriteLine(line);
						}
					}
				}
				else
				{
					Logger.LogInformation("\n==== Template target file up to date so not writing. ====");
				}
			}
		}

		public void WriteCrashlyticsResources(string UEBuildPath, string PackageName, bool bIsEmbedded, UnrealArch UnrealArch)
		{
			System.DateTime CurrentDateTime = System.DateTime.Now;
			string BuildID = Guid.NewGuid().ToString();

			string VersionDisplayName = GetVersionDisplayName(bIsEmbedded);

			StringBuilder CrashPropertiesContent = new StringBuilder("");
			CrashPropertiesContent.Append("# This file is automatically generated by Crashlytics to uniquely\n");
			CrashPropertiesContent.Append("# identify individual builds of your Android application.\n");
			CrashPropertiesContent.Append("#\n");
			CrashPropertiesContent.Append("# Do NOT modify, delete, or commit to source control!\n");
			CrashPropertiesContent.Append("#\n");
			CrashPropertiesContent.Append("# " + CurrentDateTime.ToString("D") + "\n");
			CrashPropertiesContent.Append("version_name=" + VersionDisplayName + "\n");
			CrashPropertiesContent.Append("package_name=" + PackageName + "\n");
			CrashPropertiesContent.Append("build_id=" + BuildID + "\n");
			CrashPropertiesContent.Append("version_code=" + GetStoreVersion(UnrealArch).ToString() + "\n");

			string CrashPropertiesFileName = Path.Combine(UEBuildPath, "assets", "crashlytics-build.properties");
			MakeDirectoryIfRequired(CrashPropertiesFileName);
			File.WriteAllText(CrashPropertiesFileName, CrashPropertiesContent.ToString());
			Logger.LogInformation("==== Write {CrashPropertiesFileName}  ====", CrashPropertiesFileName);

			StringBuilder BuildIDContent = new StringBuilder("");
			BuildIDContent.Append("<?xml version=\"1.0\" encoding=\"utf-8\" standalone=\"no\"?>\n");
			BuildIDContent.Append("<resources xmlns:tools=\"http://schemas.android.com/tools\">\n");
			BuildIDContent.Append("<!--\n");
			BuildIDContent.Append("  This file is automatically generated by Crashlytics to uniquely\n");
			BuildIDContent.Append("  identify individual builds of your Android application.\n");
			BuildIDContent.Append('\n');
			BuildIDContent.Append("  Do NOT modify, delete, or commit to source control!\n");
			BuildIDContent.Append("-->\n");
			BuildIDContent.Append("<string tools:ignore=\"UnusedResources, TypographyDashes\" name=\"com.crashlytics.android.build_id\" translatable=\"false\">" + BuildID + "</string>\n");
			BuildIDContent.Append("</resources>\n");

			string BuildIDFileName = Path.Combine(UEBuildPath, "res", "values", "com_crashlytics_build_id.xml");
			MakeDirectoryIfRequired(BuildIDFileName);
			File.WriteAllText(BuildIDFileName, BuildIDContent.ToString());
			Logger.LogInformation("==== Write {BuildIDFileName}  ====", BuildIDFileName);
		}

		public static string GetNDKArch(UnrealArch UnrealArch)
		{
			if (UnrealArch == UnrealArch.Arm64)
			{
				return "arm64-v8a";
			}
			else if (UnrealArch == UnrealArch.X64)
			{
				return "x86_64";
			}

			throw new BuildException("Unknown Unreal architecture '{0}'", UnrealArch);
		}

		public static UnrealArch GetUnrealArch(string NDKArch)
		{
			switch (NDKArch)
			{
				case "arm64-v8a":
				case "arm64": return UnrealArch.Arm64;
				case "x86_64":
				case "x64": return UnrealArch.X64;

				//				default: throw new BuildException("Unknown NDK architecture '{0}'", NDKArch);
				// future-proof by returning arm64 for unknown
				default: return UnrealArch.Arm64;
			}
		}

		private static void StripDebugSymbols(string SourceFileName, string TargetFileName, UnrealArch UnrealArch, ILogger Logger, bool bStripAll = false)
		{
			AndroidToolChain.StripSymbols(new FileReference(SourceFileName), new FileReference(TargetFileName), UnrealArch, Logger, bStripAll);
		}

		private static List<string> GetSODependencies(string SOFilename)
		{
			List<string> DependencyList = new List<string>();
			string ReadElfExecutable = AndroidToolChain.GetReadElfExecutablePath();
			string? Output = Utils.RunLocalProcessAndReturnStdOut(ReadElfExecutable, "--dynamic \"" + SOFilename + "\"");
			if (Output != null)
			{
				// ex:   0x0000000000000001 (NEEDED)         Shared library: [libGLESv3.so]
				int ScanStart = 0;
				int SharedIndex = Output.IndexOf("Shared library: [", ScanStart, StringComparison.Ordinal);
				while (SharedIndex >= 0)
				{
					ScanStart = SharedIndex + 17;
					int EndIndex = Output.IndexOf(']', ScanStart);
					if (EndIndex > 0)
					{
						string SharedLib = Output.Substring(ScanStart, EndIndex - ScanStart);

						// strip off lib (not needed for System.loadLibrary()
						if (SharedLib.StartsWith("lib", StringComparison.Ordinal))
						{
							SharedLib = SharedLib.Substring(3);
						}
						// strip off .so (not needed for System.loadLibrary()
						if (SharedLib.EndsWith(".so", StringComparison.Ordinal))
						{
							SharedLib = SharedLib.Substring(0, SharedLib.Length - 3);
						}

						DependencyList.Add(SharedLib);
					}
					else
					{
						// Unpaired [, break due to unexpected result
						break;
					}
					ScanStart = EndIndex + 1;
					SharedIndex = Output.IndexOf("Shared library: [", ScanStart, StringComparison.Ordinal);
				}
			}
			return DependencyList;
		}

		private static bool IsSO16KbCompatible(string FileName)
		{
			string ReadElfExecutable = AndroidToolChain.GetReadElfExecutablePath();
			string StdOutput = Utils.RunLocalProcessAndReturnStdOut(ReadElfExecutable, $"--elf-output-style=JSON --program-headers \"{FileName}\"");

			ReadElfOutput Output = default;
			bool bParsedOk = false;
			try
			{
				// expected in NDK >= 27
				List<ReadElfOutput>? JsonOutput = JsonSerializer.Deserialize<List<ReadElfOutput>>(StdOutput);
				if (JsonOutput != null)
				{
					Output = JsonOutput.First();
					bParsedOk = true;
				}
			}
			catch (Exception)
			{
				// ignored
			}

			if (!bParsedOk)
			{
				try
				{
					// expected in NDK 25
					List<Dictionary<string, ReadElfOutput>>? JsonOutput = JsonSerializer.Deserialize<List<Dictionary<string, ReadElfOutput>>>(StdOutput);
					if (JsonOutput != null)
					{
						Output = JsonOutput.First().First().Value;
						bParsedOk = true;
					}
				}
				catch (Exception)
				{
					// ignored
				}
			}

			if (!bParsedOk || Output.ProgramHeaders == null || Output.ProgramHeaders.Count == 0)
			{
				// don't fail the build if we can't check alignment
				return true;
			}

			bool bIsAlignmentOk = true;

			foreach (ReadElfProgramHeaderWrapper Wrapper in Output.ProgramHeaders)
			{
				ReadElfProgramHeader ProgramHeader = Wrapper.ProgramHeader;

				if (ProgramHeader.Type.Name == "PT_LOAD")
				{
					bIsAlignmentOk &= ProgramHeader.Alignment >= 16 * 1024;
				}
			}

			return bIsAlignmentOk;
		}

		private struct ReadElfOutput
		{
			public required List<ReadElfProgramHeaderWrapper> ProgramHeaders { get; set; }
		}

		private struct ReadElfProgramHeaderWrapper
		{
			public ReadElfProgramHeader ProgramHeader { get; set; }
		}

		private struct ReadElfProgramHeader
		{
			public ReadElfProgramHeaderType Type { get; set; }
			public int Alignment { get; set; }
		}

		private struct ReadElfProgramHeaderType
		{
			// only present in NDK >=27
			[JsonPropertyName("Name")]
			public string MaybeName { get; set; }

			// in NDK 25 this is a string, in NDK >=27 it's an int
			[JsonPropertyName("Value")]
			public object MaybeValue { get; set; }

			private string MaybeValueAsString => MaybeValue?.ToString() ?? String.Empty;

			[JsonIgnore]
			public string Name => !String.IsNullOrEmpty(MaybeName) ? MaybeName : MaybeValueAsString;
		}

		private void CheckSOInFolderAre16KbCompatible(string FolderPath)
		{
			try
			{
				IEnumerable<string> SOFiles = Directory.EnumerateFiles(FolderPath, "*.so", SearchOption.AllDirectories);

				foreach (string SOFile in SOFiles)
				{
					if (!IsSO16KbCompatible(SOFile))
					{
						Logger.LogWarning("Library '{SOFile}' is NOT compatible with 16kb page size alignment.", SOFile);
					}
				}
			}
			catch (Exception)
			{
				// ignored
			}
		}

		private void CheckSOInGradleFolderAre16KbCompatible(string GradlePath)
		{
			CheckSOInFolderAre16KbCompatible($"{GradlePath}/jniLibs/arm64-v8a");
			CheckSOInFolderAre16KbCompatible($"{GradlePath}/libs/arm64-v8a");
		}

		private static void CopySO(string FinalName, string SourceName)
		{
			// check to see if file is newer than last time we copied
			bool bFileExists = File.Exists(FinalName);
			TimeSpan Diff = File.GetLastWriteTimeUtc(FinalName) - File.GetLastWriteTimeUtc(SourceName);
			if (!bFileExists || Diff.TotalSeconds < -1 || Diff.TotalSeconds > 1)
			{
				SafeDeleteFile(FinalName);

				string? DirectoryName = Path.GetDirectoryName(FinalName);
				if (DirectoryName != null)
				{
					Directory.CreateDirectory(DirectoryName);
					File.Copy(SourceName, FinalName, true);

					// make sure it's writable if the source was readonly (e.g. autosdks)
					new FileInfo(FinalName).IsReadOnly = false;
					File.SetLastWriteTimeUtc(FinalName, File.GetLastWriteTimeUtc(SourceName));
				}
			}
		}

		public static string GetPlatformNDKHostName()
		{
			if (OperatingSystem.IsLinux())
			{
				return "linux-x86_64";
			}
			else if (OperatingSystem.IsMacOS())
			{
				return "darwin-x86_64";
			}

			return "windows-x86_64";
		}

		private static void CopySTL(string UnrealBuildPath, string NDKArch)
		{
			// copy it in!
			string SourceSTLSOName = Environment.ExpandEnvironmentVariables("%NDKROOT%/sources/cxx-stl/llvm-libc++/libs/") + NDKArch + "/libc++_shared.so";
			if (!File.Exists(SourceSTLSOName))
			{
				// NDK25 has changed a directory where it stores libs, check it instead
				string NDKTargetTripletName = (NDKArch == "x86_64") ? "x86_64-linux-android" : "aarch64-linux-android";
				SourceSTLSOName = Environment.ExpandEnvironmentVariables("%NDKROOT%/toolchains/llvm/prebuilt/") + GetPlatformNDKHostName() + "/sysroot/usr/lib/" + NDKTargetTripletName + "/libc++_shared.so";
			}
			string FinalSTLSOName = UnrealBuildPath + "/jni/" + NDKArch + "/libc++_shared.so";

			CopySO(FinalSTLSOName, SourceSTLSOName);
		}

		private static void CopyPSOService(string UnrealBuildPath, string PreBuiltPath, string NDKArch)
		{
			// copy it in!
			string SourceSOName = Path.Combine(PreBuiltPath, "PSOService/Android/Release/" + NDKArch) + "/libpsoservice.so";
			string FinalSOName = UnrealBuildPath + "/jni/" + NDKArch + "/libpsoservice.so";

			CopySO(FinalSOName, SourceSOName);
		}

		private static void CopySwappy(string UnrealBuildPath, string NDKArch, UnrealTargetConfiguration Configuration)
		{
			string BuildConfig = Configuration == UnrealTargetConfiguration.Debug ? "debug" : "release";
			string SourceSOName = Path.Combine(Unreal.EngineDirectory.ToString(), "Source", "ThirdParty", "GoogleGameSDK", "gamesdk", "libs", BuildConfig, NDKArch) + "/libswappy.so";
			string FinalSOName = UnrealBuildPath + "/jni/" + NDKArch + "/libswappy.so";

			CopySO(FinalSOName, SourceSOName);
		}

		void LogBuildSetup()
		{
			ConfigHierarchy Ini = GetConfigCacheIni(ConfigHierarchyType.Engine);
			Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bBuildForES31", out bool bBuildForES31);
			Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bSupportsVulkan", out bool bSupportsVulkan);

			Logger.LogInformation("bBuildForES31: {BuildForES31}", (bBuildForES31 ? "true" : "false"));
			Logger.LogInformation("bSupportsVulkan: {SupportsVulkan}", (bSupportsVulkan ? "true" : "false"));
		}

		void CopyVulkanValidationLayers(string UnrealBuildPath, string NDKArch, string Configuration)
		{
			ConfigHierarchy Ini = GetConfigCacheIni(ConfigHierarchyType.Engine);
			Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bSupportsVulkan", out bool bSupportsVulkan);
			Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bSupportsVulkanSM5", out bool bSupportsVulkanSM5);

			bool bCopyVulkanLayers = (bSupportsVulkan || bSupportsVulkanSM5) && (Configuration != "Shipping");
			if (bCopyVulkanLayers)
			{
				string VulkanLayersDir = Path.Combine(Unreal.EngineDirectory.ToString(), "Binaries", "ThirdParty", "Vulkan", "Android", NDKArch);
				if (Directory.Exists(VulkanLayersDir))
				{
					Logger.LogInformation("Copying {ANDROID_VULKAN_VALIDATION_LAYER} vulkan layer from {VulkanLayersDir}", ANDROID_VULKAN_VALIDATION_LAYER, VulkanLayersDir);
					string DestDir = Path.Combine(UnrealBuildPath, "libs", NDKArch);
					Directory.CreateDirectory(DestDir);
					string SourceFilename = Path.Combine(VulkanLayersDir, ANDROID_VULKAN_VALIDATION_LAYER);
					string DestFilename = Path.Combine(DestDir, ANDROID_VULKAN_VALIDATION_LAYER);
					SafeDeleteFile(DestFilename);
					File.Copy(SourceFilename, DestFilename);
					FileInfo DestFileInfo = new FileInfo(DestFilename);
					DestFileInfo.Attributes &= ~FileAttributes.ReadOnly;
					File.SetLastWriteTimeUtc(DestFilename, File.GetLastWriteTimeUtc(SourceFilename));
				}
				else
				{
					Logger.LogWarning("{ANDROID_VULKAN_VALIDATION_LAYER} vulkan layer not found at {VulkanLayersDir}, skipping", ANDROID_VULKAN_VALIDATION_LAYER, VulkanLayersDir);
				}

				AndroidPlatformSDK.GetPath(Ini, "/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "DebugVulkanLayerDirectory", out string DebugVulkanLayerDirectory);
				Logger.LogInformation("DebugVulkanLayerDirectory {LayerDir}", DebugVulkanLayerDirectory);

				if (!String.IsNullOrEmpty(DebugVulkanLayerDirectory))
				{
					DebugVulkanLayerDirectory = Environment.ExpandEnvironmentVariables(DebugVulkanLayerDirectory);
					if (!Path.IsPathRooted(DebugVulkanLayerDirectory))
					{
						DebugVulkanLayerDirectory = Path.Combine(Unreal.RootDirectory.ToString(), DebugVulkanLayerDirectory);
					}

					string LayersDir = Path.Combine(DebugVulkanLayerDirectory, NDKArch);

					if (Directory.Exists(LayersDir))
					{
						string DestDir = Path.Combine(UnrealBuildPath, "libs", NDKArch);
						Logger.LogInformation("Copying Debug vulkan layers from {DebugVulkanLayerDirectory} to {DestDir}", DebugVulkanLayerDirectory, DestDir);

						if (!Directory.Exists(DestDir))
						{
							Directory.CreateDirectory(DestDir);
						}
						CopyFileDirectory(LayersDir, DestDir);
					}
					else
					{
						Logger.LogWarning("DebugVulkanLayerDirectory vulkan layers not found at {DebugVulkanLayerDirectory}, skipping", DebugVulkanLayerDirectory);
					}
				}
			}
		}

		void CopyClangSanitizerLib(AndroidToolChain ToolChain, string UnrealBuildPath, string NDKArch, AndroidToolChain.ClangSanitizer Sanitizer)
		{
			string Architecture = "-aarch64";
			switch (NDKArch)
			{
				case "armeabi-v7a":
					Architecture = "-arm";
					break;
				case "x86_64":
					Architecture = "-x86_64";
					break;
				case "x86":
					Architecture = "-i686";
					break;
			}

			string LibName = "asan";
			switch (Sanitizer)
			{
				case AndroidToolChain.ClangSanitizer.HwAddress:
					{
						// no need to bundle asan .so in NDK r26b+
						LibName = AndroidToolChain.HasEmbeddedHWASanSupport() ? String.Empty : "hwasan";
						break;
					}
				case AndroidToolChain.ClangSanitizer.UndefinedBehavior:
					LibName = "ubsan_standalone";
					break;
				case AndroidToolChain.ClangSanitizer.UndefinedBehaviorMinimal:
					LibName = "ubsan_minimal";
					break;
				case AndroidToolChain.ClangSanitizer.Thread:
					LibName = "tsan";
					break;
			}

			string SanitizerFullLibName = String.IsNullOrEmpty(LibName) ? String.Empty : "libclang_rt." + LibName + Architecture + "-android.so";

			// NDK r26b+ needs different wrap.sh script
			string WrapShName = Sanitizer == AndroidToolChain.ClangSanitizer.HwAddress && AndroidToolChain.HasEmbeddedHWASanSupport() ? "hwasan.sh" : "asan.sh";

			string WrapShFilePath = Path.Combine(Environment.ExpandEnvironmentVariables("%NDKROOT%"), "wrap.sh", WrapShName);

			string PlatformHostName = GetPlatformNDKHostName();

			string LibsVersion = AndroidToolChain.GetClangVersionString();
			if (ToolChain.GetNdkVersionInt() >= 270000)
			{
				LibsVersion = AndroidToolChain.GetClangVersionMajor().ToString(CultureInfo.InvariantCulture);
			}

			string SanitizerLib = String.IsNullOrEmpty(SanitizerFullLibName) ?
				String.Empty :
				Path.Combine(Environment.ExpandEnvironmentVariables("%NDKROOT%"), "toolchains", "llvm", "prebuilt", PlatformHostName, (AndroidToolChain.HasEmbeddedHWASanSupport() ? "lib" : "lib64"), "clang", LibsVersion, "lib", "linux", SanitizerFullLibName);

			if (!String.IsNullOrEmpty(SanitizerLib))
			{
				if (File.Exists(SanitizerLib))
				{
					string LibDestDir = Path.Combine(UnrealBuildPath, "libs", NDKArch);
					Directory.CreateDirectory(LibDestDir);

					string LibDestFilePath = Path.Combine(LibDestDir, SanitizerFullLibName);
					Logger.LogInformation("Copying {LibName} lib from {SanitizerLib} to {LibDestFilePath}", LibName, SanitizerLib, LibDestFilePath);
					File.Copy(SanitizerLib, LibDestFilePath, true);
				}
				else
				{
					throw new BuildException("No {0} lib found in {1}", LibName, SanitizerLib);
				}
			}

			if (File.Exists(WrapShFilePath))
			{
				string WrapDestDir = Path.Combine(UnrealBuildPath, "resources", "lib", NDKArch);
				Directory.CreateDirectory(WrapDestDir);

				string WrapDestFilePath = Path.Combine(WrapDestDir, "wrap.sh");
				Logger.LogInformation("Copying wrap.sh from {WrapShFilePath} to {WrapDestFilePath}", WrapShFilePath, WrapDestFilePath);

				string WrapShContent = File.ReadAllText(WrapShFilePath);
				// replace the hwasan.sh with a debuggable script.
				// see https://developer.android.com/ndk/guides/wrap-script#debugging_when_using_wrapsh
				// first, check that this is the hwasan.sh we're expecting:
				if (WrapShFilePath.Contains("hwasan.sh", StringComparison.Ordinal))
				{
					if (WrapShContent.Replace("\n", "", StringComparison.Ordinal).Replace("\r", "", StringComparison.Ordinal) == @"#!/system/bin/shLD_HWASAN=1 exec ""$@""")
					{
						Logger.LogInformation("Substituting {WrapShFilePath} with debuggable version.", WrapShFilePath);
						WrapShContent =
@"#!/system/bin/sh
HERE=$(cd ""$(dirname ""$0"")"" && pwd)

cmd=$1
shift

# This must be called *before* `LD_PRELOAD` is set. Otherwise, if this is a 32-
# bit app running on a 64-bit device, the 64-bit getprop will fail to load
# because it will preload a 32-bit ASan runtime.
# https://github.com/android/ndk/issues/1744
os_version=$(getprop ro.build.version.sdk)

if [ ""$os_version"" -eq ""27"" ]; then
	cmd=""$cmd -Xrunjdwp:transport=dt_android_adb,suspend=n,server=y -Xcompiler-option --debuggable $@""
elif [ ""$os_version"" -eq ""28"" ]; then
	cmd=""$cmd -XjdwpProvider:adbconnection -XjdwpOptions:suspend=n,server=y -Xcompiler-option --debuggable $@""
else
	cmd=""$cmd -XjdwpProvider:adbconnection -XjdwpOptions:suspend=n,server=y $@""
fi

LD_HWASAN=1 exec $cmd";
					}
					else
					{
						Logger.LogWarning("Warning: The generated APK may not be debuggable. Unrecognized {WrapShFilePath} was not replaced with a debuggable script.", WrapShFilePath);
					}
				}
				// handle any CRLF that may have crept in from windows text file handling.
				WrapShContent = WrapShContent.Replace("\r", "", StringComparison.Ordinal);
				File.WriteAllText(WrapDestFilePath, WrapShContent);

				FileAttributes Attributes = File.GetAttributes(WrapDestFilePath);
				Attributes &= ~FileAttributes.ReadOnly;
				File.SetAttributes(WrapDestFilePath, Attributes);
			}
			else
			{
				throw new BuildException("No asan wrap.sh found in {0}", WrapShFilePath);
			}
		}

		void CopyScudoMemoryTracerLib(string UnrealBuildPath, UnrealArch UnrealArch, string NDKArch, string PackageName)
		{
			if (UnrealArch != UnrealArch.Arm64 && UnrealArch != UnrealArch.X64)
			{
				throw new BuildException("ScudoMemoryTrace not supported for arch {0}", UnrealArch.ToString());
			}

			string ScudoMemoryTraceLib = Path.Combine(Unreal.EngineDirectory.ToString(), "Build", "Android", "Prebuilt", "ScudoMemoryTrace", NDKArch, "libScudoMemoryTrace.so");

			string WrapSh = Path.Combine(Unreal.EngineDirectory.ToString(), "Build", "Android", "Prebuilt", "ScudoMemoryTrace", "wrap.sh");

			if (File.Exists(ScudoMemoryTraceLib) && File.Exists(WrapSh))
			{
				string LibDestDir = Path.Combine(UnrealBuildPath, "libs", NDKArch);
				string LibDest = Path.Combine(LibDestDir, "libScudoMemoryTrace.so");
				Directory.CreateDirectory(LibDestDir);
				if (!CopyIfDifferent(ScudoMemoryTraceLib, LibDest, true, false))
				{
					Logger.LogInformation("{LibDest} is up to date", LibDest);
				}

				string WrapDestDir = Path.Combine(UnrealBuildPath, "resources", "lib", NDKArch);
				Directory.CreateDirectory(WrapDestDir);
				string WrapDestFilePath = Path.Combine(WrapDestDir, "wrap.sh");

				Dictionary<string, string> Replacements = new Dictionary<string, string>
				{
					{ "${PACKAGE_NAME}", PackageName },
				};
				string WrapShText = File.ReadAllText(WrapSh);
				foreach (KeyValuePair<string, string> KVP in Replacements)
				{
					WrapShText = WrapShText.Replace(KVP.Key, KVP.Value, StringComparison.Ordinal);
				}

				if (!File.Exists(WrapDestFilePath) || File.ReadAllText(WrapDestFilePath) != WrapShText)
				{
					Logger.LogInformation("Writing {WrapDestFilePath}", WrapDestFilePath);
					File.WriteAllText(WrapDestFilePath, WrapShText);
				}
				else
				{
					Logger.LogInformation("{WrapDestFilePath} is up to date", WrapDestFilePath);
				}
			}
			else
			{
				throw new BuildException("No ScudoMemoryTrace found in {0} or wrap.sh in {1}", ScudoMemoryTraceLib, WrapSh);
			}
		}

		private static int RunCommandLineProgramAndReturnResultOutput(string WorkingDirectory, string Command, string Params, ILogger Logger, out string Output, string? OverrideDesc = null, bool bUseShellExecute = false)
		{
			// Process Arguments follow windows conventions in .NET Core
			// Which means single quotes ' are not considered quotes.
			// see https://github.com/dotnet/runtime/issues/29857
			// also see UE-102580
			// for rules see https://docs.microsoft.com/en-us/cpp/cpp/main-function-command-line-args
			if (Params.Contains('\'', StringComparison.Ordinal))
			{
				Params = Params.Replace("\"", "\\\"", StringComparison.Ordinal);
				Params = Params.Replace('\'', '\"');
			}

			if (OverrideDesc == null)
			{
				Logger.LogInformation("\nRunning: {Command} {Params}", Command, Params);
			}
			else if (!String.IsNullOrEmpty(OverrideDesc))
			{
				Logger.LogInformation("{Message}", OverrideDesc);
				Logger.LogDebug("\nRunning: {Command} {Params}", Command, Params);
			}

			ProcessStartInfo StartInfo = new ProcessStartInfo();
			StartInfo.WorkingDirectory = WorkingDirectory;
			StartInfo.FileName = Command;
			StartInfo.Arguments = Params;
			StartInfo.UseShellExecute = bUseShellExecute;
			StartInfo.WindowStyle = ProcessWindowStyle.Minimized;
			StartInfo.RedirectStandardOutput = true;
			StartInfo.RedirectStandardError = true;

			string ProcOutput = String.Empty;

			Process Proc = new Process();
			Proc.StartInfo = StartInfo;
			Proc.OutputDataReceived += (s, e) => ProcOutput += (e.Data != null) ? e.Data + Environment.NewLine : "";
			Proc.ErrorDataReceived += (s, e) => ProcOutput += (e.Data != null) ? e.Data + Environment.NewLine : "";

			Proc.Start();
			Proc.BeginOutputReadLine();
			Proc.BeginErrorReadLine();
			Proc.WaitForExit();

			Output = ProcOutput;
			return Proc.ExitCode;
		}

		private enum FilterAction
		{
			Skip,
			Replace,
			Error
		}

		private class FilterOperation
		{
			public FilterAction Action;
			public string Condition;
			public string Match;
			public string ReplaceWith;

			public FilterOperation(FilterAction InAction, string InCondition, string InMatch, string InReplaceWith)
			{
				Action = InAction;
				Condition = InCondition;
				Match = InMatch;
				ReplaceWith = InReplaceWith;
			}
		}

		private static List<FilterOperation>? ActiveStdOutFilter = null;

		static List<string> ParseCSVString(string Input)
		{
			List<string> Results = new List<string>();
			StringBuilder WorkString = new StringBuilder();

			int FinalIndex = Input.Length;
			int CurrentIndex = 0;
			bool InQuote = false;

			while (CurrentIndex < FinalIndex)
			{
				char CurChar = Input[CurrentIndex++];

				if (InQuote)
				{
					if (CurChar == '\\')
					{
						if (CurrentIndex < FinalIndex)
						{
							CurChar = Input[CurrentIndex++];
							WorkString.Append(CurChar);
						}
					}
					else if (CurChar == '"')
					{
						InQuote = false;
					}
					else
					{
						WorkString.Append(CurChar);
					}
				}
				else
				{
					if (CurChar == '"')
					{
						InQuote = true;
					}
					else if (CurChar == ',')
					{
						Results.Add(WorkString.ToString());
						WorkString.Clear();
					}
					else if (!Char.IsWhiteSpace(CurChar))
					{
						WorkString.Append(CurChar);
					}
				}
			}
			if (CurrentIndex > 0)
			{
				Results.Add(WorkString.ToString());
			}

			return Results;
		}

		static void ParseFilterFile(string Filename)
		{

			if (File.Exists(Filename))
			{
				ActiveStdOutFilter = new List<FilterOperation>();

				string[] FilterContents = File.ReadAllLines(Filename);
				foreach (string FileLine in FilterContents)
				{
					List<string> Parts = ParseCSVString(FileLine);

					if (Parts.Count > 1)
					{
						if (Parts[0].Equals("S", StringComparison.Ordinal))
						{
							ActiveStdOutFilter.Add(new FilterOperation(FilterAction.Skip, Parts[1], "", ""));
						}
						else if (Parts[0].Equals("R", StringComparison.Ordinal))
						{
							if (Parts.Count == 4)
							{
								ActiveStdOutFilter.Add(new FilterOperation(FilterAction.Replace, Parts[1], Parts[2], Parts[3]));
							}
						}
						else if (Parts[0].Equals("E", StringComparison.Ordinal))
						{
							if (Parts.Count == 4)
							{
								ActiveStdOutFilter.Add(new FilterOperation(FilterAction.Error, Parts[1], Parts[2], Parts[3]));
							}
						}
					}
				}

				if (ActiveStdOutFilter.Count == 0)
				{
					ActiveStdOutFilter = null;
				}
			}
		}

		static void FilterStdOutErr(DataReceivedEventArgs e, ILogger Logger)
		{
			if (e.Data != null)
			{
				if (ActiveStdOutFilter != null)
				{
					foreach (FilterOperation FilterOp in ActiveStdOutFilter)
					{
						if (e.Data.Contains(FilterOp.Condition, StringComparison.Ordinal))
						{
							switch (FilterOp.Action)
							{
								case FilterAction.Skip:
									break;

								case FilterAction.Replace:
									Logger.LogInformation("{Output}", e.Data.Replace(FilterOp.Match, FilterOp.ReplaceWith, StringComparison.Ordinal));
									break;

								case FilterAction.Error:
									Logger.LogError("{Output}", e.Data.Replace(FilterOp.Match, FilterOp.ReplaceWith, StringComparison.Ordinal));
									break;

								default:
									break;
							}
							return;
						}
					}
				}
				Logger.LogInformation("{Output}", e.Data);
			}
		}

		private static void RunCommandLineProgramWithExceptionAndFiltering(string WorkingDirectory, string Command, string Params, ILogger Logger, string? OverrideDesc = null, bool bUseShellExecute = false)
		{
			// Process Arguments follow windows conventions in .NET Core
			// Which means single quotes ' are not considered quotes.
			// see https://github.com/dotnet/runtime/issues/29857
			// also see UE-102580
			// for rules see https://docs.microsoft.com/en-us/cpp/cpp/main-function-command-line-args
			if (Params.Contains('\'', StringComparison.Ordinal))
			{
				Params = Params.Replace("\"", "\\\"", StringComparison.Ordinal);
				Params = Params.Replace('\'', '\"');
			}

			if (OverrideDesc == null)
			{
				Logger.LogInformation("\nRunning: {Command} {Params}", Command, Params);
			}
			else if (!String.IsNullOrEmpty(OverrideDesc))
			{
				Logger.LogInformation("{Message}", OverrideDesc);
				Logger.LogDebug("\nRunning: {Command} {Params}", Command, Params);
			}

			ProcessStartInfo StartInfo = new ProcessStartInfo();
			StartInfo.WorkingDirectory = WorkingDirectory;
			StartInfo.FileName = Command;
			StartInfo.Arguments = Params;
			StartInfo.UseShellExecute = bUseShellExecute;
			StartInfo.WindowStyle = ProcessWindowStyle.Minimized;
			StartInfo.RedirectStandardInput = true;
			StartInfo.RedirectStandardOutput = true;
			StartInfo.RedirectStandardError = true;

			Process Proc = new Process();
			Proc.StartInfo = StartInfo;
			Proc.OutputDataReceived += (_, e) => FilterStdOutErr(e, Logger);
			Proc.ErrorDataReceived += (_, e) => FilterStdOutErr(e, Logger);

			Proc.Start();
			Proc.BeginOutputReadLine();
			Proc.BeginErrorReadLine();

			StreamWriter StreamIn = Proc.StandardInput;
			StreamIn.WriteLine("yes");
			StreamIn.Close();

			Proc.WaitForExit();

			// android bat failure
			if (Proc.ExitCode != 0)
			{
				throw new BuildException("{0} failed with args {1}", Command, Params);
			}
		}

		private bool CheckApplicationName(string UnrealBuildPath, string ProjectName, out string? ApplicationDisplayName)
		{
			string StringsXMLPath = Path.Combine(UnrealBuildPath, "res/values/strings.xml");

			ConfigHierarchy Ini = GetConfigCacheIni(ConfigHierarchyType.Engine);
			Ini.GetString("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "ApplicationDisplayName", out ApplicationDisplayName);

			// use project name if display name is left blank
			if (String.IsNullOrWhiteSpace(ApplicationDisplayName))
			{
				ApplicationDisplayName = ProjectName;
			}

			// replace escaped characters (note: changes &# pattern before &, then patches back to allow escaped character codes in the string)
			ApplicationDisplayName = ApplicationDisplayName
				.Replace("&#", "$@#$", StringComparison.Ordinal)
				.Replace("&", "&amp;", StringComparison.Ordinal)
				.Replace("'", "\\'", StringComparison.Ordinal)
				.Replace("\"", "\\\"", StringComparison.Ordinal)
				.Replace("<", "&lt;", StringComparison.Ordinal)
				.Replace(">", "&gt;", StringComparison.Ordinal)
				.Replace("$@#$", "&#", StringComparison.Ordinal);

			// if it doesn't exist, need to repackage
			if (!File.Exists(StringsXMLPath))
			{
				return true;
			}

			// read it and see if needs to be updated
			string Contents = File.ReadAllText(StringsXMLPath);

			// find the key
			string AppNameTag = "<string name=\"app_name\">";
			int KeyIndex = Contents.IndexOf(AppNameTag, StringComparison.Ordinal);

			// if doesn't exist, need to repackage
			if (KeyIndex < 0)
			{
				return true;
			}

			// get the current value
			KeyIndex += AppNameTag.Length;
			int TagEnd = Contents.IndexOf("</string>", KeyIndex, StringComparison.Ordinal);
			if (TagEnd < 0)
			{
				return true;
			}
			string CurrentApplicationName = Contents.Substring(KeyIndex, TagEnd - KeyIndex);

			// no need to do anything if matches
			if (CurrentApplicationName == ApplicationDisplayName)
			{
				// name matches, no need to force a repackage
				return false;
			}

			// need to repackage
			return true;
		}

		private string GetAllBuildSettings(AndroidToolChain ToolChain, UnrealPluginLanguage UPL, bool bForDistribution, bool bMakeSeparateApks, bool bPackageDataInsideApk,
			bool bDisableVerifyOBBOnStartUp, bool bUseExternalFilesDir, bool bEnableBundle, bool bDontBundleLibrariesInAPK, bool bUsesZenStreaming, string TemplatesHashCode,
			bool bRequiresOBB)
		{
			// make the settings string - this will be char by char compared against last time
			StringBuilder CurrentSettings = new StringBuilder();
			CurrentSettings.AppendLine(String.Format("NDKROOT={0}", Environment.GetEnvironmentVariable("NDKROOT")));
			CurrentSettings.AppendLine(String.Format("ANDROID_HOME={0}", Environment.GetEnvironmentVariable("ANDROID_HOME")));
			CurrentSettings.AppendLine(String.Format("JAVA_HOME={0}", Environment.GetEnvironmentVariable("JAVA_HOME")));
			CurrentSettings.AppendLine(String.Format("NDKVersion={0}", ToolChain.GetNdkApiLevel()));
			CurrentSettings.AppendLine(String.Format("SDKVersion={0}", GetSdkApiLevel(ToolChain)));
			CurrentSettings.AppendLine(String.Format("bForDistribution={0}", bForDistribution));
			CurrentSettings.AppendLine(String.Format("bMakeSeparateApks={0}", bMakeSeparateApks));
			CurrentSettings.AppendLine(String.Format("bPackageDataInsideApk={0}", bPackageDataInsideApk));
			CurrentSettings.AppendLine(String.Format("bDisableVerifyOBBOnStartUp={0}", bDisableVerifyOBBOnStartUp));
			CurrentSettings.AppendLine(String.Format("bUseExternalFilesDir={0}", bUseExternalFilesDir));
			CurrentSettings.AppendLine(String.Format("UPLHashCode={0}", UPLHashCode));
			CurrentSettings.AppendLine(String.Format("TemplatesHashCode={0}", TemplatesHashCode));
			CurrentSettings.AppendLine(String.Format("bEnableBundle={0}", bEnableBundle));
			CurrentSettings.AppendLine(String.Format("bDontBundleLibrariesInAPK={0}", bDontBundleLibrariesInAPK));
			CurrentSettings.AppendLine(String.Format("bUsesZenStreaming={0}", bUsesZenStreaming));
			CurrentSettings.AppendLine(String.Format("bNoObbs={0}", bNoObbs));
			CurrentSettings.AppendLine(String.Format("bRequiresOBB={0}", bRequiresOBB));

			// all AndroidRuntimeSettings ini settings in here
			ConfigHierarchy Ini = GetConfigCacheIni(ConfigHierarchyType.Engine);
			ConfigHierarchySection Section = Ini.FindSection("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings");
			if (Section != null)
			{
				foreach (string Key in Section.KeyNames)
				{
					// filter out NDK and SDK override since actual resolved versions already written above
					if (Key.Equals("SDKAPILevelOverride", StringComparison.Ordinal) || Key.Equals("NDKAPILevelOverride", StringComparison.Ordinal))
					{
						continue;
					}

					if (Section.TryGetValues(Key, out IReadOnlyList<string>? Values))
					{
						foreach (string Value in Values)
						{
							CurrentSettings.AppendLine(String.Format("{0}={1}", Key, Value));
						}
					}
				}
			}

			Section = Ini.FindSection("/Script/AndroidPlatformEditor.AndroidSDKSettings");
			if (Section != null)
			{
				foreach (string Key in Section.KeyNames)
				{
					// filter out NDK and SDK levels since actual resolved versions already written above
					if (Key.Equals("SDKAPILevel", StringComparison.Ordinal) || Key.Equals("NDKAPILevel", StringComparison.Ordinal))
					{
						continue;
					}

					if (Section.TryGetValues(Key, out IReadOnlyList<string>? Values))
					{
						foreach (string Value in Values)
						{
							CurrentSettings.AppendLine(String.Format("{0}={1}", Key, Value));
						}
					}
				}
			}

			foreach (UnrealArch Arch in Architectures!.Architectures)
			{
				CurrentSettings.AppendFormat("Arch={0}{1}", Arch, Environment.NewLine);
			}

			// Modifying some settings in the GameMapsSettings could trigger the OBB regeneration
			// and make the cached OBBData.java mismatch to the actually data. 
			// So we insert the relevant keys into CurrentSettings to capture the change, to
			// enforce the refreshing of Android java codes
			Section = Ini.FindSection("/Script/EngineSettings.GameMapsSettings");
			if (Section != null)
			{
				foreach (string Key in Section.KeyNames)
				{
					if (!Key.Equals("GameDefaultMap", StringComparison.Ordinal) &&
						!Key.Equals("GlobalDefaultGameMode", StringComparison.Ordinal))
					{
						continue;
					}

					if (Section.TryGetValues(Key, out IReadOnlyList<string>? Values))
					{
						foreach (string Value in Values)
						{
							CurrentSettings.AppendLine(String.Format("{0}={1}", Key, Value));
						}
					}
				}
			}

			// get a list of the ini settings in UPL files that may affect the build
			// architecture doesn't matter here since this node does not use init logic
			string UPLBuildSettings = UPL.ProcessPluginNode(GetNDKArch(Architectures.Architectures[0]), "registerBuildSettings", "");
			foreach (string Line in UPLBuildSettings.Split('\n', StringSplitOptions.RemoveEmptyEntries))
			{
				string SectionName = Line.Trim();

				// needed keys are provided in [ ] separated by commas
				string[]? NeededKeys = null;
				int KeyIndex = SectionName.IndexOf('[', StringComparison.Ordinal);
				if (KeyIndex > 0)
				{
					string KeyList = SectionName.Substring(KeyIndex + 1);
					SectionName = SectionName.Substring(0, KeyIndex);
					int CloseIndex = KeyList.IndexOf(']', StringComparison.Ordinal);
					if (CloseIndex > 1)
					{
						NeededKeys = KeyList.Substring(0, CloseIndex).Split(',', StringSplitOptions.RemoveEmptyEntries);
						if (NeededKeys.Length == 0)
						{
							NeededKeys = null;
						}
					}
				}

				// write the values for the requested keys (or all if none specified)
				Section = Ini.FindSection(SectionName);
				if (Section != null)
				{
					foreach (string Key in Section.KeyNames)
					{
						if (NeededKeys != null && !NeededKeys.Contains(Key))
						{
							continue;
						}
						if (Section.TryGetValues(Key, out IReadOnlyList<string>? Values))
						{
							foreach (string Value in Values)
							{
								CurrentSettings.AppendLine(String.Format("{0}:{1}={2}", SectionName, Key, Value));
							}
						}
					}
				}
			}

			return CurrentSettings.ToString();
		}

		private bool CheckDependencies(UnrealArchitectures Architectures, string ProjectName, string ProjectDirectory, string IntermediateAndroidPath,
			string UnrealBuildFilesPath, string GameBuildFilesPath, List<string> SettingsFiles, string CookFlavor,
			string OutputPath, bool bMakeSeparateApks, bool bPackageDataInsideApk, bool bDontBundleLibrariesInAPK)
		{
			// check all input files (.so, java files, .ini files, etc)
			bool bAllInputsCurrent = true;
			foreach (UnrealArch Arch in Architectures.Architectures)
			{
				string SourceSOName = AndroidToolChain.InlineArchName(OutputPath, Arch);
				// if the source binary was UnrealGame, replace it with the new project name, when re-packaging a binary only build
				string ApkFilename = Path.GetFileNameWithoutExtension(OutputPath).Replace("UnrealGame", ProjectName, StringComparison.Ordinal);
				string DestApkName = Path.Combine(ProjectDirectory, "Binaries/Android/") + ApkFilename + ".apk";

				// if we making multiple Apks, we need to put the architecture into the name
				if (bMakeSeparateApks)
				{
					DestApkName = AndroidToolChain.InlineArchName(DestApkName, Arch);
				}

				List<string> InputFiles = new List<string>();
				// check to see if the libUnreal.so is out of date before trying the slow make apk process
				if (!bDontBundleLibrariesInAPK)
				{
					InputFiles.Add(SourceSOName);
				}

				// add all files in jni and libs subfolders
				foreach (string? SoDirName in new[] { "jni", "libs" })
				{
					string SoDirPath = Path.Combine(IntermediateAndroidPath, ArchRemapping[GetNDKArch(Arch)], SoDirName, GetNDKArch(Arch));
					if (Directory.Exists(SoDirPath))
					{
						IEnumerable<string> files = Directory.EnumerateFiles(SoDirPath, "*.*", SearchOption.AllDirectories);
						InputFiles.AddRange(files);
					}
				}

				// add all Engine and Project build files to be safe
				InputFiles.AddRange(Directory.EnumerateFiles(UnrealBuildFilesPath, "*.*", SearchOption.AllDirectories));
				if (Directory.Exists(GameBuildFilesPath))
				{
					InputFiles.AddRange(Directory.EnumerateFiles(GameBuildFilesPath, "*.*", SearchOption.AllDirectories));
				}

				// make sure changed java files will rebuild apk
				InputFiles.AddRange(SettingsFiles);

				// rebuild if .pak files exist for OBB in APK case
				if (!bNoObbs && bPackageDataInsideApk)
				{
					string PAKFileLocation = ProjectDirectory + "/Saved/StagedBuilds/Android" + CookFlavor + "/" + ProjectName + "/Content/Paks";
					if (Directory.Exists(PAKFileLocation))
					{
						IEnumerable<string> PakFiles = Directory.EnumerateFiles(PAKFileLocation, "*.pak", SearchOption.TopDirectoryOnly);
						foreach (string Name in PakFiles)
						{
							InputFiles.Add(Name);
						}
					}
				}

				// look for any newer input file
				DateTime ApkTime = File.GetLastWriteTimeUtc(DestApkName);
				foreach (string InputFileName in InputFiles)
				{
					if (File.Exists(InputFileName))
					{
						// skip .log files
						if (Path.GetExtension(InputFileName) == ".log")
						{
							continue;
						}
						DateTime InputFileTime = File.GetLastWriteTimeUtc(InputFileName);
						if (InputFileTime.CompareTo(ApkTime) > 0)
						{
							bAllInputsCurrent = false;
							Logger.LogInformation("{DestApkName} is out of date due to newer input file {InputFileName}", DestApkName, InputFileName);
						}
					}
				}
			}

			return bAllInputsCurrent;
		}

		private static int ConvertDepthBufferIniValue(string IniValue)
		{
			switch (IniValue.ToUpperInvariant())
			{
				case "BITS16":
					return 16;
				case "BITS24":
					return 24;
				case "BITS32":
					return 32;
				default:
					return 0;
			}
		}

		private static string ConvertOrientationIniValue(string IniValue)
		{
			switch (IniValue.ToUpperInvariant())
			{
				case "PORTRAIT":
					return "portrait";
				case "REVERSEPORTRAIT":
					return "reversePortrait";
				case "SENSORPORTRAIT":
					return "sensorPortrait";
				case "LANDSCAPE":
					return "landscape";
				case "REVERSELANDSCAPE":
					return "reverseLandscape";
				case "SENSORLANDSCAPE":
					return "sensorLandscape";
				case "SENSOR":
					return "sensor";
				case "FULLSENSOR":
					return "fullSensor";
				default:
					return "landscape";
			}
		}

		private string GetOrientation(string NDKArch)
		{
			ConfigHierarchy Ini = GetConfigCacheIni(ConfigHierarchyType.Engine);
			string Orientation;
			Ini.GetString("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "Orientation", out Orientation);

			// check for UPL override
			string OrientationOverride = UPL!.ProcessPluginNode(NDKArch, "orientationOverride", "");
			if (!String.IsNullOrEmpty(OrientationOverride))
			{
				Orientation = OrientationOverride;
			}

			return ConvertOrientationIniValue(Orientation);
		}

		private void DetermineScreenOrientationRequirements(string Arch, out bool bNeedPortrait, out bool bNeedLandscape)
		{
			bNeedLandscape = false;
			bNeedPortrait = false;

			switch (GetOrientation(Arch).ToUpperInvariant())
			{
				case "PORTRAIT":
					bNeedPortrait = true;
					break;
				case "REVERSEPORTRAIT":
					bNeedPortrait = true;
					break;
				case "SENSORPORTRAIT":
					bNeedPortrait = true;
					break;

				case "LANDSCAPE":
					bNeedLandscape = true;
					break;
				case "REVERSELANDSCAPE":
					bNeedLandscape = true;
					break;
				case "SENSORLANDSCAPE":
					bNeedLandscape = true;
					break;

				case "SENSOR":
					bNeedPortrait = true;
					bNeedLandscape = true;
					break;
				case "FULLSENSOR":
					bNeedPortrait = true;
					bNeedLandscape = true;
					break;

				default:
					bNeedPortrait = true;
					bNeedLandscape = true;
					break;
			}
		}

		private void PickDownloaderScreenOrientation(string UnrealBuildPath, bool bNeedPortrait, bool bNeedLandscape)
		{
			// Remove unused downloader_progress.xml to prevent missing resource
			if (!bNeedPortrait)
			{
				string LayoutPath = UnrealBuildPath + "/res/layout-port/downloader_progress.xml";
				SafeDeleteFile(LayoutPath);
			}
			if (!bNeedLandscape)
			{
				string LayoutPath = UnrealBuildPath + "/res/layout-land/downloader_progress.xml";
				SafeDeleteFile(LayoutPath);
			}

			// Loop through each of the resolutions (only /res/drawable/ is required, others are optional)
			string[] Resolutions = new string[] { "/res/drawable/", "/res/drawable-ldpi/", "/res/drawable-mdpi/", "/res/drawable-hdpi/", "/res/drawable-xhdpi/" };
			foreach (string ResolutionPath in Resolutions)
			{
				string PortraitFilename = UnrealBuildPath + ResolutionPath + "downloadimagev.png";
				if (bNeedPortrait)
				{
					if (!File.Exists(PortraitFilename) && (ResolutionPath == "/res/drawable/"))
					{
						Logger.LogWarning("Warning: Downloader screen source image {PortraitFilename} not available, downloader screen will not function properly!", PortraitFilename);
					}
				}
				else
				{
					// Remove unused image
					SafeDeleteFile(PortraitFilename);
				}

				string LandscapeFilename = UnrealBuildPath + ResolutionPath + "downloadimageh.png";
				if (bNeedLandscape)
				{
					if (!File.Exists(LandscapeFilename) && (ResolutionPath == "/res/drawable/"))
					{
						Logger.LogWarning("Warning: Downloader screen source image {LandscapeFilename} not available, downloader screen will not function properly!", LandscapeFilename);
					}
				}
				else
				{
					// Remove unused image
					SafeDeleteFile(LandscapeFilename);
				}
			}
		}

		private void PickSplashScreenOrientation(string CookFlavor, string UnrealBuildPath, bool bNeedPortrait, bool bNeedLandscape)
		{
			ConfigHierarchy Ini = GetConfigCacheIni(ConfigHierarchyType.Engine);
			Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bShowLaunchImage", out bool bShowLaunchImage);
			bool bPackageForMetaQuest = IsPackagingForMetaQuest(CookFlavor, Ini);

			//override the parameters if we are not showing a launch image or are packaging for Meta Quest
			if (bPackageForMetaQuest || !bShowLaunchImage)
			{
				bNeedPortrait = bNeedLandscape = false;
			}

			// Remove unused styles.xml to prevent missing resource
			if (!bNeedPortrait)
			{
				string StylesPath = UnrealBuildPath + "/res/values-port/styles.xml";
				SafeDeleteFile(StylesPath);
			}
			if (!bNeedLandscape)
			{
				string StylesPath = UnrealBuildPath + "/res/values-land/styles.xml";
				SafeDeleteFile(StylesPath);
			}

			if (bPackageForMetaQuest)
			{
				string LandscapeFilename = UnrealBuildPath + "/res/drawable/" + "splashscreen_landscape.png";
				string OculusSplashTargetPath = UnrealBuildPath + "/assets/vr_splash.png";

				if (bShowLaunchImage)
				{
					if (!File.Exists(LandscapeFilename))
					{
						Logger.LogWarning("Warning: Landscape splash screen source image {LandscapeFilename} not available, Oculus splash screen will not function properly!", LandscapeFilename);
					}
					else if (FilesAreDifferent(LandscapeFilename, OculusSplashTargetPath))
					{
						SafeDeleteFile(OculusSplashTargetPath);
						MakeDirectoryIfRequired(OculusSplashTargetPath);
						File.Copy(LandscapeFilename, OculusSplashTargetPath, true);
						File.SetLastWriteTimeUtc(OculusSplashTargetPath, File.GetLastWriteTimeUtc(LandscapeFilename));
						Logger.LogInformation("Copying {LandscapeFilename} to {OculusSplashTargetPath} for Oculus splash", LandscapeFilename, OculusSplashTargetPath);
					}
				}
				else
				{
					// Remove unused image
					SafeDeleteFile(OculusSplashTargetPath);
				}
			}

			// Loop through each of the resolutions (only /res/drawable/ is required, others are optional)
			string[] Resolutions = new string[] { "/res/drawable/", "/res/drawable-ldpi/", "/res/drawable-mdpi/", "/res/drawable-hdpi/", "/res/drawable-xhdpi/" };
			foreach (string ResolutionPath in Resolutions)
			{
				string PortraitFilename = UnrealBuildPath + ResolutionPath + "splashscreen_portrait.png";
				if (bNeedPortrait)
				{
					if (!File.Exists(PortraitFilename) && (ResolutionPath == "/res/drawable/"))
					{
						Logger.LogWarning("Warning: Splash screen source image {PortraitFilename} not available, splash screen will not function properly!", PortraitFilename);
					}
				}
				else
				{
					// Remove unused image
					SafeDeleteFile(PortraitFilename);

					// Remove optional extended resource
					string PortraitXmlFilename = UnrealBuildPath + ResolutionPath + "splashscreen_p.xml";
					SafeDeleteFile(PortraitXmlFilename);
				}

				string LandscapeFilename = UnrealBuildPath + ResolutionPath + "splashscreen_landscape.png";
				if (bNeedLandscape)
				{
					if (!File.Exists(LandscapeFilename) && (ResolutionPath == "/res/drawable/"))
					{
						Logger.LogWarning("Warning: Splash screen source image {LandscapeFilename} not available, splash screen will not function properly!", LandscapeFilename);
					}
				}
				else
				{
					// Remove unused image
					SafeDeleteFile(LandscapeFilename);

					// Remove optional extended resource
					string LandscapeXmlFilename = UnrealBuildPath + ResolutionPath + "splashscreen_l.xml";
					SafeDeleteFile(LandscapeXmlFilename);
				}
			}
		}

		private string? CachedPackageName = null;

		private static bool IsLetter(char Input)
		{
			return (Input >= 'A' && Input <= 'Z') || (Input >= 'a' && Input <= 'z');
		}

		private static bool IsDigit(char Input)
		{
			return (Input >= '0' && Input <= '9');
		}

		private string GetPackageName(string ProjectName)
		{
			if (CachedPackageName == null)
			{
				ConfigHierarchy Ini = GetConfigCacheIni(ConfigHierarchyType.Engine);
				string PackageName;
				Ini.GetString("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "PackageName", out PackageName);

				if (PackageName.Contains("[PROJECT]", StringComparison.Ordinal))
				{
					// project name must start with a letter
					if (!IsLetter(ProjectName[0]))
					{
						throw new BuildException("Package name segments must all start with a letter. Please replace [PROJECT] with a valid name");
					}

					// hyphens not allowed so change them to underscores in project name
					if (ProjectName.Contains('-', StringComparison.Ordinal))
					{
						Trace.TraceWarning("Project name contained hyphens, converted to underscore");
						ProjectName = ProjectName.Replace("-", "_", StringComparison.Ordinal);
					}

					// check for special characters
					for (int Index = 0; Index < ProjectName.Length; Index++)
					{
						char c = ProjectName[Index];
						if (c != '.' && c != '_' && !IsDigit(c) && !IsLetter(c))
						{
							throw new BuildException("Project name contains illegal characters (only letters, numbers, and underscore allowed); please replace [PROJECT] with a valid name");
						}
					}

					PackageName = PackageName.Replace("[PROJECT]", ProjectName, StringComparison.Ordinal);
				}

				// verify minimum number of segments
				string[] PackageParts = PackageName.Split('.');
				int SectionCount = PackageParts.Length;
				if (SectionCount < 2)
				{
					throw new BuildException("Package name must have at least 2 segments separated by periods (ex. com.projectname, not projectname); please change in Android Project Settings. Currently set to '" + PackageName + "'");
				}

				// hyphens not allowed
				if (PackageName.Contains('-', StringComparison.Ordinal))
				{
					throw new BuildException("Package names may not contain hyphens; please change in Android Project Settings. Currently set to '" + PackageName + "'");
				}

				// do not allow special characters
				for (int Index = 0; Index < PackageName.Length; Index++)
				{
					char c = PackageName[Index];
					if (c != '.' && c != '_' && !IsDigit(c) && !IsLetter(c))
					{
						throw new BuildException("Package name contains illegal characters (only letters, numbers, and underscore allowed); please change in Android Project Settings. Currently set to '" + PackageName + "'");
					}
				}

				// validate each segment
				for (int Index = 0; Index < SectionCount; Index++)
				{
					if (PackageParts[Index].Length < 1)
					{
						throw new BuildException("Package name segments must have at least one letter; please change in Android Project Settings. Currently set to '" + PackageName + "'");
					}

					if (!IsLetter(PackageParts[Index][0]))
					{
						throw new BuildException("Package name segments must start with a letter; please change in Android Project Settings. Currently set to '" + PackageName + "'");
					}

					// cannot use Java reserved keywords
					foreach (string Keyword in JavaReservedKeywords)
					{
						if (PackageParts[Index] == Keyword)
						{
							throw new BuildException("Package name segments must not be a Java reserved keyword (" + Keyword + "); please change in Android Project Settings. Currently set to '" + PackageName + "'");
						}
					}
				}

				Logger.LogInformation("Using package name: '{PackageName}'", PackageName);
				CachedPackageName = PackageName;
			}

			return CachedPackageName;
		}

		private string GetPublicKey()
		{
			ConfigHierarchy Ini = GetConfigCacheIni(ConfigHierarchyType.Engine);
			Ini.GetString("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "GooglePlayLicenseKey", out string PlayLicenseKey);
			return PlayLicenseKey;
		}

		private bool bHaveReadEngineVersion = false;
		private string EngineMajorVersion = "4";
		private string EngineMinorVersion = "0";
		private string EnginePatchVersion = "0";
		private string EngineChangelist = "0";
		private string? EngineBranch = "UE5";

		private string ReadEngineVersion()
		{
			if (!bHaveReadEngineVersion)
			{
				ReadOnlyBuildVersion Version = ReadOnlyBuildVersion.Current;

				EngineMajorVersion = Version.MajorVersion.ToString();
				EngineMinorVersion = Version.MinorVersion.ToString();
				EnginePatchVersion = Version.PatchVersion.ToString();
				EngineChangelist = Version.Changelist.ToString();
				EngineBranch = Version.BranchName;

				bHaveReadEngineVersion = true;
			}

			return EngineMajorVersion + "." + EngineMinorVersion + "." + EnginePatchVersion;
		}

		private string GenerateManifest(AndroidToolChain ToolChain, string ProjectName, TargetType InTargetType, bool bIsForDistribution, bool bPackageDataInsideApk, string GameBuildFilesPath, bool bHasOBBFiles, bool bDisableVerifyOBBOnStartUp, UnrealArch UnrealArch, string CookFlavor, bool bUseExternalFilesDir, string Configuration, int SDKLevelInt, bool bIsEmbedded, bool bEnableBundle, bool bUsesZenStreaming)
		{
			// Read the engine version
			string EngineVersion = ReadEngineVersion();

			int StoreVersion = GetStoreVersion(UnrealArch);

			string Arch = GetNDKArch(UnrealArch);
			GetMinTargetSDKVersions(ToolChain, UPL!, Arch, bEnableBundle, out int MinSDKVersion, out int TargetSDKVersion, out _);

			// get project version from ini
			ConfigHierarchy GameIni = GetConfigCacheIni(ConfigHierarchyType.Game);
			string ProjectVersion;
			GameIni.GetString("/Script/EngineSettings.GeneralProjectSettings", "ProjectVersion", out ProjectVersion);

			// ini file to get settings from
			ConfigHierarchy Ini = GetConfigCacheIni(ConfigHierarchyType.Engine);
			// Discard result but call the method for the logging side-effects to preserve previous behaviour.
			_ = GetPackageName(ProjectName);
			string VersionDisplayName = GetVersionDisplayName(bIsEmbedded);
			string DepthBufferPreference;
			Ini.GetString("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "DepthBufferPreference", out DepthBufferPreference);
			float MinAspectRatioValue;
			if (!Ini.TryGetValue("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "MinAspectRatio", out MinAspectRatioValue))
			{
				MinAspectRatioValue = 0.0f;
			}
			float MaxAspectRatioValue;
			if (!Ini.TryGetValue("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "MaxAspectRatio", out MaxAspectRatioValue))
			{
				MaxAspectRatioValue = 2.4f;
			}
			string Orientation = ConvertOrientationIniValue(GetOrientation(Arch));
			Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bFullScreen", out bool EnableFullScreen);
			Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bUseDisplayCutout", out bool bUseDisplayCutout);
			Ini.GetString("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "RoundedEdgeSafeZone", out string RoundedEdgeSafeZone);
			Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bAllowResizing", out bool bAllowResizing);
			Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bSupportSizeChanges", out bool bSupportSizeChanges);
			Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bRestoreNotificationsOnReboot", out bool bRestoreNotificationsOnReboot);
			Ini.GetArray("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "ExtraManifestNodeTags", out List<string>? ExtraManifestNodeTags);
			Ini.GetArray("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "ExtraApplicationNodeTags", out List<string>? ExtraApplicationNodeTags);
			Ini.GetArray("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "ExtraActivityNodeTags", out List<string>? ExtraActivityNodeTags);
			Ini.GetString("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "ExtraActivitySettings", out string ExtraActivitySettings);
			Ini.GetString("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "ExtraApplicationSettings", out string ExtraApplicationSettings);
			Ini.GetArray("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "ExtraPermissions", out List<string>? ExtraPermissions);
			bool bPackageForMetaQuest = IsPackagingForMetaQuest(CookFlavor, Ini);
			Ini.GetBool("OnlineSubsystemGooglePlay.Store", "bSupportsInAppPurchasing", out bool bEnableIAP);
			Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bShowLaunchImage", out bool bShowLaunchImage);
			Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bSupportAdMob", out bool bSupportAdMob);
			Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bValidateTextureFormats", out bool bValidateTextureFormats);

			Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bBuildForES31", out bool bBuildForES31);
			Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bSupportsVulkan", out bool bSupportsVulkan);

			Ini.GetInt32("/Script/Engine.RendererSettings", "r.Mobile.PropagateAlpha", out int PropagateAlpha);

			Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bAllowIMU", out bool bAllowIMU);

			Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bPublicLogFiles", out bool bPublicLogFiles);
			if (!bUseExternalFilesDir)
			{
				bPublicLogFiles = false;
			}

			string InstallLocation;
			Ini.GetString("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "InstallLocation", out InstallLocation);
			switch (InstallLocation.ToUpperInvariant())
			{
				case "PREFEREXTERNAL":
					InstallLocation = "preferExternal";
					break;
				case "AUTO":
					InstallLocation = "auto";
					break;
				default:
					InstallLocation = "internalOnly";
					break;
			}

			Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bEnableMulticastSupport", out bool bEnableMulticastSupport);

			Ini.GetBool("/Script/AndroidSingleInstanceServiceEditor.AndroidSingleInstanceServiceRuntimeSettings", "bEnableASISPlugin", out bool bASISEnabled);

			// only apply density to configChanges if using android-24 or higher and minimum sdk is 17
			bool bAddDensity = (SDKLevelInt >= 24) && (MinSDKVersion >= 17);

			// disable Meta Quest if not supported platform (in this case only arm64 for now)
			if (UnrealArch != UnrealArch.Arm64)
			{
				if (bPackageForMetaQuest)
				{
					Logger.LogInformation("Disabling Package For Meta Quest for unsupported architecture {UnrealArch}", UnrealArch);
					bPackageForMetaQuest = false;
				}
			}

			// disable splash screen for Meta Quest (for now)
			if (bPackageForMetaQuest)
			{
				if (bShowLaunchImage)
				{
					Logger.LogInformation("Disabling Show Launch Image for Meta Quest enabled application");
					bShowLaunchImage = false;
				}
			}

			//figure out the app type
			string AppType = InTargetType == TargetType.Game ? "" : InTargetType.ToString();
			if (CookFlavor.EndsWith("Client", StringComparison.Ordinal))
			{
				CookFlavor = CookFlavor.Substring(0, CookFlavor.Length - 6);
			}
			if (CookFlavor.EndsWith("Server", StringComparison.Ordinal))
			{
				CookFlavor = CookFlavor.Substring(0, CookFlavor.Length - 6);
			}

			//figure out which texture compressions are supported
			bool bETC2Enabled, bDXTEnabled, bASTCEnabled;
			bETC2Enabled = bDXTEnabled = bASTCEnabled = false;
			if (CookFlavor.Length < 1)
			{
				//All values supported
				bETC2Enabled = bDXTEnabled = bASTCEnabled = true;
			}
			else
			{
				switch (CookFlavor)
				{
					case "_Multi":
						//need to check ini to determine which are supported
						Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bMultiTargetFormat_ETC2", out bETC2Enabled);
						Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bMultiTargetFormat_DXT", out bDXTEnabled);
						Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bMultiTargetFormat_ASTC", out bASTCEnabled);
						break;
					case "_ETC2":
						bETC2Enabled = true;
						break;
					case "_DXT":
						bDXTEnabled = true;
						break;
					case "_ASTC":
					case "_OpenXR":
						bASTCEnabled = true;
						break;
					default:
						Logger.LogWarning("Invalid or unknown CookFlavor used in GenerateManifest: {CookFlavor}", CookFlavor);
						break;
				}
			}
			bool bSupportingAllTextureFormats = bETC2Enabled && bDXTEnabled && bASTCEnabled;

			// If it is only ETC2 we need to skip adding the texture format filtering and instead use ES 3.0 as minimum version (it requires ETC2)
			bool bOnlyETC2Enabled = (bETC2Enabled && !(bDXTEnabled || bASTCEnabled));

			string CookedFlavors = (bETC2Enabled ? "ETC2," : "") +
									(bDXTEnabled ? "DXT," : "") +
									(bASTCEnabled ? "ASTC," : "");
			CookedFlavors = (String.IsNullOrEmpty(CookedFlavors)) ? "" : CookedFlavors.Substring(0, CookedFlavors.Length - 1);

			StringBuilder Text = new StringBuilder();
			Text.AppendLine(XML_HEADER);
			Text.AppendLine($"<manifest {String.Join(" ", ManifestXMLNamespaces)}");

			if (ExtraManifestNodeTags != null)
			{
				foreach (string Line in ExtraManifestNodeTags)
				{
					Text.AppendLine("          " + Line);
				}
			}
			Text.AppendLine(String.Format("          android:installLocation=\"{0}\"", InstallLocation));
			Text.AppendLine(String.Format("          android:versionCode=\"{0}\"", StoreVersion));
			Text.AppendLine(String.Format("          android:versionName=\"{0}\">", VersionDisplayName));

			Text.AppendLine("");

			if (TargetSDKVersion >= 30)
			{
				Text.AppendLine("\t<queries>");
				Text.AppendLine("\t\t<intent>");
				Text.AppendLine("\t\t\t<action android:name=\"android.intent.action.VIEW\" />");
				Text.AppendLine("\t\t\t<category android:name=\"android.intent.category.BROWSABLE\" />");
				Text.AppendLine("\t\t\t<data android:scheme=\"http\" />");
				Text.AppendLine("\t\t</intent>");
				Text.AppendLine("\t\t<intent>");
				Text.AppendLine("\t\t\t<action android:name=\"android.intent.action.VIEW\" />");
				Text.AppendLine("\t\t\t<category android:name=\"android.intent.category.BROWSABLE\" />");
				Text.AppendLine("\t\t\t<data android:scheme=\"https\" />");
				Text.AppendLine("\t\t</intent>");
				Text.AppendLine("\t</queries>");
			}

			Text.AppendLine("\t<!-- Application Definition -->");
			Text.AppendLine("\t<application android:label=\"@string/app_name\"");
			Text.AppendLine("\t             android:icon=\"@drawable/icon\"");

			bool bRequestedLegacyExternalStorage = false;
			if (ExtraApplicationNodeTags != null)
			{
				foreach (string Line in ExtraApplicationNodeTags)
				{
					if (Line.Contains("requestLegacyExternalStorage", StringComparison.Ordinal))
					{
						bRequestedLegacyExternalStorage = true;
					}
					Text.AppendLine("\t             " + Line);
				}
			}
			Text.AppendLine("\t             android:hardwareAccelerated=\"true\"");
			Text.AppendLine("\t				android:name=\"com.epicgames.unreal.GameApplication\"");
			if (!bIsForDistribution && SDKLevelInt >= 29 && !bRequestedLegacyExternalStorage)
			{
				// work around scoped storage for non-distribution for SDK 29; add to ExtraApplicationNodeTags if you need it for distribution
				Text.AppendLine("\t				android:requestLegacyExternalStorage=\"true\"");
			}

			// Max supported aspect ratio
			string MaxAspectRatioString = MaxAspectRatioValue.ToString("f", System.Globalization.CultureInfo.InvariantCulture);
			if (TargetSDKVersion >= 29)
			{
				if(MinAspectRatioValue > 0.0)
				{
					string MinAspectRatioString = MinAspectRatioValue.ToString("f", System.Globalization.CultureInfo.InvariantCulture);
					Text.AppendLine(String.Format("\t\t\tandroid:minAspectRatio=\"{0}\"", MinAspectRatioString));
				}
				Text.AppendLine(String.Format("\t\t\tandroid:maxAspectRatio=\"{0}\"", MaxAspectRatioString));
			}
			Text.AppendLine("\t             android:hasCode=\"true\">");
			if (bShowLaunchImage)
			{
				// normal application settings
				Text.AppendLine("\t\t<activity android:name=\"com.epicgames.unreal.SplashActivity\"");
				Text.AppendLine("\t\t          android:exported=\"true\"");
				Text.AppendLine("\t\t          android:label=\"@string/app_name\"");
				Text.AppendLine("\t\t          android:theme=\"@style/UnrealSplashTheme\"");
				Text.AppendLine("\t\t          android:launchMode=\"singleTask\"");
				if (SDKLevelInt >= 24)
				{
					Text.AppendLine("\t\t          android:resizeableActivity=\"{0}\"", bAllowResizing ? "true" : "false");
				}
				Text.AppendLine(String.Format("\t\t          android:screenOrientation=\"{0}\"", Orientation));
				Text.AppendLine(String.Format("\t\t          android:debuggable=\"{0}\">", bIsForDistribution ? "false" : "true"));
				if (!bASISEnabled)
				{
					Text.AppendLine("\t\t\t<intent-filter>");
					Text.AppendLine("\t\t\t\t<action android:name=\"android.intent.action.MAIN\" />");
					Text.AppendLine(String.Format("\t\t\t\t<category android:name=\"android.intent.category.LAUNCHER\" />"));
					Text.AppendLine("\t\t\t</intent-filter>");
				}
				Text.AppendLine("\t\t</activity>");
				Text.AppendLine("\t\t<activity android:name=\"com.epicgames.unreal.GameActivity\"");
				Text.AppendLine("\t\t          android:exported=\"true\"");
				Text.AppendLine("\t\t          android:label=\"@string/app_name\"");
				Text.AppendLine("\t\t          android:theme=\"@style/UnrealSplashTheme\"");
				Text.AppendLine(bAddDensity ? "\t\t          android:configChanges=\"mcc|mnc|uiMode|density|screenSize|smallestScreenSize|screenLayout|orientation|keyboardHidden|keyboard|navigation|touchscreen|locale|fontScale|layoutDirection\""
											: "\t\t          android:configChanges=\"mcc|mnc|uiMode|screenSize|smallestScreenSize|screenLayout|orientation|keyboardHidden|keyboard|navigation|touchscreen|locale|fontScale|layoutDirection\"");
			}
			else
			{
				Text.AppendLine("\t\t<activity android:name=\"com.epicgames.unreal.GameActivity\"");
				Text.AppendLine("\t\t          android:exported=\"true\"");
				Text.AppendLine("\t\t          android:label=\"@string/app_name\"");
				Text.AppendLine("\t\t          android:theme=\"@style/UnrealBaseTheme\"");
				Text.AppendLine(bAddDensity ? "\t\t          android:configChanges=\"mcc|mnc|uiMode|density|screenSize|smallestScreenSize|screenLayout|orientation|keyboardHidden|keyboard|navigation|touchscreen|locale|fontScale|layoutDirection\""
											: "\t\t          android:configChanges=\"mcc|mnc|uiMode|screenSize|smallestScreenSize|screenLayout|orientation|keyboardHidden|keyboard|navigation|touchscreen|locale|fontScale|layoutDirection\"");

			}
			if (SDKLevelInt >= 24)
			{
				Text.AppendLine("\t\t          android:resizeableActivity=\"{0}\"", bAllowResizing ? "true" : "false");
			}
			Text.AppendLine("\t\t          android:launchMode=\"singleTask\"");
			Text.AppendLine(String.Format("\t\t          android:screenOrientation=\"{0}\"", Orientation));
			if (ExtraActivityNodeTags != null)
			{
				foreach (string Line in ExtraActivityNodeTags)
				{
					Text.AppendLine("\t\t          " + Line);
				}
			}
			Text.AppendLine(String.Format("\t\t          android:debuggable=\"{0}\">", bIsForDistribution ? "false" : "true"));
			Text.AppendLine("\t\t\t<meta-data android:name=\"android.app.lib_name\" android:value=\"Unreal\"/>");
			bool bNeedMainIntent = !bShowLaunchImage && !bASISEnabled;
			if (bNeedMainIntent || bUsesZenStreaming)
			{
				Text.AppendLine("\t\t\t<intent-filter>");
				if (bNeedMainIntent)
				{
					Text.AppendLine("\t\t\t\t<action android:name=\"android.intent.action.MAIN\" />");
					Text.AppendLine(String.Format("\t\t\t\t<category android:name=\"android.intent.category.LAUNCHER\" />"));
				}
				if (bUsesZenStreaming)
				{
					Text.AppendLine("\t\t\t\t<action android:name=\"com.epicgames.unreal.Zen.StreamingEnabled\" />");
				}
				Text.AppendLine("\t\t\t</intent-filter>");
			}
			if (!String.IsNullOrEmpty(ExtraActivitySettings))
			{
				ExtraActivitySettings = ExtraActivitySettings.Replace("\\n", "\n", StringComparison.Ordinal);
				foreach (string Line in ExtraActivitySettings.Split("\r\n".ToCharArray()))
				{
					Text.AppendLine("\t\t\t" + Line);
				}
			}
			string ActivityAdditionsFile = Path.Combine(GameBuildFilesPath, "ManifestActivityAdditions.txt");
			if (File.Exists(ActivityAdditionsFile))
			{
				foreach (string Line in File.ReadAllLines(ActivityAdditionsFile))
				{
					Text.AppendLine("\t\t\t" + Line);
				}
			}
			Text.AppendLine("\t\t</activity>");

			// For OBB download support
			if (bShowLaunchImage)
			{
				Text.AppendLine("\t\t<activity android:name=\".DownloaderActivity\"");
				Text.AppendLine(String.Format("\t\t          android:screenOrientation=\"{0}\"", Orientation));
				Text.AppendLine(bAddDensity ? "\t\t          android:configChanges=\"mcc|mnc|uiMode|density|screenSize|orientation|keyboardHidden|keyboard\""
											: "\t\t          android:configChanges=\"mcc|mnc|uiMode|screenSize|orientation|keyboardHidden|keyboard\"");
				Text.AppendLine("\t\t          android:theme=\"@style/UnrealSplashTheme\" />");
			}
			else
			{
				Text.AppendLine("\t\t<activity android:name=\".DownloaderActivity\" />");
			}

			// Figure out the required startup permissions if targetting devices supporting runtime permissions
			string StartupPermissions = "";
			if (TargetSDKVersion >= 23 && TargetSDKVersion < 30 && !bPackageForMetaQuest)
			{
				if (Configuration != "Shipping" || !bUseExternalFilesDir)
				{
					StartupPermissions = StartupPermissions + (StartupPermissions.Length > 0 ? "," : "") + "android.permission.WRITE_EXTERNAL_STORAGE";
				}
			}

			Text.AppendLine(String.Format("\t\t<meta-data android:name=\"com.epicgames.unreal.GameActivity.EngineVersion\" android:value=\"{0}\"/>", EngineVersion));
			Text.AppendLine(String.Format("\t\t<meta-data android:name=\"com.epicgames.unreal.GameActivity.EngineBranch\" android:value=\"{0}\"/>", EngineBranch));
			Text.AppendLine(String.Format("\t\t<meta-data android:name=\"com.epicgames.unreal.GameActivity.ProjectVersion\" android:value=\"{0}\"/>", ProjectVersion));
			Text.AppendLine(String.Format("\t\t<meta-data android:name=\"com.epicgames.unreal.GameActivity.DepthBufferPreference\" android:value=\"{0}\"/>", ConvertDepthBufferIniValue(DepthBufferPreference)));
			Text.AppendLine(String.Format("\t\t<meta-data android:name=\"com.epicgames.unreal.GameActivity.bPackageDataInsideApk\" android:value=\"{0}\"/>", bPackageDataInsideApk ? "true" : "false"));
			Text.AppendLine(String.Format("\t\t<meta-data android:name=\"com.epicgames.unreal.GameActivity.bVerifyOBBOnStartUp\" android:value=\"{0}\"/>", (bIsForDistribution && !bDisableVerifyOBBOnStartUp) ? "true" : "false"));
			Text.AppendLine(String.Format("\t\t<meta-data android:name=\"com.epicgames.unreal.GameActivity.bShouldHideUI\" android:value=\"{0}\"/>", EnableFullScreen ? "true" : "false"));
			Text.AppendLine(String.Format("\t\t<meta-data android:name=\"com.epicgames.unreal.GameActivity.ProjectName\" android:value=\"{0}\"/>", ProjectName));
			Text.AppendLine(String.Format("\t\t<meta-data android:name=\"com.epicgames.unreal.GameActivity.AppType\" android:value=\"{0}\"/>", AppType));
			Text.AppendLine(String.Format("\t\t<meta-data android:name=\"com.epicgames.unreal.GameActivity.bHasOBBFiles\" android:value=\"{0}\"/>", bHasOBBFiles ? "true" : "false"));
			Text.AppendLine(String.Format("\t\t<meta-data android:name=\"com.epicgames.unreal.GameActivity.BuildConfiguration\" android:value=\"{0}\"/>", Configuration));
			Text.AppendLine(String.Format("\t\t<meta-data android:name=\"com.epicgames.unreal.GameActivity.CookedFlavors\" android:value=\"{0}\"/>", CookedFlavors));
			Text.AppendLine(String.Format("\t\t<meta-data android:name=\"com.epicgames.unreal.GameActivity.bValidateTextureFormats\" android:value=\"{0}\"/>", bValidateTextureFormats ? "true" : "false"));
			Text.AppendLine(String.Format("\t\t<meta-data android:name=\"com.epicgames.unreal.GameActivity.bUseExternalFilesDir\" android:value=\"{0}\"/>", bUseExternalFilesDir ? "true" : "false"));
			Text.AppendLine(String.Format("\t\t<meta-data android:name=\"com.epicgames.unreal.GameActivity.bPublicLogFiles\" android:value=\"{0}\"/>", bPublicLogFiles ? "true" : "false"));
			Text.AppendLine(String.Format("\t\t<meta-data android:name=\"com.epicgames.unreal.GameActivity.bUseDisplayCutout\" android:value=\"{0}\"/>", bUseDisplayCutout ? "true" : "false"));
			Text.AppendLine(String.Format("\t\t<meta-data android:name=\"com.epicgames.unreal.GameActivity.RoundedEdgeSafeZone\" android:value=\"{0}\"/>", RoundedEdgeSafeZone));
			Text.AppendLine(String.Format("\t\t<meta-data android:name=\"com.epicgames.unreal.GameActivity.bAllowIMU\" android:value=\"{0}\"/>", bAllowIMU ? "true" : "false"));
			Text.AppendLine(String.Format("\t\t<meta-data android:name=\"com.epicgames.unreal.GameActivity.bSupportsVulkan\" android:value=\"{0}\"/>", bSupportsVulkan ? "true" : "false"));
			Text.AppendLine(String.Format("\t\t<meta-data android:name=\"com.epicgames.unreal.GameActivity.PropagateAlpha\" android:value=\"{0}\"/>", PropagateAlpha));
			Text.AppendLine(String.Format("\t\t<meta-data android:name=\"com.epicgames.unreal.GameActivity.StartupPermissions\" android:value=\"{0}\"/>", StartupPermissions));
			if (TargetSDKVersion >= 29)
			{
				Text.AppendLine(String.Format("\t\t<meta-data android:name=\"android.supports_size_changes\" android:value=\"{0}\"/>", bSupportSizeChanges ? "true" : "false"));
			}
			Text.AppendLine("\t\t<meta-data android:name=\"com.google.android.gms.games.APP_ID\"");
			Text.AppendLine("\t\t           android:value=\"@string/app_id\" />");
			Text.AppendLine("\t\t<meta-data android:name=\"com.google.android.gms.version\"");
			Text.AppendLine("\t\t           android:value=\"@integer/google_play_services_version\" />");
			if (bSupportAdMob)
			{
				Text.AppendLine("\t\t<activity android:name=\"com.google.android.gms.ads.AdActivity\"");
				Text.AppendLine("\t\t          android:configChanges=\"keyboard|keyboardHidden|orientation|screenLayout|uiMode|screenSize|smallestScreenSize\"/>");
			}
			if (!String.IsNullOrEmpty(ExtraApplicationSettings))
			{
				ExtraApplicationSettings = ExtraApplicationSettings.Replace("\\n", "\n", StringComparison.Ordinal);
				foreach (string Line in ExtraApplicationSettings.Split("\r\n".ToCharArray()))
				{
					Text.AppendLine("\t\t" + Line);
				}
			}
			string ApplicationAdditionsFile = Path.Combine(GameBuildFilesPath, "ManifestApplicationAdditions.txt");
			if (File.Exists(ApplicationAdditionsFile))
			{
				foreach (string Line in File.ReadAllLines(ApplicationAdditionsFile))
				{
					Text.AppendLine("\t\t" + Line);
				}
			}

			// Declare the 8 OpenGL program compiling services.
			Text.AppendLine(@"		<service android:name=""com.epicgames.unreal.psoservices.OGLProgramService"" android:process="":psoprogramservice"" />");
			// Declare the remaining 7 OpenGL program compiling services. (all derived from OGLProgramService)
			for (int i = 1; i < 8; i++)
			{
				string serviceLine = String.Format("		<service android:name=\"com.epicgames.unreal.psoservices.OGLProgramService{0}\" android:process=\":psoprogramservice{0}\" />", i);
				Text.AppendLine(serviceLine);
			}

			// Declare the 8 Vulkan program compiling services.
			Text.AppendLine(@"		<service android:name=""com.epicgames.unreal.psoservices.VulkanProgramService"" android:process="":psoprogramservice"" />");
			// Declare the remaining 7 Vulkan program compiling services. (all derived from VulkanProgramService)
			for (int i = 1; i < 8; i++)
			{
				string serviceLine = String.Format("		<service android:name=\"com.epicgames.unreal.psoservices.VulkanProgramService{0}\" android:process=\":psoprogramservice{0}\" />", i);
				Text.AppendLine(serviceLine);
			}

			// Required for OBB download support
			Text.AppendLine("\t\t<service android:name=\"OBBDownloaderService\" />");
			Text.AppendLine("\t\t<receiver android:name=\"AlarmReceiver\" android:exported=\"false\"/>");

			Text.AppendLine("\t\t<receiver android:name=\"com.epicgames.unreal.LocalNotificationReceiver\" android:exported=\"false\"/>");
			Text.AppendLine("\t\t<receiver android:name=\"com.epicgames.unreal.CellularReceiver\" android:exported=\"false\"/>");

			if (bRestoreNotificationsOnReboot)
			{
				Text.AppendLine("\t\t<receiver android:name=\"com.epicgames.unreal.BootCompleteReceiver\" android:exported=\"true\">");
				Text.AppendLine("\t\t\t<intent-filter>");
				Text.AppendLine("\t\t\t\t<action android:name=\"android.intent.action.BOOT_COMPLETED\" />");
				Text.AppendLine("\t\t\t\t<action android:name=\"android.intent.action.QUICKBOOT_POWERON\" />");
				Text.AppendLine("\t\t\t\t<action android:name=\"com.htc.intent.action.QUICKBOOT_POWERON\" />");
				Text.AppendLine("\t\t\t</intent-filter>");
				Text.AppendLine("\t\t</receiver>");
			}

			Text.AppendLine("\t\t<receiver android:name=\"com.epicgames.unreal.MulticastBroadcastReceiver\" android:exported=\"true\">");
			Text.AppendLine("\t\t\t<intent-filter>");
			Text.AppendLine("\t\t\t\t<action android:name=\"com.android.vending.INSTALL_REFERRER\" />");
			Text.AppendLine("\t\t\t</intent-filter>");
			Text.AppendLine("\t\t</receiver>");

			Text.AppendLine(String.Format("\t\t<meta-data android:name=\"android.max_aspect\" android:value=\"{0}\" />", MaxAspectRatioString));

			if (TargetSDKVersion >= 36)
			{
				// Opt out of Android 16+ restricted resizability so min/maxAspectRatio are honored on large-screen devices.
				// Note: this is ignored on API 37+
				Text.AppendLine("\t\t<property android:name=\"android.window.PROPERTY_COMPAT_ALLOW_RESTRICTED_RESIZABILITY\" android:value=\"true\" />");
			}

			Text.AppendLine("\t</application>");

			Text.AppendLine("");
			Text.AppendLine("\t<!-- Requirements -->");

			// check for an override for the requirements section of the manifest
			string RequirementsOverrideFile = Path.Combine(GameBuildFilesPath, "ManifestRequirementsOverride.txt");
			if (File.Exists(RequirementsOverrideFile))
			{
				foreach (string Line in File.ReadAllLines(RequirementsOverrideFile))
				{
					Text.AppendLine("\t" + Line);
				}
			}
			else
			{
				if (bPackageForMetaQuest)
				{
					// needed for virtual keyboard to work on MetaQuest
					Text.AppendLine("\t<uses-feature android:name=\"oculus.software.overlay_keyboard\" android:required=\"false\"/>");

					// need for hand tracking on MetaQuest
					Text.AppendLine("\t<uses-feature android:name=\"oculus.software.handtracking\" android:required=\"false\"/>");
					Text.AppendLine("\t<uses-permission android:name=\"com.oculus.permission.HAND_TRACKING\"/>");

					// need for passthrough on MetaQuest
					Text.AppendLine("\t<uses-feature android:name=\"com.oculus.feature.PASSTHROUGH\" android:required=\"false\" />");
				}

				Text.AppendLine("\t<uses-feature android:glEsVersion=\"" + AndroidToolChain.GetGLESVersion(bBuildForES31) + "\" android:required=\"true\" />");
				Text.AppendLine("\t<uses-permission android:name=\"android.permission.INTERNET\"/>");
				Text.AppendLine("\t<uses-permission android:name=\"android.permission.ACCESS_NETWORK_STATE\"/>");
				Text.AppendLine("\t<uses-permission android:name=\"com.android.vending.CHECK_LICENSE\"/>");
				Text.AppendLine("\t<uses-permission android:name=\"android.permission.ACCESS_WIFI_STATE\"/>");

				if (bEnableMulticastSupport)
				{
					// This permission is needed to be able to acquire a WifiManager.MulticastLock so broadcast/multcast traffic is 
					// not filtered out by the device network interface 
					Text.AppendLine("\t<uses-permission android:name=\"android.permission.CHANGE_WIFI_MULTICAST_STATE\"/>");
				}

				if (bRestoreNotificationsOnReboot)
				{
					Text.AppendLine("\t<uses-permission android:name=\"android.permission.RECEIVE_BOOT_COMPLETED\"/>");
				}

				// Apps requesting these permissions are rejected by the Quest App Store
				if (!bPackageForMetaQuest)
				{
					if (TargetSDKVersion >= 23 && TargetSDKVersion < 30)
					{
						if (Configuration != "Shipping" || !bUseExternalFilesDir)
						{
							Text.AppendLine("\t<uses-permission android:name=\"android.permission.WRITE_EXTERNAL_STORAGE\"/>");
						}
					}
					Text.AppendLine("\t<uses-permission android:name=\"android.permission.WAKE_LOCK\"/>");
					//Text.AppendLine("\t<uses-permission android:name=\"android.permission.READ_PHONE_STATE\"/>");
					Text.AppendLine("\t<uses-permission android:name=\"android.permission.MODIFY_AUDIO_SETTINGS\"/>");
					Text.AppendLine("\t<uses-permission android:name=\"android.permission.VIBRATE\"/>");
				}

				//			Text.AppendLine("\t<uses-permission android:name=\"android.permission.DISABLE_KEYGUARD\"/>");

				if (bEnableIAP)
				{
					Text.AppendLine("\t<uses-permission android:name=\"com.android.vending.BILLING\"/>");
				}
				if (ExtraPermissions != null)
				{
					foreach (string Permission in ExtraPermissions)
					{
						string TrimmedPermission = Permission.Trim(' ');
						if (!String.IsNullOrEmpty(TrimmedPermission))
						{
							string PermissionString = String.Format("\t<uses-permission android:name=\"{0}\"/>", TrimmedPermission);
							if (!Text.ToString().Contains(PermissionString, StringComparison.Ordinal))
							{
								Text.AppendLine(PermissionString);
							}
						}
					}
				}
				string RequirementsAdditionsFile = Path.Combine(GameBuildFilesPath, "ManifestRequirementsAdditions.txt");
				if (File.Exists(RequirementsAdditionsFile))
				{
					foreach (string Line in File.ReadAllLines(RequirementsAdditionsFile))
					{
						Text.AppendLine("\t" + Line);
					}
				}

				if (!bSupportingAllTextureFormats)
				{
					Text.AppendLine("\t<!-- Supported texture compression formats (cooked) -->");
					if (bETC2Enabled && !bOnlyETC2Enabled)
					{
						Text.AppendLine("\t<supports-gl-texture android:name=\"GL_COMPRESSED_RGB8_ETC2\" />");
						Text.AppendLine("\t<supports-gl-texture android:name=\"GL_COMPRESSED_RGBA8_ETC2_EAC\" />");
					}
					if (bDXTEnabled)
					{
						Text.AppendLine("\t<supports-gl-texture android:name=\"GL_EXT_texture_compression_dxt1\" />");
						Text.AppendLine("\t<supports-gl-texture android:name=\"GL_EXT_texture_compression_s3tc\" />");
						Text.AppendLine("\t<supports-gl-texture android:name=\"GL_NV_texture_compression_s3tc\" />");
					}
					if (bASTCEnabled)
					{
						Text.AppendLine("\t<supports-gl-texture android:name=\"GL_KHR_texture_compression_astc_ldr\" />");
					}
				}
			}

			Text.AppendLine("</manifest>");

			// allow plugins to modify final manifest HERE
			XDocument XDoc;
			try
			{
				XDoc = XDocument.Parse(Text.ToString());
			}
			catch (Exception e)
			{
				throw new BuildException("AndroidManifest.xml is invalid {0}\n{1}", e, Text.ToString());
			}

			UPL!.SetGlobalContextVariable("CompileSDK", SDKLevelInt.ToString());
			UPL!.ProcessPluginNode(Arch, "androidManifestUpdates", "", ref XDoc);
			return XDoc.ToString();
		}

		private string GenerateProguard(string Arch, string EngineSourcePath, string GameBuildFilesPath)
		{
			StringBuilder Text = new StringBuilder();

			string ProguardFile = Path.Combine(EngineSourcePath, "proguard-project.txt");
			if (File.Exists(ProguardFile))
			{
				foreach (string Line in File.ReadAllLines(ProguardFile))
				{
					Text.AppendLine(Line);
				}
			}

			string ProguardAdditionsFile = Path.Combine(GameBuildFilesPath, "ProguardAdditions.txt");
			if (File.Exists(ProguardAdditionsFile))
			{
				foreach (string Line in File.ReadAllLines(ProguardAdditionsFile))
				{
					Text.AppendLine(Line);
				}
			}

			// add plugin additions
			return UPL!.ProcessPluginNode(Arch, "proguardAdditions", Text.ToString());
		}

		private void ValidateGooglePlay(string UnrealBuildPath)
		{
			ConfigHierarchy Ini = GetConfigCacheIni(ConfigHierarchyType.Engine);
			bool bEnableGooglePlaySupport;
			Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bEnableGooglePlaySupport", out bEnableGooglePlaySupport);

			if (!bEnableGooglePlaySupport)
			{
				// do not need to do anything; it is fine
				return;
			}

			string IniAppId;
			bool bInvalidIniAppId = false;
			Ini.GetString("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "GamesAppID", out IniAppId);

			//validate the value found in the AndroidRuntimeSettings
			if (IniAppId.Length == 0 || !Int64.TryParse(IniAppId, out _))
			{
				bInvalidIniAppId = true;
			}

			bool bInvalid = false;
			string ReplacementId = "";
			string Filename = Path.Combine(UnrealBuildPath, "res", "values", "GooglePlayAppID.xml");
			if (File.Exists(Filename))
			{
				string[] FileContent = File.ReadAllLines(Filename);
				int LineIndex = -1;
				foreach (string Line in FileContent)
				{
					++LineIndex;

					int StartIndex = Line.IndexOf("\"app_id\">", StringComparison.Ordinal);
					if (StartIndex < 0)
					{
						continue;
					}

					StartIndex += 9;
					int EndIndex = Line.IndexOf("</string>", StringComparison.Ordinal);
					if (EndIndex < 0)
					{
						continue;
					}

					string XmlAppId = Line.Substring(StartIndex, EndIndex - StartIndex);

					//validate that the AppId matches the .ini value for the GooglePlay AppId, assuming it's valid
					if (!bInvalidIniAppId && !IniAppId.Equals(XmlAppId, StringComparison.Ordinal))
					{
						Logger.LogInformation("Replacing Google Play AppID in GooglePlayAppID.xml with AndroidRuntimeSettings .ini value");

						bInvalid = true;
						ReplacementId = IniAppId;

					}
					else if (XmlAppId.Length == 0 || !Int64.TryParse(XmlAppId, out _))
					{
						Logger.LogWarning("\nWARNING: GooglePlay Games App ID is invalid! Replacing it with \"1\"");

						//write file with something which will fail but not cause an exception if executed
						bInvalid = true;
						ReplacementId = "1";
					}

					if (bInvalid)
					{
						// remove any read only flags if invalid so it can be replaced
						FileInfo DestFileInfo = new FileInfo(Filename);
						DestFileInfo.Attributes &= ~FileAttributes.ReadOnly;

						//preserve the rest of the file, just fix up this line
						string NewLine = Line.Replace("\"app_id\">" + XmlAppId + "</string>", "\"app_id\">" + ReplacementId + "</string>", StringComparison.Ordinal);
						FileContent[LineIndex] = NewLine;

						File.WriteAllLines(Filename, FileContent);
					}

					break;
				}
			}
			else
			{
				string NewAppId;
				// if we don't have an appID to use from the config, write file with something which will fail but not cause an exception if executed
				if (bInvalidIniAppId)
				{
					Logger.LogWarning("\nWARNING: Creating GooglePlayAppID.xml using a Google Play AppID of \"1\" because there was no valid AppID in AndroidRuntimeSettings!");
					NewAppId = "1";
				}
				else
				{
					Logger.LogInformation("Creating GooglePlayAppID.xml with AndroidRuntimeSettings .ini value");
					NewAppId = IniAppId;
				}

				File.WriteAllText(Filename, XML_HEADER + "\n<resources>\n\t<string name=\"app_id\">" + NewAppId + "</string>\n</resources>\n");
			}
		}

		private static bool FilesAreDifferent(string SourceFilename, string DestFilename)
		{
			// source must exist
			FileInfo SourceInfo = new(SourceFilename);
			if (!SourceInfo.Exists)
			{
				throw new BuildException("Can't make an APK without file [{0}]", SourceFilename);
			}

			// different if destination doesn't exist
			FileInfo DestInfo = new(DestFilename);
			if (!DestInfo.Exists)
			{
				return true;
			}

			// file lengths differ?
			if (SourceInfo.Length != DestInfo.Length)
			{
				return true;
			}

			// validate timestamps
			// Allow 1s slop in case one of the files is on a FAT volume, and the other is not
			if (Math.Abs((SourceInfo.LastWriteTimeUtc - DestInfo.LastWriteTimeUtc).TotalSeconds) > 1)
			{
				return true;
			}

			// could check actual bytes just to be sure, but good enough
			return false;
		}

		private static bool FilesAreIdentical(string SourceFilename, string DestFilename)
		{
			// source must exist
			FileInfo SourceInfo = new FileInfo(SourceFilename);
			if (!SourceInfo.Exists)
			{
				throw new BuildException("Can't make an APK without file [{0}]", SourceFilename);
			}

			// different if destination doesn't exist
			FileInfo DestInfo = new FileInfo(DestFilename);
			if (!DestInfo.Exists)
			{
				return false;
			}

			// file lengths differ?
			if (SourceInfo.Length != DestInfo.Length)
			{
				return false;
			}

			using (FileStream SourceStream = new FileStream(SourceFilename, FileMode.Open, FileAccess.Read, FileShare.Read))
			using (FileStream DestStream = new FileStream(DestFilename, FileMode.Open, FileAccess.Read, FileShare.Read))
			{
				using (BinaryReader SourceReader = new BinaryReader(SourceStream))
				using (BinaryReader DestReader = new BinaryReader(DestStream))
				{
					bool bEOF = false;
					while (!bEOF)
					{
						byte[] SourceData = SourceReader.ReadBytes(32768);
						if (SourceData.Length == 0)
						{
							bEOF = true;
							break;
						}

						byte[] DestData = DestReader.ReadBytes(32768);
						if (!SourceData.SequenceEqual(DestData))
						{
							return false;
						}
					}
					return true;
				}
			}
		}

		private bool RequiresOBB(bool bDisallowPackageInAPK, string OBBLocation)
		{
			if (bNoObbs)
			{
				Logger.LogInformation("OBBs have been disabled.");
				return false;
			}
			else if (bDisallowPackageInAPK)
			{
				Logger.LogInformation("APK contains data.");
				return false;
			}
			else if (!String.IsNullOrEmpty(Environment.GetEnvironmentVariable("uebp_LOCAL_ROOT")))
			{
				Logger.LogInformation("On build machine.");
				return true;
			}
			else
			{
				Logger.LogInformation("Looking for OBB.");
				return File.Exists(OBBLocation);
			}
		}

		private bool CreateRunGradle(string GradlePath)
		{
			string RunGradleBatFilename = Path.Combine(GradlePath, "rungradle.bat");

			// check for an unused drive letter
			string UnusedDriveLetter = "";
			bool bFound = true;
			DriveInfo[] AllDrives = DriveInfo.GetDrives();
			for (char DriveLetter = 'Z'; DriveLetter >= 'A'; DriveLetter--)
			{
				UnusedDriveLetter = Char.ToString(DriveLetter) + ":";
				bFound = false;
				for (int DriveIndex = AllDrives.Length - 1; DriveIndex >= 0; DriveIndex--)
				{
					if (AllDrives[DriveIndex].Name.StartsWith(UnusedDriveLetter, StringComparison.OrdinalIgnoreCase))
					{
						bFound = true;
						break;
					}
				}
				if (!bFound)
				{
					break;
				}
			}

			if (bFound)
			{
				Logger.LogInformation("\nUnable to apply subst, using gradlew.bat directly (all drive letters in use!)");
				return false;
			}

			Logger.LogInformation("\nCreating rungradle.bat to work around commandline length limit (using unused drive letter {UnusedDriveLetter})", UnusedDriveLetter);

			// make sure rungradle.bat isn't read-only
			if (File.Exists(RunGradleBatFilename))
			{
				FileAttributes Attribs = File.GetAttributes(RunGradleBatFilename);
				if (Attribs.HasFlag(FileAttributes.ReadOnly))
				{
					File.SetAttributes(RunGradleBatFilename, Attribs & ~FileAttributes.ReadOnly);
				}
			}

			// generate new rungradle.bat with an unused drive letter for subst
			string RunGradleBatText =
					"@echo off\n" +
					"setlocal\n" +
					"set GRADLEPATH=%~dp0\n" +
					"set GRADLE_CMD_LINE_ARGS=\n" +
					":setupArgs\n" +
					"if \"\"%1\"\"==\"\"\"\" goto doneStart\n" +
					"set GRADLE_CMD_LINE_ARGS=%GRADLE_CMD_LINE_ARGS% %1\n" +
					"shift\n" +
					"goto setupArgs\n\n" +
					":doneStart\n" +
					"subst " + UnusedDriveLetter + " \"%CD%\"\n" +
					"pushd " + UnusedDriveLetter + "\n" +
					"call \"%GRADLEPATH%\\gradlew.bat\" %GRADLE_CMD_LINE_ARGS%\n" +
					"set GRADLEERROR=%ERRORLEVEL%\n" +
					"popd\n" +
					"subst " + UnusedDriveLetter + " /d\n" +
					"exit /b %GRADLEERROR%\n";

			File.WriteAllText(RunGradleBatFilename, RunGradleBatText);

			return true;
		}

		private bool BundleEnabled()
		{
			if (bFastIterate || ForceAPKGeneration)
			{
				return false;
			}
			ConfigHierarchy Ini = GetConfigCacheIni(ConfigHierarchyType.Engine);
			Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bEnableBundle", out bool bEnableBundle);
			return bEnableBundle;
		}

		private bool IsLicenseAgreementValid()
		{
			string LicensePath = Environment.ExpandEnvironmentVariables("%ANDROID_HOME%/licenses");

			// directory must exist
			if (!Directory.Exists(LicensePath))
			{
				Logger.LogInformation("Directory doesn't exist {LicensePath}", LicensePath);
				return false;
			}

			// license file must exist
			string LicenseFilename = Path.Combine(LicensePath, "android-sdk-license");
			if (!File.Exists(LicenseFilename))
			{
				Logger.LogInformation("File doesn't exist {LicenseFilename}", LicenseFilename);
				return false;
			}

			// ignore contents of hash for now (Gradle will report if it isn't valid)
			return true;
		}

		private void GetMinTargetSDKVersions(AndroidToolChain ToolChain, UnrealPluginLanguage UPL, string NDKArch, bool bEnableBundle, out int MinSDKVersion, out int TargetSDKVersion, out int NDKLevelInt)
		{
			ConfigHierarchy Ini = GetConfigCacheIni(ConfigHierarchyType.Engine);
			Ini.GetInt32("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "MinSDKVersion", out MinSDKVersion);
			Ini.GetInt32("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "TargetSDKVersion", out TargetSDKVersion);

			// Check for targetSDKOverride from UPL
			string TargetOverride = UPL.ProcessPluginNode(NDKArch, "targetSDKOverride", "");
			if (!String.IsNullOrEmpty(TargetOverride))
			{
				if (Int32.TryParse(TargetOverride, out int OverrideInt))
				{
					TargetSDKVersion = OverrideInt;
				}
			}

			if ((ToolChain.BuildWithSanitizer() != AndroidToolChain.ClangSanitizer.None) && (MinSDKVersion < 27))
			{
				MinSDKVersion = 27;
				Logger.LogInformation("Fixing minSdkVersion; requires minSdkVersion of {MinVer} for Clang's Sanitizers", MinSDKVersion);
			}

			if (bEnableBundle && MinSDKVersion < MinimumSDKLevelForBundle)
			{
				MinSDKVersion = MinimumSDKLevelForBundle;
				Logger.LogInformation("Fixing minSdkVersion; requires minSdkVersion of {MinVer} for App Bundle support", MinimumSDKLevelForBundle);
			}

			// Make sure minSdkVersion is at least 13 (need this for appcompat-v13 used by AndroidPermissions)
			// this may be changed by active plugins (Google Play Services 11.0.4 needs 14 for example)
			if (MinSDKVersion < MinimumSDKLevelForGradle)
			{
				MinSDKVersion = MinimumSDKLevelForGradle;
				Logger.LogInformation("Fixing minSdkVersion; requires minSdkVersion of {MinVer} with Gradle based on active plugins", MinimumSDKLevelForGradle);
			}

			// Get NDK API level and enforce minimum
			NDKLevelInt = ToolChain.GetNdkApiLevelInt();
			if (NDKLevelInt < AndroidToolChain.MinimumNDKAPILevel)
			{
				// 21 is required for GL ES3.1, 26 for ANativeWindow_setBuffersTransform
				NDKLevelInt = AndroidToolChain.MinimumNDKAPILevel;
			}

			// fix up the MinSdkVersion to be at least NDKLevelInt
			if (MinSDKVersion < NDKLevelInt)
			{
				Logger.LogInformation("Fixing minSdkVersion; NDK level is {NDKLevelInt} which is above minSdkVersion {MinSDKVersion}.", NDKLevelInt, MinSDKVersion);
				MinSDKVersion = NDKLevelInt;
			}

			// fix up the TargetSDK to be at least MinSdkVersion
			if (TargetSDKVersion < MinSDKVersion)
			{
				Logger.LogInformation("Fixing targetSdkVersion; minSdkVersion is {MinSDKVersion} which is above targetSdkVersion {TargetSDKVersion}.", MinSDKVersion, TargetSDKVersion);
				TargetSDKVersion = MinSDKVersion;
			}
		}

		private static readonly uint[] CRCTablesSB8 = {
			0x00000000, 0x77073096, 0xee0e612c, 0x990951ba, 0x076dc419, 0x706af48f, 0xe963a535, 0x9e6495a3, 0x0edb8832, 0x79dcb8a4, 0xe0d5e91e, 0x97d2d988, 0x09b64c2b, 0x7eb17cbd, 0xe7b82d07, 0x90bf1d91,
			0x1db71064, 0x6ab020f2, 0xf3b97148, 0x84be41de, 0x1adad47d, 0x6ddde4eb, 0xf4d4b551, 0x83d385c7, 0x136c9856, 0x646ba8c0, 0xfd62f97a, 0x8a65c9ec, 0x14015c4f, 0x63066cd9, 0xfa0f3d63, 0x8d080df5,
			0x3b6e20c8, 0x4c69105e, 0xd56041e4, 0xa2677172, 0x3c03e4d1, 0x4b04d447, 0xd20d85fd, 0xa50ab56b, 0x35b5a8fa, 0x42b2986c, 0xdbbbc9d6, 0xacbcf940, 0x32d86ce3, 0x45df5c75, 0xdcd60dcf, 0xabd13d59,
			0x26d930ac, 0x51de003a, 0xc8d75180, 0xbfd06116, 0x21b4f4b5, 0x56b3c423, 0xcfba9599, 0xb8bda50f, 0x2802b89e, 0x5f058808, 0xc60cd9b2, 0xb10be924, 0x2f6f7c87, 0x58684c11, 0xc1611dab, 0xb6662d3d,
			0x76dc4190, 0x01db7106, 0x98d220bc, 0xefd5102a, 0x71b18589, 0x06b6b51f, 0x9fbfe4a5, 0xe8b8d433, 0x7807c9a2, 0x0f00f934, 0x9609a88e, 0xe10e9818, 0x7f6a0dbb, 0x086d3d2d, 0x91646c97, 0xe6635c01,
			0x6b6b51f4, 0x1c6c6162, 0x856530d8, 0xf262004e, 0x6c0695ed, 0x1b01a57b, 0x8208f4c1, 0xf50fc457, 0x65b0d9c6, 0x12b7e950, 0x8bbeb8ea, 0xfcb9887c, 0x62dd1ddf, 0x15da2d49, 0x8cd37cf3, 0xfbd44c65,
			0x4db26158, 0x3ab551ce, 0xa3bc0074, 0xd4bb30e2, 0x4adfa541, 0x3dd895d7, 0xa4d1c46d, 0xd3d6f4fb, 0x4369e96a, 0x346ed9fc, 0xad678846, 0xda60b8d0, 0x44042d73, 0x33031de5, 0xaa0a4c5f, 0xdd0d7cc9,
			0x5005713c, 0x270241aa, 0xbe0b1010, 0xc90c2086, 0x5768b525, 0x206f85b3, 0xb966d409, 0xce61e49f, 0x5edef90e, 0x29d9c998, 0xb0d09822, 0xc7d7a8b4, 0x59b33d17, 0x2eb40d81, 0xb7bd5c3b, 0xc0ba6cad,
			0xedb88320, 0x9abfb3b6, 0x03b6e20c, 0x74b1d29a, 0xead54739, 0x9dd277af, 0x04db2615, 0x73dc1683, 0xe3630b12, 0x94643b84, 0x0d6d6a3e, 0x7a6a5aa8, 0xe40ecf0b, 0x9309ff9d, 0x0a00ae27, 0x7d079eb1,
			0xf00f9344, 0x8708a3d2, 0x1e01f268, 0x6906c2fe, 0xf762575d, 0x806567cb, 0x196c3671, 0x6e6b06e7, 0xfed41b76, 0x89d32be0, 0x10da7a5a, 0x67dd4acc, 0xf9b9df6f, 0x8ebeeff9, 0x17b7be43, 0x60b08ed5,
			0xd6d6a3e8, 0xa1d1937e, 0x38d8c2c4, 0x4fdff252, 0xd1bb67f1, 0xa6bc5767, 0x3fb506dd, 0x48b2364b, 0xd80d2bda, 0xaf0a1b4c, 0x36034af6, 0x41047a60, 0xdf60efc3, 0xa867df55, 0x316e8eef, 0x4669be79,
			0xcb61b38c, 0xbc66831a, 0x256fd2a0, 0x5268e236, 0xcc0c7795, 0xbb0b4703, 0x220216b9, 0x5505262f, 0xc5ba3bbe, 0xb2bd0b28, 0x2bb45a92, 0x5cb36a04, 0xc2d7ffa7, 0xb5d0cf31, 0x2cd99e8b, 0x5bdeae1d,
			0x9b64c2b0, 0xec63f226, 0x756aa39c, 0x026d930a, 0x9c0906a9, 0xeb0e363f, 0x72076785, 0x05005713, 0x95bf4a82, 0xe2b87a14, 0x7bb12bae, 0x0cb61b38, 0x92d28e9b, 0xe5d5be0d, 0x7cdcefb7, 0x0bdbdf21,
			0x86d3d2d4, 0xf1d4e242, 0x68ddb3f8, 0x1fda836e, 0x81be16cd, 0xf6b9265b, 0x6fb077e1, 0x18b74777, 0x88085ae6, 0xff0f6a70, 0x66063bca, 0x11010b5c, 0x8f659eff, 0xf862ae69, 0x616bffd3, 0x166ccf45,
			0xa00ae278, 0xd70dd2ee, 0x4e048354, 0x3903b3c2, 0xa7672661, 0xd06016f7, 0x4969474d, 0x3e6e77db, 0xaed16a4a, 0xd9d65adc, 0x40df0b66, 0x37d83bf0, 0xa9bcae53, 0xdebb9ec5, 0x47b2cf7f, 0x30b5ffe9,
			0xbdbdf21c, 0xcabac28a, 0x53b39330, 0x24b4a3a6, 0xbad03605, 0xcdd70693, 0x54de5729, 0x23d967bf, 0xb3667a2e, 0xc4614ab8, 0x5d681b02, 0x2a6f2b94, 0xb40bbe37, 0xc30c8ea1, 0x5a05df1b, 0x2d02ef8d
		};

		/// <summary>
		/// Replaces a SO in an existing apk, performing an alignment.
		/// This function can throw IO exceptions, ensure usage of proper try/catch handling
		/// </summary>
		/// <param name="SourceApk">The base apk to align</param>
		/// <param name="TargetApk">Path to output the newly aligned apk</param>
		/// <param name="Logger">Logger</param>
		/// <param name="SourceSOFile">SO file to use as a replacement</param>
		/// <param name="RelativeSOFile">Path of the SO file to replace, relative to the apk</param>
		public static void CopyAPKAndReplaceSO(FileReference SourceApk, FileReference TargetApk, FileReference SourceSOFile, string RelativeSOFile, ILogger Logger)
		{
			string DirectoryFilename = TargetApk.FullName + ".dir";

			using BinaryReader SourceReader = new BinaryReader(new FileStream(SourceApk.FullName, FileMode.Open, FileAccess.Read));
			using BinaryWriter DestWriter = new BinaryWriter(new FileStream(TargetApk.FullName, FileMode.Create));
			using BinaryWriter DirectoryWriter = new BinaryWriter(new FileStream(DirectoryFilename, FileMode.Create));
			{
				byte[] CopyBuffer = new byte[65536];
				ushort TotalEntries = 0;
				uint Remaining;

				uint Signature = 0x02014b50;
				ushort DirVersion = 0;
				ushort DiskNumber = 0;
				ushort DiskStart = 0;
				ushort InternalAttr = 0;
				uint ExternalAttr = 0;
				ushort CommentLen = 0;

				// copy files
				while (true)
				{
					uint Header = SourceReader.ReadUInt32();
					if (Header != 0x04034b50)
					{
						SourceReader.Close();
						break;
					}

					ushort Version = SourceReader.ReadUInt16();
					ushort Flags = SourceReader.ReadUInt16();
					ushort Compression = SourceReader.ReadUInt16();
					ushort ModTime = SourceReader.ReadUInt16();
					ushort ModDate = SourceReader.ReadUInt16();
					uint CRC32Value = SourceReader.ReadUInt32();
					uint CompressedSize = SourceReader.ReadUInt32();
					uint UncompressedSize = SourceReader.ReadUInt32();
					ushort FilenameLen = SourceReader.ReadUInt16();
					ushort ExtraLen = SourceReader.ReadUInt16();
					byte[] Filename = new byte[FilenameLen];
					SourceReader.BaseStream.ReadExactly(Filename, 0, FilenameLen);
					byte[] Extra = new byte[ExtraLen];
					SourceReader.BaseStream.ReadExactly(Extra, 0, ExtraLen);

					string FilenameStr = Encoding.UTF8.GetString(Filename, 0, FilenameLen);
					if (!FilenameStr.Equals(RelativeSOFile, StringComparison.OrdinalIgnoreCase))
					{
						TotalEntries++;
						uint Location = (uint)DestWriter.BaseStream.Position;

						// calculate location after header and any additional alignment needed (only if uncompressed)
						uint HeaderSize = (uint)(4 + 2 + 2 + 2 + 2 + 2 + 4 + 4 + 4 + 2 + 2 + FilenameLen + ExtraLen);
						uint HeaderEnd = Location + HeaderSize;
						uint DataStart = (uint)((HeaderEnd + 3) & ~3);
						ushort ExtraAlign = (ushort)(Compression != 0 ? 0 : (DataStart - HeaderEnd));

						// write local header
						DestWriter.Write(Header);
						DestWriter.Write(Version);
						DestWriter.Write(Flags);
						DestWriter.Write(Compression);
						DestWriter.Write(ModTime);
						DestWriter.Write(ModDate);
						DestWriter.Write(CRC32Value);
						DestWriter.Write(CompressedSize);
						DestWriter.Write(UncompressedSize);
						DestWriter.Write(FilenameLen);
						DestWriter.Write((ushort)(ExtraLen + ExtraAlign));
						DestWriter.BaseStream.Write(Filename, 0, FilenameLen);
						DestWriter.BaseStream.Write(Extra, 0, ExtraLen);
						while (ExtraAlign-- > 0)
						{
							DestWriter.Write((byte)0);
						}

						// copy file data as-is
						Remaining = CompressedSize;
						while (Remaining > 0)
						{
							int CopySize = Remaining < 65536 ? (int)Remaining : 65536;
							int BytesRead = SourceReader.BaseStream.Read(CopyBuffer, 0, CopySize);
							DestWriter.BaseStream.Write(CopyBuffer, 0, BytesRead);
							Remaining -= (uint)BytesRead;
						}

						// write central directory entry for this file
						DirectoryWriter.Write(Signature);
						DirectoryWriter.Write(DirVersion);
						DirectoryWriter.Write(Version);
						DirectoryWriter.Write(Flags);
						DirectoryWriter.Write(Compression);
						DirectoryWriter.Write(ModTime);
						DirectoryWriter.Write(ModDate);
						DirectoryWriter.Write(CRC32Value);
						DirectoryWriter.Write(CompressedSize);
						DirectoryWriter.Write(UncompressedSize);
						DirectoryWriter.Write(FilenameLen);
						DirectoryWriter.Write(ExtraLen);
						DirectoryWriter.Write(CommentLen);
						DirectoryWriter.Write(DiskStart);
						DirectoryWriter.Write(InternalAttr);
						DirectoryWriter.Write(ExternalAttr);
						DirectoryWriter.Write(Location);
						DirectoryWriter.BaseStream.Write(Filename, 0, FilenameLen);
						DirectoryWriter.BaseStream.Write(Extra, 0, ExtraLen);
						// write comment here if there was one
					}
					else
					{
						// replace with the new .SO
						try
						{
							FileStream SourceFile = File.OpenRead(SourceSOFile.FullName);
							uint SourceCompressedSize = CompressedSize;
							UncompressedSize = (uint)SourceFile.Length;
							CompressedSize = 0;
							Compression = 8;        // deflate

							// calculate CRC32
							CRC32Value = 0x0;
							CRC32Value = ~CRC32Value;
							Remaining = UncompressedSize;
							while (Remaining > 0)
							{
								int CopySize = Remaining < 65536 ? (int)Remaining : 65536;
								int BytesRead = SourceFile.Read(CopyBuffer, 0, CopySize);
								Remaining -= (uint)BytesRead;
								for (uint Index = 0; Index < BytesRead; Index++)
								{
									CRC32Value = (CRC32Value >> 8) ^ CRCTablesSB8[(CRC32Value ^ (uint)CopyBuffer[Index]) & 0xff];
								}
							}
							CRC32Value = ~CRC32Value;
							SourceFile.Position = 0;

							TotalEntries++;
							uint Location = (uint)DestWriter.BaseStream.Position;

							using (MemoryStream CompressedStream = new MemoryStream())
							{
								using (DeflateStream Compressor = new DeflateStream(CompressedStream, CompressionMode.Compress, leaveOpen: true))
								{
									SourceFile.CopyTo(Compressor);
									SourceFile.Close();
								}

								byte[] CompressedBuffer = CompressedStream.GetBuffer();
								CompressedSize = (uint)CompressedStream.Length;

								// write local header
								DestWriter.Write(Header);
								DestWriter.Write(Version);
								DestWriter.Write(Flags);
								DestWriter.Write(Compression);
								DestWriter.Write(ModTime);
								DestWriter.Write(ModDate);
								DestWriter.Write(CRC32Value);
								DestWriter.Write(CompressedSize);
								DestWriter.Write(UncompressedSize);
								DestWriter.Write(FilenameLen);
								DestWriter.Write(ExtraLen);
								DestWriter.BaseStream.Write(Filename, 0, FilenameLen);
								DestWriter.BaseStream.Write(Extra, 0, ExtraLen);

								// write compressed data
								DestWriter.BaseStream.Write(CompressedBuffer, 0, (int)CompressedSize);

								// advance the base stream forward by the source file size so we land on the next header
								SourceReader.BaseStream.Position += SourceCompressedSize;
							}

							// write central directory entry for this file
							DirectoryWriter.Write(Signature);
							DirectoryWriter.Write(DirVersion);
							DirectoryWriter.Write(Version);
							DirectoryWriter.Write(Flags);
							DirectoryWriter.Write(Compression);
							DirectoryWriter.Write(ModTime);
							DirectoryWriter.Write(ModDate);
							DirectoryWriter.Write(CRC32Value);
							DirectoryWriter.Write(CompressedSize);
							DirectoryWriter.Write(UncompressedSize);
							DirectoryWriter.Write(FilenameLen);
							DirectoryWriter.Write(ExtraLen);
							DirectoryWriter.Write(CommentLen);
							DirectoryWriter.Write(DiskStart);
							DirectoryWriter.Write(InternalAttr);
							DirectoryWriter.Write(ExternalAttr);
							DirectoryWriter.Write(Location);
							DirectoryWriter.BaseStream.Write(Filename, 0, FilenameLen);
							DirectoryWriter.BaseStream.Write(Extra, 0, ExtraLen);
							// write comment here if there was one
						}
						catch (IOException e)
						{
							Logger.LogInformation("Failed to add {SourceSOFile} to APK: Reason = {Reason}", SourceSOFile, e.ToString());
							DirectoryWriter.Close();
							File.Delete(DirectoryFilename);
							DestWriter.Close();
							FileReference.Delete(TargetApk);
							throw;
						}
					}
				}

				uint CentralDirectoryLocation = (uint)DestWriter.BaseStream.Position;

				// copy the directory to the end of the APK
				uint DirectorySize = (uint)DirectoryWriter.BaseStream.Position;
				DirectoryWriter.BaseStream.Position = 0;
				Remaining = DirectorySize;
				while (Remaining > 0)
				{
					int CopySize = Remaining < 65536 ? (int)Remaining : 65536;
					int BytesRead = DirectoryWriter.BaseStream.Read(CopyBuffer, 0, CopySize);
					DestWriter.BaseStream.Write(CopyBuffer, 0, BytesRead);
					Remaining -= (uint)BytesRead;
				}
				DirectoryWriter.Close();
				File.Delete(DirectoryFilename);

				// write end of central directory record
				Signature = 0x06054b50;
				DestWriter.Write(Signature);
				DestWriter.Write(DiskNumber);
				DestWriter.Write(DiskStart);
				DestWriter.Write(TotalEntries);
				DestWriter.Write(TotalEntries);
				DestWriter.Write(DirectorySize);
				DestWriter.Write(CentralDirectoryLocation);
				DestWriter.Write(CommentLen);
				DestWriter.Close();
			}
		}

		private void CreateGradlePropertiesFiles(AndroidToolChain ToolChain, int MinSDKVersion, int TargetSDKVersion, string CompileSDKVersion, string BuildToolsVersion, string PackageName,
			string? DestApkName, string NDKArch, string UnrealBuildFilesPath, string GameBuildFilesPath, string UnrealBuildGradleAppPath, string UnrealBuildPath, string UnrealBuildGradlePath,
			bool bForDistribution, bool bIsEmbedded, List<string> OBBFiles, bool bJavaPerfettoMarkers)
		{
			// Create gradle.properties
			StringBuilder GradleProperties = new StringBuilder();

			int StoreVersion = GetStoreVersion(GetUnrealArch(NDKArch));
			string VersionDisplayName = GetVersionDisplayName(bIsEmbedded);

			ConfigHierarchy Ini = GetConfigCacheIni(ConfigHierarchyType.Engine);

			Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bEnableUniversalAPK", out bool bEnableUniversalAPK);

			GradleProperties.AppendLine("org.gradle.daemon=false");
			GradleProperties.AppendLine("org.gradle.jvmargs=-XX:MaxHeapSize=4096m -Xmx9216m");
			GradleProperties.AppendLine("systemProp.org.gradle.internal.http.connectionTimeout=15000");
			GradleProperties.AppendLine("systemProp.org.gradle.internal.http.socketTimeout=30000");
			GradleProperties.AppendLine("android.injected.testOnly=false");
			GradleProperties.AppendLine("android.useAndroidX=true");
			GradleProperties.AppendLine("android.enableJetifier=true");
			GradleProperties.AppendLine(String.Format("COMPILE_SDK_VERSION={0}", CompileSDKVersion));
			GradleProperties.AppendLine(String.Format("BUILD_TOOLS_VERSION={0}", BuildToolsVersion));
			GradleProperties.AppendLine(String.Format("PACKAGE_NAME={0}", PackageName));
			GradleProperties.AppendLine(String.Format("MIN_SDK_VERSION={0}", MinSDKVersion.ToString()));
			GradleProperties.AppendLine(String.Format("TARGET_SDK_VERSION={0}", TargetSDKVersion.ToString()));
			GradleProperties.AppendLine(String.Format("STORE_VERSION={0}", StoreVersion.ToString()));
			GradleProperties.AppendLine(String.Format("VERSION_DISPLAY_NAME={0}", VersionDisplayName));

			string NDKPath = Environment.GetEnvironmentVariable("NDKROOT")!.Replace("\\", "/", StringComparison.Ordinal);
			int NDKVersionIndex = NDKPath.LastIndexOf('/');
			string NDKVersion = NDKVersionIndex > 0 ? NDKPath.Substring(NDKVersionIndex + 1) : DEFAULT_NDK_VERSION;
			GradleProperties.AppendLine(String.Format("NDK_VERSION={0}", NDKVersion));
			GradleProperties.AppendLine(String.Format("NDK_PATH={0}", NDKPath));

			if (DestApkName != null)
			{
				GradleProperties.AppendLine(String.Format("OUTPUT_PATH={0}", Path.GetDirectoryName(DestApkName)!.Replace("\\", "/", StringComparison.Ordinal)));
				GradleProperties.AppendLine(String.Format("OUTPUT_FILENAME={0}", Path.GetFileName(DestApkName)));

				string BundleFilename = Path.GetFileName(DestApkName).Replace(".apk", ".aab", StringComparison.Ordinal);
				GradleProperties.AppendLine(String.Format("OUTPUT_BUNDLEFILENAME={0}", BundleFilename));

				if (bEnableUniversalAPK)
				{
					string UniversalAPKFilename = Path.GetFileName(DestApkName).Replace(".apk", "_universal.apk", StringComparison.Ordinal);
					GradleProperties.AppendLine("OUTPUT_UNIVERSALFILENAME=" + UniversalAPKFilename);
				}
			}

			int OBBFileIndex = 0;
			GradleProperties.AppendLine(String.Format("OBB_FILECOUNT={0}", OBBFiles.Count));
			foreach (string OBBFile in OBBFiles)
			{
				GradleProperties.AppendLine(String.Format("OBB_FILE{0}={1}", OBBFileIndex++, OBBFile.Replace("\\", "/", StringComparison.Ordinal)));
			}

			GradleProperties.AppendLine("ANDROID_TOOLS_BUILD_GRADLE_VERSION={0}", ANDROID_TOOLS_BUILD_GRADLE_VERSION);
			GradleProperties.AppendLine("BUNDLETOOL_JAR=" + Path.GetFullPath(Path.Combine(UnrealBuildFilesPath, "..", "Prebuilt", "bundletool", BUNDLETOOL_JAR)).Replace("\\", "/", StringComparison.Ordinal));
			GradleProperties.AppendLine("GENUNIVERSALAPK_JAR=" + Path.GetFullPath(Path.Combine(UnrealBuildFilesPath, "..", "Prebuilt", "GenUniversalAPK", "bin", "GenUniversalAPK.jar")).Replace("\\", "/", StringComparison.Ordinal));

			// add any Gradle properties from UPL
			string GradlePropertiesUPL = UPL!.ProcessPluginNode(NDKArch, "gradleProperties", "");
			GradleProperties.AppendLine(GradlePropertiesUPL);

			// Create abi.gradle
			StringBuilder ABIGradle = new StringBuilder();
			ABIGradle.AppendLine("android {");
			ABIGradle.AppendLine("\tdefaultConfig {");
			ABIGradle.AppendLine("\t\tndk {");
			ABIGradle.AppendLine(String.Format("\t\t\tabiFilter \"{0}\"", NDKArch));
			ABIGradle.AppendLine("\t\t}");
			ABIGradle.AppendLine("\t}");
			ABIGradle.AppendLine("}");
			string ABIGradleFilename = Path.Combine(UnrealBuildGradleAppPath, "abi.gradle");
			File.WriteAllText(ABIGradleFilename, ABIGradle.ToString());

			StringBuilder GradleBuildAdditionsContent = new StringBuilder();
			GradleBuildAdditionsContent.AppendLine("apply from: 'aar-imports.gradle'");
			GradleBuildAdditionsContent.AppendLine("apply from: 'projects.gradle'");
			GradleBuildAdditionsContent.AppendLine("apply from: 'abi.gradle'");

			bool bEnableBundle, bBundleABISplit, bBundleLanguageSplit, bBundleDensitySplit;
			Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bEnableBundle", out bEnableBundle);
			Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bBundleABISplit", out bBundleABISplit);
			Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bBundleLanguageSplit", out bBundleLanguageSplit);
			Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bBundleDensitySplit", out bBundleDensitySplit);

			GradleBuildAdditionsContent.AppendLine("android {");

			Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bExtractNativeLibs", out bool bExtractNativeLibs);
			AndroidToolChain.ClangSanitizer Sanitizer = ToolChain.BuildWithSanitizer();
			// hwasan on NDK r26b+ requires wrap.sh that needs to be unpacked
			if ((Sanitizer != AndroidToolChain.ClangSanitizer.None && (Sanitizer != AndroidToolChain.ClangSanitizer.HwAddress || AndroidToolChain.HasEmbeddedHWASanSupport())) || bEnableScudoMemoryTracing)
			{
				bExtractNativeLibs = true;
			}
			if (bExtractNativeLibs)
			{
				GradleBuildAdditionsContent.AppendLine("\tpackagingOptions {");
				GradleBuildAdditionsContent.AppendLine("\t\tjniLibs {");
				GradleBuildAdditionsContent.AppendLine("\t\t\tuseLegacyPackaging=true");
				GradleBuildAdditionsContent.AppendLine("\t\t}");
				GradleBuildAdditionsContent.AppendLine("\t}");
			}

			if (!ForceAPKGeneration && bEnableBundle)
			{
				GradleBuildAdditionsContent.AppendLine("\tbundle {");
				GradleBuildAdditionsContent.AppendLine("\t\tabi { enableSplit = " + (bBundleABISplit ? "true" : "false") + " }");
				GradleBuildAdditionsContent.AppendLine("\t\tlanguage { enableSplit = " + (bBundleLanguageSplit ? "true" : "false") + " }");
				GradleBuildAdditionsContent.AppendLine("\t\tdensity { enableSplit = " + (bBundleDensitySplit ? "true" : "false") + " }");
				GradleBuildAdditionsContent.AppendLine("\t}");
			}

			string DebugKeyAlias, DebugKeyStore, DebugKeyStorePassword, DebugKeyPassword;
			Ini.GetString("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "DebugKeyStore", out DebugKeyStore);
			Ini.GetString("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "DebugKeyAlias", out DebugKeyAlias);
			Ini.GetString("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "DebugKeyStorePassword", out DebugKeyStorePassword);
			Ini.GetString("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "DebugKeyPassword", out DebugKeyPassword);

			bool bHasCustomDebugKeystore = false;
			if (!String.IsNullOrEmpty(DebugKeyStore) && !String.IsNullOrEmpty(DebugKeyAlias) && !String.IsNullOrEmpty(DebugKeyStorePassword))
			{
				if (String.IsNullOrEmpty(DebugKeyPassword) || DebugKeyPassword == "_sameaskeystore_")
				{
					DebugKeyPassword = DebugKeyStorePassword;
				}

				string DebugKeyStoreFilename = Path.Combine(UnrealBuildPath, DebugKeyStore);
				if (!File.Exists(DebugKeyStoreFilename))
				{
					Logger?.LogWarning("Debug keystore file is missing. Check the DistributionSettings section in the Android tab of Project Settings");
				}
				else
				{
					bHasCustomDebugKeystore = true;

					GradleProperties.AppendLine(String.Format("DEBUG_STORE_FILE={0}", DebugKeyStoreFilename.Replace("\\", "/", StringComparison.Ordinal)));
					GradleProperties.AppendLine(String.Format("DEBUG_STORE_PASSWORD={0}", DebugKeyStorePassword));
					GradleProperties.AppendLine(String.Format("DEBUG_KEY_ALIAS={0}", DebugKeyAlias));
					GradleProperties.AppendLine(String.Format("DEBUG_KEY_PASSWORD={0}", DebugKeyPassword));

					GradleBuildAdditionsContent.AppendLine("\tsigningConfigs {");
					GradleBuildAdditionsContent.AppendLine("\t\tdebug {");
					GradleBuildAdditionsContent.AppendLine(String.Format("\t\t\tstoreFile file('{0}')", DebugKeyStoreFilename.Replace("\\", "/", StringComparison.Ordinal)));
					GradleBuildAdditionsContent.AppendLine(String.Format("\t\t\tstorePassword '{0}'", DebugKeyStorePassword));
					GradleBuildAdditionsContent.AppendLine(String.Format("\t\t\tkeyAlias '{0}'", DebugKeyAlias));
					GradleBuildAdditionsContent.AppendLine(String.Format("\t\t\tkeyPassword '{0}'", DebugKeyPassword));
					GradleBuildAdditionsContent.AppendLine("\t\t}");
					GradleBuildAdditionsContent.AppendLine("\t}");
				}
			}

			if (bForDistribution)
			{
				string KeyAlias, KeyStore, KeyStorePassword, KeyPassword;
				Ini.GetString("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "KeyStore", out KeyStore);
				Ini.GetString("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "KeyAlias", out KeyAlias);
				Ini.GetString("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "KeyStorePassword", out KeyStorePassword);
				Ini.GetString("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "KeyPassword", out KeyPassword);

				if (String.IsNullOrEmpty(KeyStore) || String.IsNullOrEmpty(KeyAlias) || String.IsNullOrEmpty(KeyStorePassword))
				{
					throw new BuildException("DistributionSigning settings are not all set. Check the DistributionSettings section in the Android tab of Project Settings");
				}

				if (String.IsNullOrEmpty(KeyPassword) || KeyPassword == "_sameaskeystore_")
				{
					KeyPassword = KeyStorePassword;
				}

				// Make sure the keystore file exists
				string KeyStoreFilename = Path.Combine(UnrealBuildPath, KeyStore);
				if (!File.Exists(KeyStoreFilename))
				{
					throw new BuildException("Keystore file is missing. Check the DistributionSettings section in the Android tab of Project Settings");
				}

				GradleProperties.AppendLine(String.Format("STORE_FILE={0}", KeyStoreFilename.Replace("\\", "/", StringComparison.Ordinal)));
				GradleProperties.AppendLine(String.Format("STORE_PASSWORD={0}", KeyStorePassword));
				GradleProperties.AppendLine(String.Format("KEY_ALIAS={0}", KeyAlias));
				GradleProperties.AppendLine(String.Format("KEY_PASSWORD={0}", KeyPassword));

				GradleBuildAdditionsContent.AppendLine("\tsigningConfigs {");
				GradleBuildAdditionsContent.AppendLine("\t\trelease {");
				GradleBuildAdditionsContent.AppendLine(String.Format("\t\t\tstoreFile file('{0}')", KeyStoreFilename.Replace("\\", "/", StringComparison.Ordinal)));
				GradleBuildAdditionsContent.AppendLine(String.Format("\t\t\tstorePassword '{0}'", KeyStorePassword));
				GradleBuildAdditionsContent.AppendLine(String.Format("\t\t\tkeyAlias '{0}'", KeyAlias));
				GradleBuildAdditionsContent.AppendLine(String.Format("\t\t\tkeyPassword '{0}'", KeyPassword));
				GradleBuildAdditionsContent.AppendLine("\t\t}");
				GradleBuildAdditionsContent.AppendLine("\t}");

				// Generate the Proguard file contents and write it
				string ProguardContents = GenerateProguard(NDKArch, UnrealBuildFilesPath, GameBuildFilesPath);
				string ProguardFilename = Path.Combine(UnrealBuildGradleAppPath, "proguard-rules.pro");
				SafeDeleteFile(ProguardFilename);
				File.WriteAllText(ProguardFilename, ProguardContents);
			}
			else
			{
				// empty just for Gradle not to complain
				GradleProperties.AppendLine("STORE_FILE=");
				GradleProperties.AppendLine("STORE_PASSWORD=");
				GradleProperties.AppendLine("KEY_ALIAS=");
				GradleProperties.AppendLine("KEY_PASSWORD=");

				// empty just for Gradle not to complain
				GradleBuildAdditionsContent.AppendLine("\tsigningConfigs {");
				GradleBuildAdditionsContent.AppendLine("\t\trelease {");
				GradleBuildAdditionsContent.AppendLine("\t\t}");
				GradleBuildAdditionsContent.AppendLine("\t}");
			}

			GradleBuildAdditionsContent.AppendLine("\tbuildTypes {");
			GradleBuildAdditionsContent.AppendLine("\t\trelease {");
			GradleBuildAdditionsContent.AppendLine("\t\t\tsigningConfig signingConfigs.release");
			if (GradlePropertiesUPL.Contains("DISABLE_MINIFY=1", StringComparison.Ordinal))
			{
				GradleBuildAdditionsContent.AppendLine("\t\t\tminifyEnabled false");
			}
			else
			{
				GradleBuildAdditionsContent.AppendLine("\t\t\tminifyEnabled true");
			}
			if (GradlePropertiesUPL.Contains("DISABLE_PROGUARD=1", StringComparison.Ordinal))
			{
				GradleBuildAdditionsContent.AppendLine("\t\t\tuseProguard false");
			}
			else
			{
				GradleBuildAdditionsContent.AppendLine("\t\t\tproguardFiles getDefaultProguardFile('proguard-android.txt'), 'proguard-rules.pro'");
			}
			GradleBuildAdditionsContent.AppendLine("\t\t}");
			GradleBuildAdditionsContent.AppendLine("\t\tdebug {");
			GradleBuildAdditionsContent.AppendLine("\t\t\tdebuggable true");
			if (bHasCustomDebugKeystore)
			{
				GradleBuildAdditionsContent.AppendLine("\t\t\tsigningConfig signingConfigs.debug");
			}
			GradleBuildAdditionsContent.AppendLine("\t\t}");
			GradleBuildAdditionsContent.AppendLine("\t}");
			GradleBuildAdditionsContent.AppendLine("}");

			string BuildGradleAdditionsDefault = "";
			if (bJavaPerfettoMarkers)
			{
				BuildGradleAdditionsDefault = @"
dependencies.implementation('androidx.tracing:tracing:1.2.0')
";
			}

			// Add any UPL app buildGradleAdditions
			GradleBuildAdditionsContent.Append(UPL.ProcessPluginNode(NDKArch, "buildGradleAdditions", BuildGradleAdditionsDefault));

			string GradleBuildAdditionsFilename = Path.Combine(UnrealBuildGradleAppPath, "buildAdditions.gradle");
			File.WriteAllText(GradleBuildAdditionsFilename, GradleBuildAdditionsContent.ToString());

			string GradlePropertiesFilename = Path.Combine(UnrealBuildGradlePath, "gradle.properties");
			File.WriteAllText(GradlePropertiesFilename, GradleProperties.ToString());

			// Add lint if requested (note depreciation warnings can be suppressed with @SuppressWarnings("deprecation")
			string GradleBaseBuildAdditionsContents = "";
			Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bEnableLint", out bool bEnableLint);
			if (bEnableLint)
			{
				GradleBaseBuildAdditionsContents =
					"allprojects {\n" +
					"\ttasks.withType(JavaCompile) {\n" +
					"\t\toptions.compilerArgs << \"-Xlint:unchecked\" << \"-Xlint:deprecation\"\n" +
					"\t}\n" +
					"}\n\n";
			}

			// Create baseBuildAdditions.gradle from plugins baseBuildGradleAdditions
			string GradleBaseBuildAdditionsFilename = Path.Combine(UnrealBuildGradlePath, "baseBuildAdditions.gradle");
			File.WriteAllText(GradleBaseBuildAdditionsFilename, UPL.ProcessPluginNode(NDKArch, "baseBuildGradleAdditions", GradleBaseBuildAdditionsContents));

			// Create buildscriptAdditions.gradle from plugins buildscriptGradleAdditions
			string GradleBuildScriptAdditionsFilename = Path.Combine(UnrealBuildGradlePath, "buildscriptAdditions.gradle");
			File.WriteAllText(GradleBuildScriptAdditionsFilename, UPL.ProcessPluginNode(NDKArch, "buildscriptGradleAdditions", ""));
		}

		public bool GetDontBundleLibrariesInAPK(UnrealTargetConfiguration Configuration, bool bIsArchive, bool bIsFromUAT)
		{
			if (ForceDontBundleLibrariesInAPK.HasValue)
			{
				Logger?.LogInformation("bDontBundleLibrariesInAPK is force set to {ForceDontBundleLibrariesInAPK} via command line option.", ForceDontBundleLibrariesInAPK.Value);
				return ForceDontBundleLibrariesInAPK.Value;
			}

			if (bFastIterate)
			{
				return true;
			}

			// Deprecated in 5.7, remove everything below in 5.9 and just return bFastIterate
			// this feature requires extra step to install the app besides just .apk install
			// limit where the feature is enabled because we're not in control of deployment step but in VS/AGDE and UAT
			if (!(bFromMSBuild || bIsFromUAT))
			{
				Logger?.LogInformation("bDontBundleLibrariesInAPK is set to false, due to not called from MSBuild or UAT (bFromMSBuild={FromMSBuild}, bIsFromUAT={IsFromUAT}), use \"-ForceDontBundleLibrariesInAPK=true\" to override.", bFromMSBuild, bIsFromUAT);
				return false;
			}

			if (Configuration == UnrealTargetConfiguration.Shipping)
			{
				Logger?.LogInformation("bDontBundleLibrariesInAPK is set to false, due to a shipping build.");
				return false;
			}

			if (bIsArchive)
			{
				Logger?.LogInformation("bDontBundleLibrariesInAPK is set to false, due to an archive build.");
				return false;
			}

			ConfigHierarchy Ini = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, DirectoryReference.FromFile(ProjectFile), UnrealTargetPlatform.Android);
			Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bDontBundleLibrariesInAPK", out bool bDontBundleLibrariesInAPK);
			Logger?.LogInformation("bDontBundleLibrariesInAPK is set to {DontBundleLibrariesInAPK}, based on AndroidRuntimeSettings.", bDontBundleLibrariesInAPK);
			return bDontBundleLibrariesInAPK;
		}

		private static bool GetJavaPerfettoMarkers(ConfigHierarchy Ini, UnrealTargetConfiguration Configuration, bool bDistribution)
		{
			// Ship for distribution should not include markers
			if (bDistribution && Configuration == UnrealTargetConfiguration.Shipping)
			{
				return false;
			}

			Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bJavaPerfettoMarkers", out bool bJavaPerfettoMarkers);
			return bJavaPerfettoMarkers;
		}

		// Architecture remapping
		private static readonly Dictionary<string, string> ArchRemapping = new()
		{
			{"arm64-v8a", "arm64"},
			{"x86_64", "x64"}
		};

		private void MakeApk(AndroidToolChain ToolChain, string ProjectName, TargetType InTargetType, string ProjectDirectory, string OutputPath, string EngineDirectory, bool bForDistribution, string CookFlavor,
			UnrealTargetConfiguration Configuration, bool bMakeSeparateApks, bool bDisallowPackagingDataInApk, bool bDisallowExternalFilesDir, bool bSkipGradleBuild, bool bIsArchive, bool bIsFromUAT)
		{
			Logger.LogInformation("");
			Logger.LogInformation("===={Time}====PREPARING TO MAKE APK=================================================================", DateTime.Now.ToString());

			if (Architectures == null)
			{
				throw new BuildException("Called MakeApk without first calling SetAndroidPluginData");
			}

			// we do not need to really build an engine UnrealGame.apk so short-circuit it
			if (!bFastIterate &&
				!ForceAPKGeneration &&
				ProjectName == "UnrealGame" &&
				OutputPath.Replace("\\", "/", StringComparison.Ordinal).Contains("/Engine/Binaries/Android/", StringComparison.Ordinal) &&
				Path.GetFileNameWithoutExtension(OutputPath).StartsWith("UnrealGame", StringComparison.Ordinal))
			{
				if (!bSkipGradleBuild)
				{
					/*
					IEnumerable<Tuple<string, string>> TargetList = null;

					TargetList = from Arch in Arches
								from GPUArch in GPUArchitectures
								select Tuple.Create(Arch, GPUArch);

					string DestApkDirectory = Path.Combine(ProjectDirectory, "Binaries/Android");
					string ApkFilename = Path.GetFileNameWithoutExtension(OutputPath).Replace("UnrealGame", ProjectName);

					foreach (Tuple<string, string> target in TargetList)
					{
						string Arch = target.Item1;
						string GPUArchitecture = target.Item2;
						string DestApkName = Path.Combine(DestApkDirectory, ApkFilename + ".apk");
						DestApkName = AndroidToolChain.InlineArchName(DestApkName, Arch, GPUArchitecture);

						// create a dummy APK if doesn't exist
						if (!File.Exists(DestApkName))
						{
							File.WriteAllText(DestApkName, "dummyfile");
						}
					}
					*/
				}
				Logger.LogInformation("APK generation not needed for project {ProjectName} with {OutputPath}", ProjectName, OutputPath);
				Logger.LogInformation("");
				Logger.LogInformation("===={Time}====COMPLETED MAKE APK=======================================================================", DateTime.Now.ToString());
				return;
			}

			if (UPL!.GetLastError() != null)
			{
				throw new BuildException("Cannot make APK with UPL errors");
			}

			// make sure it is cached (clear unused warning)
			string EngineVersion = ReadEngineVersion();
			if (EngineVersion == null)
			{
				throw new BuildException("No engine version!");
			}

			SetMinimumSDKLevelForGradle();

			// Verify license agreement since we require Gradle
			if (!IsLicenseAgreementValid())
			{
				throw new BuildException("Android SDK license file not found.  Please agree to license in Android project settings in the editor.");
			}

			LogBuildSetup();

			ConfigHierarchy Ini = GetConfigCacheIni(ConfigHierarchyType.Engine);

			// bundles disabled for launch-on
			bool bEnableBundle = BundleEnabled() && !bDisallowPackagingDataInApk;

			bool bIsBuildMachine = Unreal.IsBuildMachine();

			bool bJavaPerfettoMarkers = GetJavaPerfettoMarkers(Ini, Configuration, bForDistribution);

			// do this here so we'll stop early if there is a problem with the SDK API level (cached so later calls will return the same)
			string SDKAPILevel = GetSdkApiLevel(ToolChain);
			string BuildToolsVersion = GetBuildToolsVersion(ToolChain);

			// cache some tools paths
			//string NDKBuildPath = Environment.ExpandEnvironmentVariables("%NDKROOT%/ndk-build" + (OperatingSystem.IsWindows() ? ".cmd" : ""));
			//bool HasNDKPath = File.Exists(NDKBuildPath);

			// set up some directory info
			string IntermediateAndroidPath = Path.Combine(ProjectDirectory, "Intermediate", "Android");
			string UnrealJavaFilePath = Path.Combine(ProjectDirectory, "Build", "Android", GetUnrealJavaSrcPath());
			string UnrealBuildFilesPath = GetUnrealBuildFilePath(EngineDirectory);
			string UnrealPreBuiltFilesPath = GetUnrealPreBuiltFilePath(EngineDirectory);
			string UnrealBuildFilesPath_NFL = GetUnrealBuildFilePath(Path.Combine(EngineDirectory, "Restricted", "NotForLicensees"));
			string UnrealBuildFilesPath_NR = GetUnrealBuildFilePath(Path.Combine(EngineDirectory, "Restricted", "NoRedist"));
			string UnrealBuildFilesPath_LA = GetUnrealBuildFilePath(Path.Combine(EngineDirectory, "Restricted", "LimitedAccess"));
			string GameBuildFilesPath = Path.Combine(ProjectDirectory, "Build", "Android");
			string GameBuildFilesPath_NFL = Path.Combine(Path.Combine(ProjectDirectory, "Restricted", "NotForLicensees"), "Build", "Android");
			string GameBuildFilesPath_NR = Path.Combine(Path.Combine(ProjectDirectory, "Restricted", "NoRedist"), "Build", "Android");
			string GameBuildFilesPath_LA = Path.Combine(Path.Combine(ProjectDirectory, "Restricted", "LimitedAccess"), "Build", "Android");

			// get a list of unique architectures enabled for build
			List<(UnrealArch UEArch, string NativeArch)> Archs = Architectures.Architectures.Select(x => (x, GetNDKArch(x))).ToList();

			// force create from scratch if on build machine
			bool bCreateFromScratch = bIsBuildMachine;

			AndroidToolChain.ClangSanitizer Sanitizer = ToolChain.BuildWithSanitizer();

			// see if last time matches the skipGradle setting
			string BuildTypeFilename = Path.Combine(IntermediateAndroidPath, "BuildType.txt");
			string BuildTypeID = bSkipGradleBuild ? "Embedded" : "Standalone";

			// hwasan on NDK r26b+ requires wrap.sh that needs to be unpacked
			if (Sanitizer != AndroidToolChain.ClangSanitizer.None && (Sanitizer != AndroidToolChain.ClangSanitizer.HwAddress || AndroidToolChain.HasEmbeddedHWASanSupport()))
			{
				BuildTypeID += Sanitizer.ToString() + "Sanitizer";
			}
			else if (bEnableScudoMemoryTracing)
			{
				BuildTypeID += Sanitizer.ToString() + "ScudoMemoryTrace";
			}
			if (File.Exists(BuildTypeFilename))
			{
				string BuildTypeContents = File.ReadAllText(BuildTypeFilename);
				if (BuildTypeID != BuildTypeContents)
				{
					Logger.LogInformation("Build type changed, forcing clean");
					bCreateFromScratch = true;
				}
			}

			// force cleanup if older UE4 project
			if (File.Exists(Path.Combine(IntermediateAndroidPath, "arm64", "jni", "arm64-v8a", "libUE4.so")) ||
				File.Exists(Path.Combine(IntermediateAndroidPath, "x64", "jni", "x86_64", "libUE4.so")))
			{
				Logger.LogInformation("Old version of library .so found, forcing clean");
				bCreateFromScratch = true;
			}

			// check if the enabled plugins has changed
			string PluginListFilename = Path.Combine(IntermediateAndroidPath, "ActiveUPL.txt");
			string PluginListContents = ActiveUPLFiles.ToString();
			if (File.Exists(PluginListFilename))
			{
				string PreviousPluginListContents = File.ReadAllText(PluginListFilename);
				if (PluginListContents != PreviousPluginListContents)
				{
					Logger.LogInformation("Active UPL files changed, forcing clean");
					bCreateFromScratch = true;
				}
			}

			// Check if bEnableBundle changed
			string BuildSettingsCacheFile = Path.Combine(IntermediateAndroidPath, "UEBuildSettings.txt");
			if (!bCreateFromScratch)
			{
				if (File.Exists(BuildSettingsCacheFile))
				{
					const string Key = "bEnableBundle=";
					string? Line = File.ReadAllLines(BuildSettingsCacheFile).FirstOrDefault(Line => Line.StartsWith(Key, StringComparison.Ordinal));
					if (Line != null && !Line[Key.Length..].Equals(bEnableBundle ? "True" : "False", StringComparison.Ordinal))
					{
						bCreateFromScratch = true;
					}
				}
			}

			if (bCreateFromScratch)
			{
				Logger.LogInformation("Cleaning {IntermediateAndroidPath}", IntermediateAndroidPath);
				DeleteDirectory(IntermediateAndroidPath, Logger);
				Directory.CreateDirectory(IntermediateAndroidPath);
			}

			if (!System.IO.Directory.Exists(IntermediateAndroidPath))
			{
				System.IO.Directory.CreateDirectory(IntermediateAndroidPath);
			}

			// write enabled plugins list
			File.WriteAllText(PluginListFilename, PluginListContents);

			// write build type
			File.WriteAllText(BuildTypeFilename, BuildTypeID);

			// cache if we want data in the Apk
			bool bPackageDataInsideApk = !bDisallowPackagingDataInApk && GetPackageDataInsideApk();
			bool bDisableVerifyOBBOnStartUp = bNoObbs || DisableVerifyOBBOnStartUp();
			bool bUseExternalFilesDir = UseExternalFilesDir(bDisallowExternalFilesDir);

			// Generate Java files
			string PackageName = GetPackageName(ProjectName);
			string TemplateDestinationBase = Path.Combine(ProjectDirectory, "Build", "Android", "src", PackageName.Replace('.', Path.DirectorySeparatorChar));
			MakeDirectoryIfRequired(TemplateDestinationBase);

			// We'll be writing the OBB data into the same location as the download service files
			string UnrealOBBDataFileName = GetUnrealJavaOBBDataFileName(TemplateDestinationBase);
			string UnrealDownloadShimFileName = GetUnrealJavaDownloadShimFileName(UnrealJavaFilePath);

			// Template generated files
			string JavaTemplateSourceDir = GetUnrealTemplateJavaSourceDir(EngineDirectory);
			// LINQ query is subsequently used multiple times, so must copy results
			TemplateFile[] templates = (from template in Directory.EnumerateFiles(JavaTemplateSourceDir, "*.template")
												  let RealName = Path.GetFileNameWithoutExtension(template)
												  select new TemplateFile(SourceFile: template, DestinationFile: GetUnrealTemplateJavaDestination(TemplateDestinationBase, RealName))).ToArray();

			// Generate the OBB and Shim files here
			string ObbFileLocation = ProjectDirectory + "/Saved/StagedBuilds/Android" + CookFlavor + ".obb";
			string PatchFileLocation = ProjectDirectory + "/Saved/StagedBuilds/Android" + CookFlavor + ".patch.obb";
			bool bRequiresOBB = RequiresOBB(bDisallowPackagingDataInApk, ObbFileLocation);
			List<string> RequiredOBBFiles = bNoObbs ? [] : [ObbFileLocation];
			if (!bNoObbs && File.Exists(PatchFileLocation))
			{
				RequiredOBBFiles.Add(PatchFileLocation);
			}

			// Generate the OBBData.java file if out of date (can skip rewriting it if packaging inside Apk in some cases)
			// Note: this may be replaced per architecture later if store version is different
			WriteJavaOBBDataFile(UnrealOBBDataFileName, PackageName, RequiredOBBFiles, CookFlavor, !bNoObbs && bPackageDataInsideApk, Architectures.Architectures[0]);

			// Make sure any existing proguard file in project is NOT used (back it up)
			string ProjectBuildProguardFile = Path.Combine(GameBuildFilesPath, "proguard-project.txt");
			if (File.Exists(ProjectBuildProguardFile))
			{
				string ProjectBackupProguardFile = Path.Combine(GameBuildFilesPath, "proguard-project.backup");
				File.Move(ProjectBuildProguardFile, ProjectBackupProguardFile);
			}

			WriteJavaDownloadSupportFiles(UnrealDownloadShimFileName, templates, new Dictionary<string, string>{
				{ "$$GameName$$", ProjectName },
				{ "$$PublicKey$$", GetPublicKey() },
				{ "$$PackageName$$",PackageName }
			});

			// Sometimes old files get left behind if things change, so we'll do a clean up pass
			foreach ((UnrealArch UnrealArch, string _) in Archs)
			{
				string UnrealBuildPath = Path.Combine(IntermediateAndroidPath, UnrealArch.ToString());

				string CleanUpBaseDir = Path.Combine(ProjectDirectory, "Build", "Android", "src");
				string ImmediateBaseDir = Path.Combine(UnrealBuildPath, "src");
				IEnumerable<string> files = Directory.EnumerateFiles(CleanUpBaseDir, "*.java", SearchOption.AllDirectories);

				Logger.LogInformation("Cleaning up files based on template dir {TemplateDestinationBase}", TemplateDestinationBase);

				// Make a set of files that are okay to clean up
				HashSet<string> cleanFiles = new HashSet<string>();
				cleanFiles.Add("DownloadShim.java");
				cleanFiles.Add("OBBData.java");
				foreach (TemplateFile template in templates)
				{
					cleanFiles.Add(Path.GetFileName(template.DestinationFile));
				}

				foreach (string filename in files)
				{
					// keep the shim if it is in the right place
					if (filename == UnrealDownloadShimFileName)
					{
						continue;
					}

					string filePath = Path.GetDirectoryName(filename)!;  // grab the file's path
					if (filePath != TemplateDestinationBase)             // and check to make sure it isn't the same as the Template directory we calculated earlier
					{
						// Only delete the files in the cleanup set
						if (!cleanFiles.Contains(Path.GetFileName(filename)))
						{
							continue;
						}

						Logger.LogInformation("Cleaning up file {File}", filename);
						SafeDeleteFile(filename, false);

						// Check to see if this file also exists in our target destination, and if so delete it too
						string DestFilename = Path.Combine(ImmediateBaseDir, Utils.MakePathRelativeTo(filename, CleanUpBaseDir));
						if (File.Exists(DestFilename))
						{
							Logger.LogInformation("Cleaning up file {DestFilename}", DestFilename);
							SafeDeleteFile(DestFilename, false);
						}
					}
				}

				// Directory clean up code (Build/Android/src)
				try
				{
					IEnumerable<string> BaseDirectories = Directory.EnumerateDirectories(CleanUpBaseDir, "*", SearchOption.AllDirectories).OrderByDescending(x => x);
					foreach (string directory in BaseDirectories)
					{
						if (Directory.Exists(directory) && Directory.GetFiles(directory, "*.*", SearchOption.AllDirectories).Length == 0)
						{
							Logger.LogInformation("Cleaning Directory {Directory} as empty.", directory);
							Directory.Delete(directory, true);
						}
					}
				}
				catch (Exception)
				{
					// likely System.IO.DirectoryNotFoundException, ignore it
				}

				// Directory clean up code (Intermediate/APK/src)
				try
				{
					IEnumerable<string> ImmediateDirectories = Directory.EnumerateDirectories(ImmediateBaseDir, "*", SearchOption.AllDirectories).OrderByDescending(x => x);
					foreach (string directory in ImmediateDirectories)
					{
						if (Directory.Exists(directory) && Directory.GetFiles(directory, "*.*", SearchOption.AllDirectories).Length == 0)
						{
							Logger.LogInformation("Cleaning Directory {Directory} as empty.", directory);
							Directory.Delete(directory, true);
						}
					}
				}
				catch (Exception)
				{
					// likely System.IO.DirectoryNotFoundException, ignore it
				}
			}

			bool bDontBundleLibrariesInAPK = GetDontBundleLibrariesInAPK(Configuration, bIsArchive, bIsFromUAT);

			string ZenProjectStoreFileName = Path.Combine(Path.GetDirectoryName(ObbFileLocation)!, Path.GetFileNameWithoutExtension(ObbFileLocation), "ue.projectstore");
			bool bUsesZenStreaming = File.Exists(ZenProjectStoreFileName);

			// check to see if any "meta information" is newer than last time we build
			string TemplatesHashCode = GenerateTemplatesHashCode(EngineDirectory);
			string CurrentBuildSettings = GetAllBuildSettings(ToolChain, UPL!, bForDistribution, bMakeSeparateApks, bPackageDataInsideApk, bDisableVerifyOBBOnStartUp, bUseExternalFilesDir, bEnableBundle, bDontBundleLibrariesInAPK, bUsesZenStreaming, TemplatesHashCode, bRequiresOBB);

			// do we match previous build settings?
			bool bBuildSettingsMatch = true;

			// get application name and whether it changed, needing to force repackage
			string? ApplicationDisplayName;
			if (CheckApplicationName(Path.Combine(IntermediateAndroidPath, ArchRemapping[Archs[0].NativeArch]), ProjectName, out ApplicationDisplayName))
			{
				bBuildSettingsMatch = false;
				Logger.LogInformation("Application display name is different than last build, forcing repackage.");
			}

			// if the manifest matches, look at other settings stored in a file
			if (bBuildSettingsMatch)
			{
				if (File.Exists(BuildSettingsCacheFile))
				{
					string PreviousBuildSettings = File.ReadAllText(BuildSettingsCacheFile);
					if (PreviousBuildSettings != CurrentBuildSettings)
					{
						bBuildSettingsMatch = false;
						Logger.LogInformation("Previous .apk file(s) were made with different build settings, forcing repackage.");
					}
				}
			}

			// only check input dependencies if the build settings already match (if we don't run gradle, there is no Apk file to check against)
			if (bBuildSettingsMatch && !bSkipGradleBuild)
			{
				// check if so's are up to date against various inputs
				List<string> JavaFiles = new List<string>{
													UnrealOBBDataFileName,
													UnrealDownloadShimFileName
												};
				// Add the generated files too
				JavaFiles.AddRange(from t in templates select t.SourceFile);
				JavaFiles.AddRange(from t in templates select t.DestinationFile);

				bBuildSettingsMatch = CheckDependencies(Architectures, ProjectName, ProjectDirectory, IntermediateAndroidPath,
					UnrealBuildFilesPath, GameBuildFilesPath, JavaFiles, CookFlavor, OutputPath,
					bMakeSeparateApks, bPackageDataInsideApk, bDontBundleLibrariesInAPK);

			}

			string CommandLineSourceFileName = Path.Combine(Path.GetDirectoryName(ObbFileLocation)!, Path.GetFileNameWithoutExtension(ObbFileLocation), "UECommandLine.txt");
			string CommandLineCacheFileName = Path.Combine(IntermediateAndroidPath, "UECommandLine.txt");
			if (bBuildSettingsMatch)
			{
				bool bCommandLineMatch =
					(!File.Exists(CommandLineSourceFileName) && !File.Exists(CommandLineCacheFileName)) ||
					BinaryFileEquals(CommandLineSourceFileName, CommandLineCacheFileName);
				if (!bCommandLineMatch)
				{
					bBuildSettingsMatch = false;
					Logger.LogInformation("Previous .apk file(s) were made with different stage/apk command line, forcing repackage.");
				}
			}

			// Initialize UPL contexts for each architecture enabled
			UPL.SetGlobalContextVariable("AndroidPackageName", PackageName);
			UPL.SetGlobalContextVariable("CookFlavor", CookFlavor);
			UPL.Init(Archs.Select(x => x.NativeArch), bForDistribution, EngineDirectory, IntermediateAndroidPath, ProjectDirectory, Configuration.ToString(), bSkipGradleBuild, bPerArchBuildDir: true, ArchRemapping: ArchRemapping);
			

			Dictionary<string, int> NDKArchToCompileSDK = new Dictionary<string, int>();
			foreach ((UnrealArch _, string NDKArch) in Archs)
			{
				// check if any plugins want to increase the required compile SDK version
				string CompileSDKVersion = SDKAPILevel.Replace("android-", "", StringComparison.Ordinal);
				string CompileSDKMin = UPL.ProcessPluginNode(NDKArch, "minimumSDKAPI", "");
				int CompileSDKVersionInt = MinimumSDKLevel;
				if (!Int32.TryParse(CompileSDKVersion, out CompileSDKVersionInt))
				{
					Logger.LogError("Failed to parse SDK version from text '{SDKVersion}'", CompileSDKVersion);
				}

				if (!String.IsNullOrEmpty(CompileSDKMin))
				{
					if (!Int32.TryParse(CompileSDKVersion, out CompileSDKVersionInt))
					{
						CompileSDKVersionInt = MinimumSDKLevel;
					}
					bool bUpdatedCompileSDK = false;
					string[] CompileSDKLines = CompileSDKMin.Split(new[] { "\r\n", "\r", "\n" }, StringSplitOptions.None);
					foreach (string CompileLine in CompileSDKLines)
					{
						//string VersionString = CompileLine.Replace("android-", "");
						int VersionInt;
						if (Int32.TryParse(CompileLine, out VersionInt))
						{
							if (VersionInt > CompileSDKVersionInt)
							{
								CompileSDKVersionInt = VersionInt;
								bUpdatedCompileSDK = true;
							}
						}
					}
					if (bUpdatedCompileSDK)
					{
						CompileSDKVersion = CompileSDKVersionInt.ToString();
						Logger.LogInformation("Building Java with SDK API Level 'android-{CompileSDKVersion}' for arch {NDKArch} due to enabled plugin requirements", CompileSDKVersion, NDKArch);
					}
				}
				NDKArchToCompileSDK.Add(NDKArch, CompileSDKVersionInt);
			}

			Tuple<UnrealArch, string>[] BuildList;

			if (!bBuildSettingsMatch || bOnlyGradleGeneration)
			{
				// LINQ query is subsequently used multiple times, so must copy results
				BuildList = (from Arch in Architectures.Architectures
							let CompileSDKLevelForArch = NDKArchToCompileSDK[GetNDKArch(Arch)]
							let manifest = GenerateManifest(ToolChain, ProjectName, InTargetType, bForDistribution, bPackageDataInsideApk, GameBuildFilesPath, bRequiresOBB, bDisableVerifyOBBOnStartUp, Arch, CookFlavor, bUseExternalFilesDir, Configuration.ToString(), CompileSDKLevelForArch, bSkipGradleBuild, bEnableBundle, bUsesZenStreaming)
							select Tuple.Create(Arch, manifest)).ToArray();
			}
			else
			{
				// LINQ query is subsequently used multiple times, so must copy results
				BuildList = (from Arch in Architectures.Architectures
							let CompileSDKLevelForArch = NDKArchToCompileSDK[GetNDKArch(Arch)]
							let manifestFile = Path.Combine(IntermediateAndroidPath, Arch + "_AndroidManifest.xml")
							let manifest = GenerateManifest(ToolChain, ProjectName, InTargetType, bForDistribution, bPackageDataInsideApk, GameBuildFilesPath, bRequiresOBB, bDisableVerifyOBBOnStartUp, Arch, CookFlavor, bUseExternalFilesDir, Configuration.ToString(), CompileSDKLevelForArch, bSkipGradleBuild, bEnableBundle, bUsesZenStreaming)
							let OldManifest = File.Exists(manifestFile) ? File.ReadAllText(manifestFile) : ""
							where manifest != OldManifest
							select Tuple.Create(Arch, manifest)).ToArray();
			}

			List<string> LLDBExtraSymbolsDirectories = new();

			// Need to do stripping here because .apk will stay the same during iteration
			if (bDontBundleLibrariesInAPK)
			{
				foreach ((UnrealArch Arch, string _) in Archs)
				{
					string SOName = AndroidToolChain.InlineArchName(OutputPath, Arch);
					FileInfo SOFile = new(SOName);
					if (!SOFile.Exists)
					{
						Logger.LogWarning("Did not find compiled .so [{SOName}]", SOName);
						continue;
					}

					string SONameStripped = Path.Combine(Path.GetDirectoryName(SOName)!, Path.GetFileNameWithoutExtension(SOName) + "-stripped" + Path.GetExtension(SOName));
					FileInfo SOFileStripped = new(SONameStripped);
					if (!SOFileStripped.Exists || SOFileStripped.CreationTime != SOFile.CreationTime || SOFileStripped.LastWriteTime != SOFile.LastWriteTime)
					{
						StripDebugSymbols(SOName, SONameStripped, Arch, Logger, true);
						SOFileStripped.CreationTime = SOFile.CreationTime;
						SOFileStripped.LastWriteTime = SOFile.LastWriteTime;
					}

					// LLDB (in Android Studio and AGDE) needs to see a file that is matching the filename on the device ("libUnreal.so") to resolve symbols,
					// otherwise it will try to download libUnreal.so and fail due to adb pull not having permissions to access it.
					// Newer lldb has "platform.plugin.remote-android.package-name" in lldb to avoid that.
#pragma warning disable CA1308 // Normalize strings to uppercase - file path should be created with the correct case
					string LLDBExtraSymbolsDirectory = Path.Combine(IntermediateAndroidPath, "LLDBSymbolsLibs", Arch.ToString().ToLowerInvariant());
#pragma warning restore CA1308

					Directory.CreateDirectory(LLDBExtraSymbolsDirectory);

					Utils.LinkOrCopyIfDifferent(Path.Combine(LLDBExtraSymbolsDirectory, "libUnreal.so"), SOName);

					LLDBExtraSymbolsDirectories.Add(LLDBExtraSymbolsDirectory);
				}
			}
			else
			{
				foreach ((UnrealArch Arch, string _) in Archs)
				{
					string SOName = AndroidToolChain.InlineArchName(OutputPath, Arch);
					string SONameStripped = Path.Combine(Path.GetDirectoryName(SOName)!, Path.GetFileNameWithoutExtension(SOName) + "-stripped" + Path.GetExtension(SOName));
#pragma warning disable CA1308 // Normalize strings to uppercase - file path should use the correct case
					string LLDBExtraSymbolsDirectory = Path.Combine(IntermediateAndroidPath, "LLDBSymbolsLibs", Arch.ToString().ToLowerInvariant());
#pragma warning restore CA1308

					SafeDeleteFile(SONameStripped);
					SafeDeleteFile(Path.Combine(LLDBExtraSymbolsDirectory, "libUnreal.so"));
				}
			}

			// Now we have to spin over all the arch/gpu combinations to make sure they all match
			int BuildListComboTotal = BuildList.Length;
			if (BuildListComboTotal == 0 && !bOnlyGradleGeneration)
			{
				Logger.LogInformation("Output .apk file(s) are up to date (dependencies and build settings are up to date)");
				return;
			}

			if (BypassGradlePackaging && !bEnableBundle && !bSkipGradleBuild && !bOnlyGradleGeneration && !bCreateFromScratch)
			{
				Logger.LogInformation("Attempting BypassGradlePackaging");
				int BuildListComboRemaining = BuildListComboTotal;

				foreach (Tuple<UnrealArch, string> build in BuildList)
				{
					UnrealArch Arch = build.Item1;
					string Manifest = build.Item2;
					string NDKArch = GetNDKArch(Arch);

					string UnrealBuildPath = Path.Combine(IntermediateAndroidPath, Arch.ToString());

					Logger.LogInformation("\n===={Time}====PREPARING NATIVE CODE====={Arch}============================================================", DateTime.Now.ToString(), Arch);

					string DestApkDirectory = Path.Combine(ProjectDirectory, "Binaries", "Android");
					string SourceSOName = AndroidToolChain.InlineArchName(OutputPath, Arch);
					// if the source binary was UnrealGame, replace it with the new project name, when re-packaging a binary only build
					string ApkFilename = Path.GetFileNameWithoutExtension(OutputPath).Replace("UnrealGame", ProjectName, StringComparison.Ordinal);
					string DestApkName = Path.Combine(DestApkDirectory, ApkFilename + ".apk");

					// As we are always making separate APKs we need to put the architecture into the name
					DestApkName = AndroidToolChain.InlineArchName(DestApkName, Arch);

					if (!File.Exists(SourceSOName))
					{
						throw new BuildException("Can't make an APK without the compiled .so [{0}]", SourceSOName);
					}

					if (!File.Exists(DestApkName))
					{
						Logger.LogInformation("Output .apk [{DestApkName}] does not exist, will do full packaging with Gradle", DestApkName);
						continue;
					}

					// verify the .so actually changed (this does involve an extra copy, we could always assume it changed for a little more speed)
					bool bVerifySOChanged = true;
					if (bVerifySOChanged)
					{
						string JNIDirectory = Path.Combine(UnrealBuildPath, "jni", NDKArch);
						if (!Directory.Exists(JNIDirectory))
						{
							Logger.LogInformation("JNI directory {JNIDirectory} does not exist, will do full packaging with Gradle", JNIDirectory);
							continue;
						}

						string FinalSOName = Path.Combine(JNIDirectory, "libUnreal.so");
						if (!Utils.LinkOrCopyIfDifferent(FinalSOName, SourceSOName))
						{
							Logger.LogInformation("{FinalSOName} unchanged, skipping repackage of {DestApkName}", FinalSOName, DestApkName);
							BuildListComboRemaining--;
							continue;
						}
					}

					// strip symbols to make libUnreal.so small enough for packaging
					string StrippedSOName = SourceSOName + ".stripped";
					StripDebugSymbols(SourceSOName, StrippedSOName, Arch, Logger, true);

					Logger.LogInformation("\n===={Time}====PERFORMING FINAL APK PACKAGE OPERATION====={Arch}===========================================", DateTime.Now.ToString(), Arch);

					string TempAPK = Path.Combine(DestApkDirectory, "TEMP_" + Path.GetFileName(DestApkName));
					try
					{
						FileReference DestApkFile = new FileReference(DestApkName);
						FileReference TempApkFile = new FileReference(TempAPK);
						FileReference StrippedSOFile = new FileReference(StrippedSOName);
						CopyAPKAndReplaceSO(DestApkFile, TempApkFile, StrippedSOFile, "lib/" + NDKArch + "/libUnreal.so", Logger);
					}
					catch { }

					// defaults for non-distribution build (debug)
					string KeyStore = Path.Combine(Environment.GetFolderPath(Environment.SpecialFolder.UserProfile), ".android", "debug.keystore");
					string KeyStorePassword = "android";
					string KeyAlias = "androiddebugkey";
					string KeyPassword = "android";

					// read project settings for keystore information if distribution build (release)
					if (bForDistribution)
					{
						Ini.GetString("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "KeyStore", out KeyStore);
						Ini.GetString("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "KeyAlias", out KeyAlias);
						Ini.GetString("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "KeyStorePassword", out KeyStorePassword);
						Ini.GetString("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "KeyPassword", out KeyPassword);

						if (String.IsNullOrEmpty(KeyStore) || String.IsNullOrEmpty(KeyAlias) || String.IsNullOrEmpty(KeyStorePassword))
						{
							throw new BuildException("DistributionSigning settings are not all set. Check the DistributionSettings section in the Android tab of Project Settings");
						}

						if (String.IsNullOrEmpty(KeyPassword) || KeyPassword == "_sameaskeystore_")
						{
							KeyPassword = KeyStorePassword;
						}

						KeyStore = Path.Combine(UnrealBuildPath, KeyStore);
					}
					else
					{
						string DebugKeyAlias, DebugKeyStore, DebugKeyStorePassword, DebugKeyPassword;
						Ini.GetString("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "DebugKeyStore", out DebugKeyStore);
						Ini.GetString("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "DebugKeyAlias", out DebugKeyAlias);
						Ini.GetString("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "DebugKeyStorePassword", out DebugKeyStorePassword);
						Ini.GetString("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "DebugKeyPassword", out DebugKeyPassword);

						if (!String.IsNullOrEmpty(DebugKeyStore) && !String.IsNullOrEmpty(DebugKeyAlias) && !String.IsNullOrEmpty(DebugKeyStorePassword))
						{
							Logger.LogInformation("Using custom debug keystore {KeyStore}", DebugKeyStore);

							KeyStore = Path.Combine(UnrealBuildPath, DebugKeyStore);
							KeyAlias = DebugKeyAlias;
							KeyStorePassword = DebugKeyStorePassword;
							KeyPassword = DebugKeyPassword;
							if (String.IsNullOrEmpty(KeyPassword) || KeyPassword == "_sameaskeystore_")
							{
								KeyPassword = KeyStorePassword;
							}
						}
						else if (!String.IsNullOrEmpty(DebugKeyStore) || !String.IsNullOrEmpty(DebugKeyAlias) || !String.IsNullOrEmpty(DebugKeyStorePassword))
						{
							throw new BuildException("Debug signing settings are not all or none set. Check the DistributionSettings section in the Android tab of Project Settings");
						}
						else
						{
							Logger.LogInformation("Using default keystore");
						}
					}

					// Make sure the keystore file exists
					if (!File.Exists(KeyStore))
					{
						throw new BuildException("Keystore file is missing. Check the DistributionSettings section in the Android tab of Project Settings");
					}

					Logger.LogInformation("Applying apksigner to apk...");
					try
					{
						ToolChain.SignApk(new FileReference(TempAPK), new FileReference(DestApkName), KeyStore, KeyStorePassword, KeyAlias, KeyPassword);
					}
					finally
					{
						SafeDeleteFile(TempAPK);
						SafeDeleteFile(DestApkName + ".idsig");
						SafeDeleteFile(StrippedSOName);
					}

					// copy .so with symbols if requested
					Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bBuildWithHiddenSymbolVisibility", out bool bBuildWithHiddenSymbolVisibility);
					Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bSaveSymbols", out bool bSaveSymbols);
					// Deprecated check of bBuildWithHiddenSymbolVisibility. Remove in 5.8+
					if (bBuildWithHiddenSymbolVisibility || bSaveSymbols)
					{
						// Copy .so with symbols to 
						int StoreVersion = GetStoreVersion(bEnableBundle ? null : Arch);
						string SymbolSODirectory = Path.Combine(DestApkDirectory, ProjectName + "_Symbols_v" + StoreVersion + "/" + ProjectName + Arch);
						string SymbolifiedSOPath = Path.Combine(SymbolSODirectory, "libUnreal.so");
						MakeDirectoryIfRequired(SymbolifiedSOPath);
						Logger.LogInformation("Writing symbols to {SymbolifiedSOPath}", SymbolifiedSOPath);

						Utils.HardLinkOrCopyIfDifferent(SymbolifiedSOPath, SourceSOName);
					}

					BuildListComboRemaining--;
				}

				if (BuildListComboRemaining == 0)
				{
					Logger.LogInformation("\n===={Time}====COMPLETED MAKE APK=======================================================================", DateTime.Now.ToString());
					return;
				}
			}

			// at this point, we can write out the cached build settings to compare for a next build
			File.WriteAllText(BuildSettingsCacheFile, CurrentBuildSettings);
			if (File.Exists(CommandLineSourceFileName))
			{
				CopyIfDifferent(CommandLineSourceFileName, CommandLineCacheFileName, true, true);
			}
			else
			{
				SafeDeleteFile(CommandLineCacheFileName);
			}

			// make up a dictionary of strings to replace in xml files (strings.xml)
			Dictionary<string, string> Replacements = new Dictionary<string, string>
			{
				{"${EXECUTABLE_NAME}", ApplicationDisplayName!},
				{"${PY_VISUALIZER_PATH}", Path.GetFullPath(Path.Combine(EngineDirectory, "Extras", "LLDBDataFormatters", "UEDataFormatters_2ByteChars.py"))},
				{"${IDEA_RUN_CONFIGURATION_SYMBOL_PATHS}", String.Join("", LLDBExtraSymbolsDirectories.Select(x => $"\n\t\t\t<symbol_dirs symbol_path=\"{x}\" />"))}
			};

			// steps run for each build combination (note: there should only be one GPU in future)
			foreach (Tuple<UnrealArch, string> build in BuildList)
			{
				UnrealArch Arch = build.Item1;
				string Manifest = build.Item2;
				string NDKArch = GetNDKArch(Arch);

				int CompileSDKLevelForArch = NDKArchToCompileSDK[NDKArch];
				string CompileSDKVersion = CompileSDKLevelForArch.ToString();

				UPL.SetGlobalContextVariable("CompileSDK", CompileSDKVersion);

				Logger.LogInformation("\n===={Time}====PREPARING NATIVE CODE====={Arch}============================================================", DateTime.Now.ToString(), Arch);

				string UnrealBuildPath = Path.Combine(IntermediateAndroidPath, Arch.ToString());
				string UnrealBuildGradlePath = Path.Combine(UnrealBuildPath, "gradle");

				bool bInApkAssetChange = false;
				if (!bNoObbs)
				{
					// If we are packaging for Amazon then we need to copy the  file to the correct location
					Logger.LogInformation("bPackageDataInsideApk = {PackageDataInsideApk}", bPackageDataInsideApk);
					if (bPackageDataInsideApk)
					{
						Logger.LogInformation("Obb location {ObbFileLocation}", ObbFileLocation);
						string ObbFileDestination = UnrealBuildPath + "/assets";
						Logger.LogInformation("Obb destination location {ObbFileDestination}", ObbFileDestination);
						if (File.Exists(ObbFileLocation))
						{
							Directory.CreateDirectory(UnrealBuildPath);
							Directory.CreateDirectory(ObbFileDestination);
							Logger.LogInformation("Obb file exists...");
							string DestFileName = Path.Combine(ObbFileDestination, "main.obb.png"); // Need a rename to turn off compression
							string SrcFileName = ObbFileLocation;
							bInApkAssetChange = CopyIfDifferent(SrcFileName, DestFileName, true, false);
						}
					}
					else // try to remove the file it we aren't packaging inside the APK
					{
						string ObbFileDestination = UnrealBuildPath + "/assets";
						string DestFileName = Path.Combine(ObbFileDestination, "main.obb.png");
						bInApkAssetChange = SafeDeleteFile(DestFileName);
					}
				}

				if (bInApkAssetChange)
				{
					// Workaround for Gradle bug whereby a change in the contents of assets triggers a patch build whereby main.obb.png is duplicated in the APK causing the APK to grow in size. 
					// Workaround deletes existing Gradle output APK to rebuild when main.obb.png is modified or deleted.
					string GradleConfig = bForDistribution ? "release" : "debug";
					string GradeOutAPK = Path.Combine(UnrealBuildGradlePath, "app", "build", "outputs", "apk", GradleConfig, "app-" + GradleConfig + ".apk");
					SafeDeleteFile(GradeOutAPK);
				}
				
				// See if we need to stage a UECommandLine.txt file in assets
				string CommandLineDestFileName = Path.Combine(UnrealBuildPath, "assets", "UECommandLine.txt");
				if (File.Exists(CommandLineSourceFileName))
				{
					Directory.CreateDirectory(UnrealBuildPath);
					Directory.CreateDirectory(Path.Combine(UnrealBuildPath, "assets"));
					Console.WriteLine("UnrealCommandLine.txt exists...");
					CopyIfDifferent(CommandLineSourceFileName, CommandLineDestFileName, true, true);
				}
				else // try to remove the file if we aren't packaging one
				{
					SafeDeleteFile(CommandLineDestFileName);
				}

				// check for Android Studio project being setup for packaging the apk/debug. If this exists then we have setup and
				//Android studio project that we intend to use to finish making the APK or for debugging.
				string GradleAppImlFilename = Path.Combine(UnrealBuildGradlePath, ".idea", "modules", "app.iml");
				bool bHasAndroidStudioProject = File.Exists(GradleAppImlFilename);

				// We need to filter out copying over our gradle files and stomping the existing gradle project if we plan to use 
				// the existing AndroidStudio project to build our APK and/or Debug.
				string[]? ExcludeFolders = null;
				if (bHasAndroidStudioProject)
				{
					//Path.Combine(UnrealBuildFilesPath, "gradle")
					ExcludeFolders = new string[] { "gradle", ".gradle", ".idea", "app", "runConfigurations" };
				}

				//Copy build files to the intermediate folder in this order (later overrides earlier):
				//	- Shared Engine
				//  - Shared Engine NoRedist (for Epic secret files)
				//  - Game
				//  - Game NoRedist (for Epic secret files)
				CopyFileDirectory(UnrealBuildFilesPath, UnrealBuildPath, Replacements, ExcludeFolders);
				CopyFileDirectory(UnrealBuildFilesPath_NFL, UnrealBuildPath, Replacements, ExcludeFolders);
				CopyFileDirectory(UnrealBuildFilesPath_NR, UnrealBuildPath, Replacements, ExcludeFolders);
				CopyFileDirectory(UnrealBuildFilesPath_LA, UnrealBuildPath, Replacements, ExcludeFolders);
				CopyFileDirectory(GameBuildFilesPath, UnrealBuildPath, Replacements, ExcludeFolders);
				CopyFileDirectory(GameBuildFilesPath_NFL, UnrealBuildPath, Replacements, ExcludeFolders);
				CopyFileDirectory(GameBuildFilesPath_NR, UnrealBuildPath, Replacements, ExcludeFolders);
				CopyFileDirectory(GameBuildFilesPath_LA, UnrealBuildPath, Replacements, ExcludeFolders);

				if (bNoObbs || !bPackageDataInsideApk)
				{
					// Try to remove main.obb.png file if we don't want OBBs
					SafeDeleteFile(Path.Combine(UnrealBuildPath, "assets", "main.obb.png"));
				}
				
				// Parse Gradle filters (may have been replaced by above copies)
				ParseFilterFile(Path.Combine(UnrealBuildPath, "GradleFilter.txt"));

				//Generate Gradle AAR dependencies
				GenerateGradleAARImports(EngineDirectory, UnrealBuildPath, Archs.Select(x => x.NativeArch));

				//Now validate GooglePlay app_id if enabled
				ValidateGooglePlay(UnrealBuildPath);

				//determine which orientation requirements this app has
				bool bNeedLandscape = false;
				bool bNeedPortrait = false;
				DetermineScreenOrientationRequirements(Archs[0].NativeArch, out bNeedPortrait, out bNeedLandscape);

				//Now keep the splash screen images matching orientation requested
				PickSplashScreenOrientation(CookFlavor, UnrealBuildPath, bNeedPortrait, bNeedLandscape);

				//Similarly, keep only the downloader screen image matching the orientation requested
				PickDownloaderScreenOrientation(UnrealBuildPath, bNeedPortrait, bNeedLandscape);

				// use Gradle for compile/package
				string UnrealBuildGradleAppPath = Path.Combine(UnrealBuildGradlePath, "app");
				string UnrealBuildGradleMainPath = Path.Combine(UnrealBuildGradleAppPath, "src", "main");

				// Write the manifest to the correct locations (cache and real)
				File.WriteAllText(Path.Combine(IntermediateAndroidPath, Arch + "_AndroidManifest.xml"), Manifest);
				File.WriteAllText(Path.Combine(UnrealBuildPath, "AndroidManifest.xml"), Manifest);

				// Check if included SplashActivity in AndroidManifest
				bool bHasSplashActivity = Manifest.Contains("com.epicgames.unreal.SplashActivity", StringComparison.Ordinal);

				// copy prebuild plugin files
				UPL.ProcessPluginNode(NDKArch, "prebuildCopies", "");

				XDocument AdditionalBuildPathFilesDoc = new XDocument(new XElement("files"));
				UPL.ProcessPluginNode(NDKArch, "additionalBuildPathFiles", "", ref AdditionalBuildPathFilesDoc);

				// Generate the OBBData.java file since different architectures may have different store version
				UnrealOBBDataFileName = GetUnrealJavaOBBDataFileName(Path.Combine(UnrealBuildPath, "src", PackageName.Replace('.', Path.DirectorySeparatorChar)));
				WriteJavaOBBDataFile(UnrealOBBDataFileName, PackageName, RequiredOBBFiles, CookFlavor, bPackageDataInsideApk, Arch);

				// we don't actually need the SO for the bSkipGradleBuild case
				string? SourceSOName = null;
				string DestApkDirectory = Path.Combine(ProjectDirectory, "Binaries", "Android");
				// If OutputPath is not at the same level with DestApkDirectory, then the target was built with binary subfolder
				string? OutputDirectory = Path.GetDirectoryName(OutputPath);
				if (OutputDirectory != null && Path.GetRelativePath(DestApkDirectory, OutputDirectory) != OutputDirectory)
				{
					// only override if not UnrealGame (BluePrint project)
					if (!Path.GetFileName(OutputPath).StartsWith("UnrealGame", StringComparison.Ordinal))
					{
						DestApkDirectory = OutputDirectory;
					}
				}
				string? DestApkName = null;
				if (bSkipGradleBuild)
				{
					SourceSOName = OutputPath;
					if (!File.Exists(SourceSOName))
					{
						Logger.LogWarning("Did not find compiled .so [{SourceSOName}]", SourceSOName);
					}
				}
				else if (bDontBundleLibrariesInAPK)
				{
					SourceSOName = AndroidToolChain.InlineArchName(OutputPath, Arch);
					string SourceSONameStripped = Path.Combine(Path.GetDirectoryName(SourceSOName)!, Path.GetFileNameWithoutExtension(SourceSOName) + "-stripped" + Path.GetExtension(SourceSOName));
					if (!bOnlyGradleGeneration && !File.Exists(SourceSONameStripped))
					{
						Logger.LogWarning("Did not find compiled .so [{FinalSONameStripped}]", SourceSONameStripped);
					}

					string ApkFilename = Path.GetFileNameWithoutExtension(OutputPath).Replace("UnrealGame", ProjectName, StringComparison.Ordinal);
					DestApkName = Path.Combine(DestApkDirectory, ApkFilename + ".apk");

					// As we are always making separate APKs we need to put the architecture into the name
					DestApkName = AndroidToolChain.InlineArchName(DestApkName, Arch);

					bool bUseAFS = true;
					if (!Ini.GetBool("/Script/AndroidFileServerEditor.AndroidFileServerRuntimeSettings", "bEnablePlugin", out bUseAFS))
					{
						bUseAFS = true;
					}

					Ini.GetString("/Script/AndroidFileServerEditor.AndroidFileServerRuntimeSettings", "SecurityToken", out string AFSToken);

					AFSToken = String.IsNullOrEmpty(AFSToken) ? "" : " -k " + AFSToken;

					string AFSExecutable = Path.Combine(Unreal.EngineDirectory.ToString(), @"Binaries/DotNET/Android/UnrealAndroidFileTool", GetAFSExecutable(BuildHostPlatform.Current.Platform, Logger));
					string AFSArguments = $"-p {PackageName}{AFSToken}";

					string? SOPushScriptLocation = Path.GetDirectoryName(SourceSOName)!;

					string FinalSONameStrippedRelative = Path.GetRelativePath(SOPushScriptLocation, SourceSONameStripped);

					// MakeApk will not be called in bDontBundleLibrariesInAPK mode, so we need to run stripping outside of it
					string SOPushScript = @$"
set ADB=adb
set AFS={AFSExecutable.Replace("/", "\\", StringComparison.Ordinal)}
set AFSARGS={AFSArguments}
set DEVICE=
if not \""%1\""==\""\"" set DEVICE=-s %1
pushd %~dp0
";
					if (bUseAFS)
					{
						SOPushScript += @$"
%AFS% %DEVICE% %AFSARGS% push {FinalSONameStrippedRelative} ""^int/libUnreal.so""
if ""%ERRORLEVEL%"" NEQ ""0"" (exit /b %ERRORLEVEL%)
";
					}
					else
					{
						SOPushScript += @$"
%ADB% %DEVICE% push -z lz4 ""{FinalSONameStrippedRelative}"" /data/local/tmp/
if ""%ERRORLEVEL%"" NEQ ""0"" (%ADB% %DEVICE% push ""{FinalSONameStrippedRelative}"" /data/local/tmp/)
if ""%ERRORLEVEL%"" NEQ ""0"" (exit /b %ERRORLEVEL%)
%ADB% %DEVICE% shell run-as {PackageName} mkdir -p ./files
%ADB% %DEVICE% shell run-as {PackageName} cp ""/data/local/tmp/{FinalSONameStrippedRelative}"" ./files/libUnreal.so
if ""%ERRORLEVEL%"" NEQ ""0"" (exit /b %ERRORLEVEL%)
%ADB% %DEVICE% shell rm ""/data/local/tmp/{FinalSONameStrippedRelative}""
";
					}

					SOPushScript +=
@"
popd
";
					// name is also used in the AGDE project, see AndroidProjectGenerator.cs
					string SOPushScriptName = Path.Combine(Path.GetDirectoryName(SourceSOName)!, "Push_" + Path.GetFileNameWithoutExtension(SourceSOName) + "_so.bat");
					File.WriteAllText(SOPushScriptName, SOPushScript);

					if (!bOnlyGradleGeneration)
					{
						// Remove libUnreal.so from gradle project if any
						string JniDir = UnrealBuildPath + "/jni/" + NDKArch;
						string TempFinalSOName = JniDir + "/libUnreal.so";
						SafeDeleteFile(TempFinalSOName);
					}
				}
				else
				{
					SourceSOName = AndroidToolChain.InlineArchName(OutputPath, Arch);
					// if the source binary was UnrealGame, replace it with the new project name, when re-packaging a binary only build
					string ApkFilename = Path.GetFileNameWithoutExtension(OutputPath).Replace("UnrealGame", ProjectName, StringComparison.Ordinal);
					DestApkName = Path.Combine(DestApkDirectory, ApkFilename + ".apk");

					// As we are always making separate APKs we need to put the architecture into the name
					DestApkName = AndroidToolChain.InlineArchName(DestApkName, Arch);

					if (!bOnlyGradleGeneration && !File.Exists(SourceSOName))
					{
						throw new BuildException("Can't make an APK without the compiled .so [{0}]", SourceSOName);
					}
					if (!bOnlyGradleGeneration && !Directory.Exists(UnrealBuildPath + "/jni"))
					{
						throw new BuildException("Can't make an APK without the jni directory [{0}/jni]", UnrealBuildFilesPath);
					}

					// Delete the push script if exists
					string SOPushScriptName = Path.Combine(Path.GetDirectoryName(DestApkName)!, "Push_" + Path.GetFileNameWithoutExtension(DestApkName) + "_so.bat");
					SafeDeleteFile(SOPushScriptName);

					// clear out libs directory like ndk-build would have
					string LibsDir = Path.Combine(UnrealBuildPath, "libs");
					DeleteDirectory(LibsDir, Logger);
					Directory.CreateDirectory(LibsDir);
					
					if (!bOnlyGradleGeneration)
					{
						string JniDir = Path.Combine(UnrealBuildPath, "jni", NDKArch);
						Directory.CreateDirectory(JniDir);
						Utils.LinkOrCopyIfDifferent(Path.Combine(JniDir, "libUnreal.so"), SourceSOName);
	
						string JniLibsDir = Path.Combine(UnrealBuildGradleMainPath, "jniLibs", NDKArch);
						Directory.CreateDirectory(JniLibsDir);
						Utils.LinkOrCopyIfDifferent(Path.Combine(JniLibsDir, "libUnreal.so"), SourceSOName);
					}
				}

				// update GameActivity.java and GameApplication.java if out of date
				UpdateGameActivity(NDKArch, EngineDirectory, UnrealBuildPath, bDontBundleLibrariesInAPK, bJavaPerfettoMarkers, Configuration, bHasSplashActivity);
				UpdateGameApplication(NDKArch, EngineDirectory, UnrealBuildPath, bJavaPerfettoMarkers, Configuration, SourceSOName);

				bool bSkipLibCpp = false;
				Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bSkipLibCpp", out bSkipLibCpp);
				if (!bSkipLibCpp)
				{
					// after ndk-build is called, we can now copy in the stl .so (ndk-build deletes old files)
					// copy libc++_shared.so to library
					CopySTL(UnrealBuildPath, NDKArch);
				}
				CopySwappy(UnrealBuildPath, NDKArch, Configuration);
				CopyPSOService(UnrealBuildPath, UnrealPreBuiltFilesPath, NDKArch);
				CopyVulkanValidationLayers(UnrealBuildPath, NDKArch, Configuration.ToString());

				if (Sanitizer != AndroidToolChain.ClangSanitizer.None)
				{
					CopyClangSanitizerLib(ToolChain, UnrealBuildPath, NDKArch, Sanitizer);

					if (bEnableScudoMemoryTracing)
					{
						throw new BuildException("ScudoMemoryTrace not supported with sanitizer enabled");
					}
				}
				else if (bEnableScudoMemoryTracing)
				{
					CopyScudoMemoryTracerLib(UnrealBuildPath, Arch, NDKArch, PackageName);
				}

				// copy postbuild plugin files
				UPL.ProcessPluginNode(NDKArch, "resourceCopies", "");

				CreateAdditonalBuildPathFiles(NDKArch, UnrealBuildPath, AdditionalBuildPathFilesDoc);

				Logger.LogInformation("\n===={Time}====PERFORMING FINAL APK PACKAGE OPERATION====={Arch}===========================================", DateTime.Now.ToString(), Arch);

				// stage files into gradle app directory
				string GradleManifest = Path.Combine(UnrealBuildGradleMainPath, "AndroidManifest.xml");
				MakeDirectoryIfRequired(GradleManifest);
				CopyIfDifferent(Path.Combine(UnrealBuildPath, "AndroidManifest.xml"), GradleManifest, true, true);

				string[] Excludes;
				switch (NDKArch)
				{
					default:
					case "arm64-v8a":
						Excludes = new string[] { "armeabi-v7a", "x86", "x86_64" };
						break;

					case "x86_64":
						Excludes = new string[] { "armeabi-v7a", "arm64-v8a", "x86" };
						break;
				}

				CleanCopyDirectory(Path.Combine(UnrealBuildPath, "jni"), Path.Combine(UnrealBuildGradleMainPath, "jniLibs"), Excludes);  // has debug symbols
				CleanCopyDirectory(Path.Combine(UnrealBuildPath, "libs"), Path.Combine(UnrealBuildGradleMainPath, "libs"), Excludes);
				if ((Sanitizer != AndroidToolChain.ClangSanitizer.None && (Sanitizer != AndroidToolChain.ClangSanitizer.HwAddress || AndroidToolChain.HasEmbeddedHWASanSupport())) || bEnableScudoMemoryTracing)
				{
					CleanCopyDirectory(Path.Combine(UnrealBuildPath, "resources"), Path.Combine(UnrealBuildGradleMainPath, "resources"), Excludes);
				}

				CleanCopyDirectory(Path.Combine(UnrealBuildPath, "assets"), Path.Combine(UnrealBuildGradleMainPath, "assets"));
				CleanCopyDirectory(Path.Combine(UnrealBuildPath, "res"), Path.Combine(UnrealBuildGradleMainPath, "res"));
				CleanCopyDirectory(Path.Combine(UnrealBuildPath, "src"), Path.Combine(UnrealBuildGradleMainPath, "java"));

				// do any plugin requested copies
				UPL.ProcessPluginNode(NDKArch, "gradleCopies", "");

				// get min and target SDK versions
				int MinSDKVersion = 0;
				int TargetSDKVersion = 0;
				int NDKLevelInt = 0;
				GetMinTargetSDKVersions(ToolChain, UPL, NDKArch, bEnableBundle, out MinSDKVersion, out TargetSDKVersion, out NDKLevelInt);

				// move JavaLibs into subprojects
				string JavaLibsDir = Path.Combine(UnrealBuildPath, "JavaLibs");
				PrepareJavaLibsForGradle(JavaLibsDir, UnrealBuildGradlePath, MinSDKVersion.ToString(), TargetSDKVersion.ToString(), NDKArch);

				// Create local.properties
				string LocalPropertiesFilename = Path.Combine(UnrealBuildGradlePath, "local.properties");
				StringBuilder LocalProperties = new StringBuilder();
//				LocalProperties.AppendLine(string.Format("ndk.dir={0}", Environment.GetEnvironmentVariable("NDKROOT")!.Replace("\\", "/")));
				LocalProperties.AppendLine(String.Format("sdk.dir={0}", Environment.GetEnvironmentVariable("ANDROID_HOME")!.Replace("\\", "/", StringComparison.Ordinal)));
				File.WriteAllText(LocalPropertiesFilename, LocalProperties.ToString());

				CreateGradlePropertiesFiles(ToolChain, MinSDKVersion, TargetSDKVersion, CompileSDKVersion, BuildToolsVersion, PackageName, DestApkName, NDKArch,
					UnrealBuildFilesPath, GameBuildFilesPath, UnrealBuildGradleAppPath, UnrealBuildPath, UnrealBuildGradlePath, bForDistribution, bSkipGradleBuild, RequiredOBBFiles, bJavaPerfettoMarkers);

				CheckSOInGradleFolderAre16KbCompatible(UnrealBuildGradleMainPath);

				if (!bSkipGradleBuild && !bOnlyGradleGeneration)
				{
					string GradleScriptPath = Path.Combine(UnrealBuildGradlePath, "gradlew");
					if (!OperatingSystem.IsWindows())
					{
						// fix permissions for Mac/Linux
						Logger.LogInformation("Fix gradlew permissions");
						File.SetUnixFileMode(GradleScriptPath, File.GetUnixFileMode(GradleScriptPath) | UnixFileMode.UserExecute | UnixFileMode.GroupExecute | UnixFileMode.OtherExecute);
					}
					else
					{
						if (CreateRunGradle(UnrealBuildGradlePath))
						{
							GradleScriptPath = Path.Combine(UnrealBuildGradlePath, "rungradle.bat");
						}
						else
						{
							GradleScriptPath = Path.Combine(UnrealBuildGradlePath, "gradlew.bat");
						}
					}

					if (!bEnableBundle)
					{
						string GradleBuildType = bForDistribution ? ":app:assembleRelease" : ":app:assembleDebug";

						// collect optional additional Gradle parameters from plugins
						string GradleOptions = UPL.ProcessPluginNode(NDKArch, "gradleParameters", GradleBuildType); //  "--stacktrace --debug " + GradleBuildType);
						string GradleSecondCallOptions = UPL.ProcessPluginNode(NDKArch, "gradleSecondCallParameters", "");

						// check for Android Studio project, call Gradle if doesn't exist (assume user will build with Android Studio)
						if (!bHasAndroidStudioProject)
						{
							// make sure destination exists
							Directory.CreateDirectory(Path.GetDirectoryName(DestApkName)!);

							// For build machine, get a list of current java.exe tasks already running to ignore for cleanup
							List<int> PreviousJavaProcesses = new List<int>();
							if (bIsBuildMachine && OperatingSystem.IsWindows())
							{
								string Output;
								int ExitCode = RunCommandLineProgramAndReturnResultOutput(UnrealBuildGradlePath, "cmd.exe", "/c tasklist.exe /NH /FI \"imagename eq java.exe\"", Logger, out Output, "Collecting existing java.exe processes");
								if (ExitCode == 0)
								{
									foreach (string Taskline in Output.Split(new[] { "\r\n", "\r", "\n" }, StringSplitOptions.None))
									{
										if (Taskline.StartsWith("java.exe", StringComparison.Ordinal))
										{
											string ProcessString = Taskline.Substring(8).TrimStart();
											ProcessString = ProcessString.Substring(0, ProcessString.IndexOf(' ', StringComparison.Ordinal));
											int ProcessId;
											if (Int32.TryParse(ProcessString, out ProcessId))
											{
												PreviousJavaProcesses.Add(ProcessId);
											}
										}
									}
								}
							}

							// Use gradle to build the .apk file
							string ShellExecutable = OperatingSystem.IsWindows() ? "cmd.exe" : "/bin/sh";
							string ShellParametersBegin = OperatingSystem.IsWindows() ? "/c " : "-c '";
							string ShellParametersEnd = OperatingSystem.IsWindows() ? "" : "'";
							RunCommandLineProgramWithExceptionAndFiltering(UnrealBuildGradlePath, ShellExecutable, ShellParametersBegin + "\"" + GradleScriptPath + "\" " + GradleOptions + ShellParametersEnd, Logger, "Making .apk with Gradle...");

							if (!String.IsNullOrEmpty(GradleSecondCallOptions))
							{
								RunCommandLineProgramWithExceptionAndFiltering(UnrealBuildGradlePath, ShellExecutable, ShellParametersBegin + "\"" + GradleScriptPath + "\" " + GradleSecondCallOptions + ShellParametersEnd, Logger, "Additional Gradle steps...");
							}

							// For build machine run a clean afterward to clean up intermediate files (does not remove final APK)
							if (bIsBuildMachine)
							{
								//GradleOptions = "tasks --all";
								//RunCommandLineProgramWithException(UnrealBuildGradlePath, ShellExecutable, ShellParametersBegin + "\"" + GradleScriptPath + "\" " + GradleOptions + ShellParametersEnd, "Listing all tasks...");

								// on Windows sometimes minifyReleaseWithR8 is keeping a lock on classes.dex so kill java.exe not previously running
								if (OperatingSystem.IsWindows())
								{
									string Output;
									int ExitCode = RunCommandLineProgramAndReturnResultOutput(UnrealBuildGradlePath, "cmd.exe", "/c tasklist.exe /NH /FI \"imagename eq java.exe\"", Logger, out Output, "Collecting existing java.exe processes");
									if (ExitCode == 0)
									{
										foreach (string Taskline in Output.Split(new[] { "\r\n", "\r", "\n" }, StringSplitOptions.None))
										{
											if (Taskline.StartsWith("java.exe", StringComparison.Ordinal))
											{
												string ProcessString = Taskline.Substring(8).TrimStart();
												ProcessString = ProcessString.Substring(0, ProcessString.IndexOf(' ', StringComparison.Ordinal));
												int ProcessId;
												if (Int32.TryParse(ProcessString, out ProcessId))
												{
													// if not in previous collected list, kill it
													if (!PreviousJavaProcesses.Contains(ProcessId))
													{
														RunCommandLineProgramAndReturnResultOutput(UnrealBuildGradlePath, "cmd.exe", "/c taskkill.exe /PID " + ProcessString + " /T /F", Logger, out Output, "Terminate java.exe process");
													}
												}
											}
										}
									}
								}

								GradleOptions = "clean";
								RunCommandLineProgramWithExceptionAndFiltering(UnrealBuildGradlePath, ShellExecutable, ShellParametersBegin + "\"" + GradleScriptPath + "\" " + GradleOptions + ShellParametersEnd, Logger, "Cleaning Gradle intermediates...");
							}
						}
						else
						{
							Logger.LogInformation("=============================================================================================");
							Logger.LogInformation("Android Studio project found, skipping Gradle; complete creation of APK in Android Studio!!!!");
							Logger.LogInformation("Delete '{GradleAppImlFilename} if you want to have UnrealBuildTool run Gradle for future runs.", GradleAppImlFilename);
							Logger.LogInformation("=============================================================================================");
						}
					}
				}

				Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bBuildWithHiddenSymbolVisibility", out bool bBuildWithHiddenSymbolVisibility);
				Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bSaveSymbols", out bool bSaveSymbols);
				// Deprecated check of bBuildWithHiddenSymbolVisibility and forced bSaveSymbols = true. Remove in 5.8+
				bSaveSymbols = true;
				if ((bBuildWithHiddenSymbolVisibility || bSaveSymbols) && !bOnlyGradleGeneration)
				{
					// Copy .so with symbols to 
					int StoreVersion = GetStoreVersion(bEnableBundle ? null : Arch);
					string SymbolSODirectory = Path.Combine(DestApkDirectory, ProjectName + "_Symbols_v" + StoreVersion, ProjectName + Arch);
					string SymbolifiedSOPath = Path.Combine(SymbolSODirectory, "libUnreal.so");
					MakeDirectoryIfRequired(SymbolifiedSOPath);
					Logger.LogInformation("Writing symbols to {SymbolifiedSOPath}", SymbolifiedSOPath);

					Utils.HardLinkOrCopyIfDifferent(SymbolifiedSOPath, SourceSOName);
				}
			}

			// Deal with generating an App Bundle
			if (bEnableBundle && !bSkipGradleBuild && !bOnlyGradleGeneration)
			{
				bool bCombinedBundleOK = true;

				// try to make a combined Gradle project for all architectures (may fail if incompatible configuration)
				string UnrealGradleDest = Path.Combine(IntermediateAndroidPath, "gradle");

				// start fresh each time for now
				DeleteDirectory(UnrealGradleDest, Logger);

				// make sure destination exists
				Directory.CreateDirectory(UnrealGradleDest);

				string ABIFilter = "";
				string[] JavaSrcRootComponents = { "app", "src", "main" };

				// loop through and merge the different architecture gradle directories
				foreach (Tuple<UnrealArch, string> build in BuildList)
				{
					UnrealArch Arch = build.Item1;
					string Manifest = build.Item2;
					string NDKArch = GetNDKArch(Arch);

					string UnrealBuildPath = Path.Combine(IntermediateAndroidPath, Arch.ToString());
					string UnrealBuildGradlePath = Path.Combine(UnrealBuildPath, "gradle");

					if (!Directory.Exists(UnrealBuildGradlePath))
					{
						Logger.LogInformation("Source directory missing: {GradlePath}", UnrealBuildGradlePath);
						bCombinedBundleOK = false;
						break;
					}

					ABIFilter += ", \"" + NDKArch + "\"";

					int UnrealBuildGradlePathLength = UnrealBuildGradlePath.Length;
					string[] SourceFiles = Directory.GetFiles(UnrealBuildGradlePath, "*.*", SearchOption.AllDirectories);
					foreach (string Filename in SourceFiles)
					{
						// make the dest filename with the same structure as it was in SourceDir
						string DestFilename = Path.Combine(UnrealGradleDest, Utils.MakePathRelativeTo(Filename, UnrealBuildGradlePath));

						// skip the build directories
						string Workname = Filename.Replace("\\", "/", StringComparison.Ordinal);
						if (Workname.Contains("/.", StringComparison.Ordinal))
						{
							continue;
						}

						string DirectoryName = Path.GetDirectoryName(Filename)!.Substring(UnrealBuildGradlePathLength);
						string[] DirectoryNameComponents = DirectoryName.Split(Path.DirectorySeparatorChar, StringSplitOptions.RemoveEmptyEntries);
						int BuildDirIndex = Array.IndexOf(DirectoryNameComponents, "build");
						// if a build directory exists
						if (BuildDirIndex >= 0)
						{
							// get the directory component index where app/src/main (JavaSrcRootComponents) begins
							int JavaSrcRootIndex = Array.IndexOf(DirectoryNameComponents, JavaSrcRootComponents[0]);
							for (int c = 1; c < JavaSrcRootComponents.Length && JavaSrcRootIndex >= 0; ++c)
							{
								int NextComponentIndex = Array.IndexOf(DirectoryNameComponents, JavaSrcRootComponents[c]);
								if (NextComponentIndex != JavaSrcRootIndex + c)
								{
									JavaSrcRootIndex = -1;
								}
							}
							// if the said build directory is not beneath the Java src root
							if (JavaSrcRootIndex == -1 || BuildDirIndex < JavaSrcRootIndex)
							{
								continue;
							}
						}

						// if destination doesn't exist, just copy it
						if (!File.Exists(DestFilename))
						{
							string DestSubdir = Path.GetDirectoryName(DestFilename)!;
							if (!Directory.Exists(DestSubdir))
							{
								Directory.CreateDirectory(DestSubdir);
							}

							// copy it
							File.Copy(Filename, DestFilename);

							// preserve timestamp and clear read-only flags
							FileInfo DestFileInfo = new FileInfo(DestFilename);
							DestFileInfo.Attributes &= ~FileAttributes.ReadOnly;
							File.SetLastWriteTimeUtc(DestFilename, File.GetLastWriteTimeUtc(Filename));

							Logger.LogInformation("Copied file {DestFilename}.", DestFilename);
							continue;
						}

						if (FilesAreIdentical(Filename, DestFilename))
						{
							continue;
						}

						// ignore abi.gradle, we're going to generate a new one
						if (Filename.EndsWith("abi.gradle", StringComparison.Ordinal))
						{
							continue;
						}

						// ignore OBBData.java, we won't use it
						if (Filename.EndsWith("OBBData.java", StringComparison.Ordinal))
						{
							continue;
						}

						// deal with AndroidManifest.xml
						if (Filename.EndsWith("AndroidManifest.xml", StringComparison.Ordinal))
						{
							// only allowed to differ by versionCode
							string[] SourceManifest = File.ReadAllLines(Filename);
							string[] DestManifest = File.ReadAllLines(DestFilename);

							if (SourceManifest.Length == DestManifest.Length)
							{
								bool bDiffers = false;
								for (int Index = 0; Index < SourceManifest.Length; Index++)
								{
									if (SourceManifest[Index] == DestManifest[Index])
									{
										continue;
									}

									int SourceVersionIndex = SourceManifest[Index].IndexOf("android:versionCode=", StringComparison.Ordinal);
									if (SourceVersionIndex < 0)
									{
										bDiffers = true;
										break;
									}

									int DestVersionIndex = DestManifest[Index].IndexOf("android:versionCode=", StringComparison.Ordinal);
									if (DestVersionIndex < 0)
									{
										bDiffers = true;
										break;
									}

									int SourceVersionIndex2 = SourceManifest[Index].Substring(SourceVersionIndex + 22).IndexOf('\"', StringComparison.Ordinal);
									string FixedSource = SourceManifest[Index].Substring(0, SourceVersionIndex + 21) + SourceManifest[Index].Substring(SourceVersionIndex + 22 + SourceVersionIndex2);

									int DestVersionIndex2 = DestManifest[Index].Substring(DestVersionIndex + 22).IndexOf('\"', StringComparison.Ordinal);
									string FixedDest = SourceManifest[Index].Substring(0, DestVersionIndex + 21) + DestManifest[Index].Substring(DestVersionIndex + 22 + DestVersionIndex2);

									if (FixedSource != FixedDest)
									{
										bDiffers = true;
										break;
									}
								}
								if (!bDiffers)
								{
									continue;
								}
							}

							// differed too much
							bCombinedBundleOK = false;
							Logger.LogInformation("AndroidManifest.xml files differ too much to combine for single AAB: '{Filename}' != '{DestFilename}'", Filename, DestFilename);
							break;
						}

						// deal with buildAdditions.gradle
						if (Filename.EndsWith("buildAdditions.gradle", StringComparison.Ordinal))
						{
							// allow store filepath to differ
							string[] SourceProperties = File.ReadAllLines(Filename);
							string[] DestProperties = File.ReadAllLines(DestFilename);

							if (SourceProperties.Length == DestProperties.Length)
							{
								bool bDiffers = false;
								for (int Index = 0; Index < SourceProperties.Length; Index++)
								{
									if (SourceProperties[Index] == DestProperties[Index])
									{
										continue;
									}

									if (SourceProperties[Index].Contains("storeFile file(", StringComparison.Ordinal) && DestProperties[Index].Contains("storeFile file(", StringComparison.Ordinal))
									{
										continue;
									}

									bDiffers = true;
									break;
								}
								if (!bDiffers)
								{
									continue;
								}
							}

							// differed too much
							bCombinedBundleOK = false;
							Logger.LogInformation("buildAdditions.gradle files differ too much to combine for single AAB: '{Filename}' != '{DestFilename}'", Filename, DestFilename);
							break;
						}

						// deal with gradle.properties
						if (Filename.EndsWith("gradle.properties", StringComparison.Ordinal))
						{
							// allow STORE_VERSION and OUTPUT_FILENAME to differ
							// only allowed to differ by versionCode
							string[] SourceProperties = File.ReadAllLines(Filename);
							string[] DestProperties = File.ReadAllLines(DestFilename);

							if (SourceProperties.Length == DestProperties.Length)
							{
								bool bDiffers = false;
								for (int Index = 0; Index < SourceProperties.Length; Index++)
								{
									if (SourceProperties[Index] == DestProperties[Index])
									{
										continue;
									}

									if (SourceProperties[Index].StartsWith("STORE_VERSION=", StringComparison.Ordinal) && DestProperties[Index].StartsWith("STORE_VERSION=", StringComparison.Ordinal))
									{
										continue;
									}

									if (SourceProperties[Index].StartsWith("STORE_FILE=", StringComparison.Ordinal) && DestProperties[Index].StartsWith("STORE_FILE=", StringComparison.Ordinal))
									{
										continue;
									}

									if (SourceProperties[Index].StartsWith("DEBUG_STORE_FILE=", StringComparison.Ordinal) && DestProperties[Index].StartsWith("DEBUG_STORE_FILE=", StringComparison.Ordinal))
									{
										continue;
									}

									if (SourceProperties[Index].StartsWith("OUTPUT_FILENAME=", StringComparison.Ordinal) && DestProperties[Index].StartsWith("OUTPUT_FILENAME=", StringComparison.Ordinal))
									{
										continue;
									}

									if (SourceProperties[Index].StartsWith("OUTPUT_BUNDLEFILENAME=", StringComparison.Ordinal) && DestProperties[Index].StartsWith("OUTPUT_BUNDLEFILENAME=", StringComparison.Ordinal))
									{
										continue;
									}

									if (SourceProperties[Index].StartsWith("OUTPUT_UNIVERSALFILENAME=", StringComparison.Ordinal) && DestProperties[Index].StartsWith("OUTPUT_UNIVERSALFILENAME=", StringComparison.Ordinal))
									{
										continue;
									}

									bDiffers = true;
									break;
								}
								if (!bDiffers)
								{
									continue;
								}
							}

							// differed too much
							bCombinedBundleOK = false;
							Logger.LogInformation("gradle.properties files differ too much to combine for single AAB: '{Filename}' != '{DestFilename}'", Filename, DestFilename);
							break;
						}

						// there are unknown differences, cannot make a single AAB
						bCombinedBundleOK = false;
						Logger.LogInformation("Gradle projects differ too much to combine for single AAB: '{Filename}' != '{DestFilename}'", Filename, DestFilename);
						break;
					}
				}

				if (bCombinedBundleOK)
				{
					string NDKArch = Archs[0].NativeArch;

					string UnrealBuildGradlePath = UnrealGradleDest;

					// write a new abi.gradle
					StringBuilder ABIGradle = new StringBuilder();
					ABIGradle.AppendLine("android {");
					ABIGradle.AppendLine("\tdefaultConfig {");
					ABIGradle.AppendLine("\t\tndk {");
					ABIGradle.AppendLine(String.Format("\t\t\tabiFilters{0}", ABIFilter.Substring(1)));
					ABIGradle.AppendLine("\t\t}");
					ABIGradle.AppendLine("\t}");
					ABIGradle.AppendLine("}");
					string ABIGradleFilename = Path.Combine(UnrealGradleDest, "app", "abi.gradle");
					File.WriteAllText(ABIGradleFilename, ABIGradle.ToString());

					// update manifest to use versionCode properly
					string BaseStoreVersion = GetStoreVersion(null).ToString();
					string ManifestFilename = Path.Combine(UnrealBuildGradlePath, Path.Combine(JavaSrcRootComponents), "AndroidManifest.xml");
					string[] ManifestContents = File.ReadAllLines(ManifestFilename);
					for (int Index = 0; Index < ManifestContents.Length; Index++)
					{
						int ManifestVersionIndex = ManifestContents[Index].IndexOf("android:versionCode=", StringComparison.Ordinal);
						if (ManifestVersionIndex < 0)
						{
							continue;
						}

						int ManifestVersionIndex2 = ManifestContents[Index].Substring(ManifestVersionIndex + 22).IndexOf('\"', StringComparison.Ordinal);
						ManifestContents[Index] = ManifestContents[Index].Substring(0, ManifestVersionIndex + 21) + BaseStoreVersion + ManifestContents[Index].Substring(ManifestVersionIndex + 22 + ManifestVersionIndex2);
						break;
					}
					File.WriteAllLines(ManifestFilename, ManifestContents);

					bool bEnableUniversalAPK = false;
					Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bEnableUniversalAPK", out bEnableUniversalAPK);

					// update gradle.properties to set STORE_VERSION properly, and OUTPUT_BUNDLEFILENAME
					string GradlePropertiesFilename = Path.Combine(UnrealBuildGradlePath, "gradle.properties");
					string GradlePropertiesContent = File.ReadAllText(GradlePropertiesFilename);
					GradlePropertiesContent += String.Format("\nSTORE_VERSION={0}\nOUTPUT_BUNDLEFILENAME={1}\n", BaseStoreVersion,
						Path.GetFileNameWithoutExtension(OutputPath).Replace("UnrealGame", ProjectName, StringComparison.Ordinal) + ".aab");
					if (bEnableUniversalAPK)
					{
						GradlePropertiesContent += String.Format("OUTPUT_UNIVERSALFILENAME={0}\n",
							Path.GetFileNameWithoutExtension(OutputPath).Replace("UnrealGame", ProjectName, StringComparison.Ordinal) + "_universal.apk");
					}
					File.WriteAllText(GradlePropertiesFilename, GradlePropertiesContent);

					string GradleScriptPath = Path.Combine(UnrealBuildGradlePath, "gradlew");
					if (!OperatingSystem.IsWindows())
					{
						// fix permissions for Mac/Linux
						Logger.LogInformation("Fix gradlew permissions");
						File.SetUnixFileMode(GradleScriptPath, File.GetUnixFileMode(GradleScriptPath) | UnixFileMode.UserExecute | UnixFileMode.GroupExecute | UnixFileMode.OtherExecute);
					}
					else
					{
						if (CreateRunGradle(UnrealBuildGradlePath))
						{
							GradleScriptPath = Path.Combine(UnrealBuildGradlePath, "rungradle.bat");
						}
						else
						{
							GradleScriptPath = Path.Combine(UnrealBuildGradlePath, "gradlew.bat");
						}
					}

					string GradleBuildType = bForDistribution ? ":app:bundleRelease" : ":app:bundleDebug";

					// collect optional additional Gradle parameters from plugins
					string GradleOptions = UPL.ProcessPluginNode(NDKArch, "gradleParameters", GradleBuildType); //  "--stacktrace --debug " + GradleBuildType);
					string GradleSecondCallOptions = UPL.ProcessPluginNode(NDKArch, "gradleSecondCallParameters", "");

					// make sure destination exists
					//Directory.CreateDirectory(Path.GetDirectoryName(DestApkName));

					// Use gradle to build the .apk file
					string ShellExecutable = OperatingSystem.IsWindows() ? "cmd.exe" : "/bin/sh";
					string ShellParametersBegin = OperatingSystem.IsWindows() ? "/c " : "-c '";
					string ShellParametersEnd = OperatingSystem.IsWindows() ? "" : "'";
					RunCommandLineProgramWithExceptionAndFiltering(UnrealBuildGradlePath, ShellExecutable, ShellParametersBegin + "\"" + GradleScriptPath + "\" " + GradleOptions + ShellParametersEnd, Logger, "Making .aab with Gradle...");

					if (!String.IsNullOrEmpty(GradleSecondCallOptions))
					{
						RunCommandLineProgramWithExceptionAndFiltering(UnrealBuildGradlePath, ShellExecutable, ShellParametersBegin + "\"" + GradleScriptPath + "\" " + GradleSecondCallOptions + ShellParametersEnd, Logger, "Additional Gradle steps...");
					}

					// For build machine run a clean afterward to clean up intermediate files (does not remove final APK)
					if (bIsBuildMachine)
					{
						//GradleOptions = "tasks --all";
						//RunCommandLineProgramWithException(UnrealBuildGradlePath, ShellExecutable, ShellParametersBegin + "\"" + GradleScriptPath + "\" " + GradleOptions + ShellParametersEnd, "Listing all tasks...");

						GradleOptions = "clean";
						RunCommandLineProgramWithExceptionAndFiltering(UnrealBuildGradlePath, ShellExecutable, ShellParametersBegin + "\"" + GradleScriptPath + "\" " + GradleOptions + ShellParametersEnd, Logger, "Cleaning Gradle intermediates...");
					}
				}
				else
				{
					// generate an AAB for each architecture separately, was unable to merge
					foreach (Tuple<UnrealArch, string> build in BuildList)
					{
						UnrealArch Arch = build.Item1;
						string Manifest = build.Item2;
						string NDKArch = GetNDKArch(Arch);

						Logger.LogInformation("\n===={Time}====GENERATING BUNDLE====={Arch}================================================================", DateTime.Now.ToString(), Arch);

						string UnrealBuildPath = Path.Combine(IntermediateAndroidPath, Arch.ToString());
						string UnrealBuildGradlePath = Path.Combine(UnrealBuildPath, "gradle");

						string GradleScriptPath = Path.Combine(UnrealBuildGradlePath, "gradlew");
						if (!OperatingSystem.IsWindows())
						{
							// fix permissions for Mac/Linux
							Logger.LogInformation("Fix gradlew permissions");
							File.SetUnixFileMode(GradleScriptPath, File.GetUnixFileMode(GradleScriptPath) | UnixFileMode.UserExecute | UnixFileMode.GroupExecute | UnixFileMode.OtherExecute);
						}
						else
						{
							if (CreateRunGradle(UnrealBuildGradlePath))
							{
								GradleScriptPath = Path.Combine(UnrealBuildGradlePath, "rungradle.bat");
							}
							else
							{
								GradleScriptPath = Path.Combine(UnrealBuildGradlePath, "gradlew.bat");
							}
						}

						string GradleBuildType = bForDistribution ? ":app:bundleRelease" : ":app:bundleDebug";

						// collect optional additional Gradle parameters from plugins
						string GradleOptions = UPL.ProcessPluginNode(NDKArch, "gradleParameters", GradleBuildType); //  "--stacktrace --debug " + GradleBuildType);
						string GradleSecondCallOptions = UPL.ProcessPluginNode(NDKArch, "gradleSecondCallParameters", "");

						// make sure destination exists
						//Directory.CreateDirectory(Path.GetDirectoryName(DestApkName));

						// Use gradle to build the .apk file
						string ShellExecutable = OperatingSystem.IsWindows() ? "cmd.exe" : "/bin/sh";
						string ShellParametersBegin = OperatingSystem.IsWindows() ? "/c " : "-c '";
						string ShellParametersEnd = OperatingSystem.IsWindows() ? "" : "'";
						RunCommandLineProgramWithExceptionAndFiltering(UnrealBuildGradlePath, ShellExecutable, ShellParametersBegin + "\"" + GradleScriptPath + "\" " + GradleOptions + ShellParametersEnd, Logger, "Making .aab with Gradle...");

						if (!String.IsNullOrEmpty(GradleSecondCallOptions))
						{
							RunCommandLineProgramWithExceptionAndFiltering(UnrealBuildGradlePath, ShellExecutable, ShellParametersBegin + "\"" + GradleScriptPath + "\" " + GradleSecondCallOptions + ShellParametersEnd, Logger, "Additional Gradle steps...");
						}

						// For build machine run a clean afterward to clean up intermediate files (does not remove final APK)
						if (bIsBuildMachine)
						{
							//GradleOptions = "tasks --all";
							//RunCommandLineProgramWithException(UnrealBuildGradlePath, ShellExecutable, ShellParametersBegin + "\"" + GradleScriptPath + "\" " + GradleOptions + ShellParametersEnd, "Listing all tasks...");

							GradleOptions = "clean";
							RunCommandLineProgramWithExceptionAndFiltering(UnrealBuildGradlePath, ShellExecutable, ShellParametersBegin + "\"" + GradleScriptPath + "\" " + GradleOptions + ShellParametersEnd, Logger, "Cleaning Gradle intermediates...");
						}
					}
				}
			}

			Logger.LogInformation("\n===={Time}====COMPLETED MAKE APK=======================================================================", DateTime.Now.ToString());
		}

		public static string GetAFSExecutable(UnrealTargetPlatform Target, ILogger Logger)
		{
			if (Target == UnrealTargetPlatform.Win64)
			{
				return "win-x64/UnrealAndroidFileTool.exe";
			}
			if (Target == UnrealTargetPlatform.Mac)
			{
				return "osx-x64/UnrealAndroidFileTool";
			}
			if (Target == UnrealTargetPlatform.Linux)
			{
				return "linux-x64/UnrealAndroidFileTool";
			}
			Logger.LogWarning("GetAFSExecutable unsupported target, assuming Win64");
			return "win-x64/UnrealAndroidFileTool.exe";
		}

		private List<string> CollectPluginDataPaths(TargetReceipt Receipt)
		{
			List<string> PluginExtras = new List<string>();
			if (Receipt == null)
			{
				Logger.LogInformation("Receipt is NULL");
				return PluginExtras;
			}

			// collect plugin extra data paths from target receipt
			IEnumerable<ReceiptProperty> Results = Receipt.AdditionalProperties.Where(x => x.Name == "AndroidPlugin");
			foreach (ReceiptProperty Property in Results)
			{
				// Keep only unique paths
				string PluginPath = Property.Value;
				if (PluginExtras.FirstOrDefault(x => x == PluginPath) == null)
				{
					PluginExtras.Add(PluginPath);
					Logger.LogInformation("AndroidPlugin: {PluginPath}", PluginPath);
				}
			}
			return PluginExtras;
		}

		public override bool PrepTargetForDeployment(TargetReceipt Receipt)
		{
			//Logger.LogInformation("$$$$$$$$$$$$$$ PrepTargetForDeployment $$$$$$$$$$$$$$$$$");

			DirectoryReference ProjectDirectory = DirectoryReference.FromFile(ProjectFile) ?? Unreal.EngineDirectory;
			string TargetName = (ProjectFile == null ? Receipt.TargetName : ProjectFile.GetFileNameWithoutAnyExtensions());

			AndroidToolChain ToolChain = (AndroidToolChain)((AndroidPlatform)UEBuildPlatform.GetBuildPlatform(Receipt.Platform)).CreateTempToolChainForProject(ProjectFile);

			// get the receipt
			SetAndroidPluginData(Receipt.Architectures, CollectPluginDataPaths(Receipt));

			bool bShouldCompileAsDll = Receipt.HasValueForAdditionalProperty("CompileAsDll", "true");

			SavePackageInfo(TargetName, ProjectDirectory.FullName, Receipt.TargetType, bShouldCompileAsDll);

			// Get the output paths
			BuildProductType ProductType = bShouldCompileAsDll ? BuildProductType.DynamicLibrary : BuildProductType.Executable;
			List<FileReference> OutputPaths = Receipt.BuildProducts.Where(x => x.Type == ProductType).Select(x => x.Path).ToList();
			if (OutputPaths.Count < 1)
			{
				throw new BuildException("Target file does not contain either executable or dynamic library .so");
			}

			// we need to strip architecture from any of the output paths
			string BaseSoName = AndroidToolChain.RemoveArchName(OutputPaths[0].FullName);

			// make an apk at the end of compiling, so that we can run without packaging (debugger, cook on the fly, etc)
			string RelativeEnginePath = Unreal.EngineDirectory.MakeRelativeTo(DirectoryReference.GetCurrentDirectory());

			MakeApk(ToolChain, TargetName, Receipt.TargetType, ProjectDirectory.FullName, BaseSoName, RelativeEnginePath, bForDistribution: false, CookFlavor: "", Configuration: Receipt.Configuration,
				bMakeSeparateApks: ShouldMakeSeparateApks(), bDisallowPackagingDataInApk: false, bDisallowExternalFilesDir: true, bSkipGradleBuild: bShouldCompileAsDll, bIsArchive: false, bIsFromUAT: false);

			// if we made any non-standard .apk files, the generated debugger settings may be wrong
			if (ShouldMakeSeparateApks() && (OutputPaths.Count > 1 || !OutputPaths[0].FullName.Contains("-armv7", StringComparison.Ordinal)))
			{
				Logger.LogInformation("================================================================================================================================");
				Logger.LogInformation("Non-default apk(s) have been made: If you are debugging, you will need to manually select one to run in the debugger properties!");
				Logger.LogInformation("================================================================================================================================");
			}
			return true;
		}

		// Store generated package name in a text file for builds that do not generate an apk file 
		public bool SavePackageInfo(string TargetName, string ProjectDirectory, TargetType InTargetType, bool bIsEmbedded)
		{
			string PackageName = GetPackageName(TargetName);
			string DestPackageNameFileName = Path.Combine(ProjectDirectory, "Binaries", "Android", "packageInfo.txt");

			string[] PackageInfoSource = new string[4];
			PackageInfoSource[0] = PackageName;
			PackageInfoSource[1] = GetStoreVersion(null).ToString();
			PackageInfoSource[2] = GetVersionDisplayName(bIsEmbedded);
			PackageInfoSource[3] = String.Format("name='com.epicgames.unreal.GameActivity.AppType' value='{0}'", InTargetType == TargetType.Game ? "" : InTargetType.ToString());

			Logger.LogInformation("Writing packageInfo pkgName:{PkgName} storeVersion:{StoreVer} versionDisplayName:{Version} to {DestFile}", PackageInfoSource[0], PackageInfoSource[1], PackageInfoSource[2], DestPackageNameFileName);

			string DestDirectory = Path.GetDirectoryName(DestPackageNameFileName)!;
			if (!Directory.Exists(DestDirectory))
			{
				Directory.CreateDirectory(DestDirectory);
			}

			File.WriteAllLines(DestPackageNameFileName, PackageInfoSource);

			return true;
		}

		public static bool ShouldMakeSeparateApks()
		{
			// @todo android fat binary: Currently, there isn't much utility in merging multiple .so's into a single .apk except for debugging,
			// but we can't properly handle multiple GPU architectures in a single .apk, so we are disabling the feature for now
			// The user will need to manually select the apk to run in their Visual Studio debugger settings (see Override APK in TADP, for instance)
			// If we change this, pay attention to <OverrideAPKPath> in AndroidProjectGenerator
			return true;

			// check to see if the project wants separate apks
			// 			ConfigCacheIni Ini = nGetConfigCacheIni("Engine");
			// 			bool bSeparateApks = false;
			// 			Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bSplitIntoSeparateApks", out bSeparateApks);
			// 
			// 			return bSeparateApks;
		}

		public bool PrepForUATPackageOrDeploy(FileReference ProjectFile, string ProjectName, DirectoryReference ProjectDirectory, string ExecutablePath, string EngineDirectory, bool bForDistribution, string CookFlavor, UnrealTargetConfiguration Configuration, bool bIsDataDeploy, bool bSkipGradleBuild, bool bIsArchive)
		{
			//Logger.LogInformation("$$$$$$$$$$$$$$ PrepForUATPackageOrDeploy $$$$$$$$$$$$$$$$$");

			TargetType Type = TargetType.Game;
			if (CookFlavor.EndsWith("Client", StringComparison.Ordinal))
			{
				Type = TargetType.Client;
			}
			else if (CookFlavor.EndsWith("Server", StringComparison.Ordinal))
			{
				Type = TargetType.Server;
			}

			// note that we cannot allow the data packaged into the APK if we are doing something like Launch On that will not make an obb
			// file and instead pushes files directly via deploy
			AndroidTargetRules TargetRules = new AndroidTargetRules();
			CommandLine.ParseArguments(Environment.GetCommandLineArgs(), TargetRules, Logger);
			ClangToolChainOptions Options = AndroidPlatform.CreateToolChainOptions(TargetRules);
			AndroidToolChain ToolChain = new AndroidToolChain(ProjectFile, Options, Logger);

			SavePackageInfo(ProjectName, ProjectDirectory.FullName, Type, bSkipGradleBuild);

			MakeApk(ToolChain, ProjectName, Type, ProjectDirectory.FullName, ExecutablePath, EngineDirectory, bForDistribution: bForDistribution, CookFlavor: CookFlavor, Configuration: Configuration,
				bMakeSeparateApks: ShouldMakeSeparateApks(), bDisallowPackagingDataInApk: bIsDataDeploy, bDisallowExternalFilesDir: bIsDataDeploy,
				bSkipGradleBuild: bSkipGradleBuild, bIsArchive: bIsArchive, bIsFromUAT: true);
			return true;
		}

		public static void OutputReceivedDataEventHandler(DataReceivedEventArgs Line, ILogger Logger)
		{
			if ((Line != null) && (Line.Data != null))
			{
				Logger.LogInformation("{Output}", Line.Data);
			}
		}

		private static string GenerateTemplatesHashCode(string EngineDir)
		{
			string SourceDirectory = Path.Combine(EngineDir, "Build", "Android", "Java");

			if (!Directory.Exists(SourceDirectory))
			{
				return "badpath";
			}

#pragma warning disable CA5351 // Not used for cryptography, would be disruptive to users to change hash algo
			MD5 md5 = MD5.Create();
#pragma warning restore CA5351
			byte[]? TotalHashBytes = null;

			string[] SourceFiles = Directory.GetFiles(SourceDirectory, "*.*", SearchOption.AllDirectories);
			foreach (string Filename in SourceFiles)
			{
				using (FileStream stream = File.OpenRead(Filename))
				{
					byte[] FileHashBytes = md5.ComputeHash(stream);
					if (TotalHashBytes != null)
					{
						int index = 0;
						foreach (byte b in FileHashBytes)
						{
							TotalHashBytes[index] ^= b;
							index++;
						}
					}
					else
					{
						TotalHashBytes = FileHashBytes;
					}
				}
			}

			if (TotalHashBytes != null)
			{
				string HashCode = "";
				foreach (byte b in TotalHashBytes)
				{
					HashCode += b.ToString("x2");
				}
				return HashCode;
			}

			return "empty";
		}

		private void UpdateGameActivity(string NDKArch, string EngineDir, string UnrealBuildPath, bool bDontBundleLibrariesInAPK,
				bool bJavaPerfettoMarkers, UnrealTargetConfiguration Configuration, bool bHasSplashActivity)
		{
			string SourceFilename = Path.Combine(EngineDir, "Build", "Android", "Java", "src", "com", "epicgames", "unreal", "GameActivity.java.template");
			string DestFilename = Path.Combine(UnrealBuildPath, "src", "com", "epicgames", "unreal", "GameActivity.java");

			// check for GameActivity.java.template override
			SourceFilename = UPL!.ProcessPluginNode(NDKArch, "gameActivityReplacement", SourceFilename);

			ConfigHierarchy Ini = GetConfigCacheIni(ConfigHierarchyType.Engine);

			string GameActivityImportAdditionsDefault = "";
			string GameActivityClassAdditionsDefault = "";
			string LoadUnrealDefault;
			string LoadLibraryDefaults = "";

			string SuperClassDefault;
			if (!Ini.GetString("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "GameActivitySuperClass", out SuperClassDefault))
			{
				SuperClassDefault = UPL.ProcessPluginNode(NDKArch, "gameActivitySuperClass", "");
				if (String.IsNullOrEmpty(SuperClassDefault))
				{
					SuperClassDefault = "NativeActivity";
				}
			}

			if (bDontBundleLibrariesInAPK)
			{
				LoadUnrealDefault = @"		System.load(GameApplication.getAppContext().getFilesDir().getAbsolutePath() + ""/libUnreal.so"");";

				GameActivityImportAdditionsDefault = @"
import dalvik.system.BaseDexClassLoader;
import java.util.Collection;
";

				GameActivityClassAdditionsDefault = @"
	@Override
	public ClassLoader getClassLoader() {
		ClassLoader baseClassLoader = super.getClassLoader();

		try {
			BaseDexClassLoader dexClassLoader = (BaseDexClassLoader) baseClassLoader;
			if (dexClassLoader.findLibrary(""Unreal"") == null) {
				Field pathListField = BaseDexClassLoader.class.getDeclaredField(""pathList"");
				pathListField.setAccessible(true);

				Object pathListObj = pathListField.get(dexClassLoader);

				Method addNativePathMethod = pathListObj.getClass().getMethod(""addNativePath"", Collection.class);

				String filesDir = getApplicationContext().getFilesDir().getAbsolutePath();
				Log.verbose(""Adding '"" + filesDir + ""' to GameActivity BaseDexClassLoader"");
				Collection<String> paths = Arrays.asList(filesDir);
				addNativePathMethod.invoke(pathListObj, paths);
			}
		} catch (Exception e) {
			Log.warn(""Failed to add native library path due to "" + e);
		}

		return baseClassLoader;
	}
";
			}
			else
			{
				LoadUnrealDefault = @"		System.loadLibrary(""Unreal"");";
			}

			if (bJavaPerfettoMarkers)
			{
				GameActivityImportAdditionsDefault += "import androidx.tracing.Trace;\n";
			}

			Dictionary<string, string> Replacements = new Dictionary<string, string>{
				{ "//$${gameActivityMemStatAdditions}$$", UPL.ProcessPluginNode(NDKArch, "gameActivityMemStatAdditions", "")},
				{ "//$${gameActivityImportAdditions}$$", UPL.ProcessPluginNode(NDKArch, "gameActivityImportAdditions", GameActivityImportAdditionsDefault)},
				{ "//$${gameActivityPostImportAdditions}$$", UPL.ProcessPluginNode(NDKArch, "gameActivityPostImportAdditions", "")},
				{ "//$${gameActivityImplementsAdditions}$$", UPL.ProcessPluginNode(NDKArch, "gameActivityImplementsAdditions", "")},
				{ "//$${gameActivityClassAdditions}$$", UPL.ProcessPluginNode(NDKArch, "gameActivityClassAdditions", GameActivityClassAdditionsDefault)},
				{ "//$${gameActivityReadMetadataAdditions}$$", UPL.ProcessPluginNode(NDKArch, "gameActivityReadMetadataAdditions", "")},
				{ "//$${gameActivityOnCreateBeforeProcessSystemInfoAdditions}$$", UPL.ProcessPluginNode(NDKArch, "gameActivityOnCreateBeforeProcessSystemInfoAdditions", "")},
				{ "//$${gameActivityProcessSystemInfoThreadBeforeConfigRulesAdditions}$$", UPL.ProcessPluginNode(NDKArch, "gameActivityProcessSystemInfoThreadBeforeConfigRulesAdditions", "")},
				{ "//$${gameActivityProcessSystemInfoThreadAfterConfigRulesAdditions}$$", UPL.ProcessPluginNode(NDKArch, "gameActivityProcessSystemInfoThreadAfterConfigRulesAdditions", "")},
				{ "//$${gameActivityOnCreateBeforeLoadLibrariesAdditions}$$", UPL.ProcessPluginNode(NDKArch, "gameActivityOnCreateBeforeLoadLibrariesAdditions", "")},
				{ "//$${gameActivityOnCreateBeginningAdditions}$$", UPL.ProcessPluginNode(NDKArch, "gameActivityOnCreateBeginningAdditions", "")},
				{ "//$${gameActivityOnCreateAdditions}$$", UPL.ProcessPluginNode(NDKArch, "gameActivityOnCreateAdditions", "")},
				{ "//$${gameActivityOnCreateFinalAdditions}$$", UPL.ProcessPluginNode(NDKArch, "gameActivityOnCreateFinalAdditions", "")},
				{ "//$${gameActivityOverrideAPKOBBPackaging}$$", UPL.ProcessPluginNode(NDKArch, "gameActivityOverrideAPKOBBPackaging", "")},
				{ "//$${gameActivityOnDestroyAdditions}$$", UPL.ProcessPluginNode(NDKArch, "gameActivityOnDestroyAdditions", "")},
				{ "//$${gameActivityonConfigurationChangedAdditions}$$", UPL.ProcessPluginNode(NDKArch, "gameActivityonConfigurationChangedAdditions", "")},
				{ "//$${gameActivityOnStartAdditions}$$", UPL.ProcessPluginNode(NDKArch, "gameActivityOnStartAdditions", "")},
				{ "//$${gameActivityOnStopAdditions}$$", UPL.ProcessPluginNode(NDKArch, "gameActivityOnStopAdditions", "")},
				{ "//$${gameActivityOnRestartAdditions}$$", UPL.ProcessPluginNode(NDKArch, "gameActivityOnRestartAdditions", "")},
				{ "//$${gameActivityOnSaveInstanceStateAdditions}$$", UPL.ProcessPluginNode(NDKArch, "gameActivityOnSaveInstanceStateAdditions", "")},
				{ "//$${gameActivityOnRequestPermissionsResultAdditions}$$", UPL.ProcessPluginNode(NDKArch, "gameActivityOnRequestPermissionsResultAdditions", "")},
				{ "//$${gameActivityOnPauseAdditions}$$", UPL.ProcessPluginNode(NDKArch, "gameActivityOnPauseAdditions", "")},
				{ "//$${gameActivityOnResumeAdditions}$$", UPL.ProcessPluginNode(NDKArch, "gameActivityOnResumeAdditions", "")},
				{ "//$${gameActivityOnTopResumeActivityAdditions}$$", UPL.ProcessPluginNode(NDKArch, "gameActivityOnTopResumeActivityAdditions", "")},
				{ "//$${gameActivityOnNewIntentAdditions}$$", UPL.ProcessPluginNode(NDKArch, "gameActivityOnNewIntentAdditions", "")},
  				{ "//$${gameActivityOnActivityResultAdditions}$$", UPL.ProcessPluginNode(NDKArch, "gameActivityOnActivityResultAdditions", "")},
  				{ "//$${gameActivityPreConfigRulesParseAdditions}$$", UPL.ProcessPluginNode(NDKArch, "gameActivityPreConfigRulesParseAdditions", "")},
  				{ "//$${gameActivityPostConfigRulesAdditions}$$", UPL.ProcessPluginNode(NDKArch, "gameActivityPostConfigRulesAdditions", "")},
  				{ "//$${gameActivityFinalizeConfigRulesAdditions}$$", UPL.ProcessPluginNode(NDKArch, "gameActivityFinalizeConfigRulesAdditions", "")},
				{ "//$${gameActivityBeforeConfigRulesAppliedAdditions}$$", UPL.ProcessPluginNode(NDKArch, "gameActivityBeforeConfigRulesAppliedAdditions", "")},
				{ "//$${gameActivityAfterMainViewCreatedAdditions}$$", UPL.ProcessPluginNode(NDKArch, "gameActivityAfterMainViewCreatedAdditions", "")},
				{ "//$${gameActivityResizeKeyboardAdditions}$$", UPL.ProcessPluginNode(NDKArch, "gameActivityResizeKeyboardAdditions", "")},
				{ "//$${gameActivityLoggerCallbackAdditions}$$", UPL.ProcessPluginNode(NDKArch, "gameActivityLoggerCallbackAdditions", "")},
				{ "//$${gameActivityGetCommandLineAdditions}$$", UPL.ProcessPluginNode(NDKArch, "gameActivityGetCommandLineAdditions", "")},
				{ "//$${gameActivityGetLoginIdAdditions}$$", UPL.ProcessPluginNode(NDKArch, "gameActivityGetLoginIdAdditions", "")},
				{ "//$${gameActivityGetFunnelIdAdditions}$$", UPL.ProcessPluginNode(NDKArch, "gameActivityGetFunnelIdAdditions", "")},
				{ "//$${gameActivityAllowedRemoteNotificationsAdditions}$$", UPL.ProcessPluginNode(NDKArch, "gameActivityAllowedRemoteNotificationsAdditions", "")},
				{ "//$${gameActivityAndroidThunkJavaIapBeginPurchase}$$", UPL.ProcessPluginNode(NDKArch, "gameActivityAndroidThunkJavaIapBeginPurchase", "")},
				{ "//$${gameActivityIapSetupServiceAdditions}$$", UPL.ProcessPluginNode(NDKArch, "gameActivityIapSetupServiceAdditions", "")},
				{ "//$${gameActivityOnRestartApplicationAdditions}$$", UPL.ProcessPluginNode(NDKArch, "gameActivityOnRestartApplicationAdditions", "")},
				{ "//$${gameActivityForceQuitAdditions}$$", UPL.ProcessPluginNode(NDKArch, "gameActivityForceQuitAdditions", "")},
				{ "//$${gameActivitySetCrashContextData}$$", UPL.ProcessPluginNode(NDKArch, "gameActivitySetCrashContextData", "")},
				{ "//$${soLoadUnreal}$$", UPL.ProcessPluginNode(NDKArch, "soLoadUnreal", LoadUnrealDefault)},
				{ "//$${soLoadLibrary}$$", UPL.ProcessPluginNode(NDKArch, "soLoadLibrary", LoadLibraryDefaults)},
				{ "$${gameActivitySuperClass}$$", SuperClassDefault},
				{ "$${gameActivityIsShippingConfig}$$", Configuration == UnrealTargetConfiguration.Shipping ? "true" : "false"},
				{ "$${gameActivityHasSplashActivity}$$", bHasSplashActivity ? "true" : "false"},
			};

			string[] TemplateSrc = File.ReadAllLines(SourceFilename);
			string[]? TemplateDest = File.Exists(DestFilename) ? File.ReadAllLines(DestFilename) : null;

			bool TemplateChanged = false;
			for (int LineIndex = 0; LineIndex < TemplateSrc.Length; ++LineIndex)
			{
				string SrcLine = TemplateSrc[LineIndex];
				bool Changed = false;

				if (!bJavaPerfettoMarkers)
				{
					if (SrcLine.StartsWith("Trace.", StringComparison.Ordinal))
					{
						TemplateSrc[LineIndex] = "";
						Changed = true;
						TemplateChanged = true;
					}
				}

				if (!Changed)
				{
					foreach (KeyValuePair<string, string> KVP in Replacements)
					{
						if (SrcLine.Contains(KVP.Key, StringComparison.Ordinal))
						{
							if (bJavaPerfettoMarkers)
							{
								SrcLine = SrcLine.Replace(KVP.Key, KVP.Value, StringComparison.Ordinal);
							}
							else
							{
								int TraceStartIndex = KVP.Value.IndexOf("Trace.", StringComparison.Ordinal);
								if (TraceStartIndex == 0 || (TraceStartIndex > 0 && KVP.Value[TraceStartIndex - 1] == '\n'))
								{
									string Addition = KVP.Value;
									do
									{
										int TraceEndIndex = Addition.IndexOf("\n", TraceStartIndex, StringComparison.Ordinal);
										if (TraceEndIndex < 0)
										{
											Addition = Addition.Substring(0, TraceStartIndex);
										}
										else
										{
											Addition = Addition.Substring(0, TraceStartIndex) + Addition.Substring(TraceEndIndex + 1);
										}
										TraceStartIndex = Addition.IndexOf("Trace.", TraceStartIndex, StringComparison.Ordinal);
									} while (TraceStartIndex >= 0);
									SrcLine = SrcLine.Replace(KVP.Key, Addition, StringComparison.Ordinal);
								}
								else
								{
									SrcLine = SrcLine.Replace(KVP.Key, KVP.Value, StringComparison.Ordinal);
								}
							}
							Changed = true;
						}
					}
					if (Changed)
					{
						TemplateSrc[LineIndex] = SrcLine;
						TemplateChanged = true;
					}
				}
			}

			if (TemplateChanged)
			{
				// deal with insertions of newlines
				TemplateSrc = String.Join("\n", TemplateSrc).Split(new[] { "\r\n", "\r", "\n" }, StringSplitOptions.None);
			}

			if (TemplateDest == null || TemplateSrc.Length != TemplateDest.Length || !TemplateSrc.SequenceEqual(TemplateDest))
			{
				Logger.LogInformation("");
				Logger.LogInformation("==== Writing new GameActivity.java file to {DestFile} ====", DestFilename);
				File.WriteAllLines(DestFilename, TemplateSrc);
			}
		}

		private void UpdateGameApplication(string NDKArch, string EngineDir, string UnrealBuildPath,
				bool bJavaPerfettoMarkers, UnrealTargetConfiguration Configuration, string SOName)
		{
			string SourceFilename = Path.Combine(EngineDir, "Build", "Android", "Java", "src", "com", "epicgames", "unreal", "GameApplication.java.template");
			string DestFilename = Path.Combine(UnrealBuildPath, "src", "com", "epicgames", "unreal", "GameApplication.java");

			string GameApplicationImportAdditionsDefault = "";
			if (bJavaPerfettoMarkers)
			{
				GameApplicationImportAdditionsDefault += "import androidx.tracing.Trace;\n";
			}

			// add dependencies to early load during SplashActivity (optional)
			List<string> UnrealDependencies = GetSODependencies(SOName);
			string GameApplicationEarlyLoad = "";
			if (UnrealDependencies.Count > 0)
			{
				// Make sure these are first then release the lock
				UnrealDependencies.Remove("GLESv3");
				UnrealDependencies.Remove("EGL");
				GameApplicationEarlyLoad += $"Trace.beginSection(\"GLESv3\");\n\t\t\tSystem.loadLibrary(\"GLESv3\");\nTrace.endSection();\n";
				GameApplicationEarlyLoad += $"Trace.beginSection(\"EGL\");\n\t\t\tSystem.loadLibrary(\"EGL\");\nTrace.endSection();\n";
				GameApplicationEarlyLoad += $"\t\t\tProcessSystemInfoLock.unlock();\n";

				foreach (string SharedLibrary in UnrealDependencies)
				{
					GameApplicationEarlyLoad += $"Trace.beginSection(\"{SharedLibrary}\");\n\t\t\tSystem.loadLibrary(\"{SharedLibrary}\");\nTrace.endSection();\n";
				}
			}

			Dictionary<string, string> Replacements = new Dictionary<string, string>{
				{ "//$${gameApplicationImportAdditions}$$", UPL!.ProcessPluginNode(NDKArch, "gameApplicationImportAdditions", GameApplicationImportAdditionsDefault)},
				{ "//$${gameApplicationImplementsAdditions}$$", UPL!.ProcessPluginNode(NDKArch, "gameApplicationImplementsAdditions", "")},
				{ "//$${gameApplicationOnCreateAdditions}$$", UPL.ProcessPluginNode(NDKArch, "gameApplicationOnCreateAdditions", "")},
				{ "//$${gameApplicationOnCreateAdditionsAlways}$$", UPL.ProcessPluginNode(NDKArch, "gameApplicationOnCreateAdditionsAlways", "")},
				{ "//$${gameApplicationAttachBaseContextAdditions}$$", UPL.ProcessPluginNode(NDKArch, "gameApplicationAttachBaseContextAdditions", "")},
				{ "//$${gameApplicationOnLowMemoryAdditions}$$", UPL.ProcessPluginNode(NDKArch, "gameApplicationOnLowMemoryAdditions", "")},
				{ "//$${gameApplicationOnTrimMemoryAdditions}$$", UPL.ProcessPluginNode(NDKArch, "gameApplicationOnTrimMemoryAdditions", "")},
				{ "//$${gameApplicationOnConfigurationChangedAdditions}$$", UPL.ProcessPluginNode(NDKArch, "gameApplicationOnConfigurationChangedAdditions", "")},
				{ "//$${gameApplicationClassAdditions}$$", UPL.ProcessPluginNode(NDKArch, "gameApplicationClassAdditions", "")},
				{ "$${gameApplicationIsShippingConfig}$$", Configuration == UnrealTargetConfiguration.Shipping ? "true" : "false"},
				{ "//$${gameApplicationEarlyLoadAdditions}$$", GameApplicationEarlyLoad},
			};

			string[] TemplateSrc = File.ReadAllLines(SourceFilename);
			string[]? TemplateDest = File.Exists(DestFilename) ? File.ReadAllLines(DestFilename) : null;

			bool TemplateChanged = false;
			for (int LineIndex = 0; LineIndex < TemplateSrc.Length; ++LineIndex)
			{
				string SrcLine = TemplateSrc[LineIndex];
				bool Changed = false;

				if (!bJavaPerfettoMarkers)
				{
					if (SrcLine.StartsWith("Trace.", StringComparison.Ordinal))
					{
						TemplateSrc[LineIndex] = "";
						Changed = true;
						TemplateChanged = true;
					}
				}

				if (!Changed)
				{
					foreach (KeyValuePair<string, string> KVP in Replacements)
					{
						if (SrcLine.Contains(KVP.Key, StringComparison.Ordinal))
						{
							if (bJavaPerfettoMarkers)
							{
								SrcLine = SrcLine.Replace(KVP.Key, KVP.Value, StringComparison.Ordinal);
							}
							else
							{
								int TraceStartIndex = KVP.Value.IndexOf("Trace.", StringComparison.Ordinal);
								if (TraceStartIndex == 0 || (TraceStartIndex > 0 && KVP.Value[TraceStartIndex - 1] == '\n'))
								{
									string Addition = KVP.Value;
									do
									{
										int TraceEndIndex = Addition.IndexOf("\n", TraceStartIndex, StringComparison.Ordinal);
										if (TraceEndIndex < 0)
										{
											Addition = Addition.Substring(0, TraceStartIndex);
										}
										else
										{
											Addition = Addition.Substring(0, TraceStartIndex) + Addition.Substring(TraceEndIndex + 1);
										}
										TraceStartIndex = Addition.IndexOf("Trace.", TraceStartIndex, StringComparison.Ordinal);
									} while (TraceStartIndex >= 0);
									SrcLine = SrcLine.Replace(KVP.Key, Addition, StringComparison.Ordinal);
								}
								else
								{
									SrcLine = SrcLine.Replace(KVP.Key, KVP.Value, StringComparison.Ordinal);
								}
							}
							Changed = true;
						}
					}
					if (Changed)
					{
						TemplateSrc[LineIndex] = SrcLine;
						TemplateChanged = true;
					}
				}
			}

			if (TemplateChanged)
			{
				// deal with insertions of newlines
				TemplateSrc = String.Join("\n", TemplateSrc).Split(new[] { "\r\n", "\r", "\n" }, StringSplitOptions.None);
			}

			if (TemplateDest == null || TemplateSrc.Length != TemplateDest.Length || !TemplateSrc.SequenceEqual(TemplateDest))
			{
				Logger.LogInformation("");
				Logger.LogInformation("==== Writing new GameApplication.java file to {DestFile} ====", DestFilename);
				File.WriteAllLines(DestFilename, TemplateSrc);
			}
		}

		private void CreateAdditonalBuildPathFiles(string NDKArch, string UnrealBuildPath, XDocument FilesToAdd)
		{
			Dictionary<string, string?> PathsAndRootEls = new Dictionary<string, string?>();

			foreach (XElement Element in FilesToAdd.Root!.Elements())
			{
				string RelPath = Element.Value;
				if (RelPath != null)
				{
					XAttribute? TypeAttr = Element.Attribute("rootEl");
					PathsAndRootEls[RelPath] = TypeAttr?.Value;
				}
			}

			foreach (KeyValuePair<string, string?> Entry in PathsAndRootEls)
			{
				string UPLNodeName = Entry.Key.Replace("/", "__", StringComparison.Ordinal).Replace(".", "__", StringComparison.Ordinal);
				string Content;
				if (Entry.Value == null)
				{
					// no root element, assume not XML
					Content = UPL!.ProcessPluginNode(NDKArch, UPLNodeName, "");
				}
				else
				{
					XDocument ContentDoc = new XDocument(new XElement(Entry.Value));
					UPL!.ProcessPluginNode(NDKArch, UPLNodeName, "", ref ContentDoc);
					Content = XML_HEADER + "\n" + ContentDoc.ToString();
				}

				string DestPath = Path.Combine(UnrealBuildPath, Entry.Key);
				if (!File.Exists(DestPath) || File.ReadAllText(DestPath) != Content)
				{
					File.WriteAllText(DestPath, Content);
				}
			}
		}

		private AndroidAARHandler CreateAARHandler(string EngineDir, string UnrealBuildPath, IEnumerable<string> NDKArches, bool HandleDependencies = true)
		{
			AndroidAARHandler AARHandler = new AndroidAARHandler();
			string ImportList = "";

			// Get some common paths
			string AndroidHome = Environment.ExpandEnvironmentVariables("%ANDROID_HOME%").TrimEnd('/', '\\');
			EngineDir = EngineDir.TrimEnd('/', '\\');

			// Add the AARs from the default aar-imports.txt
			// format: Package,Name,Version
			string ImportsFile = Path.Combine(UnrealBuildPath, "aar-imports.txt");
			if (File.Exists(ImportsFile))
			{
				ImportList = File.ReadAllText(ImportsFile);
			}

			// Run the UPL imports section for each architecture and add any new imports (duplicates will be removed)
			foreach (string NDKArch in NDKArches)
			{
				ImportList = UPL!.ProcessPluginNode(NDKArch, "AARImports", ImportList);
			}

			// Add the final list of imports and get dependencies
			foreach (string Line in ImportList.Split('\n'))
			{
				string Trimmed = Line.Trim(' ', '\r');

				if (Trimmed.StartsWith("repository ", StringComparison.Ordinal))
				{
					string DirectoryPath = Trimmed.Substring(11).Trim(' ').TrimEnd('/', '\\');
					DirectoryPath = DirectoryPath.Replace("$(ENGINEDIR)", EngineDir, StringComparison.Ordinal);
					DirectoryPath = DirectoryPath.Replace("$(ANDROID_HOME)", AndroidHome, StringComparison.Ordinal);
					DirectoryPath = DirectoryPath.Replace('\\', Path.DirectorySeparatorChar).Replace('/', Path.DirectorySeparatorChar);
					AARHandler.AddRepository(DirectoryPath, Logger);
				}
				else if (Trimmed.StartsWith("repositories ", StringComparison.Ordinal))
				{
					string DirectoryPath = Trimmed.Substring(13).Trim(' ').TrimEnd('/', '\\');
					DirectoryPath = DirectoryPath.Replace("$(ENGINEDIR)", EngineDir, StringComparison.Ordinal);
					DirectoryPath = DirectoryPath.Replace("$(ANDROID_HOME)", AndroidHome, StringComparison.Ordinal);
					DirectoryPath = DirectoryPath.Replace('\\', Path.DirectorySeparatorChar).Replace('/', Path.DirectorySeparatorChar);
					AARHandler.AddRepositories(DirectoryPath, "m2repository", Logger);
				}
				else
				{
					string[] Sections = Trimmed.Split(',');
					if (Sections.Length == 3)
					{
						string PackageName = Sections[0].Trim(' ');
						string BaseName = Sections[1].Trim(' ');
						string Version = Sections[2].Trim(' ');
						Logger.LogInformation("AARImports: {PackageName}, {BaseName}, {Version}", PackageName, BaseName, Version);
						AARHandler.AddNewAAR(PackageName, BaseName, Version, Logger, HandleDependencies);
					}
				}
			}

			return AARHandler;
		}

		private void PrepareJavaLibsForGradle(string JavaLibsDir, string UnrealBuildGradlePath, string InMinSdkVersion, string InTargetSdkVersion, string NDKArch)
		{
			StringBuilder SettingsGradleContent = new StringBuilder();
			StringBuilder ProjectDependencyContent = new StringBuilder();

			SettingsGradleContent.AppendLine("rootProject.name='app'");
			SettingsGradleContent.AppendLine("include ':app'");
			ProjectDependencyContent.AppendLine("dependencies {");

			string[] LibDirs = Directory.GetDirectories(JavaLibsDir);
			foreach (string LibDir in LibDirs)
			{
				string RelativePath = Path.GetFileName(LibDir);

				SettingsGradleContent.AppendLine(String.Format("include ':{0}'", RelativePath));
				ProjectDependencyContent.AppendLine(String.Format("\timplementation project(':{0}')", RelativePath));

				string GradleProjectPath = Path.Combine(UnrealBuildGradlePath, RelativePath);
				string GradleProjectMainPath = Path.Combine(GradleProjectPath, "src", "main");

				string ManifestFilename = Path.Combine(LibDir, "AndroidManifest.xml");
				string GradleManifest = Path.Combine(GradleProjectMainPath, "AndroidManifest.xml");
				MakeDirectoryIfRequired(GradleManifest);

				// Copy parts were they need to be
				CleanCopyDirectory(Path.Combine(LibDir, "assets"), Path.Combine(GradleProjectPath, "assets"));
				CleanCopyDirectory(Path.Combine(LibDir, "libs"), Path.Combine(GradleProjectPath, "libs"));
				CleanCopyDirectory(Path.Combine(LibDir, "res"), Path.Combine(GradleProjectMainPath, "res"));

				// If our lib already has a src/main/java folder, don't put things into a java folder
				string SrcDirectory = Path.Combine(LibDir, "src", "main");
				if (Directory.Exists(Path.Combine(SrcDirectory, "java")))
				{
					CleanCopyDirectory(SrcDirectory, GradleProjectMainPath);
				}
				else
				{
					CleanCopyDirectory(Path.Combine(LibDir, "src"), Path.Combine(GradleProjectMainPath, "java"));
				}

				// Now generate a build.gradle from the manifest
				StringBuilder BuildGradleContent = new StringBuilder();
				BuildGradleContent.AppendLine("apply plugin: 'com.android.library'");
				BuildGradleContent.AppendLine("android {");
				BuildGradleContent.AppendLine("\tndkVersion NDK_VERSION");
				BuildGradleContent.AppendLine("\tndkPath NDK_PATH");
				BuildGradleContent.AppendLine("\tcompileSdk COMPILE_SDK_VERSION.toInteger()");
				BuildGradleContent.AppendLine("\tbuildToolsVersion BUILD_TOOLS_VERSION");
				BuildGradleContent.AppendLine("\tcompileOptions {");
				BuildGradleContent.AppendLine("\t\tsourceCompatibility JavaVersion.VERSION_17");
				BuildGradleContent.AppendLine("\t\ttargetCompatibility JavaVersion.VERSION_17");
				BuildGradleContent.AppendLine("\t\tcoreLibraryDesugaringEnabled true");
				BuildGradleContent.AppendLine("\t}");
				BuildGradleContent.AppendLine("\tdefaultConfig {");
				BuildGradleContent.AppendLine("\t\tmultiDexEnabled true");

				// Try to get the SDK target from the AndroidManifest.xml
				string VersionCode = "";
				string VersionName = "";
				string MinSdkVersion = InMinSdkVersion;
				string TargetSdkVersion = InTargetSdkVersion;
				XDocument ManifestXML;
				if (File.Exists(ManifestFilename))
				{
					try
					{
						ManifestXML = XDocument.Load(ManifestFilename);

						XAttribute? VersionCodeAttr = ManifestXML.Root!.Attribute(XName.Get("versionCode", "http://schemas.android.com/apk/res/android"));
						if (VersionCodeAttr != null)
						{
							VersionCode = VersionCodeAttr.Value;
						}

						XAttribute? VersionNameAttr = ManifestXML.Root.Attribute(XName.Get("versionName", "http://schemas.android.com/apk/res/android"));
						if (VersionNameAttr != null)
						{
							VersionName = VersionNameAttr.Value;
						}

						XElement? UseSDKNode = null;
						foreach (XElement WorkNode in ManifestXML.Elements().First().Descendants("uses-sdk"))
						{
							UseSDKNode = WorkNode;

							XAttribute? MinSdkVersionAttr = WorkNode.Attribute(XName.Get("minSdkVersion", "http://schemas.android.com/apk/res/android"));
							if (MinSdkVersionAttr != null)
							{
								MinSdkVersion = MinSdkVersionAttr.Value;
							}

							XAttribute? TargetSdkVersionAttr = WorkNode.Attribute(XName.Get("targetSdkVersion", "http://schemas.android.com/apk/res/android"));
							if (TargetSdkVersionAttr != null)
							{
								TargetSdkVersion = TargetSdkVersionAttr.Value;
							}
						}

						UseSDKNode?.Remove();

						// rewrite the manifest if different
						string NewManifestText = ManifestXML.ToString();
						string OldManifestText = "";
						if (File.Exists(GradleManifest))
						{
							OldManifestText = File.ReadAllText(GradleManifest);
						}
						if (NewManifestText != OldManifestText)
						{
							File.WriteAllText(GradleManifest, NewManifestText);
						}
					}
					catch (Exception e)
					{
						Logger.LogError(e, "AAR Manifest file {FileName} parsing error! {Ex}", ManifestFilename, e);
					}
				}

				if (!String.IsNullOrEmpty(VersionCode))
				{
					BuildGradleContent.AppendLine(String.Format("\t\tversionCode {0}", VersionCode));
				}
				if (!String.IsNullOrEmpty(VersionName))
				{
					BuildGradleContent.AppendLine(String.Format("\t\tversionName \"{0}\"", VersionName));
				}
				if (!String.IsNullOrEmpty(MinSdkVersion))
				{
					BuildGradleContent.AppendLine(String.Format("\t\tminSdkVersion {0}", MinSdkVersion));
				}
				if (!String.IsNullOrEmpty(TargetSdkVersion))
				{
					BuildGradleContent.AppendLine(String.Format("\t\ttargetSdkVersion {0}", TargetSdkVersion));
				}
				BuildGradleContent.AppendLine("\t}");
				BuildGradleContent.AppendLine("}");
				BuildGradleContent.AppendLine("dependencies.coreLibraryDesugaring \"com.android.tools:desugar_jdk_libs:2.1.5\"");

				string AdditionsGradleFilename = Path.Combine(LibDir, "additions.gradle");
				if (File.Exists(AdditionsGradleFilename))
				{
					string[] AdditionsLines = File.ReadAllLines(AdditionsGradleFilename);
					foreach (string LineContents in AdditionsLines)
					{
						BuildGradleContent.AppendLine(LineContents);
					}
				}

				// rewrite the build.gradle if different
				string BuildGradleFilename = Path.Combine(GradleProjectPath, "build.gradle");
				string NewBuildGradleText = BuildGradleContent.ToString();
				string OldBuildGradleText = "";
				if (File.Exists(BuildGradleFilename))
				{
					OldBuildGradleText = File.ReadAllText(BuildGradleFilename);
				}
				if (NewBuildGradleText != OldBuildGradleText)
				{
					File.WriteAllText(BuildGradleFilename, NewBuildGradleText);
				}
			}
			ProjectDependencyContent.AppendLine("}");

			// Add any UPL settingsGradleAdditions
			SettingsGradleContent.Append(UPL!.ProcessPluginNode(NDKArch, "settingsGradleAdditions", ""));

			string SettingsGradleFilename = Path.Combine(UnrealBuildGradlePath, "settings.gradle");
			File.WriteAllText(SettingsGradleFilename, SettingsGradleContent.ToString());

			string ProjectsGradleFilename = Path.Combine(UnrealBuildGradlePath, "app", "projects.gradle");
			File.WriteAllText(ProjectsGradleFilename, ProjectDependencyContent.ToString());
		}

		private void GenerateGradleAARImports(string EngineDir, string UnrealBuildPath, IEnumerable<string> NDKArches)
		{
			AndroidAARHandler AARHandler = CreateAARHandler(EngineDir, UnrealBuildPath, NDKArches, false);
			StringBuilder AARImportsContent = new StringBuilder();

			// Add repositories
			AARImportsContent.AppendLine("repositories {");
			foreach (string Repository in AARHandler.Repositories!)
			{
				string RepositoryPath = Path.GetFullPath(Repository).Replace('\\', '/');
				AARImportsContent.AppendLine("\tmaven { url uri('" + RepositoryPath + "') }");
			}
			AARImportsContent.AppendLine("}");

			// Add dependencies
			AARImportsContent.AppendLine("dependencies {");
			foreach (AndroidAARHandler.AndroidAAREntry Dependency in AARHandler.AARList!)
			{
				AARImportsContent.AppendLine(String.Format("\timplementation '{0}:{1}:{2}'", Dependency.Filename, Dependency.BaseName, Dependency.Version));
			}
			AARImportsContent.AppendLine("}");

			string AARImportsFilename = Path.Combine(UnrealBuildPath, "gradle", "app", "aar-imports.gradle");
			File.WriteAllText(AARImportsFilename, AARImportsContent.ToString());
		}
	}
}
