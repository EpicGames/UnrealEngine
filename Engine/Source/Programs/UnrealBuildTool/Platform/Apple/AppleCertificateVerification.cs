// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Security.Cryptography.X509Certificates;
using System.Text.RegularExpressions;
using Microsoft.Extensions.Logging;

namespace UnrealBuildTool
{
	/// <summary>
	/// Utility class for verifying that you have certain Apple Certificates installed
	/// </summary>
	public partial class AppleCertificateVerification
	{
		private static bool bCertificatesVerified = false;

		[ConfigFile(ConfigHierarchyType.Engine, "/Script/MacTargetPlatform.XcodeProjectSettings", "CertificateHashes")]
		private readonly List<string> CertOfTruthHashes = [];

		[ConfigFile(ConfigHierarchyType.Engine, "/Script/MacTargetPlatform.XcodeProjectSettings", "CertificateTeamIDs")]
		private readonly List<string> CertOfTruthTeamIds = [];

		[ConfigFile(ConfigHierarchyType.Engine, "/Script/MacTargetPlatform.XcodeProjectSettings", "CertValidationFailMessage")]
		private readonly string CertValidationFailMessage = "You don't have the correct cert(s) installed for auto codesigning, please install your team's cert(s)";

		[ConfigFile(ConfigHierarchyType.Engine, "/Script/MacTargetPlatform.XcodeProjectSettings", "CodeSigningTeam")]
		private readonly string CodeSigningTeamID = String.Empty;

		private readonly ILogger Logger;

		/// <summary>
		/// Constructor
		/// </summary>
	    public AppleCertificateVerification(ILogger Logger)
	    {
	    	// the config entries shouldn't be in a platform-specific ini file, so we are just
	    	// using the mac platform because we install the certs on mac and the certs are used across
	    	// all apple platforms.
	        ConfigCache.ReadSettings(null, UnrealTargetPlatform.Mac, this);

	        this.Logger = Logger;
	    }

		/// <summary>
		/// Checks for certs that don't match the "certificates of truth" hashes but match the team ID
		/// (and thus are certs which will interfere with xcode's autosigning process)
		/// </summary>
	    private bool FoundInterferingCerts(X509Certificate2Collection CertCollection)
	    {
	    	Regex CertOrgRegex = new Regex(@"OU=([^,]+)", RegexOptions.IgnoreCase);
			Regex CertCommonNameRegex = new Regex(@"CN=([^,]+)", RegexOptions.IgnoreCase);
			HashSet<string> IdentityHashes = AppleCodeSign.GetCodeSigningIdentityHashes();

			foreach (X509Certificate2 FoundCert in CertCollection)
			{
				Match CertOrgMatch = CertOrgRegex.Match(FoundCert.Subject);
				string? CertOrgVal = CertOrgMatch.Success ? CertOrgMatch.Groups[1].Value.Trim() : null;

				Match CertCommonNameMatch = CertCommonNameRegex.Match(FoundCert.Subject);
				string? CertCommonNameVal = CertCommonNameMatch.Success ? CertCommonNameMatch.Groups[1].Value.Trim() : null;

				if (String.IsNullOrEmpty(CertOrgVal) || String.IsNullOrEmpty(CertCommonNameVal))
				{
					continue;
				}

				if (CertCommonNameVal.StartsWith("Apple Development: ", StringComparison.Ordinal) && CodeSigningTeamID == CertOrgVal &&
				    !CertOfTruthHashes.Contains(FoundCert.GetCertHashString()))
				{
					// we found a possible interfering cert, check that that cert has a valid identity it can sign with
					// otherwise it won't interfere
					if(IdentityHashes.Contains(FoundCert.GetCertHashString()))
					{
						Logger.LogError("Found interfering cert identity: {FoundCert.Subject}!", FoundCert.Subject);

						return true;
					}
				}
			}

			return false;
	    }

		private bool HasCertsOfTruthInSystemStore()
		{
			if (CertOfTruthHashes.Count == 0 || CertOfTruthTeamIds.Count == 0)
			{
				Logger.LogInformation("No certs of truth reported, skipping certificate verification...");
				return true;
			}
			else if(CertOfTruthHashes.Count != CertOfTruthTeamIds.Count)
			{
				Logger.LogError("Invalid cert of truth config! There should be a hash in CertificateHashes for each team ID in CertificateTeamIDs!");
				return false;
			}

			X509Certificate2Collection FoundCertCollection = AppleCodeSign.FindCertificateCollection();

			for (int i = 0; CertOfTruthTeamIds.Count > i; ++i)
			{
				string TeamID = CertOfTruthTeamIds[i];
				
				if (TeamID == CodeSigningTeamID)
				{
					foreach (X509Certificate2 FoundCert in FoundCertCollection)
					{
						if (FoundCert.GetCertHashString() == CertOfTruthHashes[i])
						{
							Logger.LogDebug($"Found a locally installed Apple certificate matching known Apple certificate of truth.");

							return !FoundInterferingCerts(FoundCertCollection);
						}
					}

					return false;
				}
			}

			Logger.LogInformation("No certs of truth matching current team id {CodeSigningTeamID} reported, skipping certificate verification...",
								  CodeSigningTeamID);

			return true;
		}

		/// <summary>
		/// Verifies you have the valid certificate for your associated team installed and
		/// fails the build if not.
		/// </summary> 
		public void VerifyCertificateInstallation()
		{
			if (!bCertificatesVerified)
			{
				if (HasCertsOfTruthInSystemStore())
				{
					Logger.LogInformation("Known Apple certificate for this team is installed!");
				}
				else
				{
					Logger.LogError("{CertValidationFailMessage}", CertValidationFailMessage);
					throw new BuildException("Stopping build due to broken Apple certificate setup...");
				}

				bCertificatesVerified = true;
			}
		}
	}
}
