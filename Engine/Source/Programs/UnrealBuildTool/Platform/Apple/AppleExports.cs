// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// Public Apple functions exposed to UAT
	/// </summary>
	public static class AppleExports
	{
#pragma warning disable CA1823 // unused private fields - This will be removed in a future version
#pragma warning disable IDE0052 //  unused private member - This will be removed in a future version
		[Obsolete("Deprecated in UE5.8. Modern Xcode is now the only option")]
		private static readonly bool bForceModernXcode = Environment.CommandLine.Contains("-modernxcode", StringComparison.OrdinalIgnoreCase);
		[Obsolete("Deprecated in UE5.8. Legacy Xcode is no longer supported")]
		private static readonly bool bForceLegacyXcode = Environment.CommandLine.Contains("-legacy xcode", StringComparison.OrdinalIgnoreCase);
#pragma warning restore IDE0052
#pragma warning restore CA1823

		private static readonly bool bNoEntitlements = Environment.CommandLine.Contains("-noEntitlements", StringComparison.OrdinalIgnoreCase);

		private static DirectoryReference? _MobileProvisionDir;

		/// <summary>
		///  True if we should force disable entitlements based on "-noEntitlements"
		/// </summary>
		/// <returns>bool</returns>
		public static bool ForceNoEntitlements()
		{
			return bNoEntitlements;
		}

		/// <summary>
		/// Is the current project using modern xcode?
		/// </summary>
		/// <param name="ProjectFile"></param>
		/// <returns></returns>
		[Obsolete("Deprecated in UE5.8. Use !CreatingAppOnWindows")]
		public static bool UseModernXcode(FileReference? ProjectFile)
		{
			return !CreatingAppOnWindows(ProjectFile);
		}

		/// <summary>
		/// Is the current project being built in Apple on Windows mode?
		/// </summary>
		/// <param name="ProjectFile"></param>
		/// <returns></returns>
		public static bool CreatingAppOnWindows(FileReference? ProjectFile)
		{
			if (OperatingSystem.IsWindows())
			{
				Log.TraceInformationOnce("Building in Apple on Windows mode because the host machine is running Windows...");
				return true;
			}
			
			Log.TraceVerboseOnce("Building in Xcode Mode...");
			return false;
		}

		/// <summary>
		/// Get the given project's Swift settings
		/// </summary>
		/// <param name="ProjectFile"></param>
		/// <param name="TargetType"></param>
		/// <param name="Platform"></param>
		/// <param name="bUseSwiftUIMain"></param>
		/// <param name="bCreateBridgingHeader"></param>
		public static void GetSwiftIntegrationSettings(FileReference? ProjectFile, TargetType TargetType, UnrealTargetPlatform Platform, out bool bUseSwiftUIMain, out bool bCreateBridgingHeader)
		{
			if (TargetType == TargetType.Editor)
			{
				bUseSwiftUIMain = false;
			}
			else
			{
				ConfigHierarchy Ini = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, ProjectFile?.Directory, Platform);
				Ini.TryGetValue("/Script/MacTargetPlatform.XcodeProjectSettings", "bUseSwiftUIMain", out bUseSwiftUIMain);
			}
			// for now always create bridging headers (which let's cpp call into swift)
			bCreateBridgingHeader = true;
		}

		/// <summary>
		///  This is a FilePath from UE settings, like /Game/Foo/Bar.txt
		/// </summary>
		/// <param name="ProductDirectory">Directory to use for /Game paths</param>
		/// <param name="FilePath"></param>
		/// <returns></returns>
		public static FileReference ConvertFilePath(DirectoryReference? ProductDirectory, string FilePath)
		{
			// for FilePath params, pull the path out of the struct
			if (FilePath.StartsWith("(FilePath=", StringComparison.OrdinalIgnoreCase))
			{
				string? ReadPath = ConfigHierarchy.GetStructEntry(FilePath, "FilePath", false);
				if (ReadPath == null)
				{
					throw new Exception(" Failed to read file path from config");
				}
	
				FilePath = ReadPath;
			}

			if (FilePath.StartsWith("/Engine/", StringComparison.OrdinalIgnoreCase))
			{
				return FileReference.Combine(Unreal.EngineDirectory, FilePath.Substring(8));
			}
			else if (ProductDirectory != null && FilePath.StartsWith("/Game/", StringComparison.OrdinalIgnoreCase))
			{
				return FileReference.Combine(ProductDirectory, FilePath.Substring(6));
			}
			else if (FilePath.StartsWith("/", StringComparison.OrdinalIgnoreCase))
			{
				// Absolute path
				return new FileReference(FilePath);
			}
			else
			{
				// UE-193103, using the file selector in UE will set the path relative to /Engine/Binaries/Mac
				return FileReference.Combine(Unreal.EngineDirectory, "Binaries", "Mac", FilePath);
			}
		}

		/// <summary>
		/// Convert UnrealTargetPlatform to the platform that Xcode uses for -destination
		/// </summary>
		/// <param name="Platform"></param>
		/// <param name="Architectures">The architecture we are targeting (really, this is for Simulators)</param>
		/// <returns></returns>
		public static string GetDestinationPlatform(UnrealTargetPlatform Platform, UnrealArchitectures Architectures)
		{
			if (Platform == UnrealTargetPlatform.Mac)
			{
				return "macOS";
			}
			else if (Platform == UnrealTargetPlatform.IOS)
			{
				return Architectures.SingleArchitecture == UnrealArch.IOSSimulator ? "iOS Simulator" : "iOS";
			}
			else if (Platform == UnrealTargetPlatform.TVOS)
			{
				return Architectures.SingleArchitecture == UnrealArch.TVOSSimulator ? "tvOS Simulator" : "tvOS";
			}
			else if (Platform == UnrealTargetPlatform.VisionOS)
			{
				return Architectures.SingleArchitecture == UnrealArch.IOSSimulator ? "visionOS Simulator" : "visionOS";
			}

			throw new BuildException($"Unknown plaform {Platform}");
		}

		/// <summary>
		/// Different ways that xcodebuild is run, so the scripts can behave appropriately
		/// </summary>
		public enum XcodeBuildMode
		{
			/// <summary>
			/// This is when hitting Build from in Xcode
			/// </summary>
			Default = 0,
			/// <summary>
			/// This runs after building when building on commandline directly with UBT
			/// </summary>
			PostBuildSync = 1,
			/// <summary>
			/// This runs when making a fully made .app in the Staged directory
			/// </summary>
			Stage = 2,
			/// <summary>
			/// This runs when packaging a full made .app into Project/Binaries
			/// </summary>
			Package = 3,
			/// <summary>
			/// This runs when packaging a .xcarchive for distribution
			/// </summary>
			Distribute = 4,
		}

		/// <summary>
		/// Gets the AppStoreConnect auth options for a given project. Will return empty string if the project isn't set up to use ASC
		/// </summary>
		/// <param name="ProjectFile"></param>
		/// <returns></returns>
		public static string GetXcodeBuildAuthOptions(FileReference? ProjectFile)
		{
			string Options = "";
			
			// handle AppStore Connect settings
			bool bUseAppStoreConnect;
		
			ConfigHierarchy SharedPlatformIni = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, ProjectFile?.Directory, UnrealTargetPlatform.Mac);
			SharedPlatformIni.TryGetValue("/Script/MacTargetPlatform.XcodeProjectSettings", "bUseAppStoreConnect", out bUseAppStoreConnect);
			if (bUseAppStoreConnect)
			{
				string? IssuerID, KeyID, KeyPath;
				if (SharedPlatformIni.TryGetValue("/Script/MacTargetPlatform.XcodeProjectSettings", "AppStoreConnectIssuerID", out IssuerID) &&
					SharedPlatformIni.TryGetValue("/Script/MacTargetPlatform.XcodeProjectSettings", "AppStoreConnectKeyID", out KeyID) &&
					SharedPlatformIni.TryGetValue("/Script/MacTargetPlatform.XcodeProjectSettings", "AppStoreConnectKeyPath", out KeyPath))
				{
					FileReference KeyFile = ConvertFilePath(ProjectFile?.Directory, KeyPath);
					Options += $" -authenticationKeyIssuerID {IssuerID}";
					Options += $" -authenticationKeyID {KeyID}";
					Options += $" -authenticationKeyPath \"{KeyFile}\"";
				}
			}

			return Options;
		}

		/// <summary>
		/// Generates a stub xcode project for the given project/platform/target combo, then builds or archives it
		/// </summary>
		/// <param name="ProjectFile">Project to build</param>
		/// <param name="Platform">Platform to build</param>
		/// <param name="Architectures">The architecture we are targeting (really, this is for Simulator to pass in the -destination field)</param>
		/// <param name="Configuration">Configuration to build</param>
		/// <param name="TargetName">Target to build</param>
		/// <param name="BuildMode">Sets an envvar used inside the xcode project to control certain features</param>
		/// <param name="Logger"></param>
		/// <param name="ExtraOptions">Any extra options to pass to xcodebuild</param>
		/// <param name="bForceDummySigning">If true, force signing with the - identity</param>
		/// <param name="DestinationIds">Optional list of devices to target, mainly for adding to provision if needed</param>
		/// <returns>xcode's exit code</returns>
		public static int BuildWithStubXcodeProject(FileReference? ProjectFile, UnrealTargetPlatform Platform, UnrealArchitectures Architectures, UnrealTargetConfiguration Configuration,
			string TargetName, XcodeBuildMode BuildMode, ILogger Logger, string ExtraOptions = "", bool bForceDummySigning = false, List<string>? DestinationIds = null)
		{
			DirectoryReference? GeneratedProjectFile;
			// we don't use distro flag when making a modern project
			AppleExports.GenerateRunOnlyXcodeProject(ProjectFile, Platform, TargetName, bForDistribution: false, bNoEntitlements: bNoEntitlements, Logger, out GeneratedProjectFile);

			if (GeneratedProjectFile == null)
			{
				return 1;
			}

			bool bUseAutomaticCodeSigning = false;
			if (bForceDummySigning)
			{
				ExtraOptions += " CODE_SIGN_STYLE=Manual CODE_SIGN_IDENTITY= PROVISIONING_PROFILE_SPECIFIER=";
			}
			else
			{
				// look for the special app store connect key information
				ConfigHierarchy SharedPlatformIni = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, ProjectFile?.Directory, UnrealTargetPlatform.Mac);
				SharedPlatformIni.TryGetValue("/Script/MacTargetPlatform.XcodeProjectSettings", "bUseAutomaticCodeSigning", out bUseAutomaticCodeSigning);

				// disable automatic signing, if some extra options imply manual
				if (ExtraOptions.Contains("CODE_SIGN_IDENTITY", StringComparison.Ordinal))
				{
					bUseAutomaticCodeSigning = false;
				}

				if (bUseAutomaticCodeSigning)
				{
					ExtraOptions += " -allowProvisioningUpdates";

					ExtraOptions += GetXcodeBuildAuthOptions(ProjectFile);

				}
			}

			// run xcodebuild on the generated project to make the .app
			string XcodeBuildAction = BuildMode == XcodeBuildMode.Distribute ? "archive" : "build";
			return AppleExports.FinalizeAppWithXcode(GeneratedProjectFile!, Platform, Architectures, bUseAutomaticCodeSigning, TargetName, Configuration.ToString(), XcodeBuildAction, ExtraOptions + $" UE_XCODE_BUILD_MODE={BuildMode}", Logger, DestinationIds);
		}

		/// <summary>
		/// Genearate an run-only Xcode project, that is not meant to be used for anything else besides code-signing/running/etc of the native .app bundle
		/// </summary>
		/// <param name="UProjectFile">Location of .uproject file (or null for the engine project</param>
		/// <param name="Platform">The platform to generate a project for</param>
		/// <param name="TargetName">The name of the target being built, so we can generate a more minimal project</param>
		/// <param name="bForDistribution">True if this is making a bild for uploading to app store</param>
		/// <param name="bNoEntitlements">True if we should disable entitlements</param>
		/// <param name="Logger">Logging object</param>
		/// <param name="GeneratedProjectFile">Returns the .xcworkspace that was made</param>
		public static void GenerateRunOnlyXcodeProject(FileReference? UProjectFile, UnrealTargetPlatform Platform, string TargetName, bool bForDistribution, bool bNoEntitlements, ILogger Logger, out DirectoryReference? GeneratedProjectFile)
		{
			AppleToolChain.GenerateRunOnlyXcodeProject(UProjectFile, Platform, TargetName, bForDistribution, bNoEntitlements, Logger, out GeneratedProjectFile);
		}

		/// <summary>
		/// Version of FinalizeAppWithXcode that is meant for modern xcode mode, where we assume all codesigning is setup already in the project, so nothing else is needed
		/// </summary>
		/// <param name="XcodeProject">The .xcworkspace file to build</param>
		/// <param name="Platform">THe platform to make the .app for</param>
		/// <param name="Architectures">The architecture we are targeting (really, this is for Simulator to pass in the -destination field)</param>
		/// <param name="bUseAutomaticCodeSigning">True when using automatic codesigning via xcodebuild</param>
		/// <param name="SchemeName">The name of the scheme (basically the target on the .xcworkspace)</param>
		/// <param name="Configuration">Which configuration to make (Debug, etc)</param>
		/// <param name="Action">Action (build, archive, etc)</param>
		/// <param name="ExtraOptions">Extra options to pass to xcodebuild</param>
		/// <param name="Logger">Logging object</param>
		/// <param name="DestinationIds">Optional list of devices to target, mainly for adding to provision if needed</param>
		/// <returns>xcode's exit code</returns>
		public static int FinalizeAppWithXcode(DirectoryReference XcodeProject, UnrealTargetPlatform Platform, UnrealArchitectures Architectures, bool bUseAutomaticCodeSigning, string SchemeName, string Configuration, string Action, string ExtraOptions, ILogger Logger, List<string>? DestinationIds=null)
		{
			return AppleToolChain.FinalizeAppWithXcode(XcodeProject, Platform, Architectures, bUseAutomaticCodeSigning, SchemeName, Configuration, Action, ExtraOptions, Logger, DestinationIds);
		}

		/// <summary>
		/// Pass along the call to UEBuildTarget.MakeBinaryFileName, but taking an extension instead of a binary type, since the type is hidden
		/// </summary>
		/// <param name="BinaryName"></param>
		/// <param name="Separator"></param>
		/// <param name="Platform"></param>
		/// <param name="Configuration"></param>
		/// <param name="Architectures"></param>
		/// <param name="UndecoratedConfiguration"></param>
		/// <param name="Extension"></param>
		/// <returns></returns>
		public static string MakeBinaryFileName(string BinaryName, string Separator, UnrealTargetPlatform Platform, UnrealTargetConfiguration Configuration, UnrealArchitectures Architectures, UnrealTargetConfiguration UndecoratedConfiguration, string? Extension)
		{
			string StandardBinaryName = UEBuildTarget.MakeBinaryFileName(BinaryName, Separator, Platform, Configuration, Architectures, UndecoratedConfiguration, UEBuildBinaryType.Executable);
			return System.IO.Path.ChangeExtension(StandardBinaryName, Extension);
		}

		/// <summary>
		/// Finds the latest .xcarchive for a given Target name (the .xcarchive will start with this name then have a data appended)
		/// </summary>
		/// <param name="TargetName">Name of the target to look for, this will be the prefix for the .xcarchive to search for</param>
		/// <param name="SearchPath">If null, look in Xcode Archives library, otherwiwse, look here</param>
		/// <returns></returns>
		public static DirectoryReference? FindLatestXcArchive(string TargetName, DirectoryReference? SearchPath=null)
		{
			DirectoryReference UserDir = new DirectoryReference(Environment.GetFolderPath(Environment.SpecialFolder.UserProfile));
			DirectoryReference Library = SearchPath ?? DirectoryReference.Combine(UserDir, "Library/Developer/Xcode/Archives");

			// order date named folders (use creating data, not name, but same thing)
			List<DirectoryReference> DateDirs = DirectoryReference.EnumerateDirectories(Library).ToList();
			DateDirs.SortBy(x => Directory.GetCreationTime(x.FullName));
			DateDirs.Reverse();

			// go through each folder, starting at most recent, looking for an archive for the target
			string Wildcard = $"{TargetName} *.xcarchive";
			foreach (DirectoryReference DateDir in DateDirs)
			{
				List<DirectoryReference> XcArchives = DirectoryReference.EnumerateDirectories(DateDir, Wildcard).ToList();
				if (XcArchives.Count > 0)
				{
					XcArchives.SortBy(x => Directory.GetCreationTime(x.FullName));
					DirectoryReference XcArchive = XcArchives.Last();

					return XcArchive;
				}
			}

			return null;
		}
		
		/// <summary>
		/// Returns the location of the Mobile Provisioning directory. Handles Mac and Windows.
		/// On mac, this value changed starting with Xcode16
		/// </summary>
		/// <returns>The location of the Mobile Provisioning directory</returns>
		public static DirectoryReference GetProvisionDirectory()
		{
			if (!String.IsNullOrEmpty(_MobileProvisionDir?.FullName))
			{
				return _MobileProvisionDir!;
			}
			
			if (OperatingSystem.IsMacOS())
			{
				// Default to the Xcode15 and less location
				string MobileProvisionDir = "/Library/MobileDevice/Provisioning Profiles/";
				
				// Find out the version number in Xcode.app/Contents/Info.plist
				int ExitCode;
				string XcodeVersion = Utils.RunLocalProcessAndReturnStdOut("/bin/sh",
					$"-c 'plutil -extract CFBundleShortVersionString raw $(xcode-select -p)/../Info.plist'", out ExitCode);
				if (ExitCode == 0 && XcodeVersion != null)
				{
					// Parse the major version, we don't need the minor version.
					int Major = 15;
					try
					{
						string[] Tokens = XcodeVersion!.Split(".".ToCharArray());
						if (Tokens.Length >= 2)
						{
							Major = Int32.Parse(Tokens[0]);
						}
					}
					catch (Exception)
					{
					}

					if (Major >= 16)
					{
						MobileProvisionDir = "/Library/Developer/Xcode/UserData/Provisioning Profiles/";
					}
				}

				_MobileProvisionDir = new DirectoryReference(Environment.GetEnvironmentVariable("HOME") + MobileProvisionDir);
				Console.WriteLine(" Setting Mobile Provision Profile Dir: {0}", _MobileProvisionDir.FullName ?? "Not set");
				return _MobileProvisionDir!;
			}
			else
			{
				_MobileProvisionDir = new DirectoryReference(Environment.GetFolderPath(Environment.SpecialFolder.LocalApplicationData) + "/Apple Computer/MobileDevice/Provisioning Profiles/");
				Console.WriteLine(" Setting Mobile Provision Profile Dir: {0}", _MobileProvisionDir.FullName ?? "Not set");
				return _MobileProvisionDir!;
			}
		}
	}
}
