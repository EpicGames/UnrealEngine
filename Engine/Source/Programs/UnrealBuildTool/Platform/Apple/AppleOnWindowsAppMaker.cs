// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Text;
using System.Xml.Linq;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// Helper class to make Apple apps on Windows.
	/// It contains most of the functionality that used to exist as "Legacy Xcode".
	/// </summary>
	class AppleOnWindowsAppMaker
	{
		public static bool GenerateLegacyIOSPList(FileReference? ProjectFile, string ProjectDirectory, bool bIsUnrealGame, string GameName,
			bool bIsClient, string ProjectName, string InEngineDir, string AppDirectory, UnrealPluginLanguage? UPL, string? BundleID,
			bool bBuildAsFramework, ILogger Logger)
		{
			// generate the Info.plist for future use
			string BuildDirectory = ProjectDirectory + "/Build/IOS";
			string IntermediateDirectory = (bIsUnrealGame ? InEngineDir : ProjectDirectory) + "/Intermediate/IOS";
			string PListFile = IntermediateDirectory + "/" + GameName + "-Info.plist";
			ProjectName = !String.IsNullOrEmpty(ProjectName) ? ProjectName : GameName;
			UEDeployIOS.VersionUtilities.BuildDirectory = BuildDirectory;
			UEDeployIOS.VersionUtilities.GameName = GameName;

			// read the old file
			string OldPListData = File.Exists(PListFile) ? File.ReadAllText(PListFile) : "";

			// get the settings from the ini file
			// plist replacements
			DirectoryReference? DirRef = bIsUnrealGame
				? (!String.IsNullOrEmpty(UnrealBuildTool.GetRemoteIniPath()) ? new DirectoryReference(UnrealBuildTool.GetRemoteIniPath()!) : null)
				: new DirectoryReference(ProjectDirectory);
			ConfigHierarchy Ini = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, DirRef, UnrealTargetPlatform.IOS);

			// orientations
			string InterfaceOrientation = "";
			Ini.GetString("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "PreferredLandscapeOrientation", out string PreferredLandscapeOrientation);

			string SupportedOrientations = "";
			Ini.GetBool("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bSupportsPortraitOrientation", out bool bSupportsPortrait);
			SupportedOrientations += bSupportsPortrait ? "\t\t<string>UIInterfaceOrientationPortrait</string>\n" : "";

			Ini.GetBool("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bSupportsUpsideDownOrientation", out bool bSupportsUpsideDown);
			SupportedOrientations += bSupportsUpsideDown ? "\t\t<string>UIInterfaceOrientationPortraitUpsideDown</string>\n" : "";

			Ini.GetBool("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bSupportsLandscapeLeftOrientation", out bool bSupportsLandscapeLeft);
			Ini.GetBool("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bSupportsLandscapeRightOrientation", out bool bSupportsLandscapeRight);

			if (bSupportsLandscapeLeft && bSupportsLandscapeRight)
			{
				// if both landscape orientations are present, set the UIInterfaceOrientation key
				// in the orientation list, the preferred orientation should be first
				if (PreferredLandscapeOrientation == "LandscapeLeft")
				{
					InterfaceOrientation = "\t<key>UIInterfaceOrientation</key>\n\t<string>UIInterfaceOrientationLandscapeLeft</string>\n";
					SupportedOrientations +=
						"\t\t<string>UIInterfaceOrientationLandscapeLeft</string>\n\t\t<string>UIInterfaceOrientationLandscapeRight</string>\n";
				}
				else
				{
					// by default, landscape right is the preferred orientation - Apple's UI guidlines
					InterfaceOrientation = "\t<key>UIInterfaceOrientation</key>\n\t<string>UIInterfaceOrientationLandscapeRight</string>\n";
					SupportedOrientations +=
						"\t\t<string>UIInterfaceOrientationLandscapeRight</string>\n\t\t<string>UIInterfaceOrientationLandscapeLeft</string>\n";
				}
			}
			else
			{
				// max one landscape orientation is supported
				SupportedOrientations += bSupportsLandscapeRight ? "\t\t<string>UIInterfaceOrientationLandscapeRight</string>\n" : "";
				SupportedOrientations += bSupportsLandscapeLeft ? "\t\t<string>UIInterfaceOrientationLandscapeLeft</string>\n" : "";
			}

			// ITunes file sharing
			Ini.GetBool("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bSupportsITunesFileSharing", out bool bSupportsITunesFileSharing);
			Ini.GetBool("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bSupportsFilesApp", out bool bSupportsFilesApp);

			// device family — controls install eligibility on iPhone vs iPad
			Ini.GetBool("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bSupportsIPhone", out bool bSupportsIPhone);
			Ini.GetBool("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bSupportsIPad", out bool bSupportsIPad);
			
			// bundle display name
			Ini.GetString("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "BundleDisplayName", out string BundleDisplayName);

			// bundle identifier
			Ini.GetString("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "BundleIdentifier", out string BundleIdentifier);
			if (!String.IsNullOrEmpty(BundleID))
			{
				BundleIdentifier = BundleID; // overriding bundle ID
			}

			// bundle name
			Ini.GetString("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "BundleName", out string BundleName);

			// disable https requirement
			Ini.GetBool("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bDisableHTTPS", out bool bDisableHTTPS);

			// short version string
			Ini.GetString("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "VersionInfo", out string BundleShortVersion);

			// required capabilities (arm64 always required)
			string RequiredCaps = "\t\t<string>arm64</string>\n";

			Ini.GetBool("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bSupportsMetal", out bool bSupportsMetal);
			RequiredCaps += bSupportsMetal ? "\t\t<string>metal</string>\n" : "";

			// minimum iOS version
			Ini.GetString("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "MinimumiOSVersion", out string MinVersionSetting);
			string MinVersion = UEDeployIOS.GetMinimumOSVersion(MinVersionSetting, Logger);

			// Get Google Support details
			Ini.GetBool("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bEnableGoogleSupport", out bool bEnableGoogleSupport);

			// Write the Google iOS URL Scheme if we need it.
			Ini.GetString("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "GoogleReversedClientId", out string GoogleReversedClientId);
			bEnableGoogleSupport = bEnableGoogleSupport && !String.IsNullOrWhiteSpace(GoogleReversedClientId);

			// Add remote-notifications as background mode
			Ini.GetBool("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bEnableRemoteNotificationsSupport", out bool bRemoteNotificationsSupported);

			// Add audio as background mode
			Ini.GetBool("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bSupportsBackgroundAudio", out bool bBackgroundAudioSupported);

			// Add background fetch as background mode
			Ini.GetBool("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bEnableBackgroundFetch", out bool bBackgroundFetch);

			// UIScene lifecycle adoption. Must match the UE_IOS_SCENE_LIFECYCLE compile define and the
			// equivalent block in UEDeployIOS.WritePlistFile.
			Ini.GetBool("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bUseSceneBasedLifecycle", out bool bUseSceneBasedLifecycle);

			// Get any Location Services permission descriptions added
			Ini.GetString("/Script/LocationServicesIOSEditor.LocationServicesIOSSettings", "LocationAlwaysUsageDescription",
				out string LocationAlwaysUsageDescription);
			Ini.GetString("/Script/LocationServicesIOSEditor.LocationServicesIOSSettings", "LocationWhenInUseDescription",
				out string LocationWhenInUseDescription);

			// extra plist data
			Ini.GetString("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "AdditionalPlistData", out string ExtraData);

			Ini.GetBool("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bCustomLaunchscreenStoryboard",
				out UEDeployIOS.VersionUtilities.bCustomLaunchscreenStoryboard);

			// generate the plist file
			StringBuilder Text = new StringBuilder();
			Text.AppendLine("<?xml version=\"1.0\" encoding=\"UTF-8\"?>");
			Text.AppendLine("<!DOCTYPE plist PUBLIC \"-//Apple//DTD PLIST 1.0//EN\" \"http://www.apple.com/DTDs/PropertyList-1.0.dtd\">");
			Text.AppendLine("<plist version=\"1.0\">");
			Text.AppendLine("<dict>");
			Text.AppendLine("\t<key>CFBundleURLTypes</key>");
			Text.AppendLine("\t<array>");
			Text.AppendLine("\t\t<dict>");
			Text.AppendLine("\t\t\t<key>CFBundleURLName</key>");
			Text.AppendLine("\t\t\t<string>com.Epic.Unreal</string>");
			Text.AppendLine("\t\t\t<key>CFBundleURLSchemes</key>");
			Text.AppendLine("\t\t\t<array>");
			Text.AppendLine(String.Format("\t\t\t\t<string>{0}</string>", bIsUnrealGame ? "UnrealGame" : GameName));
			if (bEnableGoogleSupport)
			{
				Text.AppendLine(String.Format("\t\t\t\t<string>{0}</string>", GoogleReversedClientId));
			}

			Text.AppendLine("\t\t\t</array>");
			Text.AppendLine("\t\t</dict>");
			Text.AppendLine("\t</array>");
			Text.AppendLine("\t<key>CFBundleDevelopmentRegion</key>");
			Text.AppendLine("\t<string>English</string>");
			Text.AppendLine("\t<key>CFBundleDisplayName</key>");
			Text.AppendLine(String.Format("\t<string>{0}</string>", UEDeployIOS.EncodeBundleName(BundleDisplayName, ProjectName)));
			Text.AppendLine("\t<key>CFBundleExecutable</key>");
			string BundleExecutable = bIsUnrealGame ? (bIsClient ? "UnrealClient" : "UnrealGame") : (bIsClient ? GameName + "Client" : GameName);
			Text.AppendLine(String.Format("\t<string>{0}</string>", BundleExecutable));
			Text.AppendLine("\t<key>CFBundleIdentifier</key>");
			string SanitizedBundleIdentifier = BundleIdentifier
				.Replace("[PROJECT_NAME]", ProjectName, StringComparison.Ordinal)
				.Replace("_", "", StringComparison.Ordinal);
			Text.AppendLine(String.Format("\t<string>{0}</string>", SanitizedBundleIdentifier));
			Text.AppendLine("\t<key>CFBundleInfoDictionaryVersion</key>");
			Text.AppendLine("\t<string>6.0</string>");
			Text.AppendLine("\t<key>CFBundleName</key>");
			Text.AppendLine(String.Format("\t<string>{0}</string>", UEDeployIOS.EncodeBundleName(BundleName, ProjectName)));
			Text.AppendLine("\t<key>CFBundlePackageType</key>");
			Text.AppendLine("\t<string>APPL</string>");
			Text.AppendLine("\t<key>CFBundleSignature</key>");
			Text.AppendLine("\t<string>????</string>");
			Text.AppendLine("\t<key>CFBundleVersion</key>");
			Text.AppendLine(String.Format("\t<string>{0}</string>", UEDeployIOS.VersionUtilities.UpdateBundleVersion(OldPListData)));
			Text.AppendLine("\t<key>CFBundleShortVersionString</key>");
			Text.AppendLine(String.Format("\t<string>{0}</string>", BundleShortVersion));
			Text.AppendLine("\t<key>LSRequiresIPhoneOS</key>");
			Text.AppendLine("\t<true/>");
			Text.AppendLine("\t<key>UIStatusBarHidden</key>");
			Text.AppendLine("\t<true/>");
			Text.AppendLine("\t<key>UIFileSharingEnabled</key>");
			Text.AppendLine(String.Format("\t<{0}/>", bSupportsITunesFileSharing ? "true" : "false"));
			if (bSupportsFilesApp)
			{
				Text.AppendLine("\t<key>LSSupportsOpeningDocumentsInPlace</key>");
				Text.AppendLine("\t<true/>");
			}

			Text.AppendLine("\t<key>UIRequiresFullScreen</key>");
			Text.AppendLine("\t<true/>");
			Text.AppendLine("\t<key>UIViewControllerBasedStatusBarAppearance</key>");
			Text.AppendLine("\t<false/>");
			if (!String.IsNullOrEmpty(InterfaceOrientation))
			{
				Text.AppendLine(InterfaceOrientation);
			}

			Text.AppendLine("\t<key>UISupportedInterfaceOrientations</key>");
			Text.AppendLine("\t<array>");
			foreach (string Line in SupportedOrientations.Split("\r\n".ToCharArray()))
			{
				if (!String.IsNullOrWhiteSpace(Line))
				{
					Text.AppendLine(Line);
				}
			}

			Text.AppendLine("\t</array>");
			Text.AppendLine("\t<key>UIRequiredDeviceCapabilities</key>");
			Text.AppendLine("\t<array>");
			foreach (string Line in RequiredCaps.Split("\r\n".ToCharArray()))
			{
				if (!String.IsNullOrWhiteSpace(Line))
				{
					Text.AppendLine(Line);
				}
			}
			Text.AppendLine("\t</array>");

			Text.AppendLine("\t<key>UIDeviceFamily</key>");
			Text.AppendLine("\t<array>");
			if (bSupportsIPhone)
			{
				Text.AppendLine("\t\t<integer>1</integer>");
			}
			if (bSupportsIPad)
			{
				Text.AppendLine("\t\t<integer>2</integer>");
			}
			Text.AppendLine("\t</array>");
			
			Text.AppendLine("\t<key>CFBundleIcons</key>");
			Text.AppendLine("\t<dict>");
			Text.AppendLine("\t\t<key>CFBundlePrimaryIcon</key>");
			Text.AppendLine("\t\t<dict>");
			Text.AppendLine("\t\t\t<key>CFBundleIconFiles</key>");
			Text.AppendLine("\t\t\t<array>");
			Text.AppendLine("\t\t\t\t<string>AppIcon60x60</string>");
			Text.AppendLine("\t\t\t</array>");
			Text.AppendLine("\t\t\t<key>CFBundleIconName</key>");
			Text.AppendLine("\t\t\t<string>AppIcon</string>");
			Text.AppendLine("\t\t\t<key>UIPrerenderedIcon</key>");
			Text.AppendLine("\t\t\t<true/>");
			Text.AppendLine("\t\t</dict>");
			Text.AppendLine("\t</dict>");
			Text.AppendLine("\t<key>CFBundleIcons~ipad</key>");
			Text.AppendLine("\t<dict>");
			Text.AppendLine("\t\t<key>CFBundlePrimaryIcon</key>");
			Text.AppendLine("\t\t<dict>");
			Text.AppendLine("\t\t\t<key>CFBundleIconFiles</key>");
			Text.AppendLine("\t\t\t<array>");
			Text.AppendLine("\t\t\t\t<string>AppIcon60x60</string>");
			Text.AppendLine("\t\t\t\t<string>AppIcon76x76</string>");
			Text.AppendLine("\t\t\t</array>");
			Text.AppendLine("\t\t\t<key>CFBundleIconName</key>");
			Text.AppendLine("\t\t\t<string>AppIcon</string>");
			Text.AppendLine("\t\t\t<key>UIPrerenderedIcon</key>");
			Text.AppendLine("\t\t\t<true/>");
			Text.AppendLine("\t\t</dict>");
			Text.AppendLine("\t</dict>");
			Text.AppendLine("\t<key>UILaunchStoryboardName</key>");
			Text.AppendLine("\t<string>LaunchScreen</string>");

			if (File.Exists(DirectoryReference.FromFile(ProjectFile) + "/Build/IOS/Resources/Interface/LaunchScreen.storyboard") &&
			    UEDeployIOS.VersionUtilities.bCustomLaunchscreenStoryboard)
			{
				string LaunchStoryboard = DirectoryReference.FromFile(ProjectFile) + "/Build/IOS/Resources/Interface/LaunchScreen.storyboard";

				if (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Mac)
				{
					string outputStoryboard = LaunchStoryboard + "c";
					string argsStoryboard = "--compile " + outputStoryboard + " " + LaunchStoryboard;
					string stdOutLaunchScreen = Utils.RunLocalProcessAndReturnStdOut("ibtool", argsStoryboard, Logger);

					Logger.LogInformation("LaunchScreen Storyboard compilation results : {Results}", stdOutLaunchScreen);
				}
				else
				{
					Logger.LogWarning("Custom Launchscreen compilation storyboard only compatible on Mac for now");
				}
			}

			// Support high refresh rates (iPhone only)
			// https://developer.apple.com/documentation/quartzcore/optimizing_promotion_refresh_rates_for_iphone_13_pro_and_ipad_pro
			Ini.GetBool("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bSupportHighRefreshRates", out bool bSupportHighRefreshRates);
			if (bSupportHighRefreshRates)
			{
				Text.AppendLine("\t<key>CADisableMinimumFrameDurationOnPhone</key><true/>");
			}

			Text.AppendLine("\t<key>CFBundleSupportedPlatforms</key>");
			Text.AppendLine("\t<array>");
			Text.AppendLine("\t\t<string>iPhoneOS</string>");
			Text.AppendLine("\t</array>");
			Text.AppendLine("\t<key>MinimumOSVersion</key>");
			Text.AppendLine(String.Format("\t<string>{0}</string>", MinVersion));
			// disable exempt encryption
			Text.AppendLine("\t<key>ITSAppUsesNonExemptEncryption</key>");
			Text.AppendLine("\t<false/>");
			// add location services descriptions if used
			if (!String.IsNullOrWhiteSpace(LocationAlwaysUsageDescription))
			{
				Text.AppendLine("\t<key>NSLocationAlwaysAndWhenInUseUsageDescription</key>");
				Text.AppendLine(String.Format("\t<string>{0}</string>", LocationAlwaysUsageDescription));
			}

			if (!String.IsNullOrWhiteSpace(LocationWhenInUseDescription))
			{
				Text.AppendLine("\t<key>NSLocationWhenInUseUsageDescription</key>");
				Text.AppendLine(String.Format("\t<string>{0}</string>", LocationWhenInUseDescription));
			}

			// disable HTTPS requirement
			if (bDisableHTTPS)
			{
				Text.AppendLine("\t<key>NSAppTransportSecurity</key>");
				Text.AppendLine("\t\t<dict>");
				Text.AppendLine("\t\t\t<key>NSAllowsArbitraryLoads</key><true/>");
				Text.AppendLine("\t\t</dict>");
			}

			if (!String.IsNullOrEmpty(ExtraData))
			{
				ExtraData = ExtraData.Replace("\\n", "\n", StringComparison.Ordinal);
				foreach (string Line in ExtraData.Split("\r\n".ToCharArray()))
				{
					if (!String.IsNullOrWhiteSpace(Line))
					{
						Text.AppendLine("\t" + Line);
					}
				}
			}

			// UIScene lifecycle manifest. UIKit only routes scene callbacks when this key is present.
			// Single-window only - UIApplicationSupportsMultipleScenes stays false.
			if (bUseSceneBasedLifecycle)
			{
				Text.AppendLine("\t<key>UIApplicationSceneManifest</key>");
				Text.AppendLine("\t<dict>");
				Text.AppendLine("\t\t<key>UIApplicationSupportsMultipleScenes</key>");
				Text.AppendLine("\t\t<false/>");
				Text.AppendLine("\t\t<key>UISceneConfigurations</key>");
				Text.AppendLine("\t\t<dict>");
				Text.AppendLine("\t\t\t<key>UIWindowSceneSessionRoleApplication</key>");
				Text.AppendLine("\t\t\t<array>");
				Text.AppendLine("\t\t\t\t<dict>");
				Text.AppendLine("\t\t\t\t\t<key>UISceneConfigurationName</key>");
				Text.AppendLine("\t\t\t\t\t<string>Default Configuration</string>");
				Text.AppendLine("\t\t\t\t\t<key>UISceneDelegateClassName</key>");
				Text.AppendLine("\t\t\t\t\t<string>IOSSceneDelegate</string>");
				Text.AppendLine("\t\t\t\t</dict>");
				Text.AppendLine("\t\t\t</array>");
				Text.AppendLine("\t\t</dict>");
				Text.AppendLine("\t</dict>");
			}

			// Add remote-notifications as background mode
			if (bRemoteNotificationsSupported || bBackgroundFetch || bBackgroundAudioSupported)
			{
				Text.AppendLine("\t<key>UIBackgroundModes</key>");
				Text.AppendLine("\t<array>");
				if (bBackgroundAudioSupported)
				{
					Text.AppendLine("\t\t<string>audio</string>");
				}

				if (bRemoteNotificationsSupported)
				{
					Text.AppendLine("\t\t<string>remote-notification</string>");
				}

				if (bBackgroundFetch)
				{
					Text.AppendLine("\t\t<string>fetch</string>");
				}

				Text.AppendLine("\t</array>");
			}

			// write the iCloud container identifier, if present in the old file
			if (!String.IsNullOrEmpty(OldPListData))
			{
				int index = OldPListData.IndexOf("ICloudContainerIdentifier", StringComparison.Ordinal);
				if (index > 0)
				{
					index = OldPListData.IndexOf("<string>", index, StringComparison.Ordinal) + 8;
					int length = OldPListData.IndexOf("</string>", index, StringComparison.Ordinal) - index;
					string ICloudContainerIdentifier = OldPListData.Substring(index, length);
					Text.AppendLine("\t<key>ICloudContainerIdentifier</key>");
					Text.AppendLine(String.Format("\t<string>{0}</string>", ICloudContainerIdentifier));
				}
			}

			Text.AppendLine("</dict>");
			Text.AppendLine("</plist>");

			// Create the intermediate directory if needed
			if (!Directory.Exists(IntermediateDirectory))
			{
				Directory.CreateDirectory(IntermediateDirectory);
			}

			if (UPL != null)
			{
				// Allow UPL to modify the plist here
				XDocument XDoc;
				try
				{
					XDoc = XDocument.Parse(Text.ToString());
				}
				catch (Exception e)
				{
					throw new BuildException("plist is invalid {0}\n{1}", e, Text.ToString());
				}

				XDoc.DocumentType!.InternalSubset = "";
				UPL.ProcessPluginNode("None", "iosPListUpdates", "", ref XDoc);
				string result = XDoc.Declaration?.ToString() + "\n" + XDoc.ToString().Replace(
					"<!DOCTYPE plist PUBLIC \"-//Apple//DTD PLIST 1.0//EN\" \"http://www.apple.com/DTDs/PropertyList-1.0.dtd\"[]>",
					"<!DOCTYPE plist PUBLIC \"-//Apple//DTD PLIST 1.0//EN\" \"http://www.apple.com/DTDs/PropertyList-1.0.dtd\">",
					StringComparison.Ordinal);
				File.WriteAllText(PListFile, result);

				Text = new StringBuilder(result);
			}

			File.WriteAllText(PListFile, Text.ToString());

			if (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Mac && !bBuildAsFramework)
			{
				if (!Directory.Exists(AppDirectory))
				{
					Directory.CreateDirectory(AppDirectory);
				}

				File.WriteAllText(AppDirectory + "/Info.plist", Text.ToString());
			}

			return true;
		}

		private static void SafeFileCopy(FileInfo SourceFile, string DestinationPath, bool bOverwrite)
		{
			FileInfo DI = new FileInfo(DestinationPath);
			if (DI.Exists && bOverwrite)
			{
				DI.IsReadOnly = false;
				DI.Delete();
			}

			Directory.CreateDirectory(Path.GetDirectoryName(DestinationPath)!);
			SourceFile.CopyTo(DestinationPath, bOverwrite);

			FileInfo DI2 = new FileInfo(DestinationPath);
			if (DI2.Exists)
			{
				DI2.IsReadOnly = false;
			}
		}

		private static void CopyFiles(string SourceDirectory, string DestinationDirectory, string TargetFiles, bool bOverwrite = false)
		{
			DirectoryInfo SourceFolderInfo = new DirectoryInfo(SourceDirectory);
			if (SourceFolderInfo.Exists)
			{
				FileInfo[] SourceFiles = SourceFolderInfo.GetFiles(TargetFiles);
				foreach (FileInfo SourceFile in SourceFiles)
				{
					string DestinationPath = Path.Combine(DestinationDirectory, SourceFile.Name);
					SafeFileCopy(SourceFile, DestinationPath, bOverwrite);
				}
			}
		}

		private static void CopyAllProvisions(string ProvisionDir, ILogger Logger)
		{
			try
			{
				FileInfo DestFileInfo;
				if (!Directory.Exists(ProvisionDir))
				{
					throw new DirectoryNotFoundException(String.Format("Provision Directory {0} not found.", ProvisionDir), null);
				}

				string LocalProvisionFolder = AppleExports.GetProvisionDirectory().FullName;
				if (!Directory.Exists(LocalProvisionFolder))
				{
					Logger.LogDebug("Local Provision Folder {LocalProvisionFolder} not found, attempting to create...", LocalProvisionFolder);
					Directory.CreateDirectory(LocalProvisionFolder);
					if (Directory.Exists(LocalProvisionFolder))
					{
						Logger.LogDebug("Local Provision Folder {LocalProvisionFolder} created successfully.", LocalProvisionFolder);
					}
					else
					{
						throw new DirectoryNotFoundException(String.Format("Local Provision Folder {0} could not be created.", LocalProvisionFolder),
							null);
					}
				}

				foreach (string Provision in Directory.EnumerateFiles(ProvisionDir, "*.mobileprovision", SearchOption.AllDirectories))
				{
					string LocalProvisionFile = Path.Combine(LocalProvisionFolder, Path.GetFileName(Provision));
					bool LocalFileExists = File.Exists(LocalProvisionFile);
					if (!LocalFileExists || File.GetLastWriteTime(LocalProvisionFile) < File.GetLastWriteTime(Provision))
					{
						if (LocalFileExists)
						{
							DestFileInfo = new FileInfo(LocalProvisionFile);
							DestFileInfo.Attributes &= ~FileAttributes.ReadOnly;
						}

						File.Copy(Provision, LocalProvisionFile, true);
						DestFileInfo = new FileInfo(LocalProvisionFile);
						DestFileInfo.Attributes &= ~FileAttributes.ReadOnly;
					}
				}
			}
			catch (Exception Ex)
			{
				Logger.LogError("{Message}", Ex.ToString());
				throw;
			}
		}

		private static void CopyCloudResources(string InEngineDir, string AppDirectory)
		{
			CopyFiles(InEngineDir + "/Build/IOS/Cloud", AppDirectory, "*.*", true);
		}

		private static void CopyCustomLaunchScreenResources(string InEngineDir, string AppDirectory, string BuildDirectory, ILogger Logger)
		{
			if (Directory.Exists(BuildDirectory + "/Resources/Interface/LaunchScreen.storyboardc"))
			{
				CopyFolder(BuildDirectory + "/Resources/Interface/LaunchScreen.storyboardc", AppDirectory + "/LaunchScreen.storyboardc", true);
				CopyFiles(BuildDirectory + "/Resources/Interface/Assets", AppDirectory, "*", true);
			}
			else
			{
				Logger.LogWarning(
					"Custom LaunchScreen Storyboard is checked but no compiled Storyboard could be found. Custom Storyboard compilation is only Mac compatible for now. Fallback to default Launchscreen");
				CopyStandardLaunchScreenResources(InEngineDir, AppDirectory, BuildDirectory);
			}
		}

		private static void CopyFolder(string SourceDirectory, string DestinationDirectory, bool bOverwrite = false,
			UEDeployIOS.FilenameFilter? Filter = null)
		{
			Directory.CreateDirectory(DestinationDirectory);
			RecursiveFolderCopy(new DirectoryInfo(SourceDirectory), new DirectoryInfo(DestinationDirectory), bOverwrite, Filter);
		}

		private static void RecursiveFolderCopy(DirectoryInfo SourceFolderInfo, DirectoryInfo DestFolderInfo, bool bOverwrite = false,
			UEDeployIOS.FilenameFilter? Filter = null)
		{
			foreach (FileInfo SourceFileInfo in SourceFolderInfo.GetFiles())
			{
				string DestinationPath = Path.Combine(DestFolderInfo.FullName, SourceFileInfo.Name);
				if (Filter != null && !Filter(DestinationPath))
				{
					continue;
				}

				SafeFileCopy(SourceFileInfo, DestinationPath, bOverwrite);
			}

			foreach (DirectoryInfo SourceSubFolderInfo in SourceFolderInfo.GetDirectories())
			{
				string DestFolderName = Path.Combine(DestFolderInfo.FullName, SourceSubFolderInfo.Name);
				Directory.CreateDirectory(DestFolderName);
				RecursiveFolderCopy(SourceSubFolderInfo, new DirectoryInfo(DestFolderName), bOverwrite);
			}
		}

		private static void CopyStandardLaunchScreenResources(string InEngineDir, string AppDirectory, string BuildDirectory)
		{
			CopyFolder(InEngineDir + "/Build/IOS/Resources/Interface/LaunchScreen.storyboardc", AppDirectory + "/LaunchScreen.storyboardc", true);

			if (File.Exists(BuildDirectory + "/Resources/Graphics/LaunchScreenIOS.png"))
			{
				CopyFiles(BuildDirectory + "/Resources/Graphics", AppDirectory, "LaunchScreenIOS.png", true);
			}
			else
			{
				CopyFiles(InEngineDir + "/Build/IOS/Resources/Graphics", AppDirectory, "LaunchScreenIOS.png", true);
			}
		}

		private static void CopyLaunchScreenResources(string InEngineDir, string AppDirectory, string BuildDirectory, ILogger Logger)
		{
			if (UEDeployIOS.VersionUtilities.bCustomLaunchscreenStoryboard)
			{
				CopyCustomLaunchScreenResources(InEngineDir, AppDirectory, BuildDirectory, Logger);
			}
			else
			{
				CopyStandardLaunchScreenResources(InEngineDir, AppDirectory, BuildDirectory);
			}

			if (!File.Exists(AppDirectory + "/LaunchScreen.storyboardc/LaunchScreen.nib"))
			{
				Logger.LogError("Launchscreen.storyboard ViewController needs an ID named LaunchScreen");
			}
		}

		public static bool PrepareForUAT(UEDeployIOS DeployContext, UnrealTargetConfiguration Config, FileReference? ProjectFile, string InProjectName,
			string InProjectDirectory, string InEngineDir, bool bCreateStubIPA, List<string> UPLScripts, string? BundleID,
			bool bBuildAsFramework, DirectoryReference BinaryPath, string PayloadDirectory, string AppDirectory, string SubDir, bool bIsUnrealGame,
			string GameExeName, string GameName, ILogger Logger, string TargetPlatformName)
		{
			DirectoryReference MobileProvisionDirRef = AppleExports.GetProvisionDirectory();

			string CookedContentDirectory = AppDirectory + "/cookeddata";
			string BuildDirectory = InProjectDirectory + "/Build/" + SubDir;
			string BuildDirectory_NFL = InProjectDirectory + "/Restricted/NotForLicensees/Build/" + SubDir;

			DirectoryReference.CreateDirectory(BinaryPath);
			Directory.CreateDirectory(PayloadDirectory);
			Directory.CreateDirectory(AppDirectory);
			Directory.CreateDirectory(BuildDirectory);
			Directory.CreateDirectory(AppleExports.GetProvisionDirectory().FullName);

			// create the entitlements file

			// delete some old files if they exist
			if (Directory.Exists(AppDirectory + "/_CodeSignature"))
			{
				Directory.Delete(AppDirectory + "/_CodeSignature", true);
			}

			if (File.Exists(AppDirectory + "/CustomResourceRules.plist"))
			{
				File.Delete(AppDirectory + "/CustomResourceRules.plist");
			}

			if (File.Exists(AppDirectory + "/embedded.mobileprovision"))
			{
				File.Delete(AppDirectory + "/embedded.mobileprovision");
			}

			if (File.Exists(AppDirectory + "/PkgInfo"))
			{
				File.Delete(AppDirectory + "/PkgInfo");
			}

			// install the provision
			FileInfo DestFileInfo;
			// always look for provisions in the IOS dir, even for TVOS
			string ProvisionWithPrefix = InEngineDir + "/Build/IOS/UnrealGame.mobileprovision";

			string ProjectProvision = InProjectName + ".mobileprovision";
			if (File.Exists(Path.Combine(BuildDirectory, ProjectProvision)))
			{
				ProvisionWithPrefix = Path.Combine(BuildDirectory, ProjectProvision);
			}
			else
			{
				if (File.Exists(Path.Combine(BuildDirectory_NFL, ProjectProvision)))
				{
					ProvisionWithPrefix = Path.Combine(BuildDirectory_NFL, BuildDirectory, ProjectProvision);
				}
				else if (!File.Exists(ProvisionWithPrefix))
				{
					ProvisionWithPrefix = Path.Combine(InEngineDir, "Restricted/NotForLicensees/Build", SubDir, "UnrealGame.mobileprovision");
				}
			}

			if (File.Exists(ProvisionWithPrefix))
			{
				Directory.CreateDirectory(MobileProvisionDirRef.FullName);

				string ProjectProvisionPath = DirectoryReference.Combine(MobileProvisionDirRef, ProjectProvision).FullName;
				if (File.Exists(ProjectProvisionPath))
				{
					DestFileInfo = new FileInfo(ProjectProvisionPath);
					DestFileInfo.Attributes &= ~FileAttributes.ReadOnly;
				}

				File.Copy(ProvisionWithPrefix, ProjectProvisionPath, true);
				DestFileInfo = new FileInfo(ProjectProvisionPath);
				DestFileInfo.Attributes &= ~FileAttributes.ReadOnly;
			}

			if (!File.Exists(ProvisionWithPrefix) || Unreal.IsBuildMachine())
			{
				// copy all provisions from the game directory, the engine directory, notforlicensees directory, and, if defined, the ProvisionDirectory.
				CopyAllProvisions(BuildDirectory, Logger);
				CopyAllProvisions(InEngineDir + "/Build/IOS", Logger);
				string? ProvisionDirectory = Environment.GetEnvironmentVariable("ProvisionDirectory");
				if (!String.IsNullOrWhiteSpace(ProvisionDirectory))
				{
					CopyAllProvisions(ProvisionDirectory, Logger);
				}
			}

			// install the distribution provision
			ProvisionWithPrefix = InEngineDir + "/Build/IOS/UnrealGame_Distro.mobileprovision";
			string ProjectDistroProvision = InProjectName + "_Distro.mobileprovision";
			if (File.Exists(Path.Combine(BuildDirectory, ProjectDistroProvision)))
			{
				ProvisionWithPrefix = Path.Combine(BuildDirectory, ProjectDistroProvision);
			}
			else
			{
				if (File.Exists(Path.Combine(BuildDirectory_NFL, ProjectDistroProvision)))
				{
					ProvisionWithPrefix = Path.Combine(BuildDirectory_NFL, ProjectDistroProvision);
				}
				else if (!File.Exists(ProvisionWithPrefix))
				{
					ProvisionWithPrefix = Path.Combine(InEngineDir, "Restricted/NotForLicensees/Build", SubDir, "UnrealGame_Distro.mobileprovision");
				}
			}

			if (File.Exists(ProvisionWithPrefix))
			{
				Directory.CreateDirectory(MobileProvisionDirRef.FullName);

				string InProjectProvisionPath = DirectoryReference.Combine(MobileProvisionDirRef, InProjectName, "_Distro.mobileprovision").FullName;
				if (File.Exists(InProjectProvisionPath))
				{
					DestFileInfo = new FileInfo(InProjectProvisionPath);
					DestFileInfo.Attributes &= ~FileAttributes.ReadOnly;
				}

				File.Copy(ProvisionWithPrefix, InProjectProvisionPath, true);
				DestFileInfo = new FileInfo(InProjectProvisionPath);
				DestFileInfo.Attributes &= ~FileAttributes.ReadOnly;
			}

			DeployContext.GeneratePList(ProjectFile, Config, InProjectDirectory, bIsUnrealGame, GameExeName, false, InProjectName, InEngineDir, AppDirectory, UPLScripts, BundleID, bBuildAsFramework);

			// ensure the destination is writable
			if (File.Exists(AppDirectory + "/" + GameName))
			{
				FileInfo GameFileInfo = new FileInfo(AppDirectory + "/" + GameName);
				GameFileInfo.Attributes &= ~FileAttributes.ReadOnly;
			}

			// copy the GameName binary
			File.Copy(BinaryPath + "/" + GameExeName, AppDirectory + "/" + GameName, true);

			//tvos support
			if (SubDir == TargetPlatformName)
			{
				string BuildDirectoryFortvOS = InProjectDirectory + "/Build/IOS";
				CopyLaunchScreenResources(InEngineDir, AppDirectory, BuildDirectoryFortvOS, Logger);
			}
			else
			{
				CopyLaunchScreenResources(InEngineDir, AppDirectory, BuildDirectory, Logger);
			}

			if (!bCreateStubIPA)
			{
				CopyCloudResources(InProjectDirectory, AppDirectory);

				// copy additional engine framework assets in
				// @todo tvos: TVOS probably needs its own assets?
				string FrameworkAssetsPath = InEngineDir + "/Intermediate/IOS/FrameworkAssets";

				// Let project override assets if they exist
				if (Directory.Exists(InProjectDirectory + "/Intermediate/IOS/FrameworkAssets"))
				{
					FrameworkAssetsPath = InProjectDirectory + "/Intermediate/IOS/FrameworkAssets";
				}

				if (Directory.Exists(FrameworkAssetsPath))
				{
					CopyFolder(FrameworkAssetsPath, AppDirectory, true);
				}

				Directory.CreateDirectory(CookedContentDirectory);
			}

			return true;
		}

		public static bool GenerateTVOSPList(string ProjectDirectory, bool bIsUnrealGame, string GameName, bool bIsClient, string ProjectName,
			string InEngineDir, string AppDirectory, string? BundleID)
		{
			// @todo tvos: THIS!

			// generate the Info.plist for future use
			string BuildDirectory = ProjectDirectory + "/Build/TVOS";
			bool bSkipDefaultPNGs = false;
			string IntermediateDirectory = (bIsUnrealGame ? InEngineDir : ProjectDirectory) + "/Intermediate/TVOS";
			string PListFile = IntermediateDirectory + "/" + GameName + "-Info.plist";
			// @todo tvos: This is really nasty - both IOS and TVOS are setting static vars
			UEDeployIOS.VersionUtilities.BuildDirectory = BuildDirectory;
			UEDeployIOS.VersionUtilities.GameName = GameName;

			// read the old file
			string OldPListData = File.Exists(PListFile) ? File.ReadAllText(PListFile) : "";

			// get the settings from the ini file
			// plist replacements
			// @todo tvos: Are we going to make TVOS specific .ini files?
			DirectoryReference? DirRef = bIsUnrealGame
				? (!String.IsNullOrEmpty(UnrealBuildTool.GetRemoteIniPath()) ? new DirectoryReference(UnrealBuildTool.GetRemoteIniPath()!) : null)
				: new DirectoryReference(ProjectDirectory);
			ConfigHierarchy Ini = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, DirRef, UnrealTargetPlatform.IOS);

			// bundle display name
			Ini.GetString("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "BundleDisplayName", out string BundleDisplayName);

			// bundle identifier
			Ini.GetString("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "BundleIdentifier", out string BundleIdentifier);
			if (!String.IsNullOrEmpty(BundleID))
			{
				BundleIdentifier = BundleID;
			}

			// bundle name
			Ini.GetString("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "BundleName", out string BundleName);

			// short version string
			Ini.GetString("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "VersionInfo", out string BundleShortVersion);

			// required capabilities
			string RequiredCaps = "\t\t<string>arm64</string>\n";

			// generate the plist file
			StringBuilder Text = new StringBuilder();
			Text.AppendLine("<?xml version=\"1.0\" encoding=\"UTF-8\"?>");
			Text.AppendLine("<!DOCTYPE plist PUBLIC \"-//Apple//DTD PLIST 1.0//EN\" \"http://www.apple.com/DTDs/PropertyList-1.0.dtd\">");
			Text.AppendLine("<plist version=\"1.0\">");
			Text.AppendLine("<dict>");
			Text.AppendLine("\t<key>CFBundleDevelopmentRegion</key>");
			Text.AppendLine("\t<string>en</string>");
			Text.AppendLine("\t<key>CFBundleDisplayName</key>");
			Text.AppendLine(String.Format("\t<string>{0}</string>", UEDeployIOS.EncodeBundleName(BundleDisplayName, ProjectName)));
			Text.AppendLine("\t<key>CFBundleExecutable</key>");
			string BundleExecutable = bIsUnrealGame ? (bIsClient ? "UnrealClient" : "UnrealGame") : (bIsClient ? GameName + "Client" : GameName);
			Text.AppendLine(String.Format("\t<string>{0}</string>", BundleExecutable));
			Text.AppendLine("\t<key>CFBundleIdentifier</key>");
			string SanitizedBundleIdentifier = BundleIdentifier
				.Replace("[PROJECT_NAME]", ProjectName, StringComparison.Ordinal)
				.Replace("_", "", StringComparison.Ordinal);
			Text.AppendLine(String.Format("\t<string>{0}</string>", SanitizedBundleIdentifier));
			Text.AppendLine("\t<key>CFBundleInfoDictionaryVersion</key>");
			Text.AppendLine("\t<string>6.0</string>");
			Text.AppendLine("\t<key>CFBundleName</key>");
			Text.AppendLine(String.Format("\t<string>{0}</string>", UEDeployIOS.EncodeBundleName(BundleName, ProjectName)));
			Text.AppendLine("\t<key>CFBundlePackageType</key>");
			Text.AppendLine("\t<string>APPL</string>");
			Text.AppendLine("\t<key>CFBundleSignature</key>");
			Text.AppendLine("\t<string>????</string>");
			Text.AppendLine("\t<key>CFBundleVersion</key>");
			Text.AppendLine(String.Format("\t<string>{0}</string>", UEDeployIOS.VersionUtilities.UpdateBundleVersion(OldPListData)));
			Text.AppendLine("\t<key>CFBundleShortVersionString</key>");
			Text.AppendLine(String.Format("\t<string>{0}</string>", BundleShortVersion));
			Text.AppendLine("\t<key>LSRequiresIPhoneOS</key>");
			Text.AppendLine("\t<true/>");
			Text.AppendLine("\t<key>UIRequiredDeviceCapabilities</key>");
			Text.AppendLine("\t<array>");
			foreach (string Line in RequiredCaps.Split("\r\n".ToCharArray()))
			{
				if (!String.IsNullOrWhiteSpace(Line))
				{
					Text.AppendLine(Line);
				}
			}

			Text.AppendLine("\t</array>");

			Text.AppendLine("\t<key>TVTopShelfImage</key>");
			Text.AppendLine("\t<dict>");
			Text.AppendLine("\t\t<key>TVTopShelfPrimaryImageWide</key>");
			Text.AppendLine("\t\t<string>Top Shelf Image Wide</string>");
			Text.AppendLine("\t</dict>");
			Text.AppendLine("\t<key>CFBundleIcons</key>");
			Text.AppendLine("\t<dict>");
			Text.AppendLine("\t\t<key>CFBundlePrimaryIcon</key>");
			Text.AppendLine("\t\t<string>App Icon</string>");
			Text.AppendLine("\t</dict>");
			Text.AppendLine("\t<key>UILaunchStoryboardName</key>");
			Text.AppendLine("\t<string>LaunchScreen</string>");

			// write the iCloud container identifier, if present in the old file
			if (!String.IsNullOrEmpty(OldPListData))
			{
				int index = OldPListData.IndexOf("ICloudContainerIdentifier", StringComparison.Ordinal);
				if (index > 0)
				{
					index = OldPListData.IndexOf("<string>", index, StringComparison.Ordinal) + 8;
					int length = OldPListData.IndexOf("</string>", index, StringComparison.Ordinal) - index;
					string ICloudContainerIdentifier = OldPListData.Substring(index, length);
					Text.AppendLine("\t<key>ICloudContainerIdentifier</key>");
					Text.AppendLine(String.Format("\t<string>{0}</string>", ICloudContainerIdentifier));
				}
			}

			Text.AppendLine("</dict>");
			Text.AppendLine("</plist>");

			// Create the intermediate directory if needed
			if (!Directory.Exists(IntermediateDirectory))
			{
				Directory.CreateDirectory(IntermediateDirectory);
			}

			File.WriteAllText(PListFile, Text.ToString());

			if (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Mac)
			{
				if (!Directory.Exists(AppDirectory))
				{
					Directory.CreateDirectory(AppDirectory);
				}

				File.WriteAllText(AppDirectory + "/Info.plist", Text.ToString());
			}

			return bSkipDefaultPNGs;
		}
	}
}