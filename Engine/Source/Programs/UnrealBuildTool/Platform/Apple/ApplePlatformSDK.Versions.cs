// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;

namespace UnrealBuildTool
{
	/////////////////////////////////////////////////////////////////////////////////////
	// If you are looking for version numbers, see Engine/Config/Apple/Apple_SDK.json
	/////////////////////////////////////////////////////////////////////////////////////

	// NOTE: These are currently only used for Mac targets

	partial class ApplePlatformSDK : UEBuildPlatformSDK
	{
		/// <summary>
		/// Get the default deployment target version for the given target type. This will be put into the .plist file for runtime check
		/// Will be in the format A.B.C
		/// </summary>
		/// <param name="Type"></param>
		/// <returns></returns>
		public string GetDeploymentTargetVersion(TargetType Type)
		{
			string? DeploymentTarget = null;
			if (Type == TargetType.Editor || Type == TargetType.Program)
			{
				DeploymentTarget = GetVersionFromConfig("EditorDeploymentTarget");
			}

			DeploymentTarget ??= GetRequiredVersionFromConfig("DeploymentTarget");

			return DeploymentTarget;
		}

		/// <summary>
		/// Get the default build target version for the given target type. This will be passed to clang when compiling/linking
		/// Will be in the format AA.BB
		/// </summary>
		/// <param name="Type"></param>
		/// <returns></returns>
		public string GetBuildTargetVersion(TargetType Type)
		{
			string? DeploymentTarget = null;
			if (Type == TargetType.Editor || Type == TargetType.Program)
			{
				DeploymentTarget = GetVersionFromConfig("EditorBuildTarget");
			}

			DeploymentTarget ??= GetRequiredVersionFromConfig("BuildTarget");

			return DeploymentTarget;
		}
	}
}
