// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics;
using System.IO;
using System.Runtime.Versioning;
using System.Text.RegularExpressions;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using Microsoft.Win32;
using UnrealBuildBase;

///////////////////////////////////////////////////////////////////
// If you are looking for supported version numbers, look in the
// ApplePlatformSDK.Versions.cs file next to this file, and
// als IOS/IOSPlatformSDK.Versions.cs
///////////////////////////////////////////////////////////////////

namespace UnrealBuildTool
{
	internal partial class ApplePlatformSDK : UEBuildPlatformSDK
	{
		public ApplePlatformSDK(ILogger Logger)
			: base(Logger)
		{
		}

		public static Lazy<DirectoryReference?> _DeveloperDir = new(() => GetDeveloperDir());

		// this shouldn't get called if there's no SDK installed
		public static DirectoryReference DeveloperDir => _DeveloperDir.Value!;

		// the version of Xcode that is installed, not any of the SDKs inside
		public static Lazy<string?> InstalledXcodeVersion => new(() => _GetInstalledSDKVersion());

		public static DirectoryReference GetToolchainDirectory()
		{
			return DirectoryReference.Combine(DeveloperDir, "Toolchains/XcodeDefault.xctoolchain/usr/bin");
		}

		public static DirectoryReference GetPlatformSDKDirectory(string AppleOSName)
		{
			return DirectoryReference.Combine(DeveloperDir, $"Platforms/{AppleOSName}.platform/Developer/SDKs/{AppleOSName}.sdk");
		}

		public static string? GetPlatformSDKVersion(string AppleOSName)
		{
			return GetVersionFromSDKDir(AppleOSName);
		}

		public static float GetPlatformSDKVersionFloat(string AppleOSName)
		{
			return Single.Parse(GetPlatformSDKVersion(AppleOSName) ?? "0.0", System.Globalization.CultureInfo.InvariantCulture);
		}

		// 8 bits per component, with high getting extra from high 32
		[GeneratedRegex(@"^(\d+).(\d+)(.(\d+))?(.(\d+))?(.(\d+))?$")]
		private static partial Regex VersionToIntRegex();

		public override bool TryConvertVersionToInt(string? StringValue, out ulong OutValue, string? Hint)
		{
			OutValue = 0;

			if (StringValue == null)
			{
				return false;
			}

			Match Result = VersionToIntRegex().Match(StringValue);
			if (Result.Success)
			{
				OutValue = UInt64.Parse(Result.Groups[1].Value) << 24 | UInt64.Parse(Result.Groups[2].Value) << 16;
				if (Result.Groups[4].Success)
				{
					OutValue |= UInt64.Parse(Result.Groups[4].Value) << 8;
				}
				if (Result.Groups[6].Success)
				{
					OutValue |= UInt64.Parse(Result.Groups[6].Value) << 0;
				}
				return true;
			}

			return false;
		}
		
		protected override string? GetInstalledSDKVersion()
		{
			return _GetInstalledSDKVersion();
		}

		private static string? _GetInstalledSDKVersion()
		{
			// get Xcode version on Mac
			if (OperatingSystem.IsMacOS())
			{
				if (_DeveloperDir.Value == null)
				{
					return null;
				}
				FileReference Plist = FileReference.Combine(DeveloperDir.ParentDirectory!, "Info.plist");
				// Find out the version number in Xcode.app/Contents/Info.plist
				int ExitCode;
				string Output = Utils.RunLocalProcessAndReturnStdOut("/bin/sh",
						$"-c 'plutil -extract CFBundleShortVersionString raw {Plist}'", out ExitCode);
				if (ExitCode == 0)
				{
					return Output;
				}
				return null;
			}

			if (OperatingSystem.IsWindows())
			{
				// otherwise, get iTunes "Version"
				string? DllPath =
					Registry.GetValue(
						"HKEY_LOCAL_MACHINE\\SOFTWARE\\Wow6432Node\\Apple Inc.\\Apple Mobile Device Support\\Shared",
						"iTunesMobileDeviceDLL", null) as string;
				if (String.IsNullOrEmpty(DllPath) || !File.Exists(DllPath))
				{
					DllPath = Registry.GetValue(
						"HKEY_LOCAL_MACHINE\\SOFTWARE\\Wow6432Node\\Apple Inc.\\Apple Mobile Device Support\\Shared",
						"MobileDeviceDLL", null) as string;
					if (String.IsNullOrEmpty(DllPath) || !File.Exists(DllPath))
					{
						// iTunes >= 12.7 doesn't have a key specifying the 32-bit DLL but it does have a ASMapiInterfaceDLL key and MobileDevice.dll is in usually in the same directory
						DllPath = Registry.GetValue(
							"HKEY_LOCAL_MACHINE\\SOFTWARE\\Wow6432Node\\Apple Inc.\\Apple Mobile Device Support\\Shared",
							"ASMapiInterfaceDLL", null) as string;
						DllPath = String.IsNullOrEmpty(DllPath)
							? null
							: DllPath.Substring(0, DllPath.LastIndexOf('\\') + 1) + "MobileDevice.dll";

						if (String.IsNullOrEmpty(DllPath) || !File.Exists(DllPath))
						{
							DllPath = FindWindowsStoreITunesDLL();
						}
					}
				}

				if (!String.IsNullOrEmpty(DllPath) && File.Exists(DllPath))
				{
					string? DllVersion = FileVersionInfo.GetVersionInfo(DllPath).FileVersion;
					// Only return the DLL version as the SDK version if we can correctly parse it
					ApplePlatformSDK? Sdk = UEBuildPlatformSDK.GetSDKForPlatformOrMakeTemp<ApplePlatformSDK>("Mac");
					if (Sdk != null && Sdk.TryConvertVersionToInt(DllVersion, out _, null))
					{
						return DllVersion;
					}
				}
			}

			return null;
		}

		[SupportedOSPlatform("windows")]
		private static string? FindWindowsStoreITunesDLL()
		{
			string? InstallPath = null;

			string PackagesKeyName = "Software\\Classes\\Local Settings\\Software\\Microsoft\\Windows\\CurrentVersion\\AppModel\\PackageRepository\\Packages";

			RegistryKey? PackagesKey = Registry.LocalMachine.OpenSubKey(PackagesKeyName);
			if (PackagesKey != null)
			{
				string[] PackageSubKeyNames = PackagesKey.GetSubKeyNames();

				foreach (string PackageSubKeyName in PackageSubKeyNames)
				{
					if (PackageSubKeyName.Contains("AppleInc.iTunes", StringComparison.Ordinal) &&
						(PackageSubKeyName.Contains("_x64", StringComparison.Ordinal) || PackageSubKeyName.Contains("_x86", StringComparison.Ordinal)))
					{
						string FullPackageSubKeyName = PackagesKeyName + "\\" + PackageSubKeyName;

						RegistryKey? iTunesKey = Registry.LocalMachine.OpenSubKey(FullPackageSubKeyName);
						if (iTunesKey != null)
						{
							object? Value = iTunesKey.GetValue("Path");
							if (Value != null)
							{
								InstallPath = (string)Value + "\\AMDS32\\MobileDevice.dll";
							}
							break;
						}
					}
				}
			}

			return InstallPath;
		}

		private static DirectoryReference? GetDeveloperDir()
		{
			int ExitCode;
			// xcode-select -p gives the currently selected Xcode location (xcodebuild -version may fail if Xcode.app is broken)
			// Example output: /Applications/Xcode.app/Contents/Developer
			string Output = Utils.RunLocalProcessAndReturnStdOut("/bin/sh", "-c 'xcode-select -p'", out ExitCode);

			if (ExitCode == 0)
			{
				return new DirectoryReference(Output);
			}

			return null;
		}

		private static string? GetVersionFromSDKDir(string AppleOSName)
		{
			string? PlatformSDKVersion = null;
			try
			{
				// loop over the subdirs and parse out the version
				int MaxSDKVersionMajor = 0;
				int MaxSDKVersionMinor = 0;
				string? MaxSDKVersionString = null;
				foreach (DirectoryReference SubDir in DirectoryReference.EnumerateDirectories(GetPlatformSDKDirectory(AppleOSName).ParentDirectory!))
				{
					string SubDirName = Path.GetFileNameWithoutExtension(SubDir.GetDirectoryName());
					if (SubDirName.StartsWith(AppleOSName, StringComparison.Ordinal))
					{
						// get the SDK version from the directory name
						string SDKString = SubDirName.Replace(AppleOSName, "", StringComparison.Ordinal);
						int Major = 0;
						int Minor = 0;

						// parse it into whole and fractional parts (since 10.10 > 10.9 in versions, but not in math)
						try
						{
							string[] Tokens = SDKString.Split(".".ToCharArray());
							if (Tokens.Length == 2)
							{
								Major = Int32.Parse(Tokens[0]);
								Minor = Int32.Parse(Tokens[1]);
							}
						}
						catch (Exception)
						{
							// weirdly formatted SDKs
							continue;
						}

						// update largest SDK version number
						if (Major > MaxSDKVersionMajor || (Major == MaxSDKVersionMajor && Minor > MaxSDKVersionMinor))
						{
							MaxSDKVersionString = SDKString;
							MaxSDKVersionMajor = Major;
							MaxSDKVersionMinor = Minor;
						}
					}
				}

				// use the largest version
				if (MaxSDKVersionString != null)
				{
					PlatformSDKVersion = MaxSDKVersionString;
				}
			}
			catch (Exception)
			{
				return null;
			}

			return PlatformSDKVersion;
		}

		protected override SDKStatus HasRequiredManualSDKInternal()
		{
			SDKStatus Status = base.HasRequiredManualSDKInternal();

			// iTunes is technically only need to deploy to and run on connected devices.
			// This code removes requirement for Windows builders to have Xcode installed.
			if (Status == SDKStatus.Invalid && !OperatingSystem.IsMacOS() && Unreal.IsBuildMachine())
			{
				Status = SDKStatus.Valid;
			}
			return Status;
		}
	}
}