// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using Microsoft.Extensions.Logging;

namespace UnrealBuildTool
{
	/// <summary>
	/// Base class for Apple-based project generators
	/// </summary>
	abstract class AppleProjectGenerator : PlatformProjectGenerator
	{
		private static readonly XcrunRunner AppleHelper = new XcrunRunner();

		/// <inheritdoc/>
		protected AppleProjectGenerator(ILogger Logger)
			: base(Logger)
		{
		}

		/// <inheritdoc/>
		public override bool HasVisualStudioSupport(VSSettings InVSSettings)
		{
			return false;
		}

		/// <inheritdoc/>
		public override IList<string> GetSystemIncludePaths(UEBuildTarget InTarget)
		{
			if (UEBuildPlatform.IsPlatformInGroup(BuildHostPlatform.Current.Platform, UnrealPlatformGroup.Apple))
			{
				// Only generate Apple system include paths when host platform is Apple OS
				return AppleHelper.GetAppleSystemIncludePaths(InTarget.Architectures.SingleArchitecture, InTarget.Platform, Logger);
			}

			// TODO: Fix case when working with MacOS on Windows host platform
			return new List<string>(0);
		}

		private sealed class XcrunRunner : IDisposable
		{
			private readonly Dictionary<string, IList<string>> CachedIncludePaths = new Dictionary<string, IList<string>>();

			private string CurrentlyProcessedSDK = String.Empty;
			private Process? XcrunProcess;
			private bool IsReadingIncludesSection;

			/// <inheritdoc/>
			public void Dispose()
			{
				XcrunProcess?.Kill(true);
				XcrunProcess?.Dispose();
				XcrunProcess = null;
			}

			public IList<string> GetAppleSystemIncludePaths(UnrealArch Architecture, UnrealTargetPlatform Platform, ILogger Logger)
			{
				if (!UEBuildPlatform.IsPlatformInGroup(Platform, UnrealPlatformGroup.Apple))
				{
					throw new InvalidOperationException("xcrun can be run only for Apple's platforms");
				}

				string SDKPath = GetSDKPath(Architecture, Platform);
				if (!CachedIncludePaths.ContainsKey(SDKPath))
				{
					CalculateSystemIncludePaths(SDKPath, Logger);
				}

				return CachedIncludePaths[SDKPath];
			}

			private void CalculateSystemIncludePaths(string SDKPath, ILogger Logger)
			{
				if (!String.IsNullOrEmpty(CurrentlyProcessedSDK))
				{
					throw new InvalidOperationException("Cannot calculate include paths for several platforms at once");
				}

				CurrentlyProcessedSDK = SDKPath;
				CachedIncludePaths[SDKPath] = new List<string>();
				using (XcrunProcess = new Process())
				{
					string AppName = "xcrun";
					string SystemRootArgument = String.IsNullOrEmpty(SDKPath) ? String.Empty : (" -isysroot " + SDKPath);
					string Arguments = "clang++ -Wp,-v -x c++ - -fsyntax-only" + SystemRootArgument;
					XcrunProcess.StartInfo.FileName = AppName;
					XcrunProcess.StartInfo.Arguments = Arguments;
					XcrunProcess.StartInfo.UseShellExecute = false;
					XcrunProcess.StartInfo.CreateNoWindow = true;

					// For some weird reason output of this command is written to error channel so we're redirecting both channels
					XcrunProcess.StartInfo.RedirectStandardOutput = true;
					XcrunProcess.StartInfo.RedirectStandardError = true;
					XcrunProcess.OutputDataReceived += OnOutputDataReceived;
					XcrunProcess.ErrorDataReceived += OnOutputDataReceived;
					XcrunProcess.Start();
					XcrunProcess.BeginOutputReadLine();
					XcrunProcess.BeginErrorReadLine();

					// xcrun is not finished on it's own. It should be killed by OnOutputDataReceived when reading is finished. But we'll add timeout as a safeguard.
					// While usually it is fast, first launch on macOS might take ~10 seconds so timeout is quite big: https://github.com/llvm/llvm-project/issues/75179
					bool HasExited = XcrunProcess.WaitForExit(30_000);
					if (!HasExited)
					{
						Logger.LogWarning("xcrun didn't finish in 30 second. List of system include paths will not be complete");
						XcrunProcess.Kill();
					}
				}

				XcrunProcess = null;
				IsReadingIncludesSection = false;
				CurrentlyProcessedSDK = String.Empty;
			}

			private void OnOutputDataReceived(object Sender, DataReceivedEventArgs Args)
			{
				if (Args.Data != null)
				{
					if (IsReadingIncludesSection)
					{
						if (Args.Data.StartsWith("End of search", StringComparison.Ordinal))
						{
							IsReadingIncludesSection = false;
							XcrunProcess!.Kill();
						}
						else
						{
							if (!Args.Data.EndsWith("(framework directory)", StringComparison.Ordinal))
							{
								CachedIncludePaths[CurrentlyProcessedSDK].Add(Args.Data.Trim(' ', '"'));
							}
						}
					}

					if (Args.Data.StartsWith("#include <...>", StringComparison.Ordinal))
					{
						IsReadingIncludesSection = true;
					}
				}
			}

			private static string GetSDKPath(UnrealArch Architecture, UnrealTargetPlatform Platform)
			{
				if (Platform == UnrealTargetPlatform.Mac)
				{
					return ApplePlatformSDK.GetPlatformSDKDirectory("MacOSX").FullName;
				}

				return ApplePlatformSDK.GetPlatformSDKDirectory(Architecture == UnrealArch.IOSSimulator ? "iPhoneSimulator" : "iPhoneOS").FullName;
			}
		}
	}
}
