// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using System.Linq;
using System.Security.Cryptography.X509Certificates;
using System.Text.Json;
using System.Text.Json.Serialization;
using System.Text.RegularExpressions;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	abstract class AppleToolChain : ClangToolChain
	{
		protected class AppleToolChainInfo : ClangToolChainInfo
		{
			readonly Tuple<Version, Version>[] AppleVersionToLLVMVersion;

			public AppleToolChainInfo(UnrealTargetPlatform Platform, DirectoryReference DeveloperDir, FileReference Clang, FileReference Archiver, ILogger Logger)
				: base(DeveloperDir, Clang, Archiver, Logger)
			{
				// get the mapping from Apple_SDK.json (turning "version ranges" into a mapping list)
				UEBuildPlatformSDK SDK = UEBuildPlatformSDK.GetSDKForPlatform(Platform.ToString())!;
				
				AppleVersionToLLVMVersion = SDK.GetVersionNumberRangeArrayFromConfig("AppleVersionToLLVMVersions").
					Select(x => new Tuple<Version, Version>(new Version(x.Min.ToString()), new Version(x.Max.ToString()))).ToArray();
			}

			// libtool doesn't provide version, just use clang's version string
			/// <inheritdoc/>
			protected override string QueryArchiverVersionString() => ClangVersionString;

			protected override Version QueryClangVersion()
			{
				// base Apple clang version against known toolchain version included with Xcode.
				Version AppleVersion = new Version(ApplePlatformSDK.InstalledXcodeVersion.Value ?? "0.0");

				// now look up this version in mappings
				for (int MappingIndex = AppleVersionToLLVMVersion.Length - 1; MappingIndex >= 0; MappingIndex--)
				{
					if (AppleVersion >= AppleVersionToLLVMVersion[MappingIndex].Item1)
					{
						Version LLVMVersion = AppleVersionToLLVMVersion[MappingIndex].Item2;
						Logger.LogDebug("Converted Apple version {AppleVersion} to LLVM version {LLVMVersion}", AppleVersion, LLVMVersion);
						return LLVMVersion;
					}
				}

				throw new BuildException($"Failed to find mapping of Apple clang version {AppleVersion} in Apple_SDK.json");
			}

			// get the actual Apple version, not the LLVM version (only Apple platform code should use this)
			public VersionNumber AppleClangVersion => VersionNumber.Parse(base.QueryClangVersion().ToString());
		}

		private readonly Lazy<AppleToolChainSettings> ToolChainSettings;
		
		protected AppleToolChainSettings Settings => ToolChainSettings.Value;

		public abstract ReadOnlyAppleTargetRules GetAppleTargetRules(ReadOnlyTargetRules Target);

		/// <summary>
		/// The SDK version being used to compile with the toolchain
		/// </summary>
		public float SDKVersionFloat => Settings.SDKVersionFloat;

		public readonly ReadOnlyTargetRules? Target;

		// cache some ini settings
		readonly bool bUseSwiftUIMain;

		/// <summary>
		/// Serializer options for serializing <see cref="ApplePostBuildSyncTarget"/> instances.
		/// </summary>
		public static readonly JsonSerializerOptions SerializerOptions = new()
		{
			IncludeFields = true,
		};

		protected ApplePortableToolChainInfo? PortableToolchain;
		private static bool CrossArchitecturePathsRegistered;

		// Platform identifier for Apple OSS headers when compiling without the real SDK
		protected abstract string XnuPlatformDefine { get; }

		public AppleToolChain(ReadOnlyTargetRules? Target, Func<AppleToolChainSettings> InCreateSettings, ClangToolChainOptions InOptions, ILogger InLogger) : base(InOptions, InLogger)
		{
			this.Target = Target;
			ProjectFile = Target?.ProjectFile;
			ToolChainSettings = new Lazy<AppleToolChainSettings>(InCreateSettings);

			AppleExports.GetSwiftIntegrationSettings(ProjectFile, Target == null ? TargetType.Game : Target.Type, Target == null ? UnrealTargetPlatform.Mac : Target.Platform, out bUseSwiftUIMain, out _);
			// Initialize portable toolchain for distributed compilation of non-SDK files
			InitializePortableToolchain(Target);
		}

		private void InitializePortableToolchain(ReadOnlyTargetRules? Target)
		{
			if (PortableToolchain != null)
			{
				return;
			}

			if (Target == null || !GetAppleTargetRules(Target).bUsePortableToolchain)
			{
				return;
			}

			ApplePortableToolChainInfo ToolChainInfo = new ApplePortableToolChainInfo(Logger);
			if (ToolChainInfo.IsValid)
			{
				PortableToolchain = ToolChainInfo;
				Logger.LogInformation("Portable toolchain initialized: {Version}", ToolChainInfo.SDKVersion);
				RegisterUBAPathsForDistribution();
			}
		}

		private static void RegisterUBAPathsForDistribution()
		{
			EpicGames.UBA.Utils.RegisterDisallowedPaths(
				new("/usr/bin/"),
				new("/Applications/Xcode.app/"),
				new("/Library/Developer/CommandLineTools/"));
		}

		public static void RegisterCrossArchitecturePathsForUBA(ILogger Logger)
		{
			if (CrossArchitecturePathsRegistered)
			{
				return;
			}
			
			CrossArchitecturePathsRegistered = true;

			DirectoryReference? SdksRoot = DirectoryReference.FromString(Environment.GetEnvironmentVariable("UE_SDKS_ROOT"));
			if (SdksRoot is null)
			{
				Logger.LogDebug("[UBA] UE_SDKS_ROOT not set, skipping Apple cross-architecture path registration");
				return;
			}

			DirectoryReference MacAppleDirectory = DirectoryReference.Combine(SdksRoot, "HostMac", "Apple");
			string? Version = ApplePortableToolChainInfo.ResolveVersion(MacAppleDirectory, Logger);
			if (Version == null)
			{
				Logger.LogDebug("[UBA] No Apple toolchain version found in {Path}", MacAppleDirectory);
				return;
			}

			DirectoryReference ClangBasePath = DirectoryReference.Combine(SdksRoot, "HostMac", "Apple", Version, "Clang");
			DirectoryReference MacBinPath = DirectoryReference.Combine(ClangBasePath, "bin");
			DirectoryReference LinuxBinPath = DirectoryReference.Combine(ClangBasePath, "binlinux");
			DirectoryReference WindowsBinPath = DirectoryReference.Combine(ClangBasePath, "binwin64");

			if (DirectoryReference.Exists(MacBinPath))
			{
				if (DirectoryReference.Exists(LinuxBinPath))
				{
					EpicGames.UBA.Utils.RegisterCrossArchitecturePath(MacBinPath, LinuxBinPath);
					Logger.LogDebug("[UBA] Early registration: {MacPath} -> {LinuxPath}", MacBinPath, LinuxBinPath);
				}
				if (DirectoryReference.Exists(WindowsBinPath))
				{
					EpicGames.UBA.Utils.RegisterCrossArchitecturePath(MacBinPath, WindowsBinPath);
					Logger.LogDebug("[UBA] Early registration: {MacPath} -> {WindowsPath}", MacBinPath, WindowsBinPath);
				}
			}
		}

		/// <inheritdoc/>
		public override void SetUpGlobalEnvironment(ReadOnlyTargetRules Target, CppCompileEnvironment GlobalCompileEnvironment, LinkEnvironment GlobalLinkEnvironment)
		{
			base.SetUpGlobalEnvironment(Target, GlobalCompileEnvironment, GlobalLinkEnvironment);

			DirectoryReference XcodeDeveloperDir = ApplePlatformSDK.DeveloperDir;
			DirectoryReference? XcodeAppDir = XcodeDeveloperDir.ParentDirectory?.ParentDirectory;

			if (XcodeAppDir != null)
			{
				GlobalCompileEnvironment.RootPaths.AddFolderName(CppRootPathFolder.PlatformSDK, "Xcode.app");
				GlobalLinkEnvironment.RootPaths.AddFolderName(CppRootPathFolder.PlatformSDK, "Xcode.app");
				GlobalCompileEnvironment.RootPaths[CppRootPathFolder.PlatformSDK] = XcodeAppDir;
				GlobalLinkEnvironment.RootPaths[CppRootPathFolder.PlatformSDK] = XcodeAppDir;

				EpicGames.UBA.Utils.RegisterPathHash(XcodeAppDir, ApplePlatformSDK.InstalledXcodeVersion.Value ?? "0");

				// Xcode sdk files are not allowed to be transferred over the network, ensure UBA will not do so
				EpicGames.UBA.Utils.RegisterDisallowedPaths(XcodeAppDir);
			}

			if (PortableToolchain != null)
			{
				// In the current setup, the parent directory of the clang binary, contains both clang, and apple oss header directories 
				GlobalCompileEnvironment.RootPaths.AddExtraPath(("PortableToolchain", PortableToolchain.ClangBasePath!.ParentDirectory!.FullName));
				GlobalLinkEnvironment.RootPaths.AddExtraPath(("PortableToolchain", PortableToolchain.ClangBasePath!.ParentDirectory!.FullName));
				
				EpicGames.UBA.Utils.RegisterPathHash(PortableToolchain.ClangBasePath!.ParentDirectory!, PortableToolchain.SDKVersion!);
			}
		}

		/// <summary>
		/// Takes an architecture string as provided by UBT for the target and formats it for Clang. Supports
		/// multiple architectures joined with '+'
		/// </summary>
		/// <param name="InArchitectures"></param>
		/// <returns></returns>
		protected static string FormatArchitectureArg(UnrealArchitectures InArchitectures)
		{
			return "-arch " + String.Join('+', InArchitectures.Architectures.Select(x => x.AppleName));
		}

		public override string GetSDKVersion()
		{
			return Settings.SDKVersion;
		}

		protected static DirectoryReference GetMacDevSrcRoot()
		{
			return Unreal.EngineSourceDirectory;
		}

		protected override CPPOutput CompileCPPFiles(CppCompileEnvironment CompileEnvironment, IEnumerable<FileItem> InputFiles, DirectoryReference OutputDir, string ModuleName, IActionGraphBuilder Graph)
		{
			if (ShouldSkipCompile(CompileEnvironment))
			{
				return new CPPOutput();
			}

			bool bUsePortableToolchain = PortableToolchain != null && !CompileEnvironment.bRequiresPlatformSDK;

			Log.TraceInformationOnce($"Compiling with {CompileEnvironment.Platform} SDK {Settings.SDKVersion}, minimum Target OS version {Settings.GetTargetVersionForTargetType(Target!.Type)}");

			List<FileItem> FrameworkTokenFiles = new List<FileItem>();
			foreach (UEBuildFramework Framework in CompileEnvironment.AdditionalFrameworks)
			{
				if (Framework.ZipFile != null)
				{
					// even in modern, we have to try to unzip in UBT, so that commandline builds extract the frameworks _before_ building (the Xcode finalize
					// happens after building). We still need the unzip in Xcode pre-build phase, because Xcode wants the frameworks around for its build graph
					ExtractFramework(Framework, CompileEnvironment.RootPaths, Graph);
					FrameworkTokenFiles.Add(Framework.ExtractedTokenFile!);
				}
			}
			
			List<string> GlobalArguments = new();

			if (bUsePortableToolchain)
			{
				GetCompileArguments_PortableToolchain(CompileEnvironment, GlobalArguments);
			}
			else
			{
				GetCompileArguments_Global(CompileEnvironment, GlobalArguments);
			}

			CPPOutput Result = new CPPOutput();
			// Create a compile action for each source file.
			List<FileItem> SwiftFiles = new List<FileItem>();

			foreach (FileItem SourceFile in InputFiles)
			{
				if (SourceFile.HasExtension(".swift"))
				{
					SwiftFiles.Add(SourceFile);
				}
				else
				{
					Action CompileAction = CompileCPPFile(CompileEnvironment, SourceFile, OutputDir, ModuleName, Graph, GlobalArguments, Result);
					CompileAction.PrerequisiteItems.UnionWith(FrameworkTokenFiles);

					if (bUsePortableToolchain)
					{
						CompileAction.CommandPath = PortableToolchain!.ClangPath!;
						CompileAction.bCanExecuteInUBACrossArchitecture = true;
					}
					else
					{
						CompileAction.bCanExecuteInUBACrossArchitecture = false;
					}
				}
			}
			if (SwiftFiles.Any())
			{
				Action CompileAction = CompileSwiftFiles(CompileEnvironment, SwiftFiles, OutputDir, ModuleName, Graph, Result);
				CompileAction.PrerequisiteItems.UnionWith(FrameworkTokenFiles);

				if (bUsePortableToolchain)
				{
					CompileAction.bCanExecuteInUBACrossArchitecture = false;
				}
			}
	
			return Result;
		}

		protected void StripSymbolsWithXcode(FileReference SourceFile, FileReference TargetFile, DirectoryReference ToolchainDir)
		{
			if (SourceFile != TargetFile)
			{
				// Strip command only works in place so we need to copy original if target is different
				File.Copy(SourceFile.FullName, TargetFile.FullName, true);
			}

			ProcessStartInfo StartInfo = CreateStripSymbolsProcessInfo(TargetFile, ToolchainDir);
			Utils.RunLocalProcessAndLogOutput(StartInfo, Logger);
		}

		protected static ProcessStartInfo CreateStripSymbolsProcessInfo(FileReference File, DirectoryReference ToolchainDir) => new()
		{
			FileName = Path.Combine(ToolchainDir.FullName, "strip"),
			Arguments = $"\"{File.FullName}\" -S",
			UseShellExecute = false,
			CreateNoWindow = true
		};

		/// <summary>
		/// Writes a versions.xcconfig file for xcode to pull in when making an app plist
		/// </summary>
		/// <param name="LinkEnvironment"></param>
		/// <param name="Prerequisite">FileItem describing the Prerequisite that this will this depends on (executable or similar) </param>
		/// <param name="Graph">List of actions to be executed. Additional actions will be added to this list.</param>
		protected FileItem UpdateVersionFile(LinkEnvironment LinkEnvironment, FileItem Prerequisite, IActionGraphBuilder Graph)
		{
			FileItem DestFile;

			// Make the compile action
			Action UpdateVersionAction = Graph.CreateAction(ActionType.CreateAppBundle);
			UpdateVersionAction.WorkingDirectory = GetMacDevSrcRoot();
			UpdateVersionAction.CommandPath = BuildHostPlatform.Current.Shell;
			UpdateVersionAction.CommandDescription = "";

			// @todo programs right nhow are sharing the Engine build version - one reason for this is that we can't get to the Engine/Programs directory from here
			// (we can't even get to the Engine/Source/Programs directory without searching on disk), and if we did, we would create a _lot_ of Engine/Programs directories
			// on disk that don't exist in p4. So, we just re-use Engine version, not Project version
			//				DirectoryReference ProductDirectory = FindProductDirectory(ProjectFile, LinkEnvironment.OutputDirectory!, Graph.Makefile.TargetType);
			DirectoryReference ProductDirectory = (ProjectFile?.Directory) ?? Unreal.EngineDirectory;
			FileReference OutputVersionFile = FileReference.Combine(ProductDirectory, "Intermediate/Build/Versions.xcconfig");
			DestFile = FileItem.GetItemByFileReference(OutputVersionFile);

			// grab a changlist version if we have it to pass to the script to use if desired
			int Changelist = 0;
			BuildVersion? Version;
			if (BuildVersion.TryRead(BuildVersion.GetDefaultFileName(), out Version))
			{
				Changelist = Version.Changelist;
			}

			// make path to the script
			FileReference VersionScript = FileReference.Combine(ProductDirectory, "Build/BatchFiles/Mac/UpdateVersionAfterBuild.sh");
			if (!FileReference.Exists(VersionScript))
			{
				VersionScript = FileReference.Combine(Unreal.EngineDirectory, "Build/BatchFiles/Mac/UpdateVersionAfterBuild.sh");
			}
			FileItem BundleScript = FileItem.GetItemByFileReference(VersionScript);
			UpdateVersionAction.CommandArguments = $"\"{BundleScript.AbsolutePath}\" \"{ProductDirectory}\" {LinkEnvironment.Platform} {Changelist}";
			UpdateVersionAction.PrerequisiteItems.Add(Prerequisite);
			UpdateVersionAction.PrerequisiteItems.Add(BundleScript);
			UpdateVersionAction.ProducedItems.Add(DestFile);
			UpdateVersionAction.StatusDescription = $"Updating version file: {OutputVersionFile}";

			return DestFile;
		}

		protected FileItem ExtractFramework(UEBuildFramework framework, CppRootPaths rootPaths, IActionGraphBuilder graph)
		{
			if (framework.ZipFile == null)
			{
				throw new BuildLogEventException("Unable to extract framework '{Name}': no zip file specified", framework.Name);
			}
			if (!framework.bHasMadeUnzipAction)
			{
				framework.bHasMadeUnzipAction = true;

				rootPaths = new(rootPaths);
				rootPaths.bUseVfs = false;

				FileItem inputFile = FileItem.GetItemByFileReference(framework.ZipFile);

				string zipOutputDirectory = NormalizeCommandLinePath(framework.ZipOutputDirectory!, rootPaths);
				string zipParentDirectory = NormalizeCommandLinePath(framework.ZipOutputDirectory!.ParentDirectory!, rootPaths);
				string zipFile = NormalizeCommandLinePath(framework.ZipFile, rootPaths);
				string extractedTokenFile = NormalizeCommandLinePath(framework.ExtractedTokenFile!, rootPaths);

				IEnumerable<string> ExtractScript = [
					$"#!{BuildHostPlatform.Current.Shell}",
					"set -e",
					//"set -x", // For debugging
					$"[ -d \"{zipOutputDirectory}\" ] && rm -rf \"{zipOutputDirectory}\"",
					$"unzip -q -o \"{zipFile}\" -d \"{zipParentDirectory}\"", // Zip contains folder with the same name, hence ParentDirectory
					$"touch \"{extractedTokenFile}\""
				];

				// gather output files
				IEnumerable<FileItem> producedItems = [framework.ExtractedTokenFile!];

				// This is writing to folder shared by all configs.
				//FileReference ScriptFile = FileReference.Combine(IntermediateDirectory, Framework.ZipOutputDirectory.GetDirectoryName() + ".sh");
				//FileItem ExtractScriptFileItem = Graph.CreateIntermediateTextFile(ScriptFile, ExtractScript.ToString());
				FileItem extractScriptFileItem = graph.CreateIntermediateTextFile(new FileReference($"{framework.ZipOutputDirectory.FullName}.sh"), ExtractScript);

				Action UnzipAction = graph.CreateAction(ActionType.BuildProject);
				UnzipAction.CommandDescription = "Unzip";
				UnzipAction.CommandPath = BuildHostPlatform.Current.Shell;
				UnzipAction.CommandArguments = $"\"{NormalizeCommandLinePath(extractScriptFileItem, rootPaths)}\"";
				UnzipAction.WorkingDirectory = Unreal.EngineSourceDirectory;
				UnzipAction.PrerequisiteItems.Add(inputFile);
				UnzipAction.PrerequisiteItems.Add(extractScriptFileItem);
				UnzipAction.ProducedItems.UnionWith(producedItems);
				UnzipAction.DeleteItems.UnionWith(producedItems);
				UnzipAction.StatusDescription = inputFile.Name;
				UnzipAction.bCanExecuteRemotely = false;
			}
			return framework.ExtractedTokenFile!;
		}

		/// <summary>
		/// If the project is a UnrealGame project, Target.ProjectDirectory refers to the engine dir, not the actual dir of the project. So this method gets the 
		/// actual directory of the project whether it is a UnrealGame project or not.
		/// </summary>
		/// <returns>The actual project directory.</returns>
		/// <param name="ProjectFile">The path to the project file</param>
		internal static DirectoryReference GetActualProjectDirectory(FileReference? ProjectFile)
		{
			DirectoryReference ProjectDirectory = (ProjectFile == null ? Unreal.EngineDirectory : DirectoryReference.FromFile(ProjectFile)!);
			return ProjectDirectory;
		}

		/// <inheritdoc/>
		protected override string EscapePreprocessorDefinition(string Definition)
		{
			return Definition.Contains('"', StringComparison.Ordinal) ? Definition.Replace("\"", "\\\"", StringComparison.Ordinal) : Definition;
		}

		protected override void GetCppStandardCompileArgument(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			if (CompileEnvironment.bEnableObjCAutomaticReferenceCounting)
			{
				Arguments.Add("-fobjc-arc");
			}

			base.GetCppStandardCompileArgument(CompileEnvironment, Arguments);
		}

		protected static void GetCompileArguments_ClangModules(List<string> Arguments)
		{
			Arguments.Add("-fmodules");
			Arguments.Add("-fno-implicit-modules");
			Arguments.Add("-fimplicit-module-maps");
			Arguments.Add("-Wno-module-import-in-extern-c");
		}

		protected override void GetCompileArguments_CPP(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			Arguments.Add("-x objective-c++");
			GetCppStandardCompileArgument(CompileEnvironment, Arguments);
			Arguments.Add("-stdlib=libc++");

			bool bUsesPortableToolchain = CompileEnvironment.bRequiresPlatformSDK && PortableToolchain != null;
			if (bUsesPortableToolchain)
			{
				GetCompileArguments_ClangModules(Arguments);
			}
		}

		protected override void GetCompileArguments_MM(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			Arguments.Add("-x objective-c++");
			GetCppStandardCompileArgument(CompileEnvironment, Arguments);
			Arguments.Add("-stdlib=libc++");
		}

		protected override void GetCompileArguments_M(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			Arguments.Add("-x objective-c");
			GetCppStandardCompileArgument(CompileEnvironment, Arguments);
			Arguments.Add("-stdlib=libc++");
		}

		protected override void GetCompileArguments_PCH(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			Arguments.Add("-x objective-c++-header");
			GetCppStandardCompileArgument(CompileEnvironment, Arguments);
			Arguments.Add("-stdlib=libc++");
			Arguments.Add("-fpch-instantiate-templates");
		}

		/// <inheritdoc/>
		protected override void GetCompileArguments_WarningsAndErrors(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			base.GetCompileArguments_WarningsAndErrors(CompileEnvironment, Arguments);

			Arguments.Add("-Wno-unknown-warning-option");
			Arguments.Add("-Wno-range-loop-analysis");
			Arguments.Add("-Wno-single-bit-bitfield-constant-conversion");

			// Various new warning from Xcode16.3. Disable for now.
			VersionNumber AppleClangVersion = ((AppleToolChainInfo)Info).AppleClangVersion;
			if (AppleClangVersion >= new VersionNumber(17))
			{
				// "implicit conversion loses integer precision:" in many files
				Arguments.Add("-Wno-shorten-64-to-32");
				
				// clang17 is detecting some uses of cxx-extensions,
				// such as: "error: variable length arrays in C++ are a Clang extension" 
				Arguments.Add("-Wno-vla-extension");
				
				// From Engine/Plugins/Runtime/Metasound/Source/MetasoundFrontend/Public/MetasoundPrimitives/Time/AudioBuffers.h
				Arguments.Add("-Wno-extra-qualification");
			}
		}

		/// <inheritdoc/>
		protected override void GetCompileArguments_Optimizations(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			base.GetCompileArguments_Optimizations(CompileEnvironment, Arguments);

			bool bStaticAnalysis = false;
			string? StaticAnalysisMode = Environment.GetEnvironmentVariable("CLANG_STATIC_ANALYZER_MODE");
			if (!String.IsNullOrEmpty(StaticAnalysisMode))
			{
				bStaticAnalysis = true;
			}

			// Optimize non- debug builds.
			if (CompileEnvironment.bOptimizeCode && !bStaticAnalysis)
			{
				// Don't over optimise if using AddressSanitizer or you'll get false positive errors due to erroneous optimisation of necessary AddressSanitizer instrumentation.
				if (Options.HasFlag(ClangToolChainOptions.EnableAddressSanitizer))
				{
					Arguments.Add("-O1");
					Arguments.Add("-g");
					Arguments.Add("-fno-optimize-sibling-calls");
					Arguments.Add("-fno-omit-frame-pointer");
				}
				else if (Options.HasFlag(ClangToolChainOptions.EnableThreadSanitizer))
				{
					Arguments.Add("-O1");
					Arguments.Add("-g");
				}
				else if (CompileEnvironment.OptimizationLevel == OptimizationMode.Size)
				{
					Arguments.Add("-Oz");
				}
				else if (CompileEnvironment.OptimizationLevel == OptimizationMode.SizeAndSpeed)
				{
					Arguments.Add("-Os");

					if (CompileEnvironment.Architecture == UnrealArch.Arm64)
					{
						Arguments.Add("-moutline");
					}
				}
				else
				{
					Arguments.Add("-O3");
				}
			}
			else
			{
				Arguments.Add("-O0");
			}
		}

		/// <inheritdoc/>
		protected override void GetCompileArguments_Debugging(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			base.GetCompileArguments_Debugging(CompileEnvironment, Arguments);

			// Create DWARF format debug info if wanted,
			if (CompileEnvironment.bCreateDebugInfo)
			{
				Arguments.Add("-gdwarf-4");

				if (CompileEnvironment.bDebugLineTablesOnly)
				{
					Arguments.Add("-gline-tables-only");
				}
			}
		}

		/// <inheritdoc/>
		protected override void GetCompileArguments_Analyze(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			base.GetCompileArguments_Analyze(CompileEnvironment, Arguments);

			// Disable all clang tidy checks
			Arguments.Add($"-Xclang -analyzer-tidy-checker=-*");
		}

		/// <inheritdoc/>
		protected override void GetCompileArguments_Global(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			base.GetCompileArguments_Global(CompileEnvironment, Arguments);

			Arguments.Add(GetRTTIFlag(CompileEnvironment));

			Arguments.Add("-fmessage-length=0");
			Arguments.Add("-fpascal-strings");

			string? Override = (CompileEnvironment.bEnableOSX109Support && CompileEnvironment.Platform == UnrealTargetPlatform.Mac) ? "10.9" : null;
			Arguments.Add($"-target {ToolChainSettings.Value.GetTargetTuple(CompileEnvironment.Architecture, Target!.Platform, Target!.Type, Override)}");
		}

		/// <summary>
		/// Adds include paths for portable toolchain compilation with Apple OSS headers.
		/// Used for distributed compilation on non-Apple workers.
		/// </summary>
		protected void AddPortableToolchainIncludePaths(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			if (PortableToolchain == null || PortableToolchain.OSSHeadersPath == null)
			{
				throw new InvalidOperationException("AddPortableToolchainIncludePaths requires initialized PortableToolchain");
			}

			DirectoryReference OSSHeadersDir = DirectoryReference.Combine(PortableToolchain.OSSHeadersPath, "usr", "include");
			if (!DirectoryReference.Exists(OSSHeadersDir))
			{
				Logger.LogError("OSS headers not found at expected path: {Path}", OSSHeadersDir.FullName);
				return;
			}

			Arguments.Add("-nostdinc");

			// FreeBSD math.h compatibility
			Arguments.Add("-D__ISO_C_VISIBLE=1999");
			Arguments.Add("-D__float_t=float");
			Arguments.Add("-D__double_t=double");

			// Apple clang generates a binary with ABI where the move constructors are not considered when determining if a type is trivially reallocable
			// To keep the portable tool chain and default toolchain binaries compatible we need to force the ABI compat version to 4
			// @note: A longer term fix would be patch our version of clang
			Arguments.Add("-fclang-abi-compat=4");

			Arguments.Add($"-D{XnuPlatformDefine}");

			// libc++ headers must come first for #include_next to work
			if (PortableToolchain.LibCxxIncludePath != null && DirectoryReference.Exists(PortableToolchain.LibCxxIncludePath))
			{
				Arguments.Add($"-isystem \"{NormalizeCommandLinePath(PortableToolchain.LibCxxIncludePath, CompileEnvironment.RootPaths)}\"");
			}
			else
			{
				Logger.LogError("libc++ headers not found");
			}

			// Clang builtin headers (stddef.h, stdarg.h, intrinsics)
			if (PortableToolchain.ClangBuiltinIncludePath != null && DirectoryReference.Exists(PortableToolchain.ClangBuiltinIncludePath))
			{
				Arguments.Add($"-isystem \"{NormalizeCommandLinePath(PortableToolchain.ClangBuiltinIncludePath,  CompileEnvironment.RootPaths)}\"");
			}
			else
			{
				Logger.LogError("Clang builtin headers not found");
			}

			// Apple OSS headers
			Arguments.Add($"-isystem \"{NormalizeCommandLinePath(OSSHeadersDir,CompileEnvironment.RootPaths)}\"");
		}

		/// <summary>
		/// Gets global compile arguments for portable toolchain (without SDK-specific paths like -isysroot, -F framework paths).
		/// Used for distributed compilation of non-Apple SDK files on Linux workers.
		/// Must be implemented by derived classes (IOSToolChain, MacToolChain).
		/// </summary>
		protected abstract void GetCompileArguments_PortableToolchain(CppCompileEnvironment CompileEnvironment, List<string> Arguments);

		public override CppCompileEnvironment CreateSharedResponseFile(CppCompileEnvironment CompileEnvironment, FileReference OutResponseFile, IActionGraphBuilder Graph)
		{
			// Temporarily turn of shared response files for apple toolchains
			return CompileEnvironment;
		}

		private static FileItem GetBridgingHeader(string ModuleName, DirectoryReference OutputDir)
		{
			string Filename = ModuleName + "-Swift.h";
			return FileItem.GetItemByFileReference(FileReference.Combine(OutputDir, "Bridging", Filename));
		}

		private void GetSharedSwiftArguments(CppCompileEnvironment CompileEnvironment, List<string> Arguments, string ModuleName)
		{
			// TODO: Fix so swiftc path supports vfs
			CppRootPaths rootPaths = new(CompileEnvironment.RootPaths);
			rootPaths.bUseVfs = false;

			Arguments.Add($"-target {ToolChainSettings.Value.GetTargetTuple(CompileEnvironment.Architecture, Target!.Platform, Target!.Type)}");
			Arguments.Add($"-sdk {ToolChainSettings.Value.GetSDKPath(CompileEnvironment.Architecture)}");

			Arguments.Add($"-swift-version 5");
			if (CompileEnvironment.bCreateDebugInfo)
			{
				Arguments.Add($"-g");
			}

			Arguments.Add($"-module-name {ModuleName}Swift");

			if (CompileEnvironment.ForceIncludeFiles.Count > 0)
			{
				Arguments.Add($"-import-objc-header \"{NormalizeCommandLinePath(CompileEnvironment.ForceIncludeFiles.First(), rootPaths)}\"");
			}

			// tell swift to define PLATFORM_FOO, and tell C++ to define PLATFORM_FOO=1
			// swift doesn't pass it's defines along to C++ when importing a C header, and it also doesn't have values for defines
			Arguments.Add($"-DPLATFORM_{Target.Platform.ToString().ToUpperInvariant()}");
			Arguments.Add($"-Xcc -DPLATFORM_{Target.Platform.ToString().ToUpperInvariant()}=1");
			if (bUseSwiftUIMain)
			{
				Arguments.Add("-DUE_USE_SWIFT_UI_MAIN");
				if (ToolChainSettings.Value.SDKVersionFloat < 2.0)
				{
					Arguments.Add("-DUE_SDK_VERSION_1");
				}
			}

			// pass the UE definitions to C++ land for the imported obj-c header
			Arguments.AddRange(CompileEnvironment.Definitions.Select(x => $"-Xcc -D{x}"));

			Console.WriteLine("Arguments: {0}", String.Join(", ", Arguments));
		}

		private Action CompileSwiftFiles(CppCompileEnvironment CompileEnvironment, IEnumerable<FileItem> InputFiles, DirectoryReference OutputDir, string ModuleName, IActionGraphBuilder Graph, CPPOutput CompileResult)
		{
			FileItem OutputFile = FileItem.GetItemByFileReference(FileReference.Combine(OutputDir, GetFileNameFromExtension(ModuleName + ".swift", ".o")));

			List<string> Arguments = new();
			foreach (FileItem SourceFile in InputFiles)
			{
				Arguments.Add($"\"{SourceFile}\"");
			}

			GetSharedSwiftArguments(CompileEnvironment, Arguments, ModuleName);
			// output file settings
			Arguments.Add("-emit-object"); // same as -c
			Arguments.Add($"-parse-as-library"); // don't create a main function (even if we use the SwiftUI @main, this still seems to be correctly working)
			Arguments.Add($"-o \"{OutputFile}\"");

			// platform settings
			Arguments.Add($"-target {Settings.GetTargetTuple(CompileEnvironment.Architecture, Target!.Platform, Target!.Type)}");
			Arguments.Add($"-sdk {Settings.GetSDKPath(CompileEnvironment.Architecture)}");

			// misc settings copied from Xcode
			Arguments.Add("-Xllvm");
			Arguments.Add("-aarch64-use-tbi");
			Arguments.Add("-stack-check");

			Arguments.Add("-enable-objc-interop");
			Arguments.Add("-cxx-interoperability-mode=default");

			Action CompileAction = Graph.CreateAction(ActionType.Compile);
			CompileAction.Weight = CompileActionWeight;
			CompileAction.CommandArguments = String.Join(" ", Arguments);
			CompileAction.CommandPath = FileReference.Combine(AppleToolChainSettings.ToolchainDir, "swift-frontend");
			foreach (FileItem SourceFile in InputFiles)
			{
				CompileAction.PrerequisiteItems.Add(SourceFile);
			}
			CompileAction.ProducedItems.Add(OutputFile);
			CompileResult.ObjectFiles.Add(OutputFile);
			CompileAction.WorkingDirectory = Unreal.EngineSourceDirectory;
			CompileAction.CommandDescription = "Compile";
			UnrealArchitectureConfig ArchConfig = UnrealArchitectureConfig.ForPlatform(CompileEnvironment.Platform);
			if (ArchConfig.Mode != UnrealArchitectureMode.SingleArchitecture)
			{
				string ReadableArch = ArchConfig.ConvertToReadableArchitecture(CompileEnvironment.Architecture);
				CompileAction.CommandDescription += $" [{ReadableArch}]";
			}
			CompileAction.StatusDescription = String.Join(" ", InputFiles.Select(x => x.Name));
			CompileAction.bIsClangCompiler = true;
			CompileAction.bCanExecuteRemotely = true;
			CompileAction.bCanExecuteInUBA = false;

			if (CompileEnvironment.ForceIncludeFiles.Count > 0)
			{
				FileItem OutputInteropHeader = GetBridgingHeader(ModuleName, OutputDir);

				// obj-c bridging header settings
				Arguments.Clear();
				foreach (FileItem SourceFile in InputFiles)
				{
					Arguments.Add($"\"{SourceFile}\"");
				}
				GetSharedSwiftArguments(CompileEnvironment, Arguments, ModuleName);

				Arguments.Add("-parse"); // this will allow it to generate the header without writing out any .o/executable
				Arguments.Add("-emit-objc-header");
				Arguments.Add($"-emit-objc-header-path \"{OutputInteropHeader}\"");
				Arguments.Add("-parse-as-library");

				//Arguments.Add("-enable-objc-interop");
				//Arguments.Add("-cxx-interoperability-mode=default");
				
				// now make an action to export the swift code as a header Obj-C bridging
				Action HeaderAction = Graph.CreateAction(ActionType.Compile);
				HeaderAction.CommandPath = FileReference.Combine(AppleToolChainSettings.ToolchainDir, "swiftc");
				HeaderAction.CommandArguments = String.Join(" ", Arguments);
				foreach (FileItem SourceFile in InputFiles)
				{
					HeaderAction.PrerequisiteItems.Add(SourceFile);
				}
				HeaderAction.ProducedItems.Add(OutputInteropHeader);
				// swiftc won't touch the output header if the generated content doesn't change, so always delete the output so that 
				// the timestamp is correct (it must be newer than the input .swift file, or it will build over and over)
				HeaderAction.DeleteItems.Add(OutputInteropHeader);
				CompileResult.GeneratedHeaderFiles.Add(OutputInteropHeader);
				HeaderAction.CommandDescription = "Generate SwiftToCPP Header";
				HeaderAction.StatusDescription = Path.GetFileName(OutputInteropHeader.AbsolutePath);
				if (ArchConfig.Mode != UnrealArchitectureMode.SingleArchitecture)
				{
					string ReadableArch = ArchConfig.ConvertToReadableArchitecture(CompileEnvironment.Architecture);
					HeaderAction.CommandDescription += $" [{ReadableArch}]";
				}

				HeaderAction.WorkingDirectory = CompileAction.WorkingDirectory;
				HeaderAction.bIsClangCompiler = CompileAction.bIsClangCompiler;
				HeaderAction.bCanExecuteRemotely = CompileAction.bCanExecuteRemotely;
				HeaderAction.bCanExecuteInUBA = CompileAction.bCanExecuteInUBA;
			}
			// this is likely ignored, but the Compile action is the important one
			return CompileAction;
		}

		public override ICollection<FileItem> PostBuild(ReadOnlyTargetRules Target, FileItem Executable, LinkEnvironment BinaryLinkEnvironment, IActionGraphBuilder Graph)
		{
			List<FileItem> OutputFiles = new List<FileItem>(base.PostBuild(Target, Executable, BinaryLinkEnvironment, Graph));

			bool bIsBuildingAppBundle = !BinaryLinkEnvironment.bIsBuildingDLL && !BinaryLinkEnvironment.bIsBuildingLibrary && !BinaryLinkEnvironment.bIsBuildingConsoleApplication;
			if (!AppleExports.CreatingAppOnWindows(Target.ProjectFile) && bIsBuildingAppBundle)
			{
				Action PostBuildAction = ApplePostBuildSyncMode.CreatePostBuildSyncAction(Target, Executable, BinaryLinkEnvironment.IntermediateDirectory!, Graph, AppleExports.ForceNoEntitlements());

				PostBuildAction.PrerequisiteItems.UnionWith(OutputFiles);
				OutputFiles.AddRange(PostBuildAction.ProducedItems);

				OutputFiles.Add(UpdateVersionFile(BinaryLinkEnvironment, FileItem.GetItemByFileReference(BinaryLinkEnvironment.OutputFilePath), Graph));
			}
			
			ReadOnlyAppleTargetRules AppleTargetRules = GetAppleTargetRules(Target);
			if (!BinaryLinkEnvironment.bIsBuildingLibrary && (AppleTargetRules.bStripSymbols || Target.Configuration == UnrealTargetConfiguration.Shipping))
			{
				// Generate the standard strip command. Should our default strip command ever change, we have a centralized place to update it.
				ProcessStartInfo StripProcessInfo = CreateStripSymbolsProcessInfo(Executable.Location, AppleToolChainSettings.ToolchainDir);

				// If building a framework, we can only strip local symbols and we need to leave global in place (-x argument)
				// We also preserve global in binaries on IOS/TVOS as we expose operators new/delete.
				bool bPreserveGlobalSymbols = BinaryLinkEnvironment.bIsBuildingDLL || (Target.Platform == UnrealTargetPlatform.IOS || Target.Platform == UnrealTargetPlatform.TVOS);
				string AddedArguments = bPreserveGlobalSymbols ? " -x" : "";
				// Determine all of the arguments we're passing to the strip command (the process info plus arguments above)
				string StripCommandArguments = $"{StripProcessInfo.Arguments}{AddedArguments}";

				// Convert the process info to an action graph entry
				Action StripAction = Graph.CreateAction(ActionType.CreateAppBundle);
				StripAction.WorkingDirectory = GetMacDevSrcRoot();
				StripAction.CommandPath = BuildHostPlatform.Current.Shell;
				StripAction.PrerequisiteItems.Add(Executable);
				StripAction.PrerequisiteItems.UnionWith(OutputFiles);
				StripAction.StatusDescription = $"Stripping symbols from {Executable.AbsolutePath}";
				StripAction.bCanExecuteRemotely = false;
				
				// Construct the strip command we want to execute
				string Command = $"\"{StripProcessInfo.FileName}\" {StripCommandArguments}";
				
				// Determine if we are creating a strip flag file.
				// This will essentially be the same location & file name, but with a ".stripped" extension.
				if (AppleTargetRules.bCreateStripFlagFile)
				{
					// Determine the name & location of the strip flag file
					FileItem StripCompleteFile = FileItem.GetItemByFileReference(FileReference.Combine(BinaryLinkEnvironment.IntermediateDirectory!, Executable.Location.GetFileName() + ".stripped"));
					
					// Create this strip flag file as part of the command we are executing
					Command += $" && touch \"{StripCompleteFile.FullName}\"";
					
					// Ensure our action identifies this as a file it is producing and it's flagged as an output file from this process
					StripAction.ProducedItems.Add(StripCompleteFile);
					OutputFiles.Add(StripCompleteFile);
				}
				else
				{
					// If we're not creating a stripped flag file, we're stripping the actual
					// target file, so we need to add that as an output from this process
					OutputFiles.Add(Executable);
				}
				
				// Additional escaping for backslashes and quotes since we are executing a shell and passing in our command as a quoted string
				string ShellCommand = $"-c \"{Command.Replace(@"\", @"\\", StringComparison.Ordinal).Replace("\"", "\\\"", StringComparison.Ordinal)}\"";
				// Finally, we assign the action our finished shell command before it executes
				StripAction.CommandArguments = ShellCommand;
			}

			return OutputFiles;
		}

		protected override void GetLinkArguments_Optimizations(LinkEnvironment linkEnvironment, List<string> arguments)
		{
			base.GetLinkArguments_Optimizations(linkEnvironment, arguments);

			bool bLTOEnabled = Options.HasFlag(ClangToolChainOptions.EnableLinkTimeOptimization);
			bool bThinLTOEnabled = Options.HasFlag(ClangToolChainOptions.EnableThinLTO);

			if (bLTOEnabled && bThinLTOEnabled)
			{
				arguments.Remove("-Wl,--thinlto-jobs=all"); // unsupported

				if (linkEnvironment.ThinLTOCacheDirectory != null)
				{
					arguments.RemoveAll(x => x.StartsWith("-Wl,--thinlto-cache", StringComparison.Ordinal));
				}
			}
		}

		protected override void GetLinkArguments_Global(LinkEnvironment LinkEnvironment, List<string> Arguments)
		{
			base.GetLinkArguments_Global(LinkEnvironment, Arguments);

			Arguments.Add($"-isysroot \"{NormalizeCommandLinePath(Settings.GetSDKPath(LinkEnvironment.Architecture), LinkEnvironment.RootPaths)}\"");

			VersionNumber AppleClangVersion = ((AppleToolChainInfo)Info).AppleClangVersion;
			// Temp solution for UE-191350
			if (AppleClangVersion >= new VersionNumber(15) && AppleClangVersion < new VersionNumber(16))
			{
				Arguments.Add("-ld_classic");
			}

			if (!LinkEnvironment.bIsBuildingDLL)
			{
				string PlatformName = LinkEnvironment.Architecture == UnrealArch.IOSSimulator ? Settings.SimulatorPlatformDirName : Settings.PlatformDirName;
				DirectoryReference libSwiftDir = DirectoryReference.Combine(ApplePlatformSDK.GetToolchainDirectory(), "..", "lib", "swift", PlatformName);
				Arguments.Add($"-L\"{NormalizeCommandLinePath(libSwiftDir, LinkEnvironment.RootPaths)}\"");
			}

			Arguments.Add("-L\"/usr/lib/swift\"");

			// Pass through architecture and OS info		
			Arguments.Add(FormatArchitectureArg(LinkEnvironment.Architectures));
			Arguments.Add($"-target {Settings.GetTargetTuple(LinkEnvironment.Architecture, Target!.Platform, Target!.Type)}");

			Arguments.Add("-dead_strip");
		}

		protected override void GetArchiveArguments_Global(LinkEnvironment LinkEnvironment, List<string> Arguments)
		{
			base.GetArchiveArguments_Global(LinkEnvironment, Arguments);
			Arguments.Add("-static");
		}

		#region Stub Xcode Projects

		internal static bool GenerateProjectFiles(FileReference? ProjectFile, string[] Arguments, string? SingleTargetName, ILogger Logger, out DirectoryReference? XcodeProjectFile)
		{
			ProjectFileGenerator.bGenerateProjectFiles = true;
			try
			{
				CommandLineArguments CmdLine = new CommandLineArguments(Arguments);

				PlatformProjectGeneratorCollection PlatformProjectGenerators = new PlatformProjectGeneratorCollection();
				PlatformProjectGenerators.RegisterPlatformProjectGenerator(UnrealTargetPlatform.Mac, new MacProjectGenerator(CmdLine, Logger), Logger);
				PlatformProjectGenerators.RegisterPlatformProjectGenerator(UnrealTargetPlatform.IOS, new IOSProjectGenerator(CmdLine, Logger), Logger);
				PlatformProjectGenerators.RegisterPlatformProjectGenerator(UnrealTargetPlatform.TVOS, new TVOSProjectGenerator(CmdLine, Logger), Logger);

				XcodeProjectFileGenerator Generator = new XcodeProjectFileGenerator(ProjectFile, new(Logger), CmdLine);
				// this could be improved if ProjectFileGenerator constructor took a Arguments param, and it could parse it there instead of the GenerateProjectFilesMode
				Generator.SingleTargetName = SingleTargetName;
				// don't need the editor data since these are stub projects
				ProjectFileGenerator.Current = Generator;
				bool bSucces = Generator.GenerateProjectFiles(PlatformProjectGenerators, Arguments, Logger);
				ProjectFileGenerator.Current = null;
				XcodeProjectFile = Generator.XCWorkspace;
				return bSucces;
			}
			catch (Exception ex)
			{
				XcodeProjectFile = null;
				Logger.LogError(ex, "{Exception}", ex.ToString());
			}
			finally
			{
				ProjectFileGenerator.bGenerateProjectFiles = false;
			}
			return false;
		}

		/// <summary>
		/// Genearate an run-only Xcode project, that is not meant to be used for anything else besides code-signing/running/etc of the native .app bundle
		/// </summary>
		/// <param name="UProjectFile">Location of .uproject file (or null for the engine project</param>
		/// <param name="Platform">The platform to generate a project for</param>
		/// <param name="TargetName">The name of the target being built, so we can generate a more minimal project</param>
		/// <param name="bForDistribution">True if this is making a bild for uploading to app store</param>
		/// <param name="bNoEntitlements">True if we shouldn't use entitlements</param>
		/// <param name="Logger">Logging object</param>
		/// <param name="GeneratedProjectFile">Returns the .xcworkspace that was made</param>
		internal static void GenerateRunOnlyXcodeProject(FileReference? UProjectFile, UnrealTargetPlatform Platform, string TargetName, bool bForDistribution, bool bNoEntitlements, ILogger Logger, out DirectoryReference? GeneratedProjectFile)
		{
			List<string> Options = new()
			{
				$"-platforms={Platform}",
				"-DeployOnly",
				"-NoIntellisense",
				"-NoDotNet",
				"-IgnoreJunk",
				bNoEntitlements ? "-noEntitlements" : "",
				bForDistribution ? "-distribution" : "-development",
				"-IncludeTempTargets",
				"-projectfileformat=XCode",
				"-automated",
			};

			if (!String.IsNullOrEmpty(TargetName))
			{
				Options.Add($"-singletarget={TargetName}");
			}

			if (UProjectFile == null || UProjectFile.IsUnderDirectory(Unreal.EngineDirectory))
			{
				// @todo do we need these? where would the bundleid come from if there's no project?
				//				Options.Add("-bundleID=" + BundleID);
				//				Options.Add("-appname=" + AppName);
				// @todo add an option to only add Engine target?
			}
			else
			{
				Options.Add($"-project=\"{UProjectFile.FullName}\"");
				Options.Add("-game");
			}

			// we need to be in Engine/Source for some build.cs files
			string CurrentCWD = Environment.CurrentDirectory;
			Environment.CurrentDirectory = DirectoryReference.Combine(Unreal.EngineDirectory, "Source").FullName;
			GenerateProjectFiles(UProjectFile, Options.ToArray(), TargetName, Logger, out GeneratedProjectFile);
			Environment.CurrentDirectory = CurrentCWD;
		}

		static string IdOrNameParam(string Id)
		{
			string OldFormat = "^[0-9,a-f]{40}$";
			string NewFormat = "^[0-9,A-F]{8}-[0-9,A-F]{16}$";
			if (Regex.Match(Id, OldFormat).Success || Regex.Match(Id, NewFormat).Success)
			{
				return $"id={Id}";
			}
			return $"name={Id}";
		}

		internal static int FinalizeAppWithXcode(DirectoryReference XcodeProject, UnrealTargetPlatform Platform, UnrealArchitectures Architectures, bool bUseAutomaticCodeSigning, string SchemeName, string Configuration, string Action, string ExtraOptions, ILogger Logger, List<string>? DestinationIds=null)
		{
			// Acquire a different mutex to the regular UBT instance, since this mode will be called as part of a build. We need the mutex to ensure that building two modular configurations 
			// in parallel don't clash over writing shared *.modules files (eg. DebugGame and Development editors).
			string MutexName = GlobalSingleInstanceMutex.GetUniqueMutexForPath("UnrealBuildTool_XcodeBuild", Unreal.RootDirectory);
			using (SingleInstanceMutex.Acquire(MutexName))
			{
				for (int Pass = 0; Pass < 2; Pass++)
				{
					string DestinationParam;
					// use generic:
					//   - for Mac, the destination seems to want name="My Mac", not the real name of the Mac
					//   - when no destinations were supplied
					//   - when trying to build iOS app to run on local mac 
					//   - when using manual codesigning because the main use of this is to update provisioning information (reducing potential for issues)
					if (Platform == UnrealTargetPlatform.Mac ||
						DestinationIds == null ||
						DestinationIds.Count == 0 ||
						DestinationIds[0].Length == 0 ||
						DestinationIds[0].Contains(AppleToolChainSettings.LocalMacUUID, StringComparison.Ordinal) ||
						!bUseAutomaticCodeSigning)
					{
						DestinationParam = $"-destination generic/platform=\"{AppleExports.GetDestinationPlatform(Platform, Architectures)}\"";
					}
					else
					{
						string DestPlatform = AppleExports.GetDestinationPlatform(Platform, Architectures);
						DestinationParam = String.Join(" ", DestinationIds.Select(x => $"-destination \"platform={DestPlatform},{IdOrNameParam(x)}\""));
					}

					if(bUseAutomaticCodeSigning && !Unreal.IsBuildMachine())
					{
						(new AppleCertificateVerification(Logger)).VerifyCertificateInstallation();
					}

					List<string> Arguments = new()
					{
						"UBT_NO_POST_DEPLOY=true",
						FileReference.Combine(ApplePlatformSDK.DeveloperDir, "usr/bin/xcodebuild").FullName,
						Action,
						$"-workspace \"{XcodeProject.FullName}\"",
						$"-scheme \"{SchemeName}\"",
						$"-configuration \"{Configuration}\"",
						DestinationParam,
						// xcode gets confused it we _just_ wrote out entitlements while generating the temp project, and it thinks it was modified _during_ building
						// but it wasn't, it was written before the build started
						"CODE_SIGN_ALLOW_ENTITLEMENTS_MODIFICATION=YES",
						ExtraOptions,
						//$"-sdk {SDKName}",
					};

					if (Pass == 0)
					{
						Arguments.Add("-hideShellScriptEnvironment");
					}

					Console.WriteLine("params: {0}", String.Join(" ", Arguments));

					Process LocalProcess = new Process();
					LocalProcess.StartInfo = new ProcessStartInfo("/usr/bin/env", String.Join(" ", Arguments));
					LocalProcess.OutputDataReceived += (Sender, Args) => { LocalProcessOutput(Args, false, Logger); };
					LocalProcess.ErrorDataReceived += (Sender, Args) =>
					{
						if (Args != null && Args.Data != null
						&& Args.Data.Contains("Failed to load profile", StringComparison.Ordinal) && Args.Data.Contains("<stdin>", StringComparison.Ordinal))
						{
							Logger.LogInformation("Silencing the following provision profile error, it is not affecting code signing:");
							LocalProcessOutput(Args, false, Logger);
						}
						else
						{
							LocalProcessOutput(Args, true, Logger);
						}
					};

					// if first pass succeeded, return 0, otherwise return whatever the exit code is
					int ExitCode = Utils.RunLocalProcess(LocalProcess);
					if (ExitCode == 0 || Pass > 0)
					{
						return ExitCode;
					}

					Logger.LogWarning("");
					Logger.LogWarning("");
					Logger.LogWarning("**************************************************************************************************************");
					Logger.LogWarning("* App finalization failed, running again with all environment variables displayed to help diagnose the error *");
					Logger.LogWarning("**************************************************************************************************************");
					Logger.LogWarning("");
					Logger.LogWarning("");
				}
			}
			return -1;
		}

		static void LocalProcessOutput(DataReceivedEventArgs? Args, bool bIsError, ILogger Logger)
		{
			if (Args != null && Args.Data != null)
			{
				if (bIsError)
				{
					Logger.LogError("{Message}", Args.Data.TrimEnd());
				}
				else
				{
					Logger.LogInformation("{Message}", Args.Data.TrimEnd());
				}
			}
		}

		#endregion
	};

	class ApplePostBuildSyncTarget
	{
		public FileReference? ProjectFile;
		public UnrealTargetPlatform Platform;
		public required UnrealArchitectures Architectures;
		public UnrealTargetConfiguration Configuration;
		public required string TargetName;

		// For iOS/TVOS
		public bool bCreateStubIPA;
		public string? RemoteImportProvision;
		public string? RemoteImportCertificate;
		public string? RemoteImportCertificatePassword;
		public required DirectoryReference ProjectIntermediateDirectory;
		public required FileReference StubOutputPath;

		[SetsRequiredMembers]
		public ApplePostBuildSyncTarget(ReadOnlyTargetRules Target, FileItem Executable, DirectoryReference IntermediateDir)
		{
			Platform = Target.Platform;
			Configuration = Target.Configuration;
			Architectures = Target.Architectures;
			ProjectFile = Target.ProjectFile;
			TargetName = Target.Name;

			bCreateStubIPA = Target.IOSPlatform.bCreateStubIPA;
			RemoteImportProvision = Target.IOSPlatform.ImportProvision;
			RemoteImportCertificate = Target.IOSPlatform.ImportCertificate;
			RemoteImportCertificatePassword = Target.IOSPlatform.ImportCertificatePassword;
			ProjectIntermediateDirectory = IntermediateDir;

			StubOutputPath = Executable.Location;
		}

		/// <summary>
		/// Constructor required for JSON serialization.
		/// </summary>
		[JsonConstructor]
		private ApplePostBuildSyncTarget()
		{
		}
	}

	internal sealed class ApplePostBuildSyncMode : IToolMode<ApplePostBuildSyncMode>
	{
		public static string Name => "ApplePostBuildSync";
		public static ToolModeOptions Options => ToolModeOptions.XmlConfig | ToolModeOptions.BuildPlatforms;

		[CommandLine("-Input=", Required = true)]
		public FileReference? InputFile = null;

		[CommandLine("-XmlConfigCache=")]
		public FileReference? XmlConfigCache = null;

		[Obsolete("Deprecated in UE5.8. Modern Xcode is now the only option")] 
		[CommandLine("-ModernXcode")]
		public bool bModernXcode = true;

		[CommandLine("-noEntitlements")]
		public bool bNoEntitlements;

		public async Task<int> ExecuteAsync(CommandLineArguments Arguments, ILogger Logger)
		{
			Arguments.ApplyTo(this);
			Arguments.CheckAllArgumentsUsed();

			// Run the PostBuildSync command
			ApplePostBuildSyncTarget Target;
			using (FileStream FileStream = FileReference.Open(InputFile!, FileMode.Open, FileAccess.Read, FileShare.Read))
			{
				ApplePostBuildSyncTarget? MaybeTarget = await JsonSerializer.DeserializeAsync<ApplePostBuildSyncTarget>(FileStream, AppleToolChain.SerializerOptions);
				Target = MaybeTarget ?? throw new BuildLogEventException("Post-build sync target file '{FilePath}' was invalid", InputFile!);
			}

			return PostBuildSync(Target, Logger);
		}

		private static int PostBuildSync(ApplePostBuildSyncTarget Target, ILogger Logger)
		{
			// generate the IOS plist file every time
			if (Target.Platform.IsInGroup(UnrealPlatformGroup.IOS))
			{
				string GameName = Target.ProjectFile == null ? "UnrealGame" : Target.ProjectFile.GetFileNameWithoutAnyExtensions();
				// most of these params are uused in modern
				UEDeployIOS.GenerateIOSPList(Target.ProjectFile, Target.Configuration, AppleToolChain.GetActualProjectDirectory(Target.ProjectFile).FullName, Target.ProjectFile == null, GameName, bIsClient: false,
					GameName, Unreal.EngineDirectory.FullName, "", null, null, false, Logger);
			}

			// if xcode is building this, it will also do the Run stuff anyway, so no need to do it here as well
			if (Environment.GetEnvironmentVariable("UE_BUILD_FROM_XCODE") == "1")
			{
				return 0;
			}

			string ExtraOptions = "";
			// for mobile builds (which need real codesigning to be able to run), we use dummy codesigning when making a .stub (which will
			// be sent to Windows an re-codesigned), or when making UnrealGame.app without a .uproject (we will always make a .app
			// again with a .uproject on the commandline to be able to get the staged data that will be pulled in to the app - we can't run 
			// without a Staged directory, so this dummy codesigned .app won't be used directly)

			// NOTE: Actually for _now_ we are using legacy-style signing with temp keychain because IPhonePackager cannot codesign
			// Frameworks, so we have to do full non-dummy signing of stubs until we get IPP working
			bool bUseDummySigning = Target.Platform != UnrealTargetPlatform.Mac && (Target.ProjectFile == null);
			bool bCreateStub = Target.Platform.IsInGroup(UnrealPlatformGroup.IOS) && Target.bCreateStubIPA;
			// if we want dummy signing (no project at all) then we don't want to use legacy signing - that is only used for remote builds from Windows
			bool bUseLegacyStubSigning = bCreateStub && !bUseDummySigning;

			if (bUseLegacyStubSigning)
			{
				// create and run a script that will make a temp keychain, and return the options needed to pass to xcodebuild to use it
				ExtraOptions += SetupRemoteCodesigning(Target);
			}

			int ExitCode = AppleExports.BuildWithStubXcodeProject(Target.ProjectFile, Target.Platform, Target.Architectures, Target.Configuration, Target.TargetName,
				AppleExports.XcodeBuildMode.PostBuildSync, Logger, ExtraOptions, bForceDummySigning: bUseDummySigning);

			// restore the keychain as soon as possible
			if (bUseLegacyStubSigning)
			{
				// cleanup the 
				CleanupRemoteCodesigning(Target);
			}

			if (ExitCode != 0)
			{
				Logger.LogError("ERROR: Failed to finalize the .app with Xcode. Check the log for more information");
			}

			if (bCreateStub)
			{
				IOSToolChain.PackageStub(Target.StubOutputPath.Directory.FullName, Target.TargetName, Target.StubOutputPath.GetFileNameWithoutExtension(), AppleExports.CreatingAppOnWindows(Target.ProjectFile), !bUseLegacyStubSigning);
			}

			return ExitCode;
		}

		// This is hopefully until we can get codesigning of Frameworks working in IPhonePackager, then we can go back to dummy codesigning, without needing
		// to mess with keychains and what not
		private static string SetupRemoteCodesigning(ApplePostBuildSyncTarget Target)
		{
			FileReference TempKeychain = FileReference.Combine(Target.ProjectIntermediateDirectory!, "TempKeychain.keychain");
			FileReference SignProjectScript = FileReference.Combine(Target.ProjectIntermediateDirectory!, "SignProject.sh");
			string MobileProvisionUUID = "";
			string SigningCertificate = "";

			using (StreamWriter Writer = new StreamWriter(SignProjectScript.FullName))
			{
				// Boilerplate
				Writer.WriteLine("#!/bin/sh");
				Writer.WriteLine("set -e");
				Writer.WriteLine("set -x");
				// Copy the mobile provision into the system store
				if (Target.RemoteImportProvision == null || Target.RemoteImportCertificate == null)
				{
					throw new BuildException("Expecting stub to be run with -ImportCertificate and -ImportProvision when using modern xcode");
				}

				// copy the provision into standard location
				Writer.WriteLine("mkdir -p {0}", Utils.EscapeShellArgument(AppleExports.GetProvisionDirectory().FullName));
				Writer.WriteLine("cp -f {0} {1}", Utils.EscapeShellArgument(Target.RemoteImportProvision), Utils.EscapeShellArgument(AppleExports.GetProvisionDirectory().FullName));
				MobileProvisionContents MobileProvision = MobileProvisionContents.Read(new FileReference(Target.RemoteImportProvision));
				MobileProvisionUUID = MobileProvision.GetUniqueId();

				// Get the signing certificate to use
				X509Certificate2 Certificate;
				try
				{
					Certificate = X509CertificateLoader.LoadPkcs12FromFile(Target.RemoteImportCertificate, Target.RemoteImportCertificatePassword ?? "");
				}
				catch (Exception Ex)
				{
					throw new BuildException(Ex, "Unable to read certificate '{0}': {1}", Target.RemoteImportCertificate, Ex.Message);
				}
				// Read the name from the certificate
				SigningCertificate = Certificate.GetNameInfo(X509NameType.SimpleName, false);

				// Install a certificate given on the command line to a temporary keychain
				Writer.WriteLine("security delete-keychain \"{0}\" || true", TempKeychain);
				Writer.WriteLine("security create-keychain -p \"A\" \"{0}\"", TempKeychain);
				Writer.WriteLine("security list-keychains -s \"{0}\"", TempKeychain);
				Writer.WriteLine("security list-keychains");
				Writer.WriteLine("security set-keychain-settings -t 3600 -l  \"{0}\"", TempKeychain);
				Writer.WriteLine("security -v unlock-keychain -p \"A\" \"{0}\"", TempKeychain);
				Writer.WriteLine("security import {0}/Build/IOS/AppleWorldwideDeveloperRelationsCA.pem -k \"{1}\"", Utils.EscapeShellArgument(Unreal.EngineDirectory.FullName), TempKeychain);
				Writer.WriteLine("security import {0} -P {1} -k \"{2}\" -T /usr/bin/codesign -T /usr/bin/security -t agg", Utils.EscapeShellArgument(Target.RemoteImportCertificate), Utils.EscapeShellArgument(Target.RemoteImportCertificatePassword!), TempKeychain);
				Writer.WriteLine("security set-key-partition-list -S apple-tool:,apple:,codesign: -s -k \"A\" -D '{0}' -t private {1}", SigningCertificate, TempKeychain);
			}

			// run the script
			Utils.RunLocalProcessAndReturnStdOut("/bin/sh", $"\"{SignProjectScript.FullName}\"");

			// Set parameters to make sure it uses the correct identity and keychain
			// pass back the comandline arguments to xcodebuild to use these certicicates
			return $" CODE_SIGN_STYLE=Manual CODE_SIGN_IDENTITY=\"{SigningCertificate}\" PROVISIONING_PROFILE_SPECIFIER={MobileProvisionUUID}";
		}

		private static void CleanupRemoteCodesigning(ApplePostBuildSyncTarget Target)
		{
			FileReference TempKeychain = FileReference.Combine(Target.ProjectIntermediateDirectory!, "TempKeychain.keychain");

			FileReference CleanProjectScript = FileReference.Combine(Target.ProjectIntermediateDirectory!, "CleanProject.sh");
			using (StreamWriter CleanWriter = new StreamWriter(CleanProjectScript.FullName))
			{
				CleanWriter.WriteLine("#!/bin/sh");
				CleanWriter.WriteLine("set -e");
				CleanWriter.WriteLine("set -x");
				// Remove the temporary keychain from the search list
				CleanWriter.WriteLine("security delete-keychain \"{0}\" || true", TempKeychain);
				// Restore the login keychain as active
				CleanWriter.WriteLine("security list-keychain -s login.keychain");
			}

			Utils.RunLocalProcessAndReturnStdOut("/bin/sh", $"\"{CleanProjectScript.FullName}\"");
		}

		private static FileItem GetPostBuildOutputFile(FileReference Executable, string TargetName, UnrealTargetPlatform Platform)
		{
			FileReference StagedExe;
			if (Platform == UnrealTargetPlatform.Mac)
			{
				StagedExe = FileReference.Combine(Executable.Directory, Executable.GetFileName() + ".app/Contents/PkgInfo");
			}
			else
			{
				StagedExe = FileReference.Combine(Executable.Directory, TargetName + ".app", TargetName);
			}
			return FileItem.GetItemByFileReference(StagedExe);
		}

		public static Action CreatePostBuildSyncAction(ReadOnlyTargetRules Target, FileItem Executable, DirectoryReference IntermediateDir, IActionGraphBuilder Graph, bool bNoEntitlements = false)
		{
			ApplePostBuildSyncTarget PostBuildSync = new(Target, Executable, IntermediateDir);
			FileReference PostBuildSyncFile = FileReference.Combine(IntermediateDir!, $"PostBuildSync_{Target.Name}.json");
			DirectoryReference.CreateDirectory(PostBuildSyncFile.Directory);
			using (FileStream stream = new(PostBuildSyncFile.FullName, FileMode.Create, FileAccess.Write))
			{
				JsonSerializer.Serialize(stream, PostBuildSync, AppleToolChain.SerializerOptions);
			}

			string PostBuildSyncArguments = String.Format(" -Input=\"{0}\" -XmlConfigCache=\"{1}\" -remoteini=\"{2}\" {3}", PostBuildSyncFile, XmlConfig.CacheFile, UnrealBuildTool.GetRemoteIniPath(), bNoEntitlements ? "-noEntitlements" : "");
			Action PostBuildSyncAction = Graph.CreateRecursiveAction<ApplePostBuildSyncMode>(ActionType.CreateAppBundle, PostBuildSyncArguments);
			PostBuildSyncAction.PrerequisiteItems.Add(Executable);
			PostBuildSyncAction.ProducedItems.Add(GetPostBuildOutputFile(Executable.Location, Target.Name, Target.Platform));
			PostBuildSyncAction.StatusDescription = $"Executing PostBuildSync [{Executable.Location}]";

			if (Target.Platform == UnrealTargetPlatform.IOS || Target.Platform == UnrealTargetPlatform.TVOS)
			{
				// @todo: do we need one per Target for IOS? Client? I dont think so for Modern
				FileReference PlistFile = FileReference.Combine(AppleToolChain.GetActualProjectDirectory(Target.ProjectFile), "Build/IOS/UBTGenerated/Info.Template.plist");
				PostBuildSyncAction.ProducedItems.Add(FileItem.GetItemByFileReference(PlistFile));

				if (PostBuildSync.bCreateStubIPA)
				{
					FileReference StubFile = FileReference.Combine(Executable.Directory.Location, Executable.Location.GetFileNameWithoutExtension() + ".stub");
					PostBuildSyncAction.ProducedItems.Add(FileItem.GetItemByFileReference(StubFile));
				}
			}

			return PostBuildSyncAction;
		}
	}
}
