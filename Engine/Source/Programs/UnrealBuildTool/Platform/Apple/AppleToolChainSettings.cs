// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// Helper class for managing Xcode paths, versions, etc. Helps to differentiate between Apple xcode platforms
	/// </summary>
	public abstract class AppleToolChainSettings
	{
		/// <summary>
		/// Cached copy of ApplePlatformSDK.GetToolchainDirectory, for existing code to work
		/// </summary>
		public static DirectoryReference ToolchainDir => ApplePlatformSDK.GetToolchainDirectory();

		/// <summary>
		/// Name for Xcode plaform directory under Toolchains
		/// </summary>
		public string PlatformDirName;

		/// <summary>
		/// Name for Xcode simulator platform directory under Toolchains
		/// </summary>
		public string SimulatorPlatformDirName = "";

		/// <summary>
		/// A portion of the target "tuple"
		/// </summary>
		private readonly string TargetOSName;

		/// <summary>
		/// The version of the SDK being used to build with
		/// </summary>
		public string SDKVersion;

		/// <summary>
		/// The version of the iOS SDK to target at build time.
		/// </summary>
		public string MinTargetVersion;

		/// <summary>
		/// The build version in a floating point value for easy comparison
		/// </summary>
		public readonly float SDKVersionFloat;

		/// <summary>
		/// The target version in a floating point value for easy comparison
		/// </summary>
		public readonly float MinTargetVersionFloat;

		/// <summary>
		/// For platforms that need a different min version for editor vs game, this can be overridden
		/// </summary>
		/// <param name="Type"></param>
		/// <returns></returns>
		public virtual string GetTargetVersionForTargetType(TargetType Type) => MinTargetVersion;

		/// <summary>
		/// Cache SDK dir for device and simulator (for non-Mac)
		/// </summary>
		readonly DirectoryReference SDKDir;
		readonly DirectoryReference? SimulatorSDKDir = null;

		/// <summary>
		/// Dummy UUID for running iOS binary natively on Mac
		/// </summary>
		public static readonly string LocalMacUUID = "10CA18AC-10CA18AC10CA18AC";

		/// <summary>
		/// Constructor, called by platform sybclasses
		/// </summary>
		/// <param name="OSPrefix">SDK name (like "MacOSX")</param>
		/// <param name="SimulatorOSPrefix">Sinulator SDK name (like "iPhoneOSSimulator")</param>
		/// <param name="TargetOSName">Platform name used in the -target parameter</param>
		/// <param name="TargetOSVersion">min OS version this project wants to target (not what it's being built with)</param>
		/// <param name="bVerbose"></param>
		/// <param name="Logger"></param>
		protected AppleToolChainSettings(string OSPrefix, string? SimulatorOSPrefix, string TargetOSName, string TargetOSVersion, bool bVerbose, ILogger Logger)
		{
			this.TargetOSName = TargetOSName;
#pragma warning disable CA1308 // Normalize strings to uppercase - file path should use the correct case
			PlatformDirName = OSPrefix.ToLowerInvariant();
#pragma warning restore CA1308

			// set up directories
			SDKDir = ApplePlatformSDK.GetPlatformSDKDirectory(OSPrefix);
			if (SimulatorOSPrefix != null)
			{
				SimulatorSDKDir = ApplePlatformSDK.GetPlatformSDKDirectory(SimulatorOSPrefix);
#pragma warning disable CA1308 // Normalize strings to uppercase - file path should use the correct case
				SimulatorPlatformDirName = SimulatorOSPrefix.ToLowerInvariant();
#pragma warning restore CA1308
			}

			MinTargetVersion = TargetOSVersion;
			MinTargetVersionFloat = Single.Parse(MinTargetVersion, System.Globalization.CultureInfo.InvariantCulture);

			// cache some info for this OS
			SDKVersion = ApplePlatformSDK.GetPlatformSDKVersion(OSPrefix) ?? "";
			SDKVersionFloat = ApplePlatformSDK.GetPlatformSDKVersionFloat(OSPrefix);

			TestXcode(bVerbose);
		}

		/// <summary>
		/// Get the path to the SDK diretory in xcode for the given architecture
		/// </summary>
		/// <param name="Architecture"></param>
		/// <returns></returns>
		public DirectoryReference GetSDKPath(UnrealArch Architecture)
		{
			// note that VisionOS uses IOSSimulator (as TVOS should eventually do as well)
			if (Architecture == UnrealArch.IOSSimulator || Architecture == UnrealArch.TVOSSimulator)
			{
				return SimulatorSDKDir!;
			}
			return SDKDir;
		}

		/// <summary>
		/// Gets the string used by xcode to taget a platform and version (will return something like "arm64-apple-ios17.0-simulator"
		/// </summary>
		/// <param name="Architecture"></param>
		/// <param name="Platform"></param>
		/// <param name="TargetType"></param>
		/// <param name="ForcedVersion"></param>
		/// <returns></returns>
		public virtual string GetTargetTuple(UnrealArch Architecture, UnrealTargetPlatform Platform, TargetType TargetType, string? ForcedVersion = null)
		{
			string Prefix = Architecture.AppleName;
			string Suffix = (Architecture == UnrealArch.IOSSimulator || Architecture == UnrealArch.TVOSSimulator) ? "-simulator" : "";
			string TargetVersion = ForcedVersion ?? GetTargetVersionForTargetType(TargetType);

			return $"{Prefix}-apple-{TargetOSName}{TargetVersion}{Suffix}";
		}

		/// <summary>
		/// Find the Xcode developer directory
		/// </summary>
		/// <param name="bVerbose"></param>
		/// <exception cref="BuildException"></exception>
		private static void TestXcode(bool bVerbose)
		{
			DirectoryReference XcodeDeveloperDir = ApplePlatformSDK.DeveloperDir;
			// make sure we get a full path
			if (DirectoryReference.Exists(XcodeDeveloperDir) == false)
			{
				throw new BuildException("Selected Xcode ('{0}') doesn't exist, cannot continue.", XcodeDeveloperDir);
			}

			if (XcodeDeveloperDir.ContainsName("CommandLineTools", 0))
			{
				throw new BuildException($"Your Mac is set to use CommandLineTools for its build tools ({XcodeDeveloperDir}). Unreal expects Xcode as the build tools. Please install Xcode if it's not already, then do one of the following:\n" +
					"  - Run Xcode, go to Settings, and in the Locations tab, choose your Xcode in Command Line Tools dropdown.\n" +
					"  - In Terminal, run 'sudo xcode-select -s /Applications/Xcode.app' (or an alternate location if you installed Xcode to a non-standard location)\n" +
					"Either way, you will need to enter your Mac password.");
			}

			if (bVerbose && !XcodeDeveloperDir.FullName.StartsWith("/Applications/Xcode.app", StringComparison.Ordinal))
			{
				Log.TraceInformationOnce("Compiling with non-standard Xcode: {0}", XcodeDeveloperDir);
			}

			// Installed engine requires Xcode 13
			if (Unreal.IsEngineInstalled())
			{
				string? InstalledSdkVersion = ApplePlatformSDK.InstalledXcodeVersion.Value;
				if (String.IsNullOrEmpty(InstalledSdkVersion))
				{
					throw new BuildException("Unable to get xcode version");
				}
				if (Int32.Parse(InstalledSdkVersion.Substring(0, 2)) < 13)
				{
					throw new BuildException("Building for macOS, iOS and tvOS requires Xcode 13.4.1 or newer, Xcode " + InstalledSdkVersion + " detected");
				}
			}
		}
	}
}
