// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Text;
using System.Text.RegularExpressions;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// Stores information about how a local directory maps to a remote directory
	/// </summary>
	[DebuggerDisplay("{LocalDirectory}")]
	class RemoteMapping
	{
		public DirectoryReference LocalDirectory;
		public string RemoteDirectory;

		public RemoteMapping(DirectoryReference LocalDirectory, string RemoteDirectory)
		{
			this.LocalDirectory = LocalDirectory;
			this.RemoteDirectory = RemoteDirectory;
		}
	}

	/// <summary>
	/// Handles uploading and building on a remote Mac
	/// </summary>
	class RemoteMac
	{
#pragma warning disable IDE0044 // Make field readonly - these private static fields are set by XML configuration.
		/// <summary>
		/// These two variables will be loaded from the XML config file in XmlConfigLoader.Init().
		/// </summary>
		[XmlConfigFile]
		private string? ServerName;

		/// <summary>
		/// The remote username.
		/// </summary>
		[XmlConfigFile]
		private string? UserName;

		/// <summary>
		/// If set, instead of looking for RemoteToolChainPrivate.key in the usual places (Documents/Unreal, Engine/UnrealBuildTool/SSHKeys or Engine/Build/SSHKeys), this private key will be used.
		/// </summary>
		[XmlConfigFile]
		private FileReference? SshPrivateKey;

		/// <summary>
		/// The authentication used for Rsync (for the -e rsync flag).
		/// </summary>
		[XmlConfigFile]
		private string RsyncAuthentication = "./ssh -i '${CYGWIN_SSH_PRIVATE_KEY}'";

		/// <summary>
		/// The authentication used for SSH (probably similar to RsyncAuthentication).
		/// </summary>
		[XmlConfigFile]
		private string SshAuthentication = "-i '${CYGWIN_SSH_PRIVATE_KEY}'";
#pragma warning restore IDE0044

		/// <summary>
		/// Save the specified port so that RemoteServerName is the machine address only
		/// </summary>
		private readonly int ServerPort = 22;   // Default ssh port

		/// <summary>
		/// Path to Rsync
		/// </summary>
		private readonly FileReference RsyncExe;

		/// <summary>
		/// Path to SSH
		/// </summary>
		private readonly FileReference SshExe;

		/// <summary>
		/// The project being built. Settings will be read from config files in this project.
		/// </summary>
		private readonly FileReference? ProjectFile;

		/// <summary>
		/// The project descriptor for the project being built.
		/// </summary>
		private readonly ProjectDescriptor? ProjectDescriptor;

		/// <summary>
		/// A set of directories containing additional paths to be built.
		/// </summary>
		private readonly List<DirectoryReference>? AdditionalPaths;

		/// <summary>
		/// The base directory on the remote machine
		/// </summary>
		private readonly string RemoteBaseDir;

		/// <summary>
		/// Mappings from local directories to remote directories
		/// </summary>
		private readonly List<RemoteMapping> Mappings;

		/// <summary>
		/// Arguments that are used by every Ssh call
		/// </summary>
		private readonly List<string> CommonSshArguments;

		/// <summary>
		/// Arguments that are used by every Rsync call
		/// </summary>
		private readonly List<string> BasicRsyncArguments;

		/// <summary>
		/// Arguments that are used by directory Rsync call
		/// </summary>
		private readonly List<string> CommonRsyncArguments;

		private List<string> RsyncArgumentsForDownload(List<string> BaseArguments)
		{
			List<string> Arguments = new List<string>(BaseArguments);
			Arguments.Add(String.Format("--rsync-path=\"{0}/rsync\"", RemoteBaseDir));
			return Arguments;
		}

		private List<string> RsyncArgumentsForUpload(List<string> BaseArguments, string RemoteDirectory)
		{
			List<string> Arguments = new List<string>(BaseArguments);
			Arguments.Add(String.Format("--rsync-path=\"mkdir -p {0} && {1}/rsync\"", RemoteDirectory, RemoteBaseDir));
			return Arguments;
		}

		private readonly string? IniBundleIdentifier = "";

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="ProjectFile">Project to read settings from</param>
		/// <param name="Logger">Logger for output</param>
		/// <param name="RemoteIniPath">Remote ini path, or null to use UnrealBuildTool.GetRemoteIniPath()</param>
		/// <param name="bIsPrimary">Is the primary Remote Mac or not. True by default.</param>
		/// <param name="bPrepareForSecondaryMac">Added arguments when preparing a secondary Mac for debug. False by default.</param>
		public RemoteMac(FileReference? ProjectFile, ILogger Logger, string? RemoteIniPath = null, bool bIsPrimary = true, bool bPrepareForSecondaryMac = false)
		{
			DirectoryReference Cwrsync = DirectoryReference.Combine(Unreal.EngineDirectory, "Extras", "ThirdPartyNotUE", "cwrsync");
			RsyncExe = FileReference.Combine(Cwrsync, "bin", "rsync.exe");
			SshExe = FileReference.Combine(Cwrsync, "bin", "ssh.exe");
			this.ProjectFile = ProjectFile;
			if (ProjectFile != null)
			{
				ProjectDescriptor = ProjectDescriptor.FromFile(ProjectFile);
				AdditionalPaths = new List<DirectoryReference>();
				ProjectDescriptor.AddAdditionalPaths(AdditionalPaths, ProjectFile.Directory);
				if (AdditionalPaths.Count == 0)
				{
					AdditionalPaths = null;
				}
			}

			// Apply settings from the XML file
			XmlConfig.ApplyTo(this);

			// Get the project config file path
			DirectoryReference? EngineIniPath = ProjectFile?.Directory;
			if (RemoteIniPath == null)
			{
				RemoteIniPath = UnrealBuildTool.GetRemoteIniPath();
			}
			if (EngineIniPath == null && RemoteIniPath != null)
			{
				EngineIniPath = new DirectoryReference(RemoteIniPath!);
			}
			ConfigHierarchy Ini = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, EngineIniPath, UnrealTargetPlatform.IOS);

			// Read the project settings if we don't have anything in the build configuration settings
			if (String.IsNullOrEmpty(ServerName))
			{
				// Read the server name
				string IniServerName;

				if ((bIsPrimary ? Ini.GetString("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "RemoteServerName", out IniServerName) : Ini.GetString("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "SecondaryRemoteServerName", out IniServerName)) && !String.IsNullOrEmpty(IniServerName))
				{
					ServerName = IniServerName;
				}
				else
				{
					throw new BuildException("Remote compiling requires a server name. Use the editor (Project Settings > IOS) to set up your remote compilation settings.");
				}

				// Parse the username
				string IniUserName;
				if ((bIsPrimary ? Ini.GetString("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "RSyncUsername", out IniUserName) : Ini.GetString("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "SecondaryRSyncUsername", out IniUserName)) && !String.IsNullOrEmpty(IniUserName))
				{
					UserName = IniUserName;
				}
			}

			// Split port out from the server name
			int PortIdx = ServerName.LastIndexOf(':');
			if (PortIdx != -1)
			{
				string Port = ServerName[(PortIdx + 1)..];
				if (!Int32.TryParse(Port, out ServerPort))
				{
					throw new BuildException("Unable to parse port number from '{0}'", ServerName);
				}
				ServerName = ServerName[..PortIdx];
			}

			// If a user name is not set, use the current user
			if (String.IsNullOrEmpty(UserName))
			{
				UserName = Environment.UserName;
			}

			// Print out the server info
			Logger.LogInformation("[Remote] Using remote server '{ServerName}' on port {ServerPort} (user '{UserName}')", ServerName, ServerPort, UserName);

			// Get the path to the SSH private key
			string OverrideSshPrivateKeyPath;
			if ((bIsPrimary ? Ini.GetString("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "SSHPrivateKeyOverridePath", out OverrideSshPrivateKeyPath) : Ini.GetString("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "SecondarySSHPrivateKeyOverridePath", out OverrideSshPrivateKeyPath)) && !String.IsNullOrEmpty(OverrideSshPrivateKeyPath))
			{
				SshPrivateKey = new FileReference(OverrideSshPrivateKeyPath);
				if (!FileReference.Exists(SshPrivateKey))
				{
					throw new BuildException("SSH private key specified in config file ({0}) does not exist.", SshPrivateKey);
				}
			}

			Ini.GetString("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "BundleIdentifier", out IniBundleIdentifier);

			// If it's not set, look in the standard locations. If that fails, spawn the batch file to generate one.
			if (SshPrivateKey == null && !TryGetSshPrivateKey(out SshPrivateKey))
			{
				Logger.LogWarning("No SSH private key found for {UserName}@{ServerName}. Launching SSH to generate one.", UserName, ServerName);

				StringBuilder CommandLine = new StringBuilder();
				CommandLine.AppendFormat("/C \"\"{0}\"", FileReference.Combine(Unreal.EngineDirectory, "Build", "BatchFiles", "MakeAndInstallSSHKey.bat"));
				CommandLine.AppendFormat(" \"{0}\"", SshExe);
				CommandLine.AppendFormat(" \"{0}\"", ServerPort);
				CommandLine.AppendFormat(" \"{0}\"", RsyncExe);
				CommandLine.AppendFormat(" \"{0}\"", UserName);
				CommandLine.AppendFormat(" \"{0}\"", ServerName);
				CommandLine.AppendFormat(" \"{0}\"", DirectoryReference.GetSpecialFolder(Environment.SpecialFolder.MyDocuments));
				CommandLine.AppendFormat(" \"{0}\"", GetLocalCygwinPath(DirectoryReference.GetSpecialFolder(Environment.SpecialFolder.MyDocuments)!));
				CommandLine.AppendFormat(" \"{0}\"", Unreal.EngineDirectory);
				CommandLine.Append('"');

				using (Process ChildProcess = Process.Start(BuildHostPlatform.Current.Shell.FullName, CommandLine.ToString()))
				{
					ChildProcess.WaitForExit();
				}

				if (!TryGetSshPrivateKey(out SshPrivateKey))
				{
					throw new BuildException("Failed to generate SSH private key for {0}@{1}.", UserName, ServerName);
				}
			}

			// Print the path to the private key
			Logger.LogInformation("[Remote] Using private key at {SshPrivateKey}", SshPrivateKey);

			// resolve the rest of the strings
			RsyncAuthentication = ExpandVariables(RsyncAuthentication);
			SshAuthentication = ExpandVariables(SshAuthentication);

			// Build a list of arguments for SSH
			CommonSshArguments = new List<string>();
			CommonSshArguments.Add("-o BatchMode=yes");
			CommonSshArguments.Add(SshAuthentication);
			CommonSshArguments.Add(String.Format("-p {0}", ServerPort));
			CommonSshArguments.Add(String.Format("\"{0}@{1}\"", UserName, ServerName));

			// Build a list of arguments for Rsync
			BasicRsyncArguments = new List<string>();
			BasicRsyncArguments.Add("--compress");
			BasicRsyncArguments.Add("--verbose");
			BasicRsyncArguments.Add(String.Format("--rsh=\"{0} -p {1}\"", RsyncAuthentication, ServerPort));
			BasicRsyncArguments.Add("--chmod=ugo=rwx");

			// Build a list of arguments for Rsync filters
			CommonRsyncArguments = new List<string>(BasicRsyncArguments);
			CommonRsyncArguments.Add("--copy-links");
			CommonRsyncArguments.Add("--recursive");
			if (!bPrepareForSecondaryMac)
			{
				CommonRsyncArguments.Add("--delete"); // Delete anything not in the source directory
				CommonRsyncArguments.Add("--delete-excluded"); // Delete anything not in the source directory
			}
			CommonRsyncArguments.Add("--times"); // Preserve modification times
			CommonRsyncArguments.Add("--omit-dir-times"); // Ignore modification times for directories
			CommonRsyncArguments.Add("--prune-empty-dirs"); // Remove empty directories from the file list

			// Get the remote base directory
			string RemoteServerOverrideBuildPath;
			if ((bIsPrimary ? Ini.GetString("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "RemoteServerOverrideBuildPath", out RemoteServerOverrideBuildPath) : Ini.GetString("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "SecondaryRemoteServerOverrideBuildPath", out RemoteServerOverrideBuildPath)) && !String.IsNullOrEmpty(RemoteServerOverrideBuildPath))
			{
				RemoteBaseDir = String.Format("{0}/{1}", RemoteServerOverrideBuildPath.Trim().TrimEnd('/'), Unreal.MachineName);
			}
			else
			{
				StringBuilder Output;
				if (ExecuteAndCaptureOutput("'echo ~'", Logger, out Output) != 0)
				{
					throw new BuildException("Unable to determine home directory for remote user. SSH output:\n{0}", StringUtils.Indent(Output.ToString(), "  "));
				}
				RemoteBaseDir = String.Format("{0}/UE5/Builds/{1}", Output.ToString().Trim().TrimEnd('/'), Unreal.MachineName);
			}

			Logger.LogInformation("[Remote] Using base directory '{RemoteBaseDir}'", RemoteBaseDir);

			// Build the list of directory mappings between the local and remote machines
			Mappings = new List<RemoteMapping>();
			Mappings.Add(new RemoteMapping(Unreal.EngineDirectory, GetRemotePath(Unreal.EngineDirectory)));
			if (ProjectFile != null && !ProjectFile.IsUnderDirectory(Unreal.EngineDirectory))
			{
				Mappings.Add(new RemoteMapping(ProjectFile.Directory, GetRemotePath(ProjectFile.Directory)));
			}
			if (AdditionalPaths != null && ProjectFile != null)
			{
				foreach (DirectoryReference AdditionalPath in AdditionalPaths)
				{
					if (!AdditionalPath.IsUnderDirectory(Unreal.EngineDirectory) &&
						!AdditionalPath.IsUnderDirectory(ProjectFile.Directory))
					{
						Mappings.Add(new RemoteMapping(AdditionalPath, GetRemotePath(AdditionalPath)));
					}
				}
			}

			string RemoteFile = GetRemotePath("rsync");

			List<string> Arguments = new List<string>(CommonRsyncArguments);
			// use MacOS default system rsync for the upload of the 2.6.9 rsync binary
			Arguments.Add(String.Format("--rsync-path=\"mkdir -p {0} && rsync\"", RemoteBaseDir));
			Arguments.Add(String.Format("\"{0}\"", GetLocalCygwinPath(FileReference.Combine(Cwrsync, "mac", "rsync"))));
			Arguments.Add(String.Format("\"{0}@{1}\":'{2}'", UserName, ServerName, RemoteFile));
			Arguments.Add("-q");

			int Result = Rsync(String.Join(" ", Arguments), Logger);
			if (Result != 0)
			{
				throw new BuildException("Error while running Rsync (exit code {0})", Result);
			}
			// Remove quaratine extended attribute if remote machine set it.
			Execute(RemoteBaseDir, "if xattr -p com.apple.quarantine rsync > /dev/null 2>&1; then xattr -d com.apple.quarantine rsync; fi", Logger);
		}

		/// <summary>
		/// Attempts to get the SSH private key from the standard locations
		/// </summary>
		/// <param name="OutPrivateKey">If successful, receives the location of the private key that was found</param>
		/// <returns>True if a private key was found, false otherwise</returns>
		private bool TryGetSshPrivateKey(out FileReference? OutPrivateKey)
		{
			// Build a list of all the places to look for a private key
			List<DirectoryReference> Locations = new List<DirectoryReference>();
			Locations.Add(DirectoryReference.Combine(DirectoryReference.GetSpecialFolder(Environment.SpecialFolder.ApplicationData)!, "Unreal Engine", "UnrealBuildTool"));
			Locations.Add(DirectoryReference.Combine(DirectoryReference.GetSpecialFolder(Environment.SpecialFolder.Personal)!, "Unreal Engine", "UnrealBuildTool"));
			if (ProjectFile != null)
			{
				Locations.Add(DirectoryReference.Combine(ProjectFile.Directory, "Restricted", "NotForLicensees", "Build"));
				Locations.Add(DirectoryReference.Combine(ProjectFile.Directory, "Restricted", "NoRedist", "Build"));
				Locations.Add(DirectoryReference.Combine(ProjectFile.Directory, "Build"));
			}
			Locations.Add(DirectoryReference.Combine(Unreal.EngineDirectory, "Restricted", "NotForLicensees", "Build"));
			Locations.Add(DirectoryReference.Combine(Unreal.EngineDirectory, "Restricted", "NoRedist", "Build"));
			Locations.Add(DirectoryReference.Combine(Unreal.EngineDirectory, "Build"));

			// Find the first that exists
			foreach (DirectoryReference Location in Locations)
			{
				FileReference KeyFile = FileReference.Combine(Location, "SSHKeys", ServerName!, UserName!, "RemoteToolChainPrivate.key");
				if (FileReference.Exists(KeyFile))
				{
					// MacOS Mojave includes a new version of SSH that generates keys that are incompatible with our version of SSH. Make sure the detected keys have the right signature.
					string Text = FileReference.ReadAllText(KeyFile);
					if (Text.Contains("---BEGIN RSA PRIVATE KEY---", StringComparison.Ordinal))
					{
						OutPrivateKey = KeyFile;
						return true;
					}
				}
			}

			// Nothing found
			OutPrivateKey = null;
			return false;
		}

		/// <summary>
		/// Expand all the variables in the given string
		/// </summary>
		/// <param name="Input">The input string</param>
		/// <returns>String with any variables expanded</returns>
		private string ExpandVariables(string Input)
		{
			string Result = Input;
			Result = Result.Replace("${SSH_PRIVATE_KEY}", SshPrivateKey!.FullName, StringComparison.Ordinal);
			Result = Result.Replace("${CYGWIN_SSH_PRIVATE_KEY}", GetLocalCygwinPath(SshPrivateKey), StringComparison.Ordinal);
			return Result;
		}

		/// <summary>
		/// Flush the remote machine, removing all existing files
		/// </summary>
		public void FlushRemote(ILogger Logger)
		{
			Logger.LogInformation("[Remote] Deleting all files under {RemoteBaseDir}...", RemoteBaseDir);
			Execute("/", String.Format("rm -rf \"{0}\"", RemoteBaseDir), Logger);
		}

		/// <summary>
		/// Returns true if the remote executor supports this target platform
		/// </summary>
		/// <param name="Platform">The platform to check</param>
		/// <returns>True if the remote mac handles this target platform</returns>
		public static bool HandlesTargetPlatform(UnrealTargetPlatform Platform)
		{
			return BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Win64 && (Platform == UnrealTargetPlatform.Mac || Platform == UnrealTargetPlatform.IOS || Platform == UnrealTargetPlatform.TVOS);
		}

		/// <summary>
		/// Clean a target remotely
		/// </summary>
		/// <param name="TargetDesc">Descriptor for the target to build</param>
		/// <param name="Logger">Logger for diagnostic output</param>
		/// <returns>True if the build succeeded, false otherwise</returns>
		public bool Clean(TargetDescriptor TargetDesc, ILogger Logger)
		{
			// Translate all the arguments for the remote
			List<string> RemoteArguments = GetRemoteArgumentsForTarget(TargetDesc, null);
			RemoteArguments.Add("-Clean");

			// Upload the workspace
			DirectoryReference TempDir = CreateTempDirectory(TargetDesc);
			UploadWorkspace(TempDir, Logger);

			// Execute the compile
			Logger.LogInformation("[Remote] Executing clean...");

			StringBuilder BuildCommandLine = new StringBuilder("Engine/Build/BatchFiles/Mac/Build.sh");
			foreach (string RemoteArgument in RemoteArguments)
			{
				BuildCommandLine.AppendFormat(" {0}", EscapeShellArgument(RemoteArgument));
			}

			int Result = Execute(GetRemotePath(Unreal.RootDirectory), BuildCommandLine.ToString(), Logger);
			return Result == 0;
		}

		/// <summary>
		/// Build a target remotely
		/// </summary>
		/// <param name="TargetDesc">Descriptor for the target to build</param>
		/// <param name="RemoteLogFile">Path to store the remote log file</param>
		/// <param name="bSkipPreBuildTargets">If true then any PreBuildTargets will be skipped</param>
		/// <param name="Logger">Logger for diagnostic output</param>
		/// <returns>True if the build succeeded, false otherwise</returns>
		public bool Build(TargetDescriptor TargetDesc, FileReference RemoteLogFile, bool bSkipPreBuildTargets, ILogger Logger)
		{
			// Compile the rules assembly
			RulesAssembly RulesAssembly = RulesCompiler.CreateTargetRulesAssembly(TargetDesc.ProjectFile, TargetDesc.Name, false, false, false, TargetDesc.ForeignPlugin, TargetDesc.bBuildPluginAsLocal, Logger);

			// Create the target rules
			TargetRules Rules = RulesAssembly.CreateTargetRules(TargetDesc.Name, TargetDesc.Platform, TargetDesc.Configuration, TargetDesc.Architectures, TargetDesc.ProjectFile, TargetDesc.AdditionalArguments, Logger);
			if (!bSkipPreBuildTargets)
			{
				foreach (TargetInfo PreBuildTargetInfo in Rules.PreBuildTargets)
				{
					RemoteMac PreBuildTargetRemoteMac = new RemoteMac(ProjectFile, Logger);
					TargetDescriptor PreBuildTargetDesc = new TargetDescriptor(PreBuildTargetInfo.ProjectFile, PreBuildTargetInfo.Name, PreBuildTargetInfo.Platform, PreBuildTargetInfo.Configuration, PreBuildTargetInfo.Architectures, PreBuildTargetInfo.Arguments);

					Logger.LogInformation("[Remote] Building pre target [{PreTarget}] for [{Target}] ", PreBuildTargetDesc.ToString(), TargetDesc.ToString());
					if (!PreBuildTargetRemoteMac.Build(PreBuildTargetDesc, RemoteLogFile, false, Logger))
					{
						return false;
					}
				}
			}

			// Get the directory for working files
			DirectoryReference TempDir = CreateTempDirectory(TargetDesc);

			// Map the path containing the remote log file
			bool bLogIsMapped = false;
			foreach (RemoteMapping Mapping in Mappings)
			{
				if (RemoteLogFile.Directory.FullName.Equals(Mapping.LocalDirectory.FullName, StringComparison.OrdinalIgnoreCase))
				{
					bLogIsMapped = true;
					break;
				}
			}
			if (!bLogIsMapped)
			{
				Mappings.Add(new RemoteMapping(RemoteLogFile.Directory, GetRemotePath(RemoteLogFile.Directory)));
			}

			// Path to the local manifest file. This has to be translated from the remote format after the build is complete.
			List<FileReference> LocalManifestFiles = new List<FileReference>();

			// Path to the remote manifest file
			FileReference RemoteManifestFile = FileReference.Combine(TempDir, "Manifest.xml");

			// Prepare the arguments we will pass to the remote build
			List<string> RemoteArguments = GetRemoteArgumentsForTarget(TargetDesc, LocalManifestFiles);
			RemoteArguments.Add(String.Format("-Log={0}", GetRemotePath(RemoteLogFile)));
			RemoteArguments.Add(String.Format("-Manifest={0}", GetRemotePath(RemoteManifestFile)));
			RemoteArguments.Add(String.Format("-SkipPreBuildTargets"));
			// Compilation has been observed to fail without forcing a UHT run.
			RemoteArguments.Add("-ForceHeaderGeneration");

			// Handle any per-platform setup that is required
			if (TargetDesc.Platform == UnrealTargetPlatform.IOS || TargetDesc.Platform == UnrealTargetPlatform.TVOS)
			{
				// Always generate a .stub
				RemoteArguments.Add("-CreateStub");

				// Cannot use makefiles, since we need PostBuildSync() to generate the IPA (and that requires a TargetRules instance)
				RemoteArguments.Add("-NoUBTMakefiles");

				// Get the provisioning data for this project
				IOSProvisioningData ProvisioningData = ((IOSPlatform)UEBuildPlatform.GetBuildPlatform(TargetDesc.Platform)).ReadProvisioningData(TargetDesc.ProjectFile, TargetDesc.AdditionalArguments.HasOption("-distribution"), IniBundleIdentifier);
				if (ProvisioningData == null || ProvisioningData.MobileProvisionFile == null)
				{
					throw new BuildException("Unable to find mobile provision for {0}. See log for more information.", TargetDesc.Name);
				}

				// Create a local copy of the provision
				FileReference MobileProvisionFile = FileReference.Combine(TempDir, ProvisioningData.MobileProvisionFile.GetFileName());
				if (FileReference.Exists(MobileProvisionFile))
				{
					FileReference.SetAttributes(MobileProvisionFile, FileAttributes.Normal);
				}
				FileReference.Copy(ProvisioningData.MobileProvisionFile, MobileProvisionFile, true);
				Logger.LogInformation("[Remote] Uploading {MobileProvisionFile}", MobileProvisionFile);
				UploadFile(MobileProvisionFile, Logger);

				// Extract the certificate for the project. Try to avoid calling IPP if we already have it.
				FileReference CertificateFile = FileReference.Combine(TempDir, "Certificate.p12");

				FileReference CertificateInfoFile = FileReference.Combine(TempDir, "Certificate.txt");
				string CertificateInfoContents = String.Format("{0}\n{1}", ProvisioningData.MobileProvisionFile, FileReference.GetLastWriteTimeUtc(ProvisioningData.MobileProvisionFile).Ticks);

				if (!FileReference.Exists(CertificateFile) || !FileReference.Exists(CertificateInfoFile) || FileReference.ReadAllText(CertificateInfoFile) != CertificateInfoContents)
				{
					Logger.LogInformation("[Remote] Exporting certificate for {ProvisioningDataMobileProvisionFile}...", ProvisioningData.MobileProvisionFile);

					StringBuilder Arguments = new StringBuilder("ExportCertificate");
					if (TargetDesc.ProjectFile == null)
					{
						Arguments.AppendFormat(" \"{0}\"", Unreal.EngineSourceDirectory);
					}
					else
					{
						Arguments.AppendFormat(" \"{0}\"", TargetDesc.ProjectFile.Directory);
					}
					Arguments.AppendFormat(" -provisionfile \"{0}\"", ProvisioningData.MobileProvisionFile);
					Arguments.AppendFormat(" -outputcertificate \"{0}\"", CertificateFile);
					if (TargetDesc.Platform == UnrealTargetPlatform.TVOS)
					{
						Arguments.Append(" -tvos");
					}

					ProcessStartInfo StartInfo = new ProcessStartInfo();
					StartInfo.FileName = FileReference.Combine(Unreal.EngineDirectory, "Binaries", "DotNET", "IOS", "IPhonePackager.exe").FullName;
					StartInfo.Arguments = Arguments.ToString();
					if (Utils.RunLocalProcessAndLogOutput(StartInfo, Logger) != 0)
					{
						throw new BuildException("IphonePackager failed.");
					}

					FileReference.WriteAllText(CertificateInfoFile, CertificateInfoContents);
				}

				// Upload the certificate to the remote
				Logger.LogInformation("[Remote] Uploading {CertificateFile}", CertificateFile);
				UploadFile(CertificateFile, Logger);

				// Tell the remote UBT instance to use them
				RemoteArguments.Add(String.Format("-ImportProvision={0}", GetRemotePath(MobileProvisionFile)));
				RemoteArguments.Add(String.Format("-ImportCertificate={0}", GetRemotePath(CertificateFile)));
				RemoteArguments.Add(String.Format("-ImportCertificatePassword=A"));
			}

			// Upload the workspace files
			UploadWorkspace(TempDir, Logger);

			// Execute the compile
			Logger.LogInformation("[Remote] Executing build");

			StringBuilder BuildCommandLine = new StringBuilder("Engine/Build/BatchFiles/Mac/Build.sh");
			foreach (string RemoteArgument in RemoteArguments)
			{
				BuildCommandLine.AppendFormat(" {0}", EscapeShellArgument(RemoteArgument));
			}

			int Result = Execute(GetRemotePath(Unreal.RootDirectory), BuildCommandLine.ToString(), Logger);
			if (Result != 0)
			{
				if (RemoteLogFile != null)
				{
					Logger.LogInformation("[Remote] Downloading {RemoteLogFile}", RemoteLogFile);
					DownloadFile(RemoteLogFile, Logger);
				}
				return false;
			}

			// Download the manifest
			Logger.LogInformation("[Remote] Downloading {RemoteManifestFile}", RemoteManifestFile);
			DownloadFile(RemoteManifestFile, Logger);

			// Convert the manifest to local form
			BuildManifest Manifest = Utils.ReadClass<BuildManifest>(RemoteManifestFile.FullName, Logger);
			for (int Idx = 0; Idx < Manifest.BuildProducts.Count; Idx++)
			{
				Manifest.BuildProducts[Idx] = GetLocalPath(Manifest.BuildProducts[Idx]).FullName;
			}

			// Download the files from the remote
			Logger.LogInformation("[Remote] Downloading build products");

			List<FileReference> FilesToDownload = new List<FileReference>();
			FilesToDownload.Add(RemoteLogFile);
			FilesToDownload.AddRange(Manifest.BuildProducts.Select(x => new FileReference(x)));
			DownloadFiles(FilesToDownload, Logger);

			// Copy remote FrameworkAssets directory as it could contain resource bundles that must be packaged locally.
			DirectoryReference BaseDir = DirectoryReference.FromFile(TargetDesc.ProjectFile) ?? Unreal.EngineDirectory;
			DirectoryReference FrameworkAssetsDir = DirectoryReference.Combine(BaseDir, "Intermediate", TargetDesc.Platform == UnrealTargetPlatform.IOS ? "IOS" : "TVOS", "FrameworkAssets");
			if (RemoteDirectoryExists(FrameworkAssetsDir, Logger))
			{
				Logger.LogInformation("[Remote] Downloading {FrameworkAssetsDir}", FrameworkAssetsDir);
				DownloadDirectory(FrameworkAssetsDir, Logger);
			}

			// Write out all the local manifests
			foreach (FileReference LocalManifestFile in LocalManifestFiles)
			{
				Logger.LogInformation("[Remote] Writing {LocalManifestFile}", LocalManifestFile);
				Utils.WriteClass<BuildManifest>(Manifest, LocalManifestFile.FullName, "", Logger);
			}

			return true;
		}

		/// <summary>
		/// Creates a temporary directory for the given target
		/// </summary>
		/// <param name="TargetDesc">The target descriptor</param>
		/// <returns>Directory to use for temporary files</returns>
		static DirectoryReference CreateTempDirectory(TargetDescriptor TargetDesc)
		{
			DirectoryReference BaseDir = DirectoryReference.FromFile(TargetDesc.ProjectFile) ?? Unreal.EngineDirectory;
			DirectoryReference TempDir = DirectoryReference.Combine(BaseDir, "Intermediate", "Remote", TargetDesc.Name, TargetDesc.Platform.ToString(), TargetDesc.Configuration.ToString());
			DirectoryReference.CreateDirectory(TempDir);
			return TempDir;
		}

		/// <summary>
		/// Translate the arguments for a target descriptor for the remote machine
		/// </summary>
		/// <param name="TargetDesc">The target descriptor</param>
		/// <param name="LocalManifestFiles">Manifest files to be output from this target</param>
		/// <return>List of remote arguments</return>
		List<string> GetRemoteArgumentsForTarget(TargetDescriptor TargetDesc, List<FileReference>? LocalManifestFiles)
		{
			List<string> RemoteArguments = new List<string>();
			RemoteArguments.Add(TargetDesc.Name);
			RemoteArguments.Add(TargetDesc.Platform.ToString());
			RemoteArguments.Add(TargetDesc.Configuration.ToString());
			RemoteArguments.Add("-SkipRulesCompile"); // Use the rules assembly built locally
			RemoteArguments.Add(String.Format("-XmlConfigCache={0}", GetRemotePath(XmlConfig.CacheFile!))); // Use the XML config cache built locally, since the remote won't have it

			string? RemoteIniPath = UnrealBuildTool.GetRemoteIniPath();
			if (!String.IsNullOrEmpty(RemoteIniPath))
			{
				RemoteArguments.Add(String.Format("-remoteini={0}", GetRemotePath(RemoteIniPath)));
			}

			if (TargetDesc.ProjectFile != null)
			{
				RemoteArguments.Add(String.Format("-Project={0}", GetRemotePath(TargetDesc.ProjectFile)));
			}

			foreach (string LocalArgument in TargetDesc.AdditionalArguments)
			{
				int EqualsIdx = LocalArgument.IndexOf('=', StringComparison.Ordinal);
				if (EqualsIdx == -1)
				{
					RemoteArguments.Add(LocalArgument);
					continue;
				}

				string Key = LocalArgument[..EqualsIdx];
				string Value = LocalArgument[(EqualsIdx + 1)..];

				if (Key.Equals("-Log", StringComparison.OrdinalIgnoreCase))
				{
					// We are already writing to the local log file. The remote will produce a different log (RemoteLogFile)
					continue;
				}
				if (Key.Equals("-Manifest", StringComparison.OrdinalIgnoreCase) && LocalManifestFiles != null)
				{
					LocalManifestFiles.Add(new FileReference(Value));
					continue;
				}

				string RemoteArgument = LocalArgument;
				foreach (RemoteMapping Mapping in Mappings)
				{
					if (Value.StartsWith(Mapping.LocalDirectory.FullName, StringComparison.OrdinalIgnoreCase))
					{
						RemoteArgument = String.Format("{0}={1}", Key, GetRemotePath(Value));
						break;
					}
				}
				RemoteArguments.Add(RemoteArgument);
			}
			return RemoteArguments;
		}

		/// <summary>
		/// Runs the actool utility on a directory to create an Assets.car file
		/// </summary>
		/// <param name="Platform">The target platform</param>
		/// <param name="InputDir">Input directory containing assets</param>
		/// <param name="OutputFile">Path to the Assets.car file to produce</param>
		/// <param name="Logger">Logger for output</param>
		public void RunAssetCatalogTool(UnrealTargetPlatform Platform, DirectoryReference InputDir, FileReference OutputFile, ILogger Logger)
		{
			Logger.LogInformation("Running asset catalog tool for {Platform}: {InputDir} -> {OutputFile}", Platform, InputDir, OutputFile);

			string RemoteInputDir = GetRemotePath(InputDir);
			UploadDirectory(InputDir, Logger);

			string RemoteOutputFile = GetRemotePath(OutputFile);
			Execute(RemoteBaseDir, String.Format("rm -f {0}", EscapeShellArgument(RemoteOutputFile)), Logger);

			string RemoteOutputDir = Path.GetDirectoryName(RemoteOutputFile)!.Replace(Path.DirectorySeparatorChar, '/');
			Execute(RemoteBaseDir, String.Format("mkdir -p {0}", EscapeShellArgument(RemoteOutputDir)), Logger);

			string RemoteArguments = IOSToolChain.GetAssetCatalogArgs(Platform, RemoteInputDir, RemoteOutputDir);
			if (Execute(RemoteBaseDir, String.Format("/usr/bin/xcrun {0}", RemoteArguments), Logger) != 0)
			{
				throw new BuildException("Failed to run actool.");
			}
			DownloadFile(OutputFile, Logger);
		}

		/// <summary>
		/// Prepare the remotely built iOS or tvOS client to de debugged on Mac
		/// </summary>
		/// <param name="CookedDataDirectory">Where the content has been previously cooked</param>
		/// <param name="Logger">The logger</param>
		public void PrepareToDebug(string CookedDataDirectory, ILogger Logger)
		{
			Logger.LogInformation("[Remote] Uploading the default uprojectdirs ...");
			UploadFile(new FileReference("Default.uprojectdirs"), Logger);

			Logger.LogInformation("[Remote] Generating an Xcode Project file...");
			string CommandLine = EscapeShellArgument(GetRemotePath(Unreal.EngineDirectory));
			CommandLine += "/Build/BatchFiles/Mac/GenerateProjectFiles.sh";

			Execute(GetRemotePath(Unreal.RootDirectory), CommandLine, Logger);

			Logger.LogInformation("[Remote] Uploading cooked data directory...");
			Logger.LogInformation("{CookedDataDirectory}", CookedDataDirectory);
			UploadDirectory(new DirectoryReference(CookedDataDirectory), Logger);

			Logger.LogInformation("An Xcode project file has been generated on your remote Mac at the above address. Please access it to debug your IPA.");
		}

		/// <summary>
		/// Retrieves data from the primary Mac to the Windows machine
		/// </summary>
		/// <param name="ProjectFile">The project file</param>
		/// <param name="Logger">The logger</param>
		/// <param name="ClientPlatform">The platfrom for the client (typically iOS or tvOS)</param>
		public void RetrieveFilesGeneratedOnPrimaryMac(FileReference ProjectFile, ILogger Logger, string ClientPlatform)
		{
			Logger.LogInformation("[Remote] Downloading files generated on the Primary Remote Mac ...");
			DownloadDirectory(DirectoryReference.Combine(Unreal.EngineDirectory, "Intermediate"), Logger);
			DownloadDirectory(DirectoryReference.Combine(Unreal.EngineDirectory, "Saved"), Logger);

			DownloadDirectory(DirectoryReference.Combine(Unreal.EngineDirectory, "Binaries", ClientPlatform), Logger);
			DownloadDirectory(DirectoryReference.Combine(ProjectFile.Directory, "Intermediate"), Logger);
			DownloadDirectory(DirectoryReference.Combine(ProjectFile.Directory, "Binaries", ClientPlatform), Logger);
			DownloadDirectory(DirectoryReference.Combine(new DirectoryReference("UE5.xcworkspace")), Logger);

		}

		/// <summary>
		/// Upload retrieved data from the primary Mac from the Windows machine to the secondary Mac
		/// </summary>
		/// <param name="ProjectFile">The project file</param>
		/// <param name="Logger">The logger</param>
		public void UploadToSecondaryMac(FileReference ProjectFile, ILogger Logger)
		{
			Logger.LogInformation("[Remote] Uploading workspace to the secondary Mac ...");

			UploadWorkspace(DirectoryReference.Combine(Unreal.EngineDirectory, ".."), Logger);
			UploadFile(new FileReference("Default.uprojectdirs"), Logger);
			UploadDirectory(DirectoryReference.Combine(ProjectFile.Directory, "Intermediate"), Logger);
			UploadDirectory(DirectoryReference.Combine(ProjectFile.Directory, "Binaries"), Logger);
			UploadDirectory(DirectoryReference.Combine(ProjectFile.Directory, "Saved"), Logger);

			Logger.LogInformation("[Remote] Generating an Xcode Project file...");
			string CommandLine = EscapeShellArgument(GetRemotePath(Unreal.EngineDirectory));
			CommandLine += "/Build/BatchFiles/Mac/GenerateProjectFiles.sh";
			Execute(GetRemotePath(Unreal.RootDirectory), CommandLine, Logger);

			UploadDirectory(new DirectoryReference("UE5.xcworkspace"), Logger);
		}

		/// <summary>
		/// Convers a remote path into local form
		/// </summary>
		/// <param name="RemotePath">The remote filename</param>
		/// <returns>Local filename corresponding to the remote path</returns>
		private FileReference GetLocalPath(string RemotePath)
		{
			foreach (RemoteMapping Mapping in Mappings)
			{
				if (RemotePath.StartsWith(Mapping.RemoteDirectory, StringComparison.InvariantCultureIgnoreCase) && RemotePath.Length > Mapping.RemoteDirectory.Length && RemotePath[Mapping.RemoteDirectory.Length] == '/')
				{
					return FileReference.Combine(Mapping.LocalDirectory, RemotePath[(Mapping.RemoteDirectory.Length + 1)..]);
				}
			}
			throw new BuildException("Unable to map remote path '{0}' to local path", RemotePath);
		}

		/// <summary>
		/// Converts a local path into a remote one
		/// </summary>
		/// <param name="LocalPath">The local path to convert</param>
		/// <returns>Equivalent remote path</returns>
		private string GetRemotePath(FileSystemReference LocalPath)
		{
			return GetRemotePath(LocalPath.FullName);
		}

		/// <summary>
		/// Converts a local path into a remote one
		/// </summary>
		/// <param name="LocalPath">The local path to convert</param>
		/// <returns>Equivalent remote path</returns>
		private string GetRemotePath(string LocalPath)
		{
			string SanitizedLocalPath = LocalPath
				.Replace(":", "", StringComparison.Ordinal)
				.Replace('\\', '/')
				.Replace(' ', '_');
			return String.Format("{0}/{1}", RemoteBaseDir, SanitizedLocalPath).Replace('(', '_').Replace(')', '_').Replace('[', '_').Replace(']', '_');
		}

		/// <summary>
		/// Gets the local path in Cygwin format (eg. /cygdrive/C/...)
		/// </summary>
		/// <param name="InPath">Local path</param>
		/// <returns>Path in cygwin format</returns>
		private static string GetLocalCygwinPath(FileSystemReference InPath)
		{
			if (InPath.FullName.Length < 2 || InPath.FullName[1] != ':')
			{
				throw new BuildException("Invalid local path for converting to cygwin format ({0}).", InPath);
			}
			return String.Format("/cygdrive/{0}{1}", InPath.FullName[..1], InPath.FullName[2..].Replace('\\', '/'));
		}

		/// <summary>
		/// Escapes spaces and brackets in a shell command argument
		/// </summary>
		/// <param name="Argument">The argument to escape</param>
		/// <returns>The escaped argument</returns>
		private static string EscapeShellArgument(string Argument)
		{
			return Argument
				.Replace(" ", "\\ ", StringComparison.Ordinal)
				.Replace("[", "\\\\[", StringComparison.Ordinal)
				.Replace("]", "\\\\]", StringComparison.Ordinal);
		}

		/// <summary>
		/// Upload a single file to the remote
		/// </summary>
		/// <param name="LocalFile">The file to upload</param>
		/// <param name="Logger">Logger for output</param>
		void UploadFile(FileReference LocalFile, ILogger Logger)
		{
			string RemoteFile = GetRemotePath(LocalFile);
			string RemoteDirectory = GetRemotePath(LocalFile.Directory);

			List<string> Arguments = RsyncArgumentsForUpload(CommonRsyncArguments, RemoteDirectory);
			Arguments.Add(String.Format("\"{0}\"", GetLocalCygwinPath(LocalFile)));
			Arguments.Add(String.Format("\"{0}@{1}\":'{2}'", UserName, ServerName, RemoteFile));
			Arguments.Add("-q");

			int Result = Rsync(String.Join(" ", Arguments), Logger);
			if (Result != 0)
			{
				throw new BuildException("Error while running Rsync (exit code {0})", Result);
			}
		}

		/// <summary>
		/// Upload a single file to the remote
		/// </summary>
		/// <param name="LocalDirectory">The base directory to copy</param>
		/// <param name="RemoteDirectory">The remote directory</param>
		/// <param name="LocalFileList">The file to upload</param>
		/// <param name="Logger">Logger for output</param>
		void UploadFiles(DirectoryReference LocalDirectory, string RemoteDirectory, FileReference LocalFileList, ILogger Logger)
		{
			List<string> Arguments = RsyncArgumentsForUpload(BasicRsyncArguments, RemoteDirectory);
			Arguments.Add(String.Format("--files-from=\"{0}\"", GetLocalCygwinPath(LocalFileList)));
			Arguments.Add(String.Format("\"{0}/\"", GetLocalCygwinPath(LocalDirectory)));
			Arguments.Add(String.Format("\"{0}@{1}\":'{2}/'", UserName, ServerName, RemoteDirectory));
			Arguments.Add("-q");

			int Result = Rsync(String.Join(" ", Arguments), Logger);
			if (Result != 0)
			{
				throw new BuildException("Error while running Rsync (exit code {0})", Result);
			}
		}

		/// <summary>
		/// Upload a single directory to the remote
		/// </summary>
		/// <param name="LocalDirectory">The local directory to upload</param>
		/// <param name="Logger">Logger for output</param>
		void UploadDirectory(DirectoryReference LocalDirectory, ILogger Logger)
		{
			string RemoteDirectory = GetRemotePath(LocalDirectory);

			List<string> Arguments = RsyncArgumentsForUpload(CommonRsyncArguments, RemoteDirectory);
			Arguments.Add(String.Format("\"{0}/\"", GetLocalCygwinPath(LocalDirectory)));
			Arguments.Add(String.Format("\"{0}@{1}\":'{2}/'", UserName, ServerName, RemoteDirectory));
			Arguments.Add("-q");

			int Result = Rsync(String.Join(" ", Arguments), Logger);
			if (Result != 0)
			{
				throw new BuildException("Error while running Rsync (exit code {0})", Result);
			}
		}

		/// <summary>
		/// Uploads a directory to the remote using a specific filter list
		/// </summary>
		/// <param name="LocalDirectory">The local directory to copy from</param>
		/// <param name="RemoteDirectory">The remote directory to copy to</param>
		/// <param name="FilterLocations">List of paths to filter</param>
		/// <param name="Logger">Logger for output</param>
		void UploadDirectory(DirectoryReference LocalDirectory, string RemoteDirectory, List<FileReference> FilterLocations, ILogger Logger)
		{
			List<string> Arguments = RsyncArgumentsForUpload(CommonRsyncArguments, RemoteDirectory);
			foreach (FileReference FilterLocation in FilterLocations)
			{
				Arguments.Add(String.Format("--filter=\"merge {0}\"", GetLocalCygwinPath(FilterLocation)));
			}
			Arguments.Add("--exclude='*'");
			Arguments.Add(String.Format("\"{0}/\"", GetLocalCygwinPath(LocalDirectory)));
			Arguments.Add(String.Format("\"{0}@{1}\":'{2}/'", UserName, ServerName, RemoteDirectory));

			int Result = Rsync(String.Join(" ", Arguments), Logger);
			if (Result != 0)
			{
				throw new BuildException("Error while running Rsync (exit code {0})", Result);
			}
		}

		/// <summary>
		/// Upload all the files in the workspace for the current project
		/// </summary>
		void UploadWorkspace(DirectoryReference TempDir, ILogger Logger)
		{
			// Path to the scripts to be uploaded
			FileReference ScriptPathsFileName = FileReference.Combine(Unreal.EngineDirectory, "Build", "Rsync", "RsyncEngineScripts.txt");

			// Read the list of scripts to be uploaded
			List<string> ScriptPaths = new List<string>();
			foreach (string Line in FileReference.ReadAllLines(ScriptPathsFileName))
			{
				string FileToUpload = Line.Trim();
				if (FileToUpload.Length > 0 && FileToUpload[0] != '#')
				{
					ScriptPaths.Add(FileToUpload);
				}
			}

			// Fixup the line endings
			List<FileReference> TargetFiles = new List<FileReference>();
			foreach (string ScriptPath in ScriptPaths)
			{
				FileReference SourceFile = FileReference.Combine(Unreal.EngineDirectory, ScriptPath.TrimStart('/'));
				if (!FileReference.Exists(SourceFile))
				{
					throw new BuildException("Missing script required for remote upload: {0}", SourceFile);
				}

				FileReference TargetFile = FileReference.Combine(TempDir, SourceFile.MakeRelativeTo(Unreal.EngineDirectory));
				if (!FileReference.Exists(TargetFile) || FileReference.GetLastWriteTimeUtc(TargetFile) < FileReference.GetLastWriteTimeUtc(SourceFile))
				{
					DirectoryReference.CreateDirectory(TargetFile.Directory);
					string ScriptText = FileReference.ReadAllText(SourceFile);
					FileReference.WriteAllText(TargetFile, ScriptText.Replace("\r\n", "\n", StringComparison.Ordinal));
				}
				TargetFiles.Add(TargetFile);
			}

			// Write a file that protects all the scripts from being overridden by the standard engine filters
			FileReference ScriptProtectList = FileReference.Combine(TempDir, "RsyncEngineScripts-Protect.txt");
			using (StreamWriter Writer = new StreamWriter(ScriptProtectList.FullName))
			{
				foreach (string ScriptPath in ScriptPaths)
				{
					Writer.WriteLine("protect {0}", ScriptPath);
				}
			}

			// Upload these files to the remote
			Logger.LogInformation("[Remote] Uploading scripts...");
			UploadFiles(TempDir, GetRemotePath(Unreal.EngineDirectory), ScriptPathsFileName, Logger);

			// Upload the config files
			Logger.LogInformation("[Remote] Uploading config files...");
			UploadFile(XmlConfig.CacheFile!, Logger);

			// Upload the engine files
			List<FileReference> EngineFilters = new List<FileReference>();
			EngineFilters.Add(ScriptProtectList);
			if (Unreal.IsEngineInstalled())
			{
				EngineFilters.Add(FileReference.Combine(Unreal.EngineDirectory, "Build", "Rsync", "RsyncEngineInstalled.txt"));

				// Upload MarketplaceRules.dll (if it exists) when in InstallBuild. The path is specified in RulesCompiler::CreateMarketplaceRulesAssembly()
				DirectoryReference MarketplaceEngineDir = DirectoryReference.Combine(Unreal.WritableEngineDirectory, "Intermediate", "Build", "BuildRules");
				if (DirectoryReference.Exists(MarketplaceEngineDir))
				{
					Logger.LogInformation("[Remote] Uploading MarketplaceRules Engine files located in {MarketplaceEngineDir}", MarketplaceEngineDir);
					UploadDirectory(MarketplaceEngineDir, Logger);
				}
			}
			EngineFilters.Add(FileReference.Combine(Unreal.EngineDirectory, "Build", "Rsync", "RsyncEngine.txt"));

			Logger.LogInformation("[Remote] Uploading engine files...");
			UploadDirectory(Unreal.EngineDirectory, GetRemotePath(Unreal.EngineDirectory), EngineFilters, Logger);

			// Upload the project files
			DirectoryReference? ProjectDir = null;
			if (ProjectFile != null && !ProjectFile.IsUnderDirectory(Unreal.EngineDirectory))
			{
				ProjectDir = ProjectFile.Directory;
			}
			else if (!String.IsNullOrEmpty(UnrealBuildTool.GetRemoteIniPath()))
			{
				ProjectDir = new DirectoryReference(UnrealBuildTool.GetRemoteIniPath()!);
				if (ProjectDir.IsUnderDirectory(Unreal.EngineDirectory))
				{
					ProjectDir = null;
				}
			}
			if (ProjectDir != null)
			{
				List<FileReference> ProjectFilters = new List<FileReference>();

				FileReference CustomFilter = FileReference.Combine(ProjectDir, "Build", "Rsync", "RsyncProject.txt");
				if (FileReference.Exists(CustomFilter))
				{
					ProjectFilters.Add(CustomFilter);
				}
				ProjectFilters.Add(FileReference.Combine(Unreal.EngineDirectory, "Build", "Rsync", "RsyncProject.txt"));

				Logger.LogInformation("[Remote] Uploading project files...");
				UploadDirectory(ProjectDir, GetRemotePath(ProjectDir), ProjectFilters, Logger);
			}

			if (AdditionalPaths != null)
			{
				foreach (DirectoryReference AdditionalPath in AdditionalPaths)
				{
					List<FileReference> CustomFilters = new List<FileReference>();

					FileReference CustomFilter = FileReference.Combine(AdditionalPath, "Build", "Rsync", "RsyncProject.txt");
					if (FileReference.Exists(CustomFilter))
					{
						CustomFilters.Add(CustomFilter);
					}
					CustomFilters.Add(FileReference.Combine(Unreal.EngineDirectory, "Build", "Rsync", "RsyncProject.txt"));

					Logger.LogInformation("[Remote] Uploading additional path files [{Dir}]...", AdditionalPath);
					UploadDirectory(AdditionalPath, GetRemotePath(AdditionalPath), CustomFilters, Logger);
				}
			}

			Execute("/", String.Format("rm -rf {0}/Intermediate/IOS/*.plist", GetRemotePath(Unreal.EngineDirectory)), Logger, true);
			Execute("/", String.Format("rm -rf {0}/Intermediate/TVOS/*.plist", GetRemotePath(Unreal.EngineDirectory)), Logger, true);
			if (ProjectFile != null)
			{
				Execute("/", String.Format("rm -rf {0}/Intermediate/IOS/*.plist", GetRemotePath(ProjectFile.Directory)), Logger, true);
				Execute("/", String.Format("rm -rf {0}/Intermediate/TVOS/*.plist", GetRemotePath(ProjectFile.Directory)), Logger, true);
			}

			// Convert CRLF to LF for all shell scripts
			Execute(RemoteBaseDir, String.Format("for i in {0}/Build/BatchFiles/Mac/*.sh; do mv $i $i.crlf; tr -d '\r' < $i.crlf > $i; done", EscapeShellArgument(GetRemotePath(Unreal.EngineDirectory))), Logger);

			// Fixup permissions on any shell scripts
			Execute(RemoteBaseDir, String.Format("chmod +x {0}/Build/BatchFiles/Mac/*.sh", EscapeShellArgument(GetRemotePath(Unreal.EngineDirectory))), Logger);
		}

		/// <summary>
		/// Downloads a single file from the remote
		/// </summary>
		/// <param name="LocalFile">The file to download</param>
		/// <param name="Logger">Logger for output</param>
		void DownloadFile(FileReference LocalFile, ILogger Logger)
		{
			RemoteMapping? Mapping = Mappings.FirstOrDefault(x => LocalFile.IsUnderDirectory(x.LocalDirectory));
			if (Mapping == null)
			{
				throw new BuildException("File for download '{0}' is not under any mapped directory.", LocalFile);
			}

			List<string> Arguments = RsyncArgumentsForDownload(CommonRsyncArguments);
			Arguments.Add(String.Format("\"{0}@{1}\":'{2}/{3}'", UserName, ServerName, Mapping.RemoteDirectory, LocalFile.MakeRelativeTo(Mapping.LocalDirectory).Replace('\\', '/')));
			Arguments.Add(String.Format("\"{0}/\"", GetLocalCygwinPath(LocalFile.Directory)));
			Arguments.Add("-q");

			int Result = Rsync(String.Join(" ", Arguments), Logger);
			if (Result != 0)
			{
				throw new BuildException("Unable to download '{0}' from the remote Mac (exit code {1}).", LocalFile, Result);
			}
		}

		/// <summary>
		/// Download multiple files from the remote Mac
		/// </summary>
		/// <param name="Files">List of local files to download</param>
		/// <param name="Logger">Logger for output</param>
		void DownloadFiles(IEnumerable<FileReference> Files, ILogger Logger)
		{
			List<FileReference>[] FileGroups = new List<FileReference>[Mappings.Count];
			for (int Idx = 0; Idx < Mappings.Count; Idx++)
			{
				FileGroups[Idx] = new List<FileReference>();
			}
			foreach (FileReference File in Files)
			{
				int MappingIdx = Mappings.FindIndex(x => File.IsUnderDirectory(x.LocalDirectory));
				if (MappingIdx == -1)
				{
					throw new BuildException("File for download '{0}' is not under the engine or project directory.", File);
				}
				FileGroups[MappingIdx].Add(File);
			}
			for (int Idx = 0; Idx < Mappings.Count; Idx++)
			{
				if (FileGroups[Idx].Count > 0)
				{
					FileReference DownloadListLocation = FileReference.Combine(Unreal.EngineDirectory, "Intermediate", "Rsync", "Download.txt");
					DirectoryReference.CreateDirectory(DownloadListLocation.Directory);
					FileReference.WriteAllLines(DownloadListLocation, FileGroups[Idx].Select(x => x.MakeRelativeTo(Mappings[Idx].LocalDirectory).Replace('\\', '/')));

					List<string> Arguments = RsyncArgumentsForDownload(CommonRsyncArguments);
					Arguments.Add(String.Format("--files-from=\"{0}\"", GetLocalCygwinPath(DownloadListLocation)));
					Arguments.Add(String.Format("\"{0}@{1}\":'{2}/'", UserName, ServerName, Mappings[Idx].RemoteDirectory));
					Arguments.Add(String.Format("\"{0}/\"", GetLocalCygwinPath(Mappings[Idx].LocalDirectory)));

					int Result = Rsync(String.Join(" ", Arguments), Logger);
					if (Result != 0)
					{
						throw new BuildException("Unable to download files from remote Mac (exit code {0})", Result);
					}
				}
			}
		}

		/// <summary>
		/// Checks whether a directory exists on the remote machine
		/// </summary>
		/// <param name="LocalDirectory">Path to the directory on the local machine</param>
		/// <param name="Logger">Logger for output</param>
		/// <returns>True if the remote directory exists</returns>
		private bool RemoteDirectoryExists(DirectoryReference LocalDirectory, ILogger Logger)
		{
			string RemoteDirectory = GetRemotePath(LocalDirectory);
			return Execute(Unreal.RootDirectory, String.Format("[ -d {0} ]", EscapeShellArgument(RemoteDirectory)), Logger) == 0;
		}

		/// <summary>
		/// Download a directory from the remote Mac
		/// </summary>
		/// <param name="LocalDirectory">Directory to download</param>
		/// <param name="Logger">Logger for output</param>
		private void DownloadDirectory(DirectoryReference LocalDirectory, ILogger Logger)
		{
			DirectoryReference.CreateDirectory(LocalDirectory);

			string RemoteDirectory = GetRemotePath(LocalDirectory);

			List<string> Arguments = RsyncArgumentsForDownload(CommonRsyncArguments);
			Arguments.Add(String.Format("\"{0}@{1}\":'{2}/'", UserName, ServerName, RemoteDirectory));
			Arguments.Add(String.Format("\"{0}/\"", GetLocalCygwinPath(LocalDirectory)));

			int Result = Rsync(String.Join(" ", Arguments), Logger);
			if (Result != 0)
			{
				throw new BuildException("Unable to download '{0}' from the remote Mac (exit code {1}).", LocalDirectory, Result);
			}
		}

		/// <summary>
		/// Execute Rsync
		/// </summary>
		/// <param name="Arguments">Arguments for the Rsync command</param>
		/// <param name="Logger">Logger for output</param>
		/// <returns>Exit code from Rsync</returns>
		private int Rsync(string Arguments, ILogger Logger)
		{
			using (Process RsyncProcess = new Process())
			{
				DataReceivedEventHandler OutputHandler = (E, Args) => { RsyncOutput(Args, false, Logger); };
				DataReceivedEventHandler ErrorHandler = (E, Args) => { RsyncOutput(Args, true, Logger); };

				RsyncProcess.StartInfo.FileName = RsyncExe.FullName;
				RsyncProcess.StartInfo.Arguments = Arguments;
				RsyncProcess.StartInfo.WorkingDirectory = SshExe.Directory.FullName;
				RsyncProcess.OutputDataReceived += OutputHandler;
				RsyncProcess.ErrorDataReceived += ErrorHandler;

				Logger.LogDebug("[Rsync] {File} {Args}", Utils.MakePathSafeToUseWithCommandLine(RsyncProcess.StartInfo.FileName), RsyncProcess.StartInfo.Arguments);
				return Utils.RunLocalProcess(RsyncProcess);
			}
		}

		/// <summary>
		/// Handles data output by rsync
		/// </summary>
		/// <param name="Args">The received data</param>e
		/// <param name="bStdErr">whether the data was received on stderr</param>
		/// <param name="Logger">Logger for output</param>
		private static void RsyncOutput(DataReceivedEventArgs Args, bool bStdErr, ILogger Logger)
		{
			if (Args.Data != null)
			{
				if (bStdErr)
				{
					Logger.LogError("  {Output}", Args.Data);
				}
				else
				{
					Logger.LogInformation("  {Output}", Args.Data);
				}
			}
		}

		/// <summary>
		/// Execute a command on the remote in the remote equivalent of a local directory
		/// </summary>
		/// <param name="WorkingDir"></param>
		/// <param name="Command"></param>
		/// <param name="Logger">Logger for output</param>
		/// <param name="bSilent"></param>
		/// <returns></returns>
		public int Execute(DirectoryReference WorkingDir, string Command, ILogger Logger, bool bSilent = false)
		{
			return Execute(GetRemotePath(WorkingDir), Command, Logger, bSilent);
		}

		/// <summary>
		/// Execute a remote command, capturing the output text
		/// </summary>
		/// <param name="WorkingDirectory">The remote working directory</param>
		/// <param name="Command">Command to be executed</param>
		/// <param name="Logger">Logger for output</param>
		/// <param name="bSilent">If true, logging is suppressed</param>
		/// <returns></returns>
		protected int Execute(string WorkingDirectory, string Command, ILogger Logger, bool bSilent = false)
		{
			string FullCommand = String.Format("cd {0} && {1}", EscapeShellArgument(WorkingDirectory), Command);
			using (Process SSHProcess = new Process())
			{
				DataReceivedEventHandler OutputHandler = (E, Args) => { SshOutput(Args, false, Logger); };
				DataReceivedEventHandler ErrorHandler = (E, Args) => { SshOutput(Args, true, Logger); };

				SSHProcess.StartInfo.FileName = SshExe.FullName;
				SSHProcess.StartInfo.WorkingDirectory = SshExe.Directory.FullName;
				SSHProcess.StartInfo.Arguments = String.Format("{0} {1}", String.Join(" ", CommonSshArguments), FullCommand);
				if (!bSilent)
				{
					SSHProcess.OutputDataReceived += OutputHandler;
					SSHProcess.ErrorDataReceived += ErrorHandler;
				}

				Logger.LogDebug("[SSH] {Exe} {Args}", Utils.MakePathSafeToUseWithCommandLine(SSHProcess.StartInfo.FileName), SSHProcess.StartInfo.Arguments);
				return Utils.RunLocalProcess(SSHProcess);
			}
		}

		/// <summary>
		/// Handler for output from running remote SSH commands
		/// </summary>
		/// <param name="Args"></param>
		/// <param name="bStdErr">whether the data was received on stderr</param>
		/// <param name="Logger">Logger for output</param>
		private void SshOutput(DataReceivedEventArgs Args, bool bStdErr, ILogger Logger)
		{
			if (Args.Data != null)
			{
				string FormattedOutput = ConvertRemotePathsToLocal(Args.Data);
				if (bStdErr)
				{
					Logger.LogError("  {Output}", FormattedOutput);
				}
				else
				{
					Logger.LogInformation("  {Output}", FormattedOutput);
				}
			}
		}

		/// <summary>
		/// Execute a remote command, capturing the output text
		/// </summary>
		/// <param name="Command">Command to be executed</param>
		/// <param name="Logger">Logger for output</param>
		/// <param name="Output">Receives the output text</param>
		/// <returns></returns>
		protected int ExecuteAndCaptureOutput(string Command, ILogger Logger, out StringBuilder Output)
		{
			StringBuilder FullCommand = new StringBuilder();
			foreach (string CommonSshArgument in CommonSshArguments)
			{
				FullCommand.AppendFormat("{0} ", CommonSshArgument);
			}
			FullCommand.Append(Command.Replace("\"", "\\\"", StringComparison.Ordinal));

			using (Process SSHProcess = new Process())
			{
				Output = new StringBuilder();

				StringBuilder OutputLocal = Output;
				DataReceivedEventHandler OutputHandler = (E, Args) =>
				{
					if (Args.Data != null)
					{
						OutputLocal.Append(Args.Data);
					}
				};

				SSHProcess.StartInfo.FileName = SshExe.FullName;
				SSHProcess.StartInfo.WorkingDirectory = SshExe.Directory.FullName;
				SSHProcess.StartInfo.Arguments = FullCommand.ToString();
				SSHProcess.OutputDataReceived += OutputHandler;
				SSHProcess.ErrorDataReceived += OutputHandler;

				Logger.LogDebug("[SSH] {Exe} {Args}", Utils.MakePathSafeToUseWithCommandLine(SSHProcess.StartInfo.FileName), SSHProcess.StartInfo.Arguments);
				return Utils.RunLocalProcess(SSHProcess);
			}
		}

		/// <summary>
		/// Converts any remote paths within the given string to local format
		/// </summary>
		/// <param name="Text">The text containing strings to convert</param>
		/// <returns>The string with paths converted to local format</returns>
		private string ConvertRemotePathsToLocal(string Text)
		{
			// Try to match any source file with the remote base directory in front of it
			string Pattern = String.Format("(?<![a-zA-Z=]){0}[^:]*\\.(?:cpp|inl|h|hpp|hh|txt)(?![a-zA-Z])", Regex.Escape(RemoteBaseDir));

			// Find the matches, and early out if there are none
			MatchCollection Matches = Regex.Matches(Text, Pattern, RegexOptions.IgnoreCase);
			if (Matches.Count == 0)
			{
				return Text;
			}

			// Replace any remote paths with local ones
			StringBuilder Result = new StringBuilder();
			int StartIdx = 0;
			foreach (Match? Match in Matches)
			{
				// Append the text leading up to this path
				Result.Append(Text, StartIdx, Match!.Index - StartIdx);

				// Try to convert the path
				string Path = Match.Value;
				foreach (RemoteMapping Mapping in Mappings)
				{
					if (Path.StartsWith(Mapping.RemoteDirectory, StringComparison.Ordinal))
					{
						Path = Mapping.LocalDirectory + Path[Mapping.RemoteDirectory.Length..].Replace('/', '\\');
						break;
					}
				}

				// Append the path to the output string
				Result.Append(Path);

				// Move past this match
				StartIdx = Match.Index + Match.Length;
			}
			Result.Append(Text, StartIdx, Text.Length - StartIdx);
			return Result.ToString();
		}
	}
}
