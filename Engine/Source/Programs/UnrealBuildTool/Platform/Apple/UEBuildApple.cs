// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

// ReSharper disable once CheckNamespace
namespace UnrealBuildTool
{
	/// <summary>
	/// Apple-specific target settings
	/// </summary>
	public abstract class AppleTargetRules
	{ 
		/// <summary>
		/// Whether to strip symbols or not (implied by Shipping config).
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		[CommandLine("-stripsymbols", Value = "true")]
		public bool bStripSymbols = false;
		
		/// <summary>
		/// Whether to create a ".stripped" file when stripping symbols or not.
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		[CommandLine("-createstripflagfile", Value = "true")]
		public bool bCreateStripFlagFile = false;

		/// <summary>
		/// Disables clang build verification checks on static libraries
		/// </summary>
		[CommandLine("-skipclangvalidation", Value = "true")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bSkipClangValidation = false;
		
		/// <summary>
		/// Enables address sanitizer (ASan).
		/// </summary>
		[CommandLine("-EnableASan")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bEnableAddressSanitizer = false;

		/// <summary>
		/// Enables thread sanitizer (TSan).
		/// </summary>
		[CommandLine("-EnableTSan")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bEnableThreadSanitizer = false;

		/// <summary>
		/// Enables undefined behavior sanitizer (UBSan).
		/// </summary>
		[CommandLine("-EnableUBSan")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bEnableUndefinedBehaviorSanitizer = false;

		/// <summary>
		/// Enables FAppleContactManager support by linking to the Contacts framework and adding the addressbook entitlement if needed. 
		/// NOTE: simply turning this on may introduce more paperwork during the App Store review process, so only enable this if you 
		/// need to use that class!
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Game, "/Script/AppleTargetPlatform.AppleTargetSettings", "bEnableContactsManager")]
		public bool bEnableContactsManager = false;

		/// <summary>
		/// Experimental: Enable portable toolchain for distributed compilation. Separates SDK-dependent
		/// files (local) from portable files (remote). Requires Auto SDK (UE_SDKS_ROOT).
		/// </summary>
		[CommandLine("-UsePortableToolchain")]
		[CommandLine("-NoPortableToolchain", Value = "false")]
		[XmlConfigFile(Category = "BuildConfiguration")]
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/AppleTargetPlatform.AppleTargetSettings", "bUsePortableToolchain")]
		public bool bUsePortableToolchain = false;
	}

	/// <summary>
	/// Read-only wrapper for Apple-specific target settings
	/// </summary>
	public class ReadOnlyAppleTargetRules
	{
		/// <summary>
		/// The private mutable settings object
		/// </summary>
		private readonly AppleTargetRules Inner;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="Inner">The settings object to wrap</param>
		public ReadOnlyAppleTargetRules(AppleTargetRules Inner)
		{
			this.Inner = Inner;
		}

		/// <summary>
		/// Accessors for fields on the inner TargetRules instance
		/// </summary>

		#region Read-only accessor properties
#pragma warning disable CS1591
		public bool bStripSymbols => Inner.bStripSymbols;
		public bool bCreateStripFlagFile => Inner.bCreateStripFlagFile;
		public bool bSkipClangValidation => Inner.bSkipClangValidation;
		public bool bEnableAddressSanitizer => Inner.bEnableAddressSanitizer;
		public bool bEnableThreadSanitizer => Inner.bEnableThreadSanitizer;
		public bool bEnableUndefinedBehaviorSanitizer => Inner.bEnableUndefinedBehaviorSanitizer;
		public bool bEnableContactsManager => Inner.bEnableContactsManager;
		public bool bUsePortableToolchain => Inner.bUsePortableToolchain;
#pragma warning restore CS1591
		#endregion
	}
	
	/// <summary>
	/// Architecture config for Apple platforms
	/// </summary>
	class AppleArchitectureConfig : UnrealArchitectureConfig
	{
		public AppleArchitectureConfig(IEnumerable<UnrealArch> SupportedArchitectures)
			: base(UnrealArchitectureMode.SingleTargetCompileSeparately, SupportedArchitectures)
		{
		}
	}
	
	abstract class AppleBuildPlatform : UEBuildPlatform
	{
		public AppleBuildPlatform(UnrealTargetPlatform Platform, UEBuildPlatformSDK SDK, UnrealArchitectureConfig ArchitectureConfig, ILogger Logger)
			: base(Platform, SDK, ArchitectureConfig, Logger)
		{
		}

		public abstract AppleTargetRules GetAppleTargetRules(TargetRules Target);

		public abstract ReadOnlyAppleTargetRules GetAppleTargetRules(ReadOnlyTargetRules Target);

		public override void GetExternalBuildMetadata(FileReference? ProjectFile, StringBuilder Metadata)
		{
			base.GetExternalBuildMetadata(ProjectFile, Metadata);
			Metadata.AppendLine("xcode-select: {0}", ApplePlatformSDK.DeveloperDir);
		}

		public override bool IsBuildProduct(string FileName, string[] NamePrefixes, string[] NameSuffixes)
		{
			return IsBuildProductName(FileName, NamePrefixes, NameSuffixes, "")
			       || IsBuildProductName(FileName, NamePrefixes, NameSuffixes, ".dylib")
			       || IsBuildProductName(FileName, NamePrefixes, NameSuffixes, ".dSYM");
		}

		public override bool CanUseFASTBuild()
		{
			return true;
		}

		public override void ResetTarget(TargetRules Target)
		{
			base.ResetTarget(Target);

			// always strip in shipping configuration (commandline could have set it also)
			if (Target.Configuration == UnrealTargetConfiguration.Shipping)
			{
				GetAppleTargetRules(Target).bStripSymbols = true;
			}
		}

		public override void ValidateTarget(TargetRules Target)
		{
			base.ValidateTarget(Target);
			
			if (!String.IsNullOrWhiteSpace(Environment.GetEnvironmentVariable("CLANG_STATIC_ANALYZER_MODE")))
			{
				Target.StaticAnalyzer = StaticAnalyzer.Default;
				Target.StaticAnalyzerOutputType = (Environment.GetEnvironmentVariable("CLANG_ANALYZER_OUTPUT")?.Contains("html", StringComparison.OrdinalIgnoreCase) == true) ? StaticAnalyzerOutputType.Html : StaticAnalyzerOutputType.Text;
				Target.StaticAnalyzerMode = String.Equals(Environment.GetEnvironmentVariable("CLANG_STATIC_ANALYZER_MODE"), "shallow", StringComparison.OrdinalIgnoreCase) ? StaticAnalyzerMode.Shallow : StaticAnalyzerMode.Deep;
			}
			else if (Target.StaticAnalyzer == StaticAnalyzer.Clang)
			{
				Target.StaticAnalyzer = StaticAnalyzer.Default;
			}

			// Disable linking and ignore build outputs if we're using a static analyzer
			if (Target.StaticAnalyzer == StaticAnalyzer.Default)
			{
				Target.bDisableLinking = true;
				Target.bIgnoreBuildOutputs = true;

				// Clang static analysis requires non unity builds
				Target.bUseUnityBuild = false;
				if (Target.bStaticAnalyzerIncludeGenerated)
				{
					Target.bAlwaysUseUnityForGeneratedFiles = false;
				}

				// Disable chaining PCHs for the moment because it is crashing clang
				Target.bChainPCHs = false;
			}

			if (Target.bCompileAgainstEngine)
			{
				Target.GlobalDefinitions.Add("HAS_METAL=1");
				Target.ExtraModuleNames.Add("MetalRHI");
			}
			else
			{
				Target.GlobalDefinitions.Add("HAS_METAL=0");
			}
			
			// Force using the ANSI allocator if ASan is enabled
			if (GetAppleTargetRules(Target).bEnableAddressSanitizer)
			{
				Target.StaticAllocator = StaticAllocatorType.Ansi;
			}
		}

		/// <summary>
		/// Get the extension to use for the given binary type
		/// </summary>
		/// <param name="InBinaryType"> The binary type being built</param>
		/// <returns>string    The binary extenstion (ie 'exe' or 'dll')</returns>
		public override string GetBinaryExtension(UEBuildBinaryType InBinaryType)
		{
			switch (InBinaryType)
			{
				case UEBuildBinaryType.DynamicLinkLibrary:
					return ".dylib";
				case UEBuildBinaryType.Executable:
					return "";
				case UEBuildBinaryType.StaticLibrary:
					return ".a";
			}

			return base.GetBinaryExtension(InBinaryType);
		}

		public override void SetUpEnvironment(ReadOnlyTargetRules Target, CppCompileEnvironment CompileEnvironment, LinkEnvironment LinkEnvironment)
		{
			// shared stuff for all apple platforms
			
			CompileEnvironment.Definitions.Add("PLATFORM_APPLE=1");
			CompileEnvironment.Definitions.Add("UE_PLATFORM_APPLE=1");

			AppleExports.GetSwiftIntegrationSettings(Target.ProjectFile, Target.Type, Platform, out bool bUseSwiftUIMain, out _);
			CompileEnvironment.Definitions.Add("UE_USE_SWIFT_UI_MAIN=" + (bUseSwiftUIMain ? "1" : "0"));

			CompileEnvironment.Definitions.Add("WITH_TTS=0");
			CompileEnvironment.Definitions.Add("WITH_SPEECH_RECOGNITION=0");
		}
		
		protected ClangToolChainOptions GetToolChainOptionsForSanitizers(ReadOnlyTargetRules Target)
		{
			ReadOnlyAppleTargetRules AppleTarget = GetAppleTargetRules(Target);
			ClangToolChainOptions Options = ClangToolChainOptions.None;
			if (AppleTarget.bEnableAddressSanitizer)
			{
				Options |= ClangToolChainOptions.EnableAddressSanitizer;
			}
			if (AppleTarget.bEnableThreadSanitizer)
			{
				Options |= ClangToolChainOptions.EnableThreadSanitizer;
			}
			if (AppleTarget.bEnableUndefinedBehaviorSanitizer)
			{
				Options |= ClangToolChainOptions.EnableUndefinedBehaviorSanitizer;
			}
			return Options;
		}
	}
}
