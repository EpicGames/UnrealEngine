// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.IO;
using System.Linq;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// Single
	/// </summary>
	internal class ClangSpecificFileAction : Action, ISpecificFileAction
	{
		readonly DirectoryReference SourceDir;
		readonly DirectoryReference OutputDir;
		readonly IEnumerable<string> RspLines;

		readonly Dictionary<string, List<FileItem>> SingleFiles = new();

		internal ClangSpecificFileAction(DirectoryReference Source, DirectoryReference Output, Action Action, IEnumerable<string> ContentLines) : base(Action)
		{
			ProducedItems.Clear();
			DependencyListFile = null;

			SourceDir = Source;
			OutputDir = Output;
			RspLines = ContentLines;
			ArtifactMode = ArtifactMode.None;
		}

		public ClangSpecificFileAction(BinaryArchiveReader Reader) : base(Reader)
		{
			SourceDir = Reader.ReadCompactDirectoryReference();
			OutputDir = Reader.ReadCompactDirectoryReference();
			RspLines = Reader.ReadList(() => Reader.ReadString())!;
		}

		public new void Write(BinaryArchiveWriter Writer)
		{
			base.Write(Writer);
			Writer.WriteCompactDirectoryReference(SourceDir);
			Writer.WriteCompactDirectoryReference(OutputDir);
			Writer.WriteList(RspLines.ToList(), (Str) => Writer.WriteString(Str));
		}

		public DirectoryReference RootDirectory => SourceDir;

		public IExternalAction? CreateAction(FileItem SourceFile, ILogger Logger)
		{
			DirectoryReference.CreateDirectory(DirectoryReference.Combine(OutputDir, "SingleFile"));

			// Keep track of all specific files, so the output file can be renamed if there's a naming conflict
			string Filename = $"SingleFile/{SourceFile.Name}";
			if (!SingleFiles.ContainsKey(SourceFile.Name))
			{
				SingleFiles[SourceFile.Name] = new();
			}
			else
			{
				Filename = $"SingleFile/{Path.GetFileNameWithoutExtension(SourceFile.Name)}{SingleFiles[SourceFile.Name].Count}{Path.GetExtension(SourceFile.Name)}";
			}
			SingleFiles[SourceFile.Name].Add(SourceFile);

			string DummyName = "SingleFile.cpp";
			string UniqueDummyName = Filename;

			int FileNameIndex = CommandArguments.IndexOf(DummyName, System.StringComparison.Ordinal);
			string DummyPath = CommandArguments.Substring(2, FileNameIndex + DummyName.Length - 2);

			List<string> NewRspLines = new();

			if (SourceFile.HasExtension(".h"))
			{
				ClangWarnings.GetHeaderDisabledWarnings(NewRspLines);

				string IncludeFileString = SourceFile.AbsolutePath;
				if (SourceFile.Location.IsUnderDirectory(Unreal.RootDirectory))
				{
					IncludeFileString = SourceFile.Location.MakeRelativeTo(Unreal.EngineSourceDirectory);
				}

				List<string> GeneratedHeaderCppContents = UEBuildModuleCPP.GenerateHeaderCpp(SourceFile.Name, IncludeFileString);
				SourceFile = FileItem.GetItemByFileReference(FileReference.Combine(OutputDir, $"{UniqueDummyName}.cpp"));
				File.WriteAllLines(SourceFile.FullName, GeneratedHeaderCppContents);
			}

			foreach (string L in RspLines)
			{
				string Line = L;
				if (Line.Contains(".cpp.bc", System.StringComparison.Ordinal) ||
					Line.Contains(".cpp.d", System.StringComparison.Ordinal) ||
					Line.Contains(".cpp.i", System.StringComparison.Ordinal) ||
					Line.Contains(".cpp.json", System.StringComparison.Ordinal) ||
					Line.Contains(".cpp.o", System.StringComparison.Ordinal))
				{
					Line = Line.Replace("SingleFile.cpp", UniqueDummyName, System.StringComparison.Ordinal);
				}
				else
				{
					Line = Line.Replace(DummyPath, SourceFile.FullName.Replace('\\', '/'), System.StringComparison.Ordinal);
				}
				NewRspLines.Add(Line);
			}

			Action Action = new Action(this);
			Action.CommandArguments = CommandArguments.Replace(DummyName, UniqueDummyName, System.StringComparison.Ordinal);
			Action.DependencyListFile = null;
			Action.StatusDescription = SourceFile.Name;

			// We have to add a produced item so this action is not skipped.
			// Note we on purpose use a different extension than what the compiler produce because otherwise up-to-date checker might see it as up-to-date
			// even though we want it to always be built
			FileItem ProducedItem = FileItem.GetItemByFileReference(FileReference.Combine(OutputDir, UniqueDummyName + ".n"));
			Action.ProducedItems.Add(ProducedItem);

			if (RspLines.Any(x => x.Contains(".cpp.i", System.StringComparison.Ordinal)))
			{
				FileItem PreprocessedItem = FileItem.GetItemByFileReference(FileReference.Combine(OutputDir, UniqueDummyName + ".i"));
				Action.ProducedItems.Add(PreprocessedItem);
			}

			FileReference ResponseFile = RootPaths.GetLocalPath(new(Action.CommandArguments.Substring(1).Trim('"')));
			File.WriteAllLines(ResponseFile.FullName, NewRspLines);

			return Action;
		}
	}

	class ClangSpecificFileActionSerializer : ActionSerializerBase<ClangSpecificFileAction>
	{
		/// <inheritdoc/>
		public override ClangSpecificFileAction Read(BinaryArchiveReader Reader)
		{
			return new ClangSpecificFileAction(Reader);
		}

		/// <inheritdoc/>
		public override void Write(BinaryArchiveWriter Writer, ClangSpecificFileAction Action)
		{
			Action.Write(Writer);
		}
	}

	class ClangSpecificFileActionGraphBuilder : ForwardingActionGraphBuilder
	{
		public ClangSpecificFileActionGraphBuilder(ILogger Logger) : base(new NullActionGraphBuilder(Logger))
		{
		}
		public override void CreateIntermediateTextFile(FileItem Location, IEnumerable<string> ContentLines, bool AllowAsync = true)
		{
			this.ContentLines = ContentLines;
		}

		public IEnumerable<string> ContentLines = Enumerable.Empty<string>();
	}
}
