// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using EpicGames.Core;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	abstract partial class ClangToolChain : ISPCToolChain
	{
		protected virtual void GetLinkArguments_Optimizations(LinkEnvironment linkEnvironment, List<string> arguments)
		{
			bool bLTOEnabled = Options.HasFlag(ClangToolChainOptions.EnableLinkTimeOptimization);
			bool bThinLTOEnabled = Options.HasFlag(ClangToolChainOptions.EnableThinLTO);
			bool bSPGOEnabled = Options.HasFlag(ClangToolChainOptions.SampleBasedPGO);

			// Profile Guided Optimization (PGO)
			// Disable PGOOptimize when no profdata available
			if (linkEnvironment.bPGOOptimize)
			{
				if (linkEnvironment.PGODirectory != null && linkEnvironment.PGOFilenamePrefix != null)
				{
					DirectoryReference PGODir = new(linkEnvironment.PGODirectory);
					FileItem PGOFile = FileItem.GetItemByFileReference(FileReference.Combine(PGODir, linkEnvironment.PGOFilenamePrefix));

					if (!PGOFile.Exists)
					{
						Log.TraceInformationOnce("Missing ProfileData, Profile Guided Optimization Disabled (PGO).");
						linkEnvironment.bPGOOptimize = false;
					}
				}
				else
				{
					Log.TraceInformationOnce("Missing ProfileData, Profile Guided Optimization Disabled (PGO).");
					linkEnvironment.bPGOOptimize = false;
				}
			}

			if (linkEnvironment.bPGOOptimize && linkEnvironment.PGODirectory != null && linkEnvironment.PGOFilenamePrefix != null)
			{
				DirectoryReference PGODir = new(linkEnvironment.PGODirectory);

				if (bSPGOEnabled)
				{
					arguments.Add($"-fprofile-sample-use=\"{NormalizeCommandLinePath(DirectoryReference.Combine(PGODir, linkEnvironment.PGOFilenamePrefix), linkEnvironment.RootPaths)}\"");
				}
				else
				{
					arguments.Add($"-fprofile-use=\"{NormalizeCommandLinePath(DirectoryReference.Combine(PGODir, linkEnvironment.PGOFilenamePrefix), linkEnvironment.RootPaths)}\"");
				}

				//
				// Clang emits a warning for each compiled function that doesn't have a matching entry in the profile data.
				// This can happen when the profile data is older than the binaries we're compiling.
				//
				// Disable this warning. It's far too verbose.
				//
				arguments.Add("-Wno-profile-instr-out-of-date");
				arguments.Add("-Wno-profile-instr-unprofiled");
				arguments.Add("-Wno-backend-plugin");
			}
			else if (linkEnvironment.bPGOProfile)
			{
				arguments.Add("-fprofile-generate");

				// generate minimal debugging information for profiling
				arguments.Add("-gline-tables-only");
			}

			// Link Time Optimization (LTO)
			if (bLTOEnabled)
			{
				if (bThinLTOEnabled)
				{
					arguments.Add("-flto=thin");

					// lld should consider logical cores when determining how many threads to use
					arguments.Add("-Wl,--thinlto-jobs=all");

					DirectoryReference? thinLTOCacheDir = DirectoryReference.FromString(linkEnvironment.ThinLTOCacheDirectory);
					if (thinLTOCacheDir != null)
					{
						arguments.Add($"-Wl,--thinlto-cache-dir=\"{NormalizeCommandLinePath(thinLTOCacheDir, linkEnvironment.RootPaths)}\"");
						string? thinLTOCachePruningArgs = linkEnvironment.ThinLTOCachePruningArguments;
						if (thinLTOCachePruningArgs != null)
						{
							arguments.Add($"-Wl,--thinlto-cache-policy,{thinLTOCachePruningArgs}");
						}
					}
				}
				else
				{
					arguments.Add("-flto");
				}

				AddDisableAutoUpgradeDebugInfoFlags(arguments);
			}
		}

		protected virtual void GetLinkArguments_Sanitizers(LinkEnvironment linkEnvironment, List<string> arguments)
		{
			// ASan
			if (Options.HasFlag(ClangToolChainOptions.EnableAddressSanitizer))
			{
				arguments.Add("-fsanitize=address");
				arguments.Add("-fsanitize-recover=address");
			}

			// TSan
			if (Options.HasFlag(ClangToolChainOptions.EnableThreadSanitizer))
			{
				arguments.Add("-fsanitize=thread");
			}

			// UBSan
			if (Options.HasFlag(ClangToolChainOptions.EnableUndefinedBehaviorSanitizer))
			{
				arguments.Add("-fsanitize=undefined");
			}

			// MSan
			if (Options.HasFlag(ClangToolChainOptions.EnableMemorySanitizer))
			{
				arguments.Add("-fsanitize=memory");
			}

			// LibFuzzer
			if (Options.HasFlag(ClangToolChainOptions.EnableLibFuzzer))
			{
				arguments.Add("-fsanitize=fuzzer");
			}
		}

		protected virtual void GetLinkArguments_WarningsAndErrors(LinkEnvironment linkEnvironment, List<string> arguments)
		{
			// always use absolute paths for errors, this can help IDEs go to the error properly
			arguments.Add("-fdiagnostics-absolute-paths");  // https://clang.llvm.org/docs/ClangCommandLineReference.html#cmdoption-clang-fdiagnostics-absolute-paths

			// https://clang.llvm.org/docs/ClangCommandLineReference.html#cmdoption-cla-fcolor-diagnostics
			if (Log.ColorConsoleOutput)
			{
				arguments.Add("-fdiagnostics-color");
			}

			// https://clang.llvm.org/docs/ClangCommandLineReference.html#cmdoption-clang-fdiagnostics-format
			if (OperatingSystem.IsWindows())
			{
				arguments.Add("-fdiagnostics-format=msvc");
			}

			DirectoryReference? crashDiagnosticDirectory = DirectoryReference.FromString(linkEnvironment.CrashDiagnosticDirectory);
			if (crashDiagnosticDirectory != null)
			{
				if (DirectoryReference.Exists(crashDiagnosticDirectory))
				{
					arguments.Add($"-fcrash-diagnostics-dir=\"{NormalizeCommandLinePath(crashDiagnosticDirectory, linkEnvironment.RootPaths)}\"");
				}
				else
				{
					Log.TraceWarningOnce("CrashDiagnosticDirectory has been specified but directory \"{CrashDiagnosticDirectory}\" does not exist. Linker argument \"-fcrash-diagnostics-dir\" has been discarded.", crashDiagnosticDirectory);
				}
			}
		}

		protected virtual void GetArchiveArguments_Global(LinkEnvironment linkEnvironment, List<string> arguments)
		{
			GetLinkArguments_WarningsAndErrors(linkEnvironment, arguments);
		}

		protected virtual void GetLinkArguments_Global(LinkEnvironment linkEnvironment, List<string> arguments)
		{
			GetLinkArguments_Optimizations(linkEnvironment, arguments);

			GetLinkArguments_Sanitizers(linkEnvironment, arguments);

			GetLinkArguments_WarningsAndErrors(linkEnvironment, arguments);
		}

		protected virtual void AddDisableAutoUpgradeDebugInfoFlags(List<string> Arguments)
		{
			// we build everything with the same toolchain, so we don't want to upgrade debug info because it runs validation and is expensive
			Arguments.Add("-Wl,-mllvm,-disable-auto-upgrade-debug-info");
		}
	}
}
