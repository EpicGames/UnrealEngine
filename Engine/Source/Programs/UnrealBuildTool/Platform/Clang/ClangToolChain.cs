// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text.RegularExpressions;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// Common option flags for the Clang toolchains.
	/// Usage of these flags is currently inconsistent between various toolchains.
	/// </summary>
	[Flags]
	enum ClangToolChainOptions
	{
		/// <summary>
		/// No custom options
		/// </summary>
		None = 0,

		/// <summary>
		/// Enable address sanitizer
		/// </summary>
		EnableAddressSanitizer = 1 << 0,

		/// <summary>
		/// Enable hardware address sanitizer
		/// </summary>
		EnableHWAddressSanitizer = 1 << 1,

		/// <summary>
		/// Enable thread sanitizer
		/// </summary>
		EnableThreadSanitizer = 1 << 2,

		/// <summary>
		/// Enable undefined behavior sanitizer
		/// </summary>
		EnableUndefinedBehaviorSanitizer = 1 << 3,

		/// <summary>
		/// Enable minimal undefined behavior sanitizer
		/// </summary>
		EnableMinimalUndefinedBehaviorSanitizer = 1 << 4,

		/// <summary>
		/// Enable memory sanitizer
		/// </summary>
		EnableMemorySanitizer = 1 << 5,

		/// <summary>
		/// Enable AutoRTFM sanitizer
		/// </summary>
		EnableAutoRTFMSanitizer = 1 << 6,
		
		/// <summary>
		/// Enable Shared library for the Sanitizers otherwise defaults to Statically linked
		/// </summary>
		EnableSharedSanitizer = 1 << 7,

		/// <summary>
		/// Enables link time optimization (LTO). Link times will significantly increase.
		/// </summary>
		EnableLinkTimeOptimization = 1 << 8,

		/// <summary>
		/// Enable thin LTO
		/// </summary>
		EnableThinLTO = 1 << 9,

		/// <summary>
		/// Enable thin LTO
		/// </summary>
		ThinLTODistributed = 1 << 10,

		/// <summary>
		/// Enable tuning of debug info for LLDB
		/// </summary>
		TuneDebugInfoForLLDB = 1 << 11,

		/// <summary>
		/// Whether or not to preserve the portable symbol file produced by dump_syms
		/// </summary>
		PreservePSYM = 1 << 12,

		/// <summary>
		/// (Apple toolchains) Whether we're outputting a dylib instead of an executable
		/// </summary>
		OutputDylib = 1 << 13,

		/// <summary>
		/// Enables the creation of custom symbol files used for runtime symbol resolution.
		/// </summary>
		GenerateSymbols = 1 << 14,

		/// <summary>
		/// Enables dead code/data stripping and common code folding.
		/// </summary>
		EnableDeadStripping = 1 << 15,

		/// <summary>
		/// Indicates that the target is a moduler build i.e. Target.LinkType == TargetLinkType.Modular
		/// </summary>
		ModularBuild = 1 << 16,

		/// <summary>
		/// Disable Dump Syms step for faster iteration
		/// </summary>
		DisableDumpSyms = 1 << 17,

		/// <summary>
		/// Indicates that the AutoRTFM Clang compiler should be used instead of the standard clang compiler
		/// </summary>
		UseAutoRTFMCompiler = 1 << 18,

		/// <summary>
		/// Enable LibFuzzer
		/// </summary>
		EnableLibFuzzer = 1 << 19,

		/// <summary>
		/// Modify code generation to help with debugging optimized builds e.g. by extending lifetimes of local variables.
		/// It may slightly reduce performance. Thus, it's meant to be used during development only.
		/// Supported only on some platforms.
		/// </summary>
		OptimizeForDebugging = 1 << 20,

		/// <summary>
		/// Enables compressing the debug sections if the platform supports this
		/// </summary>
		CompressDebugFile = 1 << 21,

		/// <summary>
		/// Disable Symbol stripping, for use when an external part of the build pipeline requires symbols to be present
		/// and will strip them in a later step
		/// </summary>
		DisableStripSymbols = 1 << 22,

		/// <summary>
		/// If specified along with -PGOProfile or -PGOOptimize, use sample-based PGO instead of instrumented.
		/// </summary>
		SampleBasedPGO = 1 << 23,
	}

	abstract partial class ClangToolChain : ISPCToolChain
	{
		protected partial class ClangToolChainInfo
		{
			protected ILogger Logger { get; init; }
			public DirectoryReference? BasePath { get; init; }
			public FileReference Clang { get; init; }
			public FileReference Archiver { get; init; }

			public Version ClangVersion => LazyClangVersion.Value;
			public string ClangVersionString => LazyClangVersionString.Value;
			public string ArchiverVersionString => LazyArchiverVersionString.Value;
			public HashSet<string> ValidAnalyzerCheckers => LazyValidAnalyzerCheckers.Value;

			readonly Lazy<Version> LazyClangVersion;
			readonly Lazy<string> LazyClangVersionString;
			readonly Lazy<string> LazyArchiverVersionString;
			readonly Lazy<HashSet<string>> LazyValidAnalyzerCheckers;

			/// <summary>
			/// Constructor for ClangToolChainInfo
			/// </summary>
			/// <param name="BasePath">The base path to the clang sdk root, if available</param>
			/// <param name="Clang">The path to the compiler</param>
			/// <param name="Archiver">The path to the archiver</param>
			/// <param name="Logger">Logging interface</param>
			public ClangToolChainInfo(DirectoryReference? BasePath, FileReference Clang, FileReference Archiver, ILogger Logger)
			{
				this.Logger = Logger;
				this.BasePath = BasePath;
				this.Clang = Clang;
				this.Archiver = Archiver;

				LazyClangVersion = new Lazy<Version>(() => QueryClangVersion());
				LazyClangVersionString = new Lazy<string>(() => QueryClangVersionString());
				LazyArchiverVersionString = new Lazy<string>(() => QueryArchiverVersionString());
				LazyValidAnalyzerCheckers = new Lazy<HashSet<string>>(() => ClangWarnings.ValidAnalyzerCheckers(Clang));
			}

			/// <summary>
			/// Lazily query the clang version. Will only be executed once.
			/// </summary>
			/// <returns>The version of clang</returns>
			/// <exception cref="BuildException"></exception>
			protected virtual Version QueryClangVersion()
			{
				Match MatchClangVersion = ClangVersionRegex().Match(ClangVersionString);
				if (MatchClangVersion.Success && Version.TryParse(MatchClangVersion.Groups["full"].Value, out Version? ClangVersion))
				{
					return ClangVersion;
				}
				throw new BuildException($"Failed to query the Clang version number! Got: {ClangVersionString}");
			}

			/// <summary>
			/// Lazily query the clang version output. Will only be executed once.
			/// </summary>
			/// <returns>The standard output when running Clang --version</returns>
			protected virtual string QueryClangVersionString() => Utils.RunLocalProcessAndReturnStdOut(Clang.FullName, "--version", null);

			/// <summary>
			/// Lazily query the archiver version output. Will only be executed once.
			/// </summary>
			/// <returns>The standard output when running Archiver --version</returns>
			protected virtual string QueryArchiverVersionString() => Utils.RunLocalProcessAndReturnStdOut(Archiver.FullName, "--version", null);

			/// <summary>
			/// Regex to parse clang version from console output
			/// </summary>
			/// <returns></returns>
			[GeneratedRegex(@"clang version (?<full>(?<major>\d+)\.(?<minor>\d+)\.(?<patch>\d+))")]
			private static partial Regex ClangVersionRegex();
		}

		// The Clang version being used to compile
		readonly Lazy<ClangToolChainInfo> LazyInfo;
		protected ClangToolChainInfo Info => LazyInfo.Value;

		protected ClangToolChainOptions Options;

		protected bool bOptimizeForDebugging => Options.HasFlag(ClangToolChainOptions.OptimizeForDebugging);

		// Dummy define to work around clang compilation related to the windows maximum path length limitation
		private const string DummyStart = "-D \"DUMMY_DEFINE";
		private const string DummyEnd = "\"";
		protected static readonly string ClangDummyDefine = DummyStart + "_".PadRight(ClangCmdLineMaxSize - ClangCmdlineDangerZone - DummyStart.Length - DummyEnd.Length, 'X') + DummyEnd;
		protected const int ClangCmdLineMaxSize = 32 * 1024;
		protected const int ClangCmdlineDangerZone = 30 * 1024;

		// Target settings
		protected bool ShowIncludes = false;
		protected StaticAnalyzer StaticAnalyzer = StaticAnalyzer.None;
		protected StaticAnalyzerMode StaticAnalyzerMode = StaticAnalyzerMode.Deep;
		protected StaticAnalyzerOutputType StaticAnalyzerOutputType = StaticAnalyzerOutputType.Text;
		protected float CompileActionWeight = 1.0f;

		public ClangToolChain(ClangToolChainOptions InOptions, ILogger InLogger)
			: base(InLogger)
		{
			Options = InOptions;

			LazyInfo = new Lazy<ClangToolChainInfo>(() => { return GetToolChainInfo(); }); // Don't change to => GetToolChainInfo().. it doesnt produce correct code
		}

		public override void GetVersionInfo(List<string> Lines)
		{
			base.GetVersionInfo(Lines);

			Lines.Add($"Using Clang compiler {Info.ClangVersion} ({Info.Clang})");
			Lines.Add($"  {Info.ClangVersionString.ReplaceLineEndings(" ")}");
			Lines.Add($"Using Clang archiver ({Info.Archiver})");
			Lines.Add($"  {Info.ArchiverVersionString.ReplaceLineEndings(" ")}");

			if (TargetRules?.bPGOOptimize == true)
			{
				if (Options.HasFlag(ClangToolChainOptions.SampleBasedPGO))
				{
					Lines.Add("Using Sample-Based PGO (profile guided optimization)");
				}
				else
				{
					Lines.Add("Using PGO (profile guided optimization)");
				}
			}
			else if (TargetRules?.bPGOProfile == true)
			{
				Lines.Add("Using PGI (profile guided instrumentation)");
			}

			if (Options.HasFlag(ClangToolChainOptions.EnableLinkTimeOptimization))
			{
				if (Options.HasFlag(ClangToolChainOptions.EnableThinLTO))
				{
					Logger.LogInformation("Using ThinLTO (link-time optimization)");
				}
				else
				{
					Logger.LogInformation("Using LTO (link-time optimization)");
				}
			}

			if (Options.HasFlag(ClangToolChainOptions.EnableAddressSanitizer) ||
				Options.HasFlag(ClangToolChainOptions.EnableThreadSanitizer) ||
				Options.HasFlag(ClangToolChainOptions.EnableUndefinedBehaviorSanitizer) ||
				Options.HasFlag(ClangToolChainOptions.EnableMemorySanitizer) ||
				Options.HasFlag(ClangToolChainOptions.EnableAutoRTFMSanitizer) ||
				Options.HasFlag(ClangToolChainOptions.EnableLibFuzzer))
			{
				string SanitizerInfo = "Building with:";
				string StaticOrShared = Options.HasFlag(ClangToolChainOptions.EnableSharedSanitizer) ? " dynamically" : " statically";

				SanitizerInfo += Options.HasFlag(ClangToolChainOptions.EnableAddressSanitizer) ? StaticOrShared + " linked AddressSanitizer" : "";
				SanitizerInfo += Options.HasFlag(ClangToolChainOptions.EnableThreadSanitizer) ? StaticOrShared + " linked ThreadSanitizer" : "";
				SanitizerInfo += Options.HasFlag(ClangToolChainOptions.EnableUndefinedBehaviorSanitizer) ? StaticOrShared + " linked UndefinedBehaviorSanitizer" : "";
				SanitizerInfo += Options.HasFlag(ClangToolChainOptions.EnableMemorySanitizer) ? StaticOrShared + " linked MemorySanitizer" : "";
				SanitizerInfo += Options.HasFlag(ClangToolChainOptions.EnableAutoRTFMSanitizer) ? " AutoRTFMSanitizer" : "";
				SanitizerInfo += Options.HasFlag(ClangToolChainOptions.EnableLibFuzzer) ? StaticOrShared + " linked LibFuzzer" : "";

				Lines.Add(SanitizerInfo);
			}
		}

		/// <inheritdoc/>
		public override void SetEnvironmentVariables()
		{
			// Don't allow the INCLUDE environment variables to propagate.
			Environment.SetEnvironmentVariable("C_INCLUDE_PATH", null);
			Environment.SetEnvironmentVariable("OBJC_INCLUDE_PATH", null);
			Environment.SetEnvironmentVariable("CPLUS_INCLUDE_PATH", null);
			Environment.SetEnvironmentVariable("OBJCPLUS_INCLUDE_PATH", null);
		}

		/// <summary>
		/// Create toolchain info. Do not call this function, use "Info" to get results
		/// </summary>
		protected abstract ClangToolChainInfo GetToolChainInfo();

		public override void GetExternalDependencies(HashSet<FileItem> ExternalDependencies)
		{
			base.GetExternalDependencies(ExternalDependencies);
			ExternalDependencies.Add(FileItem.GetItemByFileReference(Info.Clang));
			ExternalDependencies.Add(FileItem.GetItemByFileReference(Info.Archiver));
		}

		public override FileReference? GetCppCompilerPath()
		{
			return LazyInfo.Value.Clang;
		}

		public override void SetUpGlobalEnvironment(ReadOnlyTargetRules Target, CppCompileEnvironment GlobalCompileEnvironment, LinkEnvironment GlobalLinkEnvironment)
		{
			base.SetUpGlobalEnvironment(Target, GlobalCompileEnvironment, GlobalLinkEnvironment);

			if (Target.bAllowLTCG)
			{
				Options |= ClangToolChainOptions.EnableLinkTimeOptimization;
				if (Target.bPreferThinLTO)
				{
					Options |= ClangToolChainOptions.EnableThinLTO;
				}
			}

			if (LazyInfo.Value.BasePath != null)
			{
				GlobalCompileEnvironment.RootPaths.AddFolderName(CppRootPathFolder.PlatformSDK, $"{Target.Platform}SDK");
				GlobalLinkEnvironment.RootPaths.AddFolderName(CppRootPathFolder.PlatformSDK, $"{Target.Platform}SDK");
				GlobalCompileEnvironment.RootPaths[CppRootPathFolder.PlatformSDK] = LazyInfo.Value.BasePath;
				GlobalLinkEnvironment.RootPaths[CppRootPathFolder.PlatformSDK] = LazyInfo.Value.BasePath;

				string sdkVersion = UEBuildPlatformSDK.GetSDKForPlatform(Target.Platform.ToString())?.GetInstalledVersion() ?? "0";
				string pathHash = $"{Target.Platform}-{sdkVersion}-{LazyInfo.Value.ClangVersion}";
				FileReference versionFile = FileReference.Combine(LazyInfo.Value.BasePath, "..", "Version.txt"); // Optional AutoSDK version, may not exist
				if (FileReference.Exists(versionFile))
				{
					string contents = FileReference.ReadAllText(versionFile).ReplaceLineEndings("-").Trim();
					pathHash = $"{pathHash}-{contents}";
				}
				EpicGames.UBA.Utils.RegisterPathHash(LazyInfo.Value.BasePath, pathHash);
			}

			ShowIncludes = Target.bShowIncludes;
			StaticAnalyzer = Target.StaticAnalyzer;
			StaticAnalyzerMode = Target.StaticAnalyzerMode;
			StaticAnalyzerOutputType = Target.StaticAnalyzerOutputType;
			CompileActionWeight = Target.ClangCompileActionWeight;
		}

		public override void FinalizeOutput(ReadOnlyTargetRules Target, TargetMakefileBuilder MakefileBuilder, DirectoryReference ExeDir)
		{
			if (Target.bPrintToolChainTimingInfo && Target.bParseTimingInfoForTracing)
			{
				TargetMakefile Makefile = MakefileBuilder.Makefile;
				List<IExternalAction> CompileActions = [.. Makefile.Actions.Where(x => x.ActionType == ActionType.Compile && x.ProducedItems.Any(i => i.HasExtension(".json")))];
				List<FileItem> TimingJsonFiles = [.. CompileActions.SelectMany(a => a.ProducedItems.Where(i => i.HasExtension(".json")))];

				// Handing generating aggregate timing information if we compiled more than one file.
				if (TimingJsonFiles.Count > 0)
				{
					// Generate the file manifest for the aggregator.
					FileReference ManifestFile = FileReference.Combine(Makefile.ProjectIntermediateDirectory, $"{Target.Name}TimingManifest.csv");
					if (!DirectoryReference.Exists(ManifestFile.Directory))
					{
						DirectoryReference.CreateDirectory(ManifestFile.Directory);
					}
					File.WriteAllLines(ManifestFile.FullName, TimingJsonFiles.Select(f => f.FullName[..^".json".Length]));

					FileItem AggregateOutputFile = FileItem.GetItemByFileReference(FileReference.Combine(Makefile.ProjectIntermediateDirectory, $"{Target.Name}.trace.csv"));
					FileItem HeadersOutputFile = FileItem.GetItemByFileReference(FileReference.Combine(Makefile.ProjectIntermediateDirectory, $"{Target.Name}.headers.csv"));
					List<string> AggregateActionArgs =
					[
						$"-ManifestFile={ManifestFile.FullName}",
						$"-AggregateFile={AggregateOutputFile.FullName}",
						$"-HeadersFile={HeadersOutputFile.FullName}",
					];

					Action AggregateTimingInfoAction = MakefileBuilder.CreateRecursiveAction<AggregateClangTimingInfo>(ActionType.ParseTimingInfo, String.Join(" ", AggregateActionArgs));
					AggregateTimingInfoAction.StatusDescription = $"Aggregating {TimingJsonFiles.Count} Timing File(s)";
					AggregateTimingInfoAction.PrerequisiteItems.UnionWith(TimingJsonFiles);

					AggregateTimingInfoAction.ProducedItems.Add(AggregateOutputFile);
					AggregateTimingInfoAction.ProducedItems.Add(HeadersOutputFile);
					Makefile.OutputItems.AddRange(AggregateTimingInfoAction.ProducedItems);

					FileItem ArchiveOutputFile = FileItem.GetItemByFileReference(FileReference.Combine(Makefile.ProjectIntermediateDirectory, $"{Target.Name}.traces.zip"));
					List<string> ArchiveActionArgs =
					[
						$"-ManifestFile={ManifestFile.FullName}",
						$"-ArchiveFile={ArchiveOutputFile.FullName}",
					];

					Action ArchiveTimingInfoAction = MakefileBuilder.CreateRecursiveAction<AggregateClangTimingInfo>(ActionType.ParseTimingInfo, String.Join(" ", ArchiveActionArgs));
					ArchiveTimingInfoAction.StatusDescription = $"Archiving {TimingJsonFiles.Count} Timing File(s)";
					ArchiveTimingInfoAction.PrerequisiteItems.UnionWith(TimingJsonFiles);

					ArchiveTimingInfoAction.ProducedItems.Add(ArchiveOutputFile);
					Makefile.OutputItems.AddRange(ArchiveTimingInfoAction.ProducedItems);

					// Extract CompileScore data from traces
					FileReference ScoreDataExtractor = FileReference.Combine(Unreal.RootDirectory, "Engine", "Extras", "ThirdPartyNotUE", "CompileScore", "ScoreDataExtractor.exe");
					if (OperatingSystem.IsWindows() && FileReference.Exists(ScoreDataExtractor))
					{
						FileItem CompileScoreOutput = FileItem.GetItemByFileReference(FileReference.Combine(Makefile.ProjectIntermediateDirectory, $"{Target.Name}.scor"));

						Action CompileScoreExtractorAction = MakefileBuilder.CreateAction(ActionType.ParseTimingInfo);
						CompileScoreExtractorAction.WorkingDirectory = Unreal.EngineSourceDirectory;
						CompileScoreExtractorAction.StatusDescription = $"Extracting CompileScore";
						CompileScoreExtractorAction.bCanExecuteRemotely = false;
						CompileScoreExtractorAction.bCanExecuteRemotelyWithSNDBS = false;
						CompileScoreExtractorAction.bCanExecuteInUBA = false; // TODO: Unknown if supported
						CompileScoreExtractorAction.PrerequisiteItems.UnionWith(TimingJsonFiles);
						CompileScoreExtractorAction.CommandPath = ScoreDataExtractor;
						CompileScoreExtractorAction.CommandArguments = $"-clang -verbosity 0 -timelinepack 1000000 -extract -i \"{NormalizeCommandLinePath(Makefile.ProjectIntermediateDirectory)}\" -o \"{NormalizeCommandLinePath(CompileScoreOutput)}\"";

						CompileScoreExtractorAction.ProducedItems.Add(CompileScoreOutput);
						CompileScoreExtractorAction.ProducedItems.Add(FileItem.GetItemByFileReference(FileReference.Combine(Makefile.ProjectIntermediateDirectory, $"{Target.Name}.scor.gbl")));
						CompileScoreExtractorAction.ProducedItems.Add(FileItem.GetItemByFileReference(FileReference.Combine(Makefile.ProjectIntermediateDirectory, $"{Target.Name}.scor.incl")));
						CompileScoreExtractorAction.ProducedItems.Add(FileItem.GetItemByFileReference(FileReference.Combine(Makefile.ProjectIntermediateDirectory, $"{Target.Name}.scor.t0000")));
						Makefile.OutputItems.AddRange(CompileScoreExtractorAction.ProducedItems);
					}
				}
			}
		}

		/// <summary>
		/// Sanitizes a preprocessor definition argument if needed.
		/// </summary>
		/// <param name="Definition">A string in the format "foo=bar" or "foo".</param>
		/// <returns>An escaped string</returns>
		protected virtual string EscapePreprocessorDefinition(string Definition)
		{
			// By default don't modify preprocessor definition, handle in platform overrides.
			return Definition;
		}

		/// <summary>
		/// Checks if compiler version matches the requirements
		/// </summary>
		protected bool CompilerVersionGreaterOrEqual(int Major, int Minor, int Patch) => Info.ClangVersion >= new Version(Major, Minor, Patch);

		/// <summary>
		/// Checks if compiler version matches the requirements
		/// </summary>
		protected bool CompilerVersionLessThan(int Major, int Minor, int Patch) => Info.ClangVersion < new Version(Major, Minor, Patch);

		protected static bool IsPreprocessing(CppCompileEnvironment CompileEnvironment) =>
				CompileEnvironment.PrecompiledHeaderAction != PrecompiledHeaderAction.Create
			&& CompileEnvironment.bPreprocessOnly;

		protected bool IsAnalyzing(CppCompileEnvironment CompileEnvironment) =>
				StaticAnalyzer == StaticAnalyzer.Default
			&& CompileEnvironment.PrecompiledHeaderAction != PrecompiledHeaderAction.Create
			&& !CompileEnvironment.bDisableStaticAnalysis;

		protected bool ShouldSkipCompile(CppCompileEnvironment CompileEnvironment) =>
				StaticAnalyzer == StaticAnalyzer.Default
			&& CompileEnvironment.PrecompiledHeaderAction != PrecompiledHeaderAction.Create
			&& CompileEnvironment.bDisableStaticAnalysis;

		protected virtual void GetCppStandardCompileArgument(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			// https://clang.llvm.org/cxx_status.html
			switch (CompileEnvironment.CppStandard)
			{
				case CppStandardVersion.Cpp20:
					Arguments.Add("-std=c++20");
					break;
				case CppStandardVersion.Cpp23:
					Arguments.Add("-std=c++23");
					break;
				case CppStandardVersion.Latest:
					Arguments.Add("-std=c++2c");
					break;
				default:
					throw new BuildException($"Unsupported C++ standard type set: {CompileEnvironment.CppStandard}");
			}

			if (CompileEnvironment.bEnableCoroutines)
			{
				if (!CompileEnvironment.bEnableExceptions)
				{
					Arguments.Add("-Wno-coroutine-missing-unhandled-exception");
				}
			}

			if (CompileEnvironment.PrecompiledHeaderAction != PrecompiledHeaderAction.None && CompilerVersionGreaterOrEqual(11, 0, 0))
			{
				// Validate PCH inputs by content if mtime check fails
				Arguments.Add("-Xclang -fno-pch-timestamp"); // This is needed to prevent check on timestamp stored inside pch
				Arguments.Add("-fpch-validate-input-files-content");
			}

			if (CompileEnvironment.bEnableAutoRTFMInstrumentation)
			{
				Arguments.Add("-fautortfm");

				if (CompileEnvironment.bEnableAutoRTFMVerification)
				{
					Arguments.Add("-fautortfm-verify");
				}
			}

			if (CompileEnvironment.bAutoRTFMVerify)
			{
				Arguments.Add("-Xclang -mllvm -Xclang -autortfm-verify");
			}

			if (CompileEnvironment.bAutoRTFMClosedStaticLinkage)
			{
				Arguments.Add("-Xclang -mllvm -Xclang -autortfm-closed-static-linkage");
			}

			if (CompileEnvironment.bAutoRTFMRedirectionInstrumentation)
			{
				Arguments.Add("-Xclang -mllvm -Xclang -autortfm-enable-redirection-instrumentation");
			}
		}

		protected virtual void GetCStandardCompileArgument(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			switch (CompileEnvironment.CStandard)
			{
				case CStandardVersion.None:
					break;
				case CStandardVersion.C89:
					Arguments.Add("-std=c89");
					break;
				case CStandardVersion.C99:
					Arguments.Add("-std=c99");
					break;
				case CStandardVersion.C11:
					Arguments.Add("-std=c11");
					break;
				case CStandardVersion.C17:
					Arguments.Add("-std=c17");
					break;
				case CStandardVersion.Latest:
					Arguments.Add("-std=c2x");
					break;
				default:
					throw new BuildException($"Unsupported C standard type set: {CompileEnvironment.CStandard}");
			}

			if (CompileEnvironment.bEnableAutoRTFMInstrumentation)
			{
				Arguments.Add("-fautortfm");

				if (CompileEnvironment.bEnableAutoRTFMVerification)
				{
					Arguments.Add("-fautortfm-verify");
				}
			}
		}

		protected virtual void GetCompileArguments_H(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			GetCompileArguments_CPP(CompileEnvironment, Arguments);
			ClangWarnings.GetHeaderDisabledWarnings(Arguments);
		}

		protected virtual void GetCompileArguments_CPP(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			Arguments.Add("-x c++");
			GetCppStandardCompileArgument(CompileEnvironment, Arguments);
		}

		protected virtual void GetCompileArguments_C(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			Arguments.Add("-x c");
			GetCStandardCompileArgument(CompileEnvironment, Arguments);
		}

		protected virtual void GetCompileArguments_MM(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			Arguments.Add("-x objective-c++");
			GetCppStandardCompileArgument(CompileEnvironment, Arguments);
		}

		protected virtual void GetCompileArguments_M(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			Arguments.Add("-x objective-c");
			GetCStandardCompileArgument(CompileEnvironment, Arguments);
		}

		protected virtual void GetCompileArguments_PCH(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			Arguments.Add("-x c++-header");
			if (CompilerVersionGreaterOrEqual(11, 0, 0))
			{
				Arguments.Add("-fpch-instantiate-templates");
			}
			GetCppStandardCompileArgument(CompileEnvironment, Arguments);
		}

		// Conditionally enable (default disabled) generation of information about every class with virtual functions for use by the C++ runtime type identification features
		// (`dynamic_cast' and `typeid'). If you don't use those parts of the language, you can save some space by using -fno-rtti.
		// Note that exception handling uses the same information, but it will generate it as needed.
		protected virtual string GetRTTIFlag(CppCompileEnvironment CompileEnvironment)
		{
			return CompileEnvironment.bUseRTTI ? "-frtti" : "-fno-rtti";
		}

		protected virtual string GetUserIncludePathArgument(DirectoryReference IncludePath, CppCompileEnvironment CompileEnvironment)
		{
			return $"-I\"{NormalizeCommandLinePath(IncludePath, CompileEnvironment.RootPaths)}\"";
		}

		protected virtual string GetSystemIncludePathArgument(DirectoryReference IncludePath, CppCompileEnvironment CompileEnvironment)
		{
			return $"-isystem\"{NormalizeCommandLinePath(IncludePath, CompileEnvironment.RootPaths)}\"";
		}

		protected virtual DirectoryReference GetResourceDir(CppCompileEnvironment CompileEnvironment)
		{
			return DirectoryReference.Combine(Info.BasePath!, "lib", "clang", Info.ClangVersion.Major.ToString());
		}

		protected virtual void GetCompileArguments_IncludePaths(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
            Arguments.AddRange(CompileEnvironment.UserIncludePaths.Select(IncludePath => GetUserIncludePathArgument(IncludePath, CompileEnvironment)));
			Arguments.AddRange(CompileEnvironment.SystemIncludePaths.Select(IncludePath => GetSystemIncludePathArgument(IncludePath, CompileEnvironment)));

			if (ShowIncludes)
			{
				Arguments.Add("-H");
			}
		}

		protected virtual string GetPreprocessorDefinitionArgument(string Definition)
		{
			return $"-D\"{EscapePreprocessorDefinition(Definition)}\"";
		}

		protected virtual void GetCompileArguments_PreprocessorDefinitions(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			Arguments.AddRange(CompileEnvironment.Definitions.Select(Definition => GetPreprocessorDefinitionArgument(Definition)));

			if (CompileEnvironment.PrecompiledHeaderAction == PrecompiledHeaderAction.Create && StaticAnalyzer == StaticAnalyzer.Default)
			{
				Arguments.Add(GetPreprocessorDefinitionArgument("__clang_analyzer__"));
			}
		}

		protected virtual string GetForceIncludeFileArgument(FileReference ForceIncludeFile, CppCompileEnvironment CompileEnvironment)
		{
			return $"-include \"{NormalizeCommandLinePath(ForceIncludeFile, CompileEnvironment.RootPaths)}\"";
		}

		protected virtual string GetForceIncludeFileArgument(FileItem ForceIncludeFile, CppCompileEnvironment CompileEnvironment)
		{
			return GetForceIncludeFileArgument(ForceIncludeFile.Location, CompileEnvironment);
		}

		protected virtual string GetIncludePCHFileArgument(FileReference IncludePCHFile, CppCompileEnvironment CompileEnvironment)
		{
			return $"-include-pch \"{NormalizeCommandLinePath(IncludePCHFile, CompileEnvironment.RootPaths)}\"";
		}

		protected virtual string GetIncludePCHFileArgument(FileItem IncludePCHFile, CppCompileEnvironment CompileEnvironment)
		{
			return GetIncludePCHFileArgument(IncludePCHFile.Location, CompileEnvironment);
		}

		protected virtual void GetCompileArguments_ForceInclude(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			// Add the precompiled header file's path to the force include path
			// This needs to be before the other force include paths to ensure clang uses it instead of the source header file.
			if (CompileEnvironment.PrecompiledHeaderAction == PrecompiledHeaderAction.Include)
			{
				Arguments.Add(GetIncludePCHFileArgument(CompileEnvironment.PrecompiledHeaderFile!, CompileEnvironment));
			}
			else if (CompileEnvironment.PrecompiledHeaderAction == PrecompiledHeaderAction.Create && CompileEnvironment.ParentPCHInstance != null)
			{
				Arguments.Add(GetIncludePCHFileArgument(CompileEnvironment.ParentPrecompiledHeaderFile!, CompileEnvironment));
			}

			Arguments.AddRange(CompileEnvironment.ForceIncludeFiles.Select(ForceIncludeFile => GetForceIncludeFileArgument(ForceIncludeFile, CompileEnvironment)));
		}

		protected virtual string GetSourceFileArgument(FileItem SourceFile, CppCompileEnvironment CompileEnvironment)
		{
			return $"\"{NormalizeCommandLinePath(SourceFile, CompileEnvironment.RootPaths)}\"";
		}

		protected virtual string GetOutputFileArgument(FileItem OutputFile, CppRootPaths RootPaths)
		{
			return $"-o \"{NormalizeCommandLinePath(OutputFile, RootPaths)}\"";
		}

        protected virtual string GetOutputFileArgument(FileItem OutputFile, CppCompileEnvironment CompileEnvironment)
        {
            return GetOutputFileArgument(OutputFile, CompileEnvironment.RootPaths);
        }

        protected virtual string GetDepencenciesListFileArgument(FileItem DependencyListFile, CppCompileEnvironment CompileEnvironment)
		{
			return $"-MD -MF\"{NormalizeCommandLinePath(DependencyListFile, CompileEnvironment.RootPaths)}\"";
		}

		protected virtual string GetPreprocessDepencenciesListFileArgument(FileItem DependencyListFile, CppCompileEnvironment CompileEnvironment)
		{
			return $"-M -MF\"{NormalizeCommandLinePath(DependencyListFile, CompileEnvironment.RootPaths)}\"";
		}

		protected virtual string GetResponseFileArgument(FileItem ResponseFile, CppRootPaths RootPaths)
		{
			return $"@\"{NormalizeCommandLinePath(ResponseFile, RootPaths)}\"";
		}

		/// <summary>
		/// Common compile arguments that control which warnings are enabled.
		/// https://clang.llvm.org/docs/DiagnosticsReference.html
		/// </summary>
		/// <param name="CompileEnvironment"></param>
		/// <param name="Arguments"></param>
		protected virtual void GetCompileArguments_WarningsAndErrors(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			Arguments.Add("-Wall");                                     // https://clang.llvm.org/docs/DiagnosticsReference.html#wall
			Arguments.Add("-Werror");                                   // https://clang.llvm.org/docs/UsersManual.html#cmdoption-werror
			Arguments.Add("-ftabstop=4");                               // Clang defaults to -ftabstop=8 which can cause misleading indentation warnings if tabs and spaces are mixed

			VersionNumber clangVersionNumber = new(Info.ClangVersion.Major, Info.ClangVersion.Minor, Info.ClangVersion.Build);
			ClangWarnings.GetWarnings(CompileEnvironment, clangVersionNumber, Arguments);

			// always use absolute paths for errors, this can help IDEs go to the error properly
			Arguments.Add("-fdiagnostics-absolute-paths");  // https://clang.llvm.org/docs/ClangCommandLineReference.html#cmdoption-clang-fdiagnostics-absolute-paths

			// https://clang.llvm.org/docs/ClangCommandLineReference.html#cmdoption-cla-fcolor-diagnostics
			if (Log.ColorConsoleOutput)
			{
				Arguments.Add("-fdiagnostics-color");
			}

			// https://clang.llvm.org/docs/ClangCommandLineReference.html#cmdoption-clang-fdiagnostics-format
			if (OperatingSystem.IsWindows())
			{
				Arguments.Add("-fdiagnostics-format=msvc");
			}

			// Set the output directory for crashes
			// https://clang.llvm.org/docs/ClangCommandLineReference.html#cmdoption-clang-fcrash-diagnostics-dir
			DirectoryReference? CrashDiagnosticDirectory = DirectoryReference.FromString(CompileEnvironment.CrashDiagnosticDirectory);
			if (CrashDiagnosticDirectory != null)
			{
				if (DirectoryReference.Exists(CrashDiagnosticDirectory))
				{
					Arguments.Add($"-fcrash-diagnostics-dir=\"{NormalizeCommandLinePath(CrashDiagnosticDirectory, CompileEnvironment.RootPaths)}\"");
				}
				else
				{
					Log.TraceWarningOnce("CrashDiagnosticDirectory has been specified but directory \"{CrashDiagnosticDirectory}\" does not exist. Compiler argument \"-fcrash-diagnostics-dir\" has been discarded.", CrashDiagnosticDirectory);
				}
			}
		}

		/// <summary>
		/// Compile arguments for FP semantics
		/// </summary>
		/// <param name="CompileEnvironment"></param>
		/// <param name="Arguments"></param>
		protected virtual void GetCompileArguments_FPSemantics(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			switch (CompileEnvironment.FPSemantics)
			{
				case FPSemanticsMode.Default: // Default to precise FP semantics.
				case FPSemanticsMode.Precise:
					// Clang defaults to -ffp-contract=on, which allows fusing multiplications and additions into FMAs.
					Arguments.Add("-ffp-contract=off");
					break;
				case FPSemanticsMode.Imprecise:
					Arguments.Add("-ffast-math");
					Arguments.Add("-fhonor-infinities");
					if (Info.ClangVersion.Major >= 19)
					{
						Arguments.Add("-fhonor-nans");
					}
					Arguments.Add("-fno-reciprocal-math");
					break;
				default:
					throw new BuildException($"Unsupported FP semantics: {CompileEnvironment.FPSemantics}");
			}
		}

		/// <summary>
		/// Compile arguments for optimization settings, such as profile guided optimization and link time optimization
		/// </summary>
		/// <param name="CompileEnvironment"></param>
		/// <param name="Arguments"></param>
		protected virtual void GetCompileArguments_Optimizations(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			bool bLTOEnabled = Options.HasFlag(ClangToolChainOptions.EnableLinkTimeOptimization);
			bool bSPGOEnabled = Options.HasFlag(ClangToolChainOptions.SampleBasedPGO);

			// Link Time Optimization (LTO)
			if (bLTOEnabled)
			{
				if (Options.HasFlag(ClangToolChainOptions.EnableThinLTO))
				{
					Log.TraceInformationOnce("Enabling Thin Link Time Optimization (ThinLTO). Linking may take a while.");
				}
				else
				{
					Log.TraceInformationOnce("Enabling Full Link Time Optimization (LTO). Linking will take a long time.");
				}

				Arguments.Add(Options.HasFlag(ClangToolChainOptions.EnableThinLTO) ? "-flto=thin" : "-flto");
			}

			// Profile Guided Optimization (PGO) and Link Time Optimization (LTO)
			// Disable PGOOptimize when no profdata available
			if (CompileEnvironment.bPGOOptimize)
			{
				CompileEnvironment.bPGOOptimize = false;

				if (CompileEnvironment.PGODirectory != null && CompileEnvironment.PGOFilenamePrefix != null)
				{
					DirectoryReference? PGODir = DirectoryReference.FromString(CompileEnvironment.PGODirectory!);
					if (PGODir != null)
					{
						FileItem PGOFile = FileItem.GetItemByFileReference(FileReference.Combine(PGODir, CompileEnvironment.PGOFilenamePrefix));

						if (PGOFile.Exists)
						{
							CompileEnvironment.bPGOOptimize = true;
						}
					}
				}
				
				if (!CompileEnvironment.bPGOOptimize)
				{
					Log.TraceInformationOnce("Missing ProfileData, Profile Guided Optimization Disabled (PGO).");
				}
			}

			if (CompileEnvironment.bPGOOptimize)
			{
				if (bSPGOEnabled)
				{
					Log.TraceInformationOnce("Enabling Sampling Based Profile Guided Optimization (SPGO). Linking will take a while.");
					DirectoryReference? PGODir = DirectoryReference.FromString(CompileEnvironment.PGODirectory!);
					Arguments.Add($"-fprofile-sample-use=\"{NormalizeCommandLinePath(DirectoryReference.Combine(PGODir!, CompileEnvironment.PGOFilenamePrefix!), CompileEnvironment.RootPaths)}\"");
				}
				else
				{
					Log.TraceInformationOnce("Enabling Profile Guided Optimization (PGO). Linking will take a while.");
					DirectoryReference? PGODir = DirectoryReference.FromString(CompileEnvironment.PGODirectory!);
					Arguments.Add($"-fprofile-use=\"{NormalizeCommandLinePath(DirectoryReference.Combine(PGODir!, CompileEnvironment.PGOFilenamePrefix!), CompileEnvironment.RootPaths)}\"");
				}
			}
			else if (CompileEnvironment.bPGOProfile)
			{
				if (bSPGOEnabled)
				{
					Log.TraceInformationOnce("Enabling Sample Based Profile Guided Instrumentation (SPGI). Linking will take a while.");
					Arguments.Add("-gdwarf");
					Arguments.Add("-gsplit-dwarf");
					Arguments.Add("-gline-tables-only");
					Arguments.Add("-fdebug-info-for-profiling");
					Arguments.Add("-funique-internal-linkage-names");
				}
				else
				{
					Log.TraceInformationOnce("Enabling Profile Guided Instrumentation (PGI). Linking will take a while.");
					Arguments.Add("-fprofile-generate");
				}
			}

			if (!CompileEnvironment.bUseInlining)
			{
				Arguments.Add("-fno-inline-functions");
			}

			// We have 'this' vs nullptr comparisons that get optimized away for newer versions of Clang, which is undesirable until we refactor these checks.
			Arguments.Add("-fno-delete-null-pointer-checks");
		}

		/// <summary>
		/// Compile arguments for architecture specific settings
		/// </summary>
		/// <param name="CompileEnvironment"></param>
		/// <param name="Arguments"></param>
		protected virtual void GetCompileArguments_Architecture(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			// architecture (all but None are AVX)
			if (CompileEnvironment.Architecture == UnrealArch.X64 && CompileEnvironment.MinCpuArchX64 != MinimumCpuArchitectureX64.None)
			{
				// The binary created will be targeting AVX instructions. Machines without AVX support will crash on any AVX instructions if they run this compilation unit.
				MinimumCpuArchitectureX64 MinCpuArchX64 = CompileEnvironment.MinCpuArchX64;
				VersionNumber ClangVersion = new(Info.ClangVersion.Major, Info.ClangVersion.Minor, Info.ClangVersion.Build);
				// Downgrade AVX if unsupported by Clang version
				if (ClangVersion < new VersionNumber(20, 1) && MinCpuArchX64 >= MinimumCpuArchitectureX64.AVX10_2)
				{
					MinCpuArchX64 = MinimumCpuArchitectureX64.AVX10_1;
				}
				if (ClangVersion < new VersionNumber(18, 1) && MinCpuArchX64 >= MinimumCpuArchitectureX64.AVX10_1)
				{
					MinCpuArchX64 = MinimumCpuArchitectureX64.AVX512;
				}

				// AVX available implies sse4 and sse2 available.
				// Inform Unreal code that we have sse2, sse4, and AVX, both available to compile and available to run
				// By setting the ALWAYS_HAS defines, we we direct Unreal code to skip cpuid checks to verify that the running hardware supports sse/avx.
				Arguments.Add("-DPLATFORM_ENABLE_VECTORINTRINSICS=1");

				if (MinCpuArchX64 >= MinimumCpuArchitectureX64.AVX)
				{
					// Apparently MSVC enables (a subset?) of BMI (bit manipulation instructions) when /arch:AVX is set. Some code relies on this, so mirror it by enabling BMI1
					Arguments.Add("-mavx");
					Arguments.Add("-mbmi");
					// Inform Unreal code that we have sse2, sse4, and AVX, both available to compile and available to run
					Arguments.Add("-DPLATFORM_MAYBE_HAS_AVX=1");
					// By setting the ALWAYS_HAS defines, we we direct Unreal code to skip cpuid checks to verify that the running hardware supports sse/avx.
					Arguments.Add("-DPLATFORM_ALWAYS_HAS_AVX=1");
				}

				if (MinCpuArchX64 >= MinimumCpuArchitectureX64.AVX2)
				{
					Arguments.Add("-mavx2");
					Arguments.Add("-mfma"); // Matching MSVC which includes FMA with /arch:AVX2
					Arguments.Add("-DPLATFORM_ALWAYS_HAS_AVX_2=1");
				}

				if (MinCpuArchX64 >= MinimumCpuArchitectureX64.AVX512)
				{
					// Match MSVC which says (https://learn.microsoft.com/en-us/cpp/build/reference/arch-x64?view=msvc-170):
					// > The __AVX512F__, __AVX512CD__, __AVX512BW__, __AVX512DQ__ and __AVX512VL__ preprocessor symbols are defined when the /arch:AVX512 compiler option is specified
					if (MinCpuArchX64 < MinimumCpuArchitectureX64.AVX10_1)
					{
						Arguments.Add("-mavx512f");
						Arguments.Add("-mavx512cd");
						Arguments.Add("-mavx512bw");
						Arguments.Add("-mavx512dq");
						Arguments.Add("-mavx512vl");
					}

					Arguments.Add("-DPLATFORM_ALWAYS_HAS_AVX_512=1");
				}

				if (MinCpuArchX64 >= MinimumCpuArchitectureX64.AVX10_1)
				{
					Arguments.Add("-DPLATFORM_ALWAYS_HAS_AVX_10_1=1");
					Arguments.Add("-mavx10.1");
				}

				if (MinCpuArchX64 >= MinimumCpuArchitectureX64.AVX10_2)
				{
					Arguments.Add("-DPLATFORM_ALWAYS_HAS_AVX_10_2=1");
					Arguments.Add("-mavx10.2");
				}
			}

			// Compile with support for APX on X64 architectures
			if (CompileEnvironment.Architecture == UnrealArch.X64 && CompileEnvironment.bMinCpuArchAPX)
			{
				Arguments.Add("-mapxf");
			}

			// Treat chars as signed on arm64
			if (CompileEnvironment.Architecture == UnrealArch.Arm64)
			{
				Arguments.Add("-fsigned-char");

				// For the Graviton targets, desired optional features are enabled by default,
				// with the exception of crypto, which may be disabled by default due to export
				// restrictions.
				switch (CompileEnvironment.MinArm64CpuTarget)
				{
					case Arm64TargetCpuType.Graviton2:
						Arguments.Add("-march=armv8.2-a+fp16+rcpc+dotprod+crypto");
						Arguments.Add("-mtune=neoverse-n1");
						break;

					case Arm64TargetCpuType.GravitonSVE:
						Arguments.Add("-mcpu=neoverse-512tvb+crypto");
						break;

					case Arm64TargetCpuType.Graviton3:
						Arguments.Add("-mcpu=neoverse-512tvb+bf16+rng+crypto");
						Arguments.Add("-mtune=neoverse-v1");
						break;

					case Arm64TargetCpuType.Graviton4:
						Arguments.Add("-mcpu=neoverse-v2+crypto");
						break;
				}
			}
		}

		/// <summary>
		/// Compile arguments for debug settings
		/// </summary>
		/// <param name="CompileEnvironment"></param>
		/// <param name="Arguments"></param>
		protected virtual void GetCompileArguments_Debugging(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			// Optionally enable exception handling (off by default since it generates extra code needed to propagate exceptions)
			if (CompileEnvironment.bEnableExceptions)
			{
				Arguments.Add("-fexceptions");
				Arguments.Add("-DPLATFORM_EXCEPTIONS_DISABLED=0");
			}
			else
			{
				Arguments.Add("-fno-exceptions");
				Arguments.Add("-DPLATFORM_EXCEPTIONS_DISABLED=1");
			}

			if (bOptimizeForDebugging)
			{
				Arguments.Add("-fextend-lifetimes");
			}

			if (CompileEnvironment.bCreateDebugInfo)
			{
				if (CompileEnvironment.bDebugNoInlineLineTables)
				{
					Arguments.Add("-gno-inline-line-tables");
				}

				if (CompileEnvironment.bDebugSimpleTemplateNames)
				{
					Arguments.Add("-gsimple-template-names");
				}
			}
		}

		/// <summary>
		/// Compile arguments for sanitizers
		/// </summary>
		/// <param name="CompileEnvironment"></param>
		/// <param name="Arguments"></param>
		protected virtual void GetCompilerArguments_Sanitizers(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			// ASan
			if (Options.HasFlag(ClangToolChainOptions.EnableAddressSanitizer))
			{
				Arguments.Add("-fsanitize=address");
				Arguments.Add("-fsanitize-recover=address");
			}

			// TSan
			if (Options.HasFlag(ClangToolChainOptions.EnableThreadSanitizer))
			{
				Arguments.Add("-fsanitize=thread");
			}

			// UBSan
			if (Options.HasFlag(ClangToolChainOptions.EnableUndefinedBehaviorSanitizer))
			{
				Arguments.Add("-fsanitize=undefined");
			}

			// MSan
			if (Options.HasFlag(ClangToolChainOptions.EnableMemorySanitizer))
			{
				Arguments.Add("-fsanitize=memory");
			}

			// AutoRTFMSan
			if (Options.HasFlag(ClangToolChainOptions.EnableAutoRTFMSanitizer) &&
				CompileEnvironment.bUseAutoRTFMCompiler)
			{
				Arguments.Add("-fsanitize=autortfm");
			}

			// LibFuzzer
			if (Options.HasFlag(ClangToolChainOptions.EnableLibFuzzer))
			{
				Arguments.Add("-fsanitize=fuzzer");
			}
		}

		/// <summary>
		/// Additional compile arguments.
		/// </summary>
		/// <param name="CompileEnvironment"></param>
		/// <param name="Arguments"></param>
		protected virtual void GetCompileArguments_AdditionalArgs(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			if (!String.IsNullOrWhiteSpace(CompileEnvironment.AdditionalArguments))
			{
				Arguments.Add(CompileEnvironment.AdditionalArguments);
			}
		}

		/// <summary>
		/// Compile arguments for running clang-analyze.
		/// </summary>
		/// <param name="CompileEnvironment"></param>
		/// <param name="Arguments"></param>
		protected virtual void GetCompileArguments_Analyze(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			Arguments.Add("-Wno-unused-command-line-argument");

			// Enable the static analyzer with default checks.
			Arguments.Add("--analyze");

			// Write out a pretty web page with navigation to understand how the analysis was derived if HTML is enabled.
#pragma warning disable CA1308 // Normalize strings to uppercase - Clang arg may be case-sensitive
			Arguments.Add($"-Xclang -analyzer-output={StaticAnalyzerOutputType.ToString().ToLowerInvariant()}");
#pragma warning restore CA1308

			// Needed for some of the C++ checkers.
			Arguments.Add("-Xclang -analyzer-config -Xclang aggressive-binary-operation-simplification=true");

			// If writing to HTML, use the source filename as a basis for the report filename.
			Arguments.Add("-Xclang -analyzer-config -Xclang stable-report-filename=true");
			Arguments.Add("-Xclang -analyzer-config -Xclang report-in-main-source-file=true");
			Arguments.Add("-Xclang -analyzer-config -Xclang path-diagnostics-alternate=true");

			// Run shallow analyze if requested.
			if (StaticAnalyzerMode == StaticAnalyzerMode.Shallow)
			{
				Arguments.Add("-Xclang -analyzer-config -Xclang mode=shallow");
			}

			VersionNumber ClangVersion = new(Info.ClangVersion.Major, Info.ClangVersion.Minor, Info.ClangVersion.Build);

			if (CompileEnvironment.StaticAnalyzerCheckers.Count > 0)
			{
				// Disable all default checks
				Arguments.Add("--analyzer-no-default-checks");

				// Only enable specific checkers.
				foreach (string Checker in CompileEnvironment.StaticAnalyzerCheckers.Where(Info.ValidAnalyzerCheckers.Contains))
				{
					Arguments.Add($"-Xclang -analyzer-checker -Xclang {Checker}");
				}
			}
			else
			{
				// Disable default checks.
				foreach (string Checker in CompileEnvironment.StaticAnalyzerDisabledCheckers.Where(Info.ValidAnalyzerCheckers.Contains))
				{
					Arguments.Add($"-Xclang -analyzer-disable-checker -Xclang {Checker}");
				}
				// Enable additional non-default checks.
				foreach (string Checker in CompileEnvironment.StaticAnalyzerAdditionalCheckers.Where(Info.ValidAnalyzerCheckers.Contains))
				{
					Arguments.Add($"-Xclang -analyzer-checker -Xclang {Checker}");
				}
			}
		}

		/// <summary>
		/// Common compile arguments for all files in a module.
		/// Override and call base.GetCompileArguments_Global() in derived classes.
		/// </summary>
		///
		/// <param name="CompileEnvironment"></param>
		/// <param name="Arguments"></param>
		protected virtual void GetCompileArguments_Global(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			// build up the commandline common to C and C++
			Arguments.Add("-c");
			Arguments.Add("-pipe");

			if (CompileEnvironment.Architecture.bIsX64)
			{
				// UE5 minspec is 4.2
				Arguments.Add("-msse4.2");
			}

			// Add include paths to the argument list.
			GetCompileArguments_IncludePaths(CompileEnvironment, Arguments);

			// Add preprocessor definitions to the argument list.
			GetCompileArguments_PreprocessorDefinitions(CompileEnvironment, Arguments);

			// Add warning and error flags to the argument list.
			GetCompileArguments_WarningsAndErrors(CompileEnvironment, Arguments);

			// Add FP semantics flags to the argument list.
			GetCompileArguments_FPSemantics(CompileEnvironment, Arguments);

			// Add optimization flags to the argument list.
			GetCompileArguments_Optimizations(CompileEnvironment, Arguments);

			// Add architecture flags to the argument list.
			GetCompileArguments_Architecture(CompileEnvironment, Arguments);

			// Add debugging flags to the argument list.
			GetCompileArguments_Debugging(CompileEnvironment, Arguments);

			// Add sanitizer flags to the argument list.
			GetCompilerArguments_Sanitizers(CompileEnvironment, Arguments);

			// Add analysis flags to the argument list.
			if (IsAnalyzing(CompileEnvironment))
			{
				GetCompileArguments_Analyze(CompileEnvironment, Arguments);
			}
		}

		protected virtual string GetFileNameFromExtension(string AbsolutePath, string Extension)
		{
			return Path.GetFileName(AbsolutePath) + Extension;
		}

		/// <summary>
		/// Compile arguments for specific files in a module. Also updates Action and CPPOutput results.
		/// </summary>
		/// <param name="CompileEnvironment"></param>
		/// <param name="SourceFile"></param>
		/// <param name="OutputDir"></param>
		/// <param name="Arguments"></param>
		/// <param name="CompileAction"></param>
		/// <param name="CompileResult"></param>
		/// <returns>Path to the target file (such as .o)</returns>
		protected virtual FileItem GetCompileArguments_FileType(CppCompileEnvironment CompileEnvironment, FileItem SourceFile, DirectoryReference OutputDir, List<string> Arguments, Action CompileAction, CPPOutput CompileResult)
		{
			// Add the additional response files
			foreach (FileItem AdditionalResponseFile in CompileEnvironment.AdditionalResponseFiles)
			{
				Arguments.Add(GetResponseFileArgument(AdditionalResponseFile, CompileEnvironment.RootPaths));
			}

			// Add force include paths to the argument list.
			GetCompileArguments_ForceInclude(CompileEnvironment, Arguments);

			// Add the C++ source file and its included files to the prerequisite item list.
			CompileAction.PrerequisiteItems.UnionWith(CompileEnvironment.ForceIncludeFiles);
			CompileAction.PrerequisiteItems.UnionWith(CompileEnvironment.AdditionalPrerequisites);
			CompileAction.PrerequisiteItems.Add(SourceFile);

			if (CompileEnvironment.FileInlineSourceMap.TryGetValue(SourceFile, out HashSet<FileItem>? InlinedFiles))
			{
				CompileAction.PrerequisiteItems.UnionWith(InlinedFiles);
			}

			string Extension = Path.GetExtension(SourceFile.AbsolutePath).ToUpperInvariant();
			if (CompileEnvironment.PrecompiledHeaderAction == PrecompiledHeaderAction.Create)
			{
				GetCompileArguments_PCH(CompileEnvironment, Arguments);
			}
			else if (Extension == ".C")
			{
				// Compile the file as C code.
				GetCompileArguments_C(CompileEnvironment, Arguments);
			}
			else if (Extension == ".MM")
			{
				// Compile the file as Objective-C++ code.
				GetCompileArguments_MM(CompileEnvironment, Arguments);
			}
			else if (Extension == ".M")
			{
				// Compile the file as Objective-C code.
				GetCompileArguments_M(CompileEnvironment, Arguments);
			}
			else if (Extension == ".H")
			{
				// Compile the file as C++ code with some additional arguments
				GetCompileArguments_H(CompileEnvironment, Arguments);
			}
			else
			{
				// Compile the file as C++ code.
				GetCompileArguments_CPP(CompileEnvironment, Arguments);
			}

			if (CompileEnvironment.PrecompiledHeaderAction == PrecompiledHeaderAction.Include)
			{
				CompileAction.PrerequisiteItems.Add(FileItem.GetItemByFileReference(CompileEnvironment.PrecompiledHeaderIncludeFilename!));

				PrecompiledHeaderInstance? PCHInstance = CompileEnvironment.PCHInstance;
				while (PCHInstance != null)
				{
					CompileAction.PrerequisiteItems.Add(PCHInstance.Output.GetPrecompiledHeaderFile(CompileEnvironment.Architecture)!);
					CompileAction.PrerequisiteItems.Add(PCHInstance.HeaderFile!);
					PCHInstance = PCHInstance.ParentPCHInstance;
				}
			}

			string FileName = SourceFile.Name;
			if (CompileEnvironment.CollidingNames != null && CompileEnvironment.CollidingNames.Contains(SourceFile))
			{
				string HashString = ContentHash.MD5(SourceFile.AbsolutePath[Unreal.RootDirectory.FullName.Length..]).GetHashCode().ToString("X4");
				FileName = Path.GetFileNameWithoutExtension(FileName) + "_" + HashString + Path.GetExtension(FileName);
			}

			FileItem OutputFile;
			if (CompileEnvironment.PrecompiledHeaderAction == PrecompiledHeaderAction.Create)
			{
				OutputFile = FileItem.GetItemByFileReference(FileReference.Combine(OutputDir, GetFileNameFromExtension(FileName, ".gch")));
				CompileResult.PrecompiledHeaderFile = OutputFile;

				PrecompiledHeaderInstance? ParentPCHInstance = CompileEnvironment.ParentPCHInstance;
				while (ParentPCHInstance != null)
				{
					CompileAction.PrerequisiteItems.Add(ParentPCHInstance.Output.GetPrecompiledHeaderFile(CompileEnvironment.Architecture)!);
					CompileAction.PrerequisiteItems.Add(ParentPCHInstance.HeaderFile!);
					ParentPCHInstance = ParentPCHInstance.ParentPCHInstance;
				}
			}
			else if (IsPreprocessing(CompileEnvironment))
			{
				OutputFile = FileItem.GetItemByFileReference(FileReference.Combine(OutputDir, GetFileNameFromExtension(FileName, ".i")));
				CompileResult.ObjectFiles.Add(OutputFile);

				// Clang does EITHER pre-process or object file.
				Arguments.Add("-E"); // Only run the preprocessor
				Arguments.Add("-fuse-line-directives"); // Use #line in preprocessed output
			}
			else if (IsAnalyzing(CompileEnvironment))
			{
				// Clang analysis does not actually create an object, use the dependency list as the response filename
				OutputFile = FileItem.GetItemByFileReference(FileReference.Combine(OutputDir, GetFileNameFromExtension(FileName, ".d")));
				CompileResult.ObjectFiles.Add(OutputFile);
			}
			else
			{
				string ObjectFileExtension = (CompileEnvironment.AdditionalArguments?.Contains("-emit-llvm", StringComparison.Ordinal) ?? false) ? ".bc" : ".o";
				OutputFile = FileItem.GetItemByFileReference(FileReference.Combine(OutputDir, GetFileNameFromExtension(FileName, ObjectFileExtension)));
				CompileResult.ObjectFiles.Add(OutputFile);
			}

			// Add profdata as a prerequisite for pgo optimize
			if (CompileEnvironment.bPGOOptimize && CompileEnvironment.PGODirectory != null && CompileEnvironment.PGOFilenamePrefix != null)
			{
				DirectoryReference PGODir = new(CompileEnvironment.PGODirectory);
				CompileAction.PrerequisiteItems.Add(FileItem.GetItemByFileReference(FileReference.Combine(PGODir, CompileEnvironment.PGOFilenamePrefix)));
			}

			// FPassPlugin
			foreach (string FPassPlugin in CompileEnvironment.FPassPlugins)
			{
			FileReference? FPassPluginFile = FileReference.FromString(FPassPlugin);
			if (FPassPluginFile != null && FileReference.Exists(FPassPluginFile))
			{
				Arguments.Add($"-fpass-plugin=\"{NormalizeCommandLinePath(FPassPluginFile, CompileEnvironment.RootPaths)}\"");
				CompileAction.PrerequisiteItems.Add(FileItem.GetItemByFileReference(FPassPluginFile));
			}
			}

			// Add the output file to the produced item list.
			CompileAction.ProducedItems.Add(OutputFile);

			// Add the source file path to the command-line.
			Arguments.Add(GetSourceFileArgument(SourceFile, CompileEnvironment));

			// Generate the included header dependency list
			{
				FileItem DependencyListFile = FileItem.GetItemByFileReference(FileReference.Combine(OutputDir, GetFileNameFromExtension(FileName, ".d")));
				Arguments.Add(GetDepencenciesListFileArgument(DependencyListFile, CompileEnvironment));
				CompileAction.DependencyListFile = DependencyListFile;
				CompileAction.ProducedItems.Add(DependencyListFile);
			}

			if (!IsAnalyzing(CompileEnvironment))
			{
				// Generate the timing info
				if (CompileEnvironment.bPrintTimingInfo)
				{
					FileItem TraceFile = FileItem.GetItemByFileReference(FileReference.Combine(OutputDir, GetFileNameFromExtension(FileName, ".json")));
					Arguments.Add("-ftime-trace");
					CompileAction.ProducedItems.Add(TraceFile);
				}

				// Add the parameters needed to compile the output file to the command-line.
				Arguments.Add(GetOutputFileArgument(OutputFile, CompileEnvironment));
			}

			// Add additional arguments to the argument list, must be the final arguments added
			GetCompileArguments_AdditionalArgs(CompileEnvironment, Arguments);

			return OutputFile;
		}

		protected virtual List<string> ExpandResponseFileContents(List<string> ResponseFileContents)
		{
			// This code is optimized for the scenario where ResponseFile has no variables to expand
			List<string> NewList = ResponseFileContents;
			for (int I = 0; I != NewList.Count; ++I)
			{
				string Line = ResponseFileContents[I];
				string NewLine = Utils.ExpandVariables(Line);
				if (NewLine == Line)
				{
					continue;
				}

				lock (ResponseFileContents)
				{
					if (NewList == ResponseFileContents)
					{
						NewList = [.. ResponseFileContents];
					}
				}
				NewList[I] = NewLine;
			}
			return NewList;
		}

		public override IEnumerable<string> GetGlobalCommandLineArgs(CppCompileEnvironment CompileEnvironment)
		{
			List<string> Arguments = [];
			GetCompileArguments_Global(new CppCompileEnvironment(CompileEnvironment), Arguments);
			return Arguments;
		}

		public override IEnumerable<string> GetCPPCommandLineArgs(CppCompileEnvironment CompileEnvironment)
		{
			List<string> Arguments = [];
			GetCompileArguments_CPP(CompileEnvironment, Arguments);
			return Arguments;
		}

		public override IEnumerable<string> GetCCommandLineArgs(CppCompileEnvironment CompileEnvironment)
		{
			List<string> Arguments = [];
			GetCompileArguments_C(CompileEnvironment, Arguments);
			return Arguments;
		}

		public override CppCompileEnvironment CreateSharedResponseFile(CppCompileEnvironment CompileEnvironment, FileReference OutResponseFile, IActionGraphBuilder Graph)
		{
			CppCompileEnvironment NewCompileEnvironment = new(CompileEnvironment);
			List<string> Arguments = [];

			GetCompileArguments_Global(CompileEnvironment, Arguments);

			// Stash shared include paths for validation purposes
			NewCompileEnvironment.SharedUserIncludePaths = [.. NewCompileEnvironment.UserIncludePaths];
			NewCompileEnvironment.SharedSystemIncludePaths = [.. NewCompileEnvironment.SystemIncludePaths];

			NewCompileEnvironment.UserIncludePaths.Clear();
			NewCompileEnvironment.SystemIncludePaths.Clear();

			Arguments = ExpandResponseFileContents(Arguments);

			FileItem FileItem = FileItem.GetItemByFileReference(OutResponseFile);
			Graph.CreateIntermediateTextFile(FileItem, Arguments);

			NewCompileEnvironment.AdditionalPrerequisites.Add(FileItem);
			NewCompileEnvironment.AdditionalResponseFiles.Add(FileItem);
			NewCompileEnvironment.bHasSharedResponseFile = true;

			return NewCompileEnvironment;
		}

		public override void CreateSpecificFileAction(CppCompileEnvironment CompileEnvironment, DirectoryReference SourceDir, DirectoryReference OutputDir, IActionGraphBuilder Graph)
		{
			// This is not supported for now.. If someone wants it we can implement it
			if (CompileEnvironment.Architectures.bIsMultiArch)
			{
				return;
			}

			List<string> GlobalArguments = [];
			if (!CompileEnvironment.bHasSharedResponseFile)
			{
				GetCompileArguments_Global(CompileEnvironment, GlobalArguments);
			}

			string DummyName = "SingleFile.cpp";
			FileItem DummyFile = FileItem.GetItemByFileReference(FileReference.Combine(OutputDir, DummyName));
			CPPOutput Result = new();
			ClangSpecificFileActionGraphBuilder GraphBuilder = new(Logger);
			Action Action = CompileCPPFile(CompileEnvironment, DummyFile, OutputDir, "<Unknown>", GraphBuilder, GlobalArguments, Result);
			Action.PrerequisiteItems.RemoveWhere(File => File.Name.Contains(DummyName, StringComparison.Ordinal));
			Action.bCanExecuteRemotely = true;
			Action.bCanExecuteRemotelyWithSNDBS = Action.bCanExecuteRemotely && !CompileEnvironment.bBuildLocallyWithSNDBS;
			Action.Weight = CompileActionWeight;

			Graph.AddAction(new ClangSpecificFileAction(SourceDir, OutputDir, Action, GraphBuilder.ContentLines));
		}

		protected virtual Action CompileCPPFile(CppCompileEnvironment CompileEnvironment, FileItem SourceFile, DirectoryReference OutputDir, string ModuleName, IActionGraphBuilder Graph, IReadOnlyCollection<string> GlobalArguments, CPPOutput Result)
		{
			Action CompileAction = Graph.CreateAction(ActionType.Compile);
			CompileAction.RootPaths = CompileEnvironment.RootPaths;

			CompileAction.Weight = CompileActionWeight;

			// copy the global arguments into the file arguments, so GetCompileArguments_FileType can remove entries if needed (special case but can be important)
			List<string> FileArguments = [.. GlobalArguments];

			// If we are using the AutoRTFM compiler, we make the compile action depend on the version of the compiler itself.
			// This lets us update the compiler (which might not cause a version update of the compiler, which instead tracks
			// the LLVM versioning scheme that Clang uses), but ensure that we rebuild the source if the compiler has changed.
			// AutoRTFM also depends on external mapping files, which requires a full rebuild when changed.
			if (CompileEnvironment.bUseAutoRTFMCompiler)
			{
				CompileAction.PrerequisiteItems.Add(FileItem.GetItemByFileReference(Info.Clang));
				foreach (FileItem File in CompileEnvironment.AutoRTFMExternalMappingFiles)
				{
					CompileAction.PrerequisiteItems.Add(File);
					FileArguments.Add($"-Xclang -autortfm-mappings -Xclang \"{NormalizeCommandLinePath(File.Location, CompileEnvironment.RootPaths)}\"");
				}
			}

			// Add C or C++ specific compiler arguments and get the target file so we can get the correct output path.
			FileItem TargetFile = GetCompileArguments_FileType(CompileEnvironment, SourceFile, OutputDir, FileArguments, CompileAction, Result);

			// Creates the path to the response file using the name of the output file and creates its contents.
			FileReference ResponseFileName = GetResponseFileName(TargetFile);
			List<string> ResponseFileContents = ExpandResponseFileContents(FileArguments);

			if (OperatingSystem.IsWindows())
			{
				// Enables Clang Cmd Line Work Around
				// Clang reads the response file and computes the total cmd line length.
				// Depending on the length it will then use either cmd line or response file mode.
				// Clang currently underestimates the total size of the cmd line, which can trigger the following clang error in cmd line mode:
				// >>>> xxx-clang.exe': The filename or extension is too long.  (0xCE)
				// Clang processes and modifies the response file contents and this makes the final cmd line size hard for us to predict.
				// To be conservative we add a dummy define to inflate the response file size and force clang to use the response file mode when we are close to the limit.
				int CmdLineLength = Info.Clang.ToString().Length;
				foreach (string Line in ResponseFileContents)
				{
					CmdLineLength += 1 + Line.Length;
				}

				bool bIsInDangerZone = CmdLineLength >= ClangCmdlineDangerZone && CmdLineLength <= ClangCmdLineMaxSize;
				if (bIsInDangerZone)
				{
					ResponseFileContents.Add(ClangDummyDefine);
				}
			}

			// Adds the response file to the compiler input.
			CompileAction.ResponseFileContents = [.. ResponseFileContents];
			FileItem CompilerResponseFileItem = Graph.CreateIntermediateTextFile(ResponseFileName, CompileAction.ResponseFileContents);
			string CommandArguments = GetResponseFileArgument(CompilerResponseFileItem, CompileEnvironment.RootPaths);

			if (bMergeModules && CompileEnvironment.PrecompiledHeaderAction != PrecompiledHeaderAction.Create)
			{
				// EXTRACTEXPORTS can only be interpreted by UBA.. so this action won't build outside uba
				CommandArguments += " /EXTRACTEXPORTS";
				FileItem SymFile = FileItem.GetItemByFileReference(FileReference.Combine(OutputDir, Path.GetFileName(SourceFile.AbsolutePath) + ".exi"));
				CompileAction.ProducedItems.Add(SymFile);
			}

			CompileAction.CommandArguments = CommandArguments;
			CompileAction.PrerequisiteItems.Add(CompilerResponseFileItem);

			CompileAction.WorkingDirectory = Unreal.EngineSourceDirectory;
			CompileAction.CommandPath = Info.Clang;
			CompileAction.CommandVersion = Info.ClangVersionString;
			if (bAllowUbaCompression)
			{
				CompileAction.CommandVersion = $"{CompileAction.CommandVersion} Compressed";
			}
			CompileAction.CommandDescription = IsPreprocessing(CompileEnvironment) ? "Preprocess" : IsAnalyzing(CompileEnvironment) ? "Analyze" : "Compile";
			UnrealArchitectureConfig ArchConfig = UnrealArchitectureConfig.ForPlatform(CompileEnvironment.Platform);
			if (ArchConfig.Mode != UnrealArchitectureMode.SingleArchitecture)
			{
				string ReadableArch = ArchConfig.ConvertToReadableArchitecture(CompileEnvironment.Architecture);
				CompileAction.CommandDescription += $" [{ReadableArch}]";
			}
			CompileAction.StatusDescription = Path.GetFileName(SourceFile.AbsolutePath);
			CompileAction.bIsClangCompiler = true;

			// Don't farm out creation of pre-compiled headers as it is the critical path task.
			if (CompileEnvironment.PrecompiledHeaderAction == PrecompiledHeaderAction.Create)
			{
				CompileAction.bCanExecuteRemotely = CompileEnvironment.bAllowRemotelyCompiledPCHs;
			}
			else
			{
				// Don't farm out preprocess actions since they are more or less only IO
				CompileAction.bCanExecuteRemotely = !CompileEnvironment.bPreprocessOnly;
			}

			CompileAction.ArtifactMode = ArtifactMode.Enabled;
			CompileAction.CacheBucket = GetCacheBucket(TargetRules, CompileEnvironment);

			if (CompileEnvironment.PrecompiledHeaderAction != PrecompiledHeaderAction.None)
			{
				CompileAction.ArtifactMode |= ArtifactMode.AbsolutePath; // Unfortunately we require matching absolute paths for pch to be cached
			}

			if (IsAnalyzing(CompileEnvironment) && CompileEnvironment.bWarningsAsErrors)
			{
				CompileAction.bForceWarningsAsError = true;
				CompileAction.bDeleteProducedItemsOnError = true;
			}

			return CompileAction;
		}

		protected override CPPOutput CompileCPPFiles(CppCompileEnvironment CompileEnvironment, IEnumerable<FileItem> InputFiles, DirectoryReference OutputDir, string ModuleName, IActionGraphBuilder Graph)
		{
			if (ShouldSkipCompile(CompileEnvironment))
			{
				return new CPPOutput();
			}

			List<string> GlobalArguments = [];

			if (!CompileEnvironment.bHasSharedResponseFile)
			{
				GetCompileArguments_Global(CompileEnvironment, GlobalArguments);
			}

			// Create a compile action for each source file.
			CPPOutput Result = new();
			foreach (FileItem SourceFile in InputFiles)
			{
				CompileCPPFile(CompileEnvironment, SourceFile, OutputDir, ModuleName, Graph, GlobalArguments, Result);
			}

			return Result;
		}

		/// <summary>
		/// Used by other tools to get the extra arguments to run vanilla clang for a particular platform.
		/// </summary>
		/// <param name="ExtraArguments">List of extra arguments to add to.</param>
		public virtual void AddExtraToolArguments(IList<string> ExtraArguments)
		{
		}
	}
}
