// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Linq;
using System.Text.RegularExpressions;
using EpicGames.Core;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	internal static class ClangWarnings
	{
		internal static void GetWarnings(CppCompileEnvironment CompileEnvironment, VersionNumber ClangVersion, List<string> Arguments)
		{
			Arguments.AddRange(CompileEnvironment.CppCompileWarnings.GenerateWarningCommandLineArgs(CompileEnvironment, typeof(ClangToolChain), ClangVersion));
		}

		internal static void GetHeaderDisabledWarnings(List<string> Arguments)
		{
			// This warning was to catch #pragma once inside a source file.
			// If we're compiling a header directly, we should always have the pragma once, so we need to ignore this warning.
			Arguments.Add("-Wno-pragma-once-outside-header");
			Arguments.Add("-Wno-#pragma-messages");
		}

		static readonly ConcurrentDictionary<FileReference, HashSet<string>> s_validClangCheckers = [];

		internal static HashSet<string> ValidAnalyzerCheckers(FileReference clang)
		{
			lock (FileItem.GetItemByFileReference(clang))
			{
				return s_validClangCheckers.GetOrAdd(clang, QueryValidAnalyzerCheckers);
			}
		}

		static HashSet<string> QueryValidAnalyzerCheckers(FileReference clang)
		{
			string[] args = ["-analyzer-checker-help", "-analyzer-checker-help-alpha", "-analyzer-checker-help-developer"];
			HashSet<string> checkers = [];
			foreach (string arg in args)
			{
				string output = Utils.RunLocalProcessAndReturnStdOut(clang.FullName, $"-cc1 {arg}", null);
				checkers.AddRange(Regex.Matches(output, "^\\s\\s(\\S*?)\\s", RegexOptions.Multiline)
					.Select(x => x.Groups[1].Value)
					.Where(x => !String.IsNullOrWhiteSpace(x)));
			}
			return checkers;
		}
	}
}
