// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.Globalization;
using System.IO;
using System.Linq;
using System.Security.Cryptography.X509Certificates;
using System.Text;
using System.Text.RegularExpressions;
using System.Xml;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// Utilities related to code sign for Apple Platforms
	/// </summary>
	public static class AppleCodeSign
	{
		/// <summary>
		/// The shared provision library directory (on PC)
		/// </summary>
		public static readonly string ProvisionDirectory = AppleExports.GetProvisionDirectory().FullName;

		/// <summary>
		/// Get the name from the Certificate
		/// </summary>
		public static string GetCommonNameFromCert(X509Certificate2 Cert)
		{
			// Make sure we have a useful friendly name
			string CommonName = "(no common name present)";

			string FullName = Cert.SubjectName.Name;
			char[] SplitChars = { ',' };
			string[] NameParts = FullName.Split(SplitChars);

			foreach (string Part in NameParts)
			{
				string CleanPart = Part.Trim();
				if (CleanPart.StartsWith("CN=", StringComparison.Ordinal))
				{
					CommonName = CleanPart.Substring(3);
				}
			}

			return CommonName;
		}

		/// <summary>
		/// Get the friendly name from the Certificate
		/// </summary>
		public static string GetFriendlyNameFromCert(X509Certificate2 Cert)
		{
			if (OperatingSystem.IsMacOS())
			{
				return GetCommonNameFromCert(Cert);
			}
			else
			{
				// Make sure we have a useful friendly name
				string FriendlyName = Cert.FriendlyName;
				if (String.IsNullOrEmpty(FriendlyName))
				{
					FriendlyName = GetCommonNameFromCert(Cert);
				}

				return FriendlyName;
			}
		}

		private static string CertToolData = "";

		/// <summary>
		/// When receiving the cert tool process call
		/// </summary>
		public static void OutputReceivedCertToolProcessCall(object Sender, DataReceivedEventArgs Line)
		{
			if ((Line != null) && !String.IsNullOrEmpty(Line.Data))
			{
				CertToolData += Line.Data + "\n";
			}
		}

		/// <summary>
		/// Finds all identites that could be used for apple code signing and gets their certificate hashes
		/// </summary>
		public static HashSet<string> GetCodeSigningIdentityHashes()
		{
			CertToolData = "";
			HashSet<string> Identities = new HashSet<string>();
			Process CertTool = new Process();
			CertTool.StartInfo.FileName = "/usr/bin/security";
			CertTool.StartInfo.UseShellExecute = false;
			CertTool.StartInfo.Arguments = String.Format("find-identity -v -p codesigning");
			CertTool.StartInfo.RedirectStandardOutput = true;
			CertTool.StartInfo.StandardOutputEncoding = Encoding.UTF8;
			CertTool.OutputDataReceived += new DataReceivedEventHandler(OutputReceivedCertToolProcessCall);
			CertTool.Start();
			CertTool.BeginOutputReadLine();
			CertTool.WaitForExit();

			if (CertTool.ExitCode == 0)
			{
				const string IdentityHashPattern = @"^\s*\d+\)\s+([A-F0-9]{40})\s+";
				Identities = 
					Regex.Matches(CertToolData, IdentityHashPattern, RegexOptions.Multiline).Cast<Match>()
					.Select(Match => Match.Groups[1].Value)
					.ToHashSet();
			}

			CertTool.Dispose();

			return Identities;
		}

		/// <summary>
		/// Finds all installed certificates with known Apple development/distro prefixes as a X509Certificate2Collection
		/// </summary>
		public static X509Certificate2Collection FindCertificateCollection()
		{
			string[] ValidCertificatePrefixes = { "iPhone Developer", "iPhone Distribution", "Apple Development", "Apple Distribution" };

			X509Certificate2Collection FoundCerts = new X509Certificate2Collection();

			if (OperatingSystem.IsMacOS())
			{
				foreach (string SearchPrefix in ValidCertificatePrefixes)
				{
					// run certtool y to get the currently installed certificates
					CertToolData = "";
					Process CertTool = new Process();
					CertTool.StartInfo.FileName = "/usr/bin/security";
					CertTool.StartInfo.UseShellExecute = false;
					CertTool.StartInfo.Arguments = String.Format("find-certificate -a -c \"{0}\" -p", SearchPrefix);
					CertTool.StartInfo.RedirectStandardOutput = true;
					CertTool.StartInfo.StandardOutputEncoding = Encoding.UTF8;
					CertTool.OutputDataReceived += new DataReceivedEventHandler(OutputReceivedCertToolProcessCall);
					CertTool.Start();
					CertTool.BeginOutputReadLine();
					CertTool.WaitForExit();
					if (CertTool.ExitCode == 0)
					{
						string header = "-----BEGIN CERTIFICATE-----\n";
						string footer = "-----END CERTIFICATE-----";
						int start = CertToolData.IndexOf(header, StringComparison.Ordinal);
						while (start != -1)
						{
							start += header.Length;
							int end = CertToolData.IndexOf(footer, start, StringComparison.Ordinal);
							string base64 = CertToolData.Substring(start, (end - start));
							byte[] certData = Convert.FromBase64String(base64);
							X509Certificate2 cert = X509CertificateLoader.LoadCertificate(certData);
							FoundCerts.Add(cert);
							start = CertToolData.IndexOf(header, start, StringComparison.Ordinal);
						}
					}
				}
			}
			else
			{
				// Open the personal certificate store on this machine
				X509Store Store = new X509Store();
				Store.Open(OpenFlags.ReadOnly);

				foreach (string SearchPrefix in ValidCertificatePrefixes)
				{
					FoundCerts.AddRange(Store.Certificates.Find(X509FindType.FindBySubjectName, SearchPrefix, false));
				}

				Store.Close();
			}

			return FoundCerts;
		}

		/// <summary>
		/// Finds all valid installed certificates
		/// </summary>
		public static List<string> FindCertificates()
		{
			X509Certificate2Collection FoundCerts = FindCertificateCollection();
			
			List<string> Certs = new();
			foreach (X509Certificate2 TestCert in FoundCerts)
			{
				DateTime EffectiveDate = TestCert.NotBefore.ToUniversalTime();
				DateTime ExpirationDate = TestCert.NotAfter.ToUniversalTime();
				DateTime Now = DateTime.UtcNow;

				bool bCertTimeIsValid = (EffectiveDate < Now) && (ExpirationDate > Now);
				string CertLine = String.Format("CERTIFICATE-Name:{0},Validity:{1},StartDate:{2},EndDate:{3}", GetFriendlyNameFromCert(TestCert), bCertTimeIsValid ? "VALID" : "EXPIRED", EffectiveDate.ToString("o"), ExpirationDate.ToString("o"));
				Log.Logger.LogInformation("{Certificate}", CertLine);
				Certs.Add(CertLine);
			}
			return Certs;
		}

		/// <summary>
		/// Finds all valid installed provisions
		/// </summary>
		public static List<FileReference> FindProvisions(string CFBundleIdentifier, bool bForDistribution, out FileReference? MatchedProvision)
		{
			CodeSigningConfig.bForDistribution = bForDistribution;

			List<FileReference> Provisions = new();

			if (!Directory.Exists(ProvisionDirectory))
			{
				Log.Logger.LogError("Could not find provision directory '{ProvisionDirectory}'.", ProvisionDirectory);
				MatchedProvision = null;
				return Provisions;
			}

			// cache the provision library
			string SelectedProvision = "";
			string SelectedCert = "";
			string SelectedFile = "";
			int FoundName = -1;

			Dictionary<string, MobileProvision> ProvisionLibrary = new Dictionary<string, MobileProvision>();

			foreach (string ProvisionFile in Directory.EnumerateFiles(ProvisionDirectory, "*.mobileprovision"))
			{
				MobileProvision Provision = MobileProvisionParser.ParseFile(ProvisionFile);
				ProvisionLibrary.Add(ProvisionFile, Provision);
			}

			// first sort all profiles so we look at newer ones first.
			IEnumerable<string> ProfileKeys = ProvisionLibrary.Select(KV => KV.Key)
				.OrderByDescending(K => ProvisionLibrary[K].CreationDate)
				.ToArray();

			// note - all of this is a near duplicate of code in MobileProvisionUtilities, which other functions
			// in this class call to do similar work! 
			// @todo - unify all of this.

			foreach (string ProvisionFile in ProfileKeys)
			{
				MobileProvision p = ProvisionLibrary[ProvisionFile];

				DateTime EffectiveDate = p.CreationDate;
				DateTime ExpirationDate = p.ExpirationDate;
				DateTime Now = DateTime.UtcNow;

				bool bCertTimeIsValid = (EffectiveDate < Now) && (ExpirationDate > Now);
				bool bValid = false;
				X509Certificate2? Cert = FindCertificate(p);
				if (Cert != null)
				{
					bValid = (Cert.NotBefore.ToUniversalTime() < Now) && (Cert.NotAfter.ToUniversalTime() > Now);
				}
				bool bPassesNameCheck = p.ApplicationIdentifier.Substring(p.ApplicationIdentifierPrefix.Length + 1) == CFBundleIdentifier;
				bool bPassesCompanyCheck = false;
				bool bPassesWildCardCheck = false;
				if (p.ApplicationIdentifier.Contains('*', StringComparison.Ordinal))
				{
					string CompanyName = p.ApplicationIdentifier.Substring(p.ApplicationIdentifierPrefix.Length + 1);

					if (CompanyName != "*")
					{
						CompanyName = CompanyName.Substring(0, CompanyName.LastIndexOf('.'));
						bPassesCompanyCheck = CFBundleIdentifier.StartsWith(CompanyName, StringComparison.Ordinal);
					}
					else
					{
						bPassesWildCardCheck = true;
					}
				}
				bool bIsManaged = false;
				if (p.ProvisionName == "iOS Team Provisioning Profile: " + CFBundleIdentifier)
				{
					bIsManaged = true;
				}
				bool bDistribution = ((p?.ProvisionedDeviceIDs?.Count == 0) && !(p?.bDebug ?? false));
				string Validity = "VALID";
				if (!bCertTimeIsValid)
				{
					Validity = "EXPIRED";
				}
				else if (!bValid)
				{
					Validity = "NO_CERT";
				}
				else if (!bPassesNameCheck && !bPassesWildCardCheck && !bPassesCompanyCheck)
				{
					Validity = "NO_MATCH";
				}
				if (bIsManaged)
				{
					Validity = "MANAGED";
				}
				if ((String.IsNullOrWhiteSpace(SelectedProvision) || FoundName < 2) && Validity == "VALID" && !bDistribution)
				{
					int Prev = FoundName;
					if (bPassesNameCheck)
					{
						FoundName = 2;
					}
					else if (bPassesCompanyCheck && FoundName < 1)
					{
						FoundName = 1;
					}
					else if (bPassesWildCardCheck && FoundName == -1)
					{
						FoundName = 0;
					}
					if (Cert is not null && FoundName != Prev)
					{
						SelectedProvision = p?.ProvisionName ?? "";
						SelectedFile = Path.GetFileName(ProvisionFile) ?? "";
						SelectedCert = GetFriendlyNameFromCert(Cert);
					}
				}
				if (p != null)
				{
					Log.Logger.LogInformation("PROVISION-File:{Filename},Name:{Name},Validity:{Validity},StartDate:{StartDate},EndDate:{EndDate},Type:{Type}", Path.GetFileName(ProvisionFile), p.ProvisionName, Validity, EffectiveDate.ToString(), ExpirationDate.ToString(), bDistribution ? "DISTRIBUTION" : "DEVELOPMENT");
				}
				if (ProvisionFile != null)
				{
					Provisions.Add(new FileReference(ProvisionFile));
				}
			}
			Log.Logger.LogInformation(" MATCHED-Provision:{Name},File:{Filename},Cert:{Cert} ", SelectedProvision, SelectedFile, SelectedCert);
			if (!String.IsNullOrEmpty(SelectedProvision))
			{
				MatchedProvision = new FileReference(SelectedProvision);
			}
			else
			{
				MatchedProvision = null;
			}

			return Provisions;
		}

		/// <summary>
		/// Tries to find a matching certificate on this machine from the the serial number of one of the
		/// certificates in the mobile provision (the one in the mobileprovision is missing the public/private key pair)
		/// </summary>
		public static X509Certificate2? FindCertificate(MobileProvision ProvisionToWorkFrom)
		{
			Log.Logger.LogInformation("  Looking for a certificate that matches the application identifier '{ApplicationIdentifier}'", ProvisionToWorkFrom.ApplicationIdentifier);

			X509Certificate2? Result = null;

			if (Environment.OSVersion.Platform == PlatformID.Unix || Environment.OSVersion.Platform == PlatformID.MacOSX)
			{
				// run certtool y to get the currently installed certificates
				CertToolData = "";
				Process CertTool = new Process();
				CertTool.StartInfo.FileName = "/usr/bin/security";
				CertTool.StartInfo.UseShellExecute = false;
				CertTool.StartInfo.Arguments = "find-identity -p codesigning -v";
				CertTool.StartInfo.RedirectStandardOutput = true;
				CertTool.StartInfo.StandardOutputEncoding = Encoding.UTF8;
				CertTool.OutputDataReceived += new DataReceivedEventHandler(OutputReceivedCertToolProcessCall);
				CertTool.Start();
				CertTool.BeginOutputReadLine();
				CertTool.WaitForExit();

				Log.Logger.LogInformation("  Running {CertToolPath} {CertToolArguments}\n{CertToolData}", CertTool.StartInfo.FileName, CertTool.StartInfo.Arguments, CertToolData);

				if (CertTool.ExitCode == 0)
				{
					Log.Logger.LogInformation("  Provisioning profile contains the following certificate hashes:");
					foreach (X509Certificate2 SourceCert in ProvisionToWorkFrom.DeveloperCertificates)
					{
						X509Certificate2? ValidInTimeCert = null;
						// see if certificate can be found by serial number
						string CertHash = SourceCert.GetCertHashString();

						Log.Logger.LogInformation("    {CertHash}", CertHash);

						if (CertToolData.Contains(CertHash, StringComparison.Ordinal))
						{
							ValidInTimeCert = SourceCert;
						}

						if (ValidInTimeCert != null)
						{
							// Found a cert in the valid time range, quit now!
							Result = ValidInTimeCert;
							Log.Logger.LogInformation("  Matches!");
							break;
						}
					}
				}
			}
			else
			{
				// Open the personal certificate store on this machine
				X509Store Store = new X509Store();
				Store.Open(OpenFlags.ReadOnly);

				// Try finding a matching certificate from the serial number (the one in the mobileprovision is missing the public/private key pair)
				foreach (X509Certificate2 SourceCert in ProvisionToWorkFrom.DeveloperCertificates)
				{
					X509Certificate2Collection FoundCerts = Store.Certificates.Find(X509FindType.FindBySerialNumber, SourceCert.SerialNumber, false);

					Log.Logger.LogInformation("  .. Provision entry SN '{SerialNumber}' matched {NumFoundCerts} installed certificate(s)", SourceCert.SerialNumber, FoundCerts.Count);

					X509Certificate2? ValidInTimeCert = null;
					foreach (X509Certificate2 TestCert in FoundCerts)
					{
						DateTime EffectiveDate = TestCert.NotBefore.ToUniversalTime();
						DateTime ExpirationDate = TestCert.NotAfter.ToUniversalTime();
						DateTime Now = DateTime.UtcNow;

						bool bCertTimeIsValid = (EffectiveDate < Now) && (ExpirationDate > Now);

						Log.Logger.LogInformation("  .. .. Installed certificate '{CertFriendlyName}' is {Validity} (range '{StartDate}' to '{EndDate}')", GetFriendlyNameFromCert(TestCert), bCertTimeIsValid ? "valid (choosing it)" : "EXPIRED", TestCert.GetEffectiveDateString(), TestCert.GetExpirationDateString());
						if (bCertTimeIsValid)
						{
							ValidInTimeCert = TestCert;
							break;
						}
					}

					if (ValidInTimeCert != null)
					{
						// Found a cert in the valid time range, quit now!
						Result = ValidInTimeCert;
						break;
					}
				}

				Store.Close();
			}

			if (Result == null)
			{
				Log.Logger.LogInformation("  .. Failed to find a valid certificate that was in date");
			}

			return Result;
		}

		/// <summary>
		/// Makes sure the required files for code signing exist and can be found
		/// </summary>
		public static bool FindRequiredFiles(out MobileProvision? Provision, out X509Certificate2? Cert, out bool bHasOverridesFile, out bool bNameMatch, bool bNameCheck = true)
		{
			Provision = null;
			Cert = null;
			bHasOverridesFile = File.Exists(CodeSigningConfig.GetPlistOverrideFilename());
			bNameMatch = false;

			string? CFBundleIdentifier = CodeSigningConfig.OverrideBundleName;

			if (String.IsNullOrEmpty(CFBundleIdentifier))
			{
				// Load Info.plist, which guides nearly everything else
				string plistFile = CodeSigningConfig.EngineBuildDirectory + "/UnrealGame-Info.plist";
				if (!String.IsNullOrEmpty(CodeSigningConfig.ProjectFile))
				{
					plistFile = Path.GetDirectoryName(CodeSigningConfig.ProjectFile) + "/Intermediate/" + CodeSigningConfig.OSString + "/" + Path.GetFileNameWithoutExtension(CodeSigningConfig.ProjectFile) + "-Info.plist";

					if (!File.Exists(plistFile))
					{
						plistFile = CodeSigningConfig.IntermediateDirectory + "/UnrealGame-Info.plist";
						if (!File.Exists(plistFile))
						{
							plistFile = CodeSigningConfig.EngineBuildDirectory + "/UnrealGame-Info.plist";
						}
					}
				}
				PListHelper Info = new PListHelper();
				try
				{
					string RawInfoPList = File.ReadAllText(plistFile, Encoding.UTF8);
					Info = new PListHelper(RawInfoPList);
				}
				catch (Exception ex)
				{
					Console.WriteLine(ex.Message);
				}

				// Get the name of the bundle
				Info.GetString("CFBundleIdentifier", out CFBundleIdentifier);
				if (CFBundleIdentifier == null)
				{
					return false;
				}
				else
				{
					string ProjectFilenameNoExt = Path.GetFileNameWithoutExtension(CodeSigningConfig.ProjectFile);
					CFBundleIdentifier = CFBundleIdentifier.Replace("${BUNDLE_IDENTIFIER}", ProjectFilenameNoExt, StringComparison.Ordinal);
				}
			}

			// Check for a mobile provision
			try
			{
				string MobileProvisionFilename = MobileProvision.FindCompatibleProvision(CFBundleIdentifier, out bNameMatch);
				Provision = MobileProvisionParser.ParseFile(MobileProvisionFilename);
			}
			catch (Exception)
			{
			}

			// if we have a null provision see if we can find a compatible provision without checking for a certificate
			if (Provision == null)
			{
				try
				{
					string MobileProvisionFilename = MobileProvision.FindCompatibleProvision(CFBundleIdentifier, out bNameMatch, false);
					Provision = MobileProvisionParser.ParseFile(MobileProvisionFilename);
				}
				catch (Exception)
				{
				}

				// if we have a null provision see if we can find a valid provision without checking for name match
				if (Provision == null && !bNameCheck)
				{
					try
					{
						string MobileProvisionFilename = MobileProvision.FindCompatibleProvision(CFBundleIdentifier, out bNameMatch, false, false);
						Provision = MobileProvisionParser.ParseFile(MobileProvisionFilename);
					}
					catch (Exception)
					{
					}
				}
			}

			// Check for a suitable signature to match the mobile provision
			if (Provision != null)
			{
				Cert = AppleCodeSign.FindCertificate(Provision);
			}

			return true;
		}

		/// <summary>
		/// Find the certs and provision
		/// </summary>
		public static bool FindCertAndProvision(string BundleID, out FileReference? Provision, out string CertName)
		{
			X509Certificate2? Cert;
			MobileProvision? MobileProvision;
			MobileProvision.CacheMobileProvisions();

			if (!String.IsNullOrEmpty(BundleID))
			{
				CodeSigningConfig.OverrideBundleName = BundleID;
			}

			if (FindRequiredFiles(out MobileProvision, out Cert, out _, out _) && Cert != null)
			{
				string? FileName = MobileProvision?.FileName;
				if (FileName != null)
				{
					// print out the provision and cert name
					Provision = new FileReference(FileName);
					CertName = GetFriendlyNameFromCert(Cert!);
					Log.Logger.LogInformation("CERTIFICATE-{Name},PROVISION-{ProvisionFilePath}", CertName, Provision.FullName);
					return true;
				}
			}

			Log.Logger.LogInformation("No matching Signing Data found!");
			Provision = null;
			CertName = "";
			return false;
		}
	}

	internal class FileOperations
	{
		/// <summary>
		/// Find a prefixable file (checks first with the prefix and then without it)
		/// </summary>
		/// <param name="BasePath">The path that the file should be in</param>
		/// <param name="FileName">The filename to check for (with and without SigningPrefix prepended to it)</param>
		/// <returns>The path to the most desirable file (may still not exist)</returns>
		public static string FindPrefixedFile(string BasePath, string FileName)
		{
			return FindPrefixedFileOrDirectory(BasePath, FileName, false);
		}

		/// <summary>
		/// Find a prefixable file or directory (checks first with the prefix and then without it)
		/// </summary>
		/// <param name="BasePath">The path that the file should be in</param>
		/// <param name="Name">The name to check for (with and without SigningPrefix prepended to it)</param>
		/// <param name="bIsDirectory">Is the desired name a directory?</param>
		/// 
		/// <returns>The path to the most desirable file (may still not exist)</returns>
		public static string FindPrefixedFileOrDirectory(string BasePath, string Name, bool bIsDirectory)
		{
			string PrefixedPath = Path.Combine(BasePath, CodeSigningConfig.SigningPrefix + Name);

			if ((bIsDirectory && Directory.Exists(PrefixedPath)) || (!bIsDirectory && File.Exists(PrefixedPath)))
			{
				return PrefixedPath;
			}
			else
			{
				return Path.Combine(BasePath, Name);
			}
		}
	}

	/// <summary>
	/// The mobile provision
	/// </summary>
	public class MobileProvision
	{
		/// <summary>
		/// Tag
		/// </summary>
		public object? Tag;

		/// <summary>
		/// ApplicationIdentifierPrefix
		/// </summary>
		public string ApplicationIdentifierPrefix = "";

		/// <summary>
		/// ApplicationIdentifier
		/// </summary>
		public string ApplicationIdentifier = "";

		/// <summary>
		/// DeveloperCertificates
		/// </summary>
		public List<X509Certificate2> DeveloperCertificates = new List<X509Certificate2>();

		/// <summary>
		/// ProvisionedDeviceIDs
		/// </summary>
		public List<string>? ProvisionedDeviceIDs;

		/// <summary>
		/// ProvisionName
		/// </summary>
		public string? ProvisionName;

		/// <summary>
		/// bDebugn
		/// </summary>
		public bool bDebug;

		/// <summary>
		/// Data
		/// </summary>
		public PListHelper? Data;

		/// <summary>
		/// CreationDate
		/// </summary>
		public DateTime CreationDate;

		/// <summary>
		/// ExpirationDate
		/// </summary>
		public DateTime ExpirationDate;

		/// <summary>
		/// FileName
		/// </summary>
		public string? FileName;

		/// <summary>
		/// UUID
		/// </summary>
		public string? UUID;

		/// <summary>
		/// The Platform
		/// </summary>
		public string? Platform;

		/// <summary>
		/// The logger
		/// </summary>
		static ILogger? _logger;

		/// <summary>
		/// The constructor
		/// </summary>
		public MobileProvision()
		{
			ILoggerFactory loggerFactory = LoggerFactory.Create(builder =>
			{
				builder.AddConsole();
			});
			_logger = loggerFactory.CreateLogger("MobileProvision");
		}

		/// <summary>
		/// Cache the mobile provisions
		/// </summary>
		public static void CacheMobileProvisions()
		{
			_logger?.LogInformation("Caching provisions");
			string ProvisionDirectory = AppleCodeSign.ProvisionDirectory;
			if (!Directory.Exists(ProvisionDirectory))
			{
				_logger?.LogInformation("Provision Folder {ProvisionDirectory} doesn't exist, creating..", ProvisionDirectory);
				Directory.CreateDirectory(ProvisionDirectory);
			}

			List<string> ProvisionCopySrcDirectories = new List<string>();

			// Paths for provisions under game directory 
			if (!String.IsNullOrEmpty(CodeSigningConfig.ProjectFile))
			{
				ProvisionCopySrcDirectories.Add(Path.GetDirectoryName(CodeSigningConfig.ProjectFile) + "/Build/" + CodeSigningConfig.OSString + "/");
				ProvisionCopySrcDirectories.Add(Path.GetDirectoryName(CodeSigningConfig.ProjectFile) + "/Restricted/NoRedist/Build/" + CodeSigningConfig.OSString + "/");
				ProvisionCopySrcDirectories.Add(Path.GetDirectoryName(CodeSigningConfig.ProjectFile) + "/Restricted/NotForLicensees/Build/" + CodeSigningConfig.OSString + "/");
			}

			// Paths for provisions under the Engine directory
			string? OverrideProvisionDirectory = Environment.GetEnvironmentVariable("ProvisionDirectory");
			if (!String.IsNullOrEmpty(OverrideProvisionDirectory))
			{
				ProvisionCopySrcDirectories.Add(OverrideProvisionDirectory);
			}
			else
			{
				ProvisionCopySrcDirectories.Add(CodeSigningConfig.EngineBuildDirectory);
				ProvisionCopySrcDirectories.Add(CodeSigningConfig.EngineDirectory + "/Restricted/NoRedist/Build/" + CodeSigningConfig.OSString + "/");
				ProvisionCopySrcDirectories.Add(CodeSigningConfig.EngineDirectory + "/Restricted/NotForLicensees/Build/" + CodeSigningConfig.OSString + "/");
			}

			// Copy all provisions from the above paths to the library
			foreach (string ProvisionCopySrcDirectory in ProvisionCopySrcDirectories)
			{
				if (Directory.Exists(ProvisionCopySrcDirectory))
				{
					_logger?.LogInformation("Finding provisions in {ProvisionCopySrcDirectory}", ProvisionCopySrcDirectory);
					foreach (string Provision in Directory.EnumerateFiles(ProvisionCopySrcDirectory, "*.mobileprovision", SearchOption.AllDirectories))
					{
						string TargetFile = Path.Combine(ProvisionDirectory, Path.GetFileName(Provision));
						if (!File.Exists(TargetFile) || File.GetLastWriteTime(TargetFile) < File.GetLastWriteTime(Provision))
						{
							FileInfo DestFileInfo;
							if (File.Exists(TargetFile))
							{
								DestFileInfo = new FileInfo(TargetFile);
								DestFileInfo.Attributes &= ~FileAttributes.ReadOnly;
							}
							_logger?.LogInformation("  Copying {ProvisionFilePath} -> {TargetFilePath}", Provision, TargetFile);
							File.Copy(Provision, TargetFile, true);
							DestFileInfo = new FileInfo(TargetFile);
							DestFileInfo.Attributes &= ~FileAttributes.ReadOnly;
							if (!File.Exists(TargetFile))
							{
								_logger?.LogInformation("ERROR: Failed to copy {ProvisionFilePath} -> {TargetFilePath}", Provision, TargetFile);
							}
						}
						else
						{
							_logger?.LogInformation("  Not copying {ProvisionFilePath} as {TargetFilePath} already exists and is not older", Provision, TargetFile);
						}
					}
				}
			}
		}

		/// <summary>
		/// Clean the mobile provisions
		/// </summary>
		public static void CleanMobileProvisions()
		{
			if (!Directory.Exists(AppleCodeSign.ProvisionDirectory))
			{
				_logger?.LogInformation("Provision Folder {ProvisionDirectory} doesn't exist, nothing to do.", AppleCodeSign.ProvisionDirectory);
			}
			else
			{
				_logger?.LogInformation("Cleaning out contents of Provision Folder {ProvisionDirectory}", AppleCodeSign.ProvisionDirectory);
				foreach (string Provision in Directory.GetFiles(AppleCodeSign.ProvisionDirectory))
				{
					File.Delete(Provision);
				}
			}
		}

		/// <summary>
		/// Fond compatible provision
		/// </summary>
		public static string FindCompatibleProvision(string CFBundleIdentifier, out bool bNameMatch, bool bCheckCert = true, bool bCheckIdentifier = true, bool bCheckDistro = true)
		{
			bNameMatch = false;

			// remap the gamename if necessary
			string GameName = CodeSigningConfig.Program_GameName;
			if (GameName == "UnrealGame")
			{
				if (CodeSigningConfig.ProjectFile.Length > 0)
				{
					GameName = Path.GetFileNameWithoutExtension(CodeSigningConfig.ProjectFile);
				}
			}

			// ensure the provision directory exists
			string ProvisionDirectory = AppleCodeSign.ProvisionDirectory;
			if (!Directory.Exists(ProvisionDirectory))
			{
				Directory.CreateDirectory(ProvisionDirectory);
			}

			if (CodeSigningConfig.bProvision)
			{
				if (File.Exists(ProvisionDirectory + "/" + CodeSigningConfig.Provision))
				{
					return ProvisionDirectory + "/" + CodeSigningConfig.Provision;
				}
			}

			#region remove after we provide an install mechanism
			MobileProvision.CacheMobileProvisions();
			#endregion

			// cache the provision library
			Dictionary<string, MobileProvision> ProvisionLibrary = new Dictionary<string, MobileProvision>();
			foreach (string Provision in Directory.EnumerateFiles(ProvisionDirectory, "*.mobileprovision"))
			{
				MobileProvision? p = MobileProvisionParser.ParseFile(Provision);
				if (p != null)
				{
					ProvisionLibrary.Add(Provision, p);
					if (p.FileName != null && p.UUID != null && p.FileName.Contains(p.UUID, StringComparison.Ordinal) && !File.Exists(Path.Combine(ProvisionDirectory, "UE4_" + p.UUID + ".mobileprovision")))
					{
						File.Copy(Provision, Path.Combine(ProvisionDirectory, "UE4_" + p.UUID + ".mobileprovision"));
						p = MobileProvisionParser.ParseFile(Path.Combine(ProvisionDirectory, "UE4_" + p.UUID + ".mobileprovision"));
						if (p != null)
						{
							ProvisionLibrary.Add(Path.Combine(ProvisionDirectory, "UE4_" + p.UUID + ".mobileprovision"), p);
						}
					}
				}
			}

			_logger?.LogInformation(
				"Searching for mobile provisions that match the game '{GameName}' (distribution: {ForDistribution}) with CFBundleIdentifier='{CFBundleIdentifier}' in '{ProvisionDirectory}'",
				GameName,
				CodeSigningConfig.bForDistribution,
				CFBundleIdentifier,
				ProvisionDirectory);

			// first sort all profiles so we look at newer ones first.
			IEnumerable<string> ProfileKeys = ProvisionLibrary.Select(KV => KV.Key)
				.OrderByDescending(K => ProvisionLibrary[K].CreationDate)
				.ToArray();

			// check the cache for a provision matching the app id (com.company.Game)
			// First checking for a contains match and then for a wildcard match
			for (int Phase = -1; Phase < 3; ++Phase)
			{
				if (Phase == -1 && String.IsNullOrEmpty(CodeSigningConfig.ProvisionUUID))
				{
					continue;
				}
				foreach (string Key in ProfileKeys)
				{
					string DebugName = Path.GetFileName(Key);
					MobileProvision TestProvision = ProvisionLibrary[Key];

					// make sure the file is not managed by Xcode
					if (TestProvision.FileName != null && TestProvision.UUID != null &&
					Path.GetFileName(TestProvision.FileName).Equals($"{TestProvision.UUID}.mobileprovision", StringComparison.OrdinalIgnoreCase))
					{
						continue;
					}

					_logger?.LogInformation(" Phase {Phase} considering provision '{Filename}' named '{Name}'", Phase, DebugName, TestProvision.ProvisionName);

					if (TestProvision.ProvisionName == "iOS Team Provisioning Profile: " + CFBundleIdentifier)
					{
						_logger?.LogInformation("  Failing as provisioning is automatic");
						continue;
					}

					// check to see if the platform is the same as what we are looking for
					if (!String.IsNullOrEmpty(TestProvision.Platform) && TestProvision.Platform != CodeSigningConfig.OSString && !String.IsNullOrEmpty(CodeSigningConfig.OSString))
					{
						_logger?.LogInformation("  Failing platform {Platform} Config: {OS}", TestProvision.Platform, CodeSigningConfig.OSString);
						continue;
					}

					// Validate the name
					bool bPassesNameCheck = false;
					if (Phase == -1)
					{
						bPassesNameCheck = TestProvision.UUID == CodeSigningConfig.ProvisionUUID;
						bNameMatch = bPassesNameCheck;
					}
					else if (Phase == 0)
					{
						bPassesNameCheck = TestProvision.ApplicationIdentifier.Substring(TestProvision.ApplicationIdentifierPrefix.Length + 1) == CFBundleIdentifier;
						bNameMatch = bPassesNameCheck;
					}
					else if (Phase == 1)
					{
						if (TestProvision.ApplicationIdentifier.Contains('*', StringComparison.Ordinal))
						{
							string CompanyName = TestProvision.ApplicationIdentifier.Substring(TestProvision.ApplicationIdentifierPrefix.Length + 1);
							if (CompanyName != "*")
							{
								CompanyName = CompanyName.Substring(0, CompanyName.LastIndexOf('.'));
								bPassesNameCheck = CFBundleIdentifier.StartsWith(CompanyName, StringComparison.Ordinal);
							}
						}
					}
					else
					{
						if (TestProvision.ApplicationIdentifier.Contains('*', StringComparison.Ordinal))
						{
							string CompanyName = TestProvision.ApplicationIdentifier.Substring(TestProvision.ApplicationIdentifierPrefix.Length + 1);
							bPassesNameCheck = CompanyName == "*";
						}
					}
					if (!bPassesNameCheck && bCheckIdentifier)
					{
						_logger?.LogInformation("  .. Failed phase {Phase} name check (provision app ID was {ApplicationIdentifier})", Phase, TestProvision.ApplicationIdentifier);
						continue;
					}

					if (CodeSigningConfig.bForDistribution)
					{
						// Check to see if this is a distribution provision. get-task-allow must be false for distro profiles.
						// TestProvision.ProvisionedDeviceIDs.Count==0 is not a valid check as ad-hoc distro profiles do list devices.
						bool bDistroProv = !TestProvision.bDebug;
						if (!bDistroProv)
						{
							_logger?.LogInformation(
								"  .. Failed distribution check (mode={ForDistribution}, get-task-allow={Debug}, #devices={NumDevices})",
								CodeSigningConfig.bForDistribution,
								TestProvision.bDebug,
								TestProvision.ProvisionedDeviceIDs?.Count);
							continue;
						}
					}
					else
					{
						if (bCheckDistro)
						{
							bool bPassesDebugCheck = TestProvision.bDebug;
							if (!bPassesDebugCheck)
							{
								_logger?.LogInformation(
									" .. Failed debugging check (mode={ForDistribution}, get-task-allow={Debug}, #devices={NumDevices})",
									CodeSigningConfig.bForDistribution,
									TestProvision.bDebug,
									TestProvision.ProvisionedDeviceIDs?.Count ?? 0);
								continue;
							}
						}
						else if (!TestProvision.bDebug)
						{
							CodeSigningConfig.bForceStripSymbols = true;
						}
					}

					DateTime CurrentUTCTime = DateTime.UtcNow;
					bool bPassesDateCheck = (CurrentUTCTime >= TestProvision.CreationDate) && (CurrentUTCTime < TestProvision.ExpirationDate);
					if (!bPassesDateCheck)
					{
						_logger?.LogInformation("  .. Failed time period check (valid from {StartDate} to {EndDate}, but UTC time is now {CurrentDate})",
						TestProvision.CreationDate, TestProvision.ExpirationDate, CurrentUTCTime);
						continue;
					}

					// check to see if we have a certificate for this provision
					bool bPassesHasMatchingCertCheck = false;
					if (bCheckCert)
					{

						X509Certificate2? Cert;
						try
						{
							Cert = AppleCodeSign.FindCertificate(TestProvision);
						}
						catch (Exception ex)
						{
							// handle the exception
							Console.WriteLine($"Error finding certificate: {ex.Message}");
							return "";
						}
						bPassesHasMatchingCertCheck = (Cert != null);
						if (bPassesHasMatchingCertCheck && CodeSigningConfig.bCert)
						{
							bPassesHasMatchingCertCheck &= (AppleCodeSign.GetFriendlyNameFromCert(Cert!) == CodeSigningConfig.Certificate);
						}
					}
					else
					{
						bPassesHasMatchingCertCheck = true;
					}

					if (!bPassesHasMatchingCertCheck)
					{
						_logger?.LogInformation("  .. Failed to find a matching certificate that was in date");
						continue;
					}

					// Made it past all the tests
					_logger?.LogInformation(
						"  Picked '{Filename}' with AppID '{ApplicationIdentifier}' and Name '{Name}' as a matching provision for the game '{GameName}'",
						DebugName,
						TestProvision.ApplicationIdentifier,
						TestProvision.ProvisionName,
						GameName);
					return Key;
				}
			}

			// check to see if there is already an embedded provision
			string EmbeddedMobileProvisionFilename = Path.Combine(CodeSigningConfig.RepackageStagingDirectory, "embedded.mobileprovision");

			_logger?.LogWarning("Failed to find a valid matching mobile provision, will attempt to use the embedded mobile provision instead if present");
			return EmbeddedMobileProvisionFilename;
		}

		/// <summary>
		/// Extracts the dict values for the Entitlements key and creates a new full .plist file
		/// from them (with outer plist and dict keys as well as doctype, etc...)
		/// </summary>
		public string GetEntitlementsString(string CFBundleIdentifier, out string TeamIdentifier)
		{
			PListHelper XCentPList = new PListHelper();

			Data?.ProcessValueForKey("Entitlements", "dict", (XmlNode ValueNode) =>
			{
				if (ValueNode != null)
				{
					XCentPList = PListHelper.CloneDictionaryRootedAt(ValueNode)!;
				}
			});

			// Modify the application-identifier to be fully qualified if needed
			string CurrentApplicationIdentifier;
			XCentPList.GetString("application-identifier", out CurrentApplicationIdentifier);
			XCentPList.GetString("com.apple.developer.team-identifier", out TeamIdentifier);

			//			if (CurrentApplicationIdentifier.Contains("*"))
			{
				// Replace the application identifier
				string NewApplicationIdentifier = String.Format("{0}.{1}", ApplicationIdentifierPrefix, CFBundleIdentifier);
				XCentPList.SetString("application-identifier", NewApplicationIdentifier);

				// Replace the keychain access groups
				// Note: This isn't robust, it ignores the existing value in the wildcard and uses the same value for
				// each entry.  If there is a legitimate need for more than one entry in the access group list, then
				// don't use a wildcard!
				List<string> KeyGroups = XCentPList.GetArray("keychain-access-groups", "string");

				for (int i = 0; i < KeyGroups.Count; ++i)
				{
					string Entry = KeyGroups[i];
					if (Entry.Contains('*', StringComparison.Ordinal))
					{
						Entry = NewApplicationIdentifier;
					}
					KeyGroups[i] = Entry;
				}

				XCentPList.SetValueForKey("keychain-access-groups", KeyGroups);
			}

			// must have CloudKit and CloudDocuments for com.apple.developer.icloud-services
			// otherwise the game will not be listed in the Settings->iCloud apps menu on the device
			{
				// iOS only
				if (Platform == "IOS" && XCentPList.HasKey("com.apple.developer.icloud-services"))
				{
					List<string> ServicesGroups = XCentPList.GetArray("com.apple.developer.icloud-services", "string");
					ServicesGroups.Clear();

					ServicesGroups.Add("CloudKit");
					ServicesGroups.Add("CloudDocuments");
					XCentPList.SetValueForKey("com.apple.developer.icloud-services", ServicesGroups);
				}

				// For distribution builds, the entitlements from mobileprovisioning have a modified syntax
				if (CodeSigningConfig.bForDistribution)
				{
					// remove the wildcards from the ubiquity-kvstore-identifier string
					if (XCentPList.HasKey("com.apple.developer.ubiquity-kvstore-identifier"))
					{
						string UbiquityKvstoreString;
						XCentPList.GetString("com.apple.developer.ubiquity-kvstore-identifier", out UbiquityKvstoreString);

						int DotPosition = UbiquityKvstoreString.LastIndexOf('*');
						if (DotPosition >= 0)
						{
							string TeamPrefix = DotPosition > 1 ? UbiquityKvstoreString.Substring(0, DotPosition - 1) : TeamIdentifier;
							string NewUbiquityKvstoreIdentifier = String.Format("{0}.{1}", TeamPrefix, CFBundleIdentifier);
							XCentPList.SetValueForKey("com.apple.developer.ubiquity-kvstore-identifier", NewUbiquityKvstoreIdentifier);
						}
					}

					// remove the wildcards from the ubiquity-container-identifiers array
					if (XCentPList.HasKey("com.apple.developer.ubiquity-container-identifiers"))
					{
						List<string> UbiquityContainerIdentifiersGroups = XCentPList.GetArray("com.apple.developer.ubiquity-container-identifiers", "string");

						for (int i = 0; i < UbiquityContainerIdentifiersGroups.Count; i++)
						{
							int DotPosition = UbiquityContainerIdentifiersGroups[i].LastIndexOf('*');
							if (DotPosition >= 0)
							{
								string TeamPrefix = DotPosition > 1 ? UbiquityContainerIdentifiersGroups[i].Substring(0, DotPosition - 1) : TeamIdentifier;
								string NewUbiquityContainerIdentifier = String.Format("{0}.{1}", TeamPrefix, CFBundleIdentifier);
								UbiquityContainerIdentifiersGroups[i] = NewUbiquityContainerIdentifier;
							}
						}

						if (UbiquityContainerIdentifiersGroups.Count == 0)
						{
							string NewUbiquityKvstoreIdentifier = String.Format("{0}.{1}", TeamIdentifier, CFBundleIdentifier);
							UbiquityContainerIdentifiersGroups.Add(NewUbiquityKvstoreIdentifier);
						}

						XCentPList.SetValueForKey("com.apple.developer.ubiquity-container-identifiers", UbiquityContainerIdentifiersGroups);
					}

					// remove the wildcards from the developer.associated-domains array or string
					if (XCentPList.HasKey("com.apple.developer.associated-domains"))
					{
						string AssociatedDomainsString;
						XCentPList.GetString("com.apple.developer.associated-domains", out AssociatedDomainsString);

						//check if the value is string
						if (AssociatedDomainsString != null && AssociatedDomainsString.Contains('*', StringComparison.Ordinal))
						{
							XCentPList.RemoveKeyValue("com.apple.developer.associated-domains");
						}
						else
						{
							//check if the value is an array
							List<string> AssociatedDomainsGroup = XCentPList.GetArray("com.apple.developer.associated-domains", "string");

							if (AssociatedDomainsGroup.Count == 1 && AssociatedDomainsGroup[0].Contains('*', StringComparison.Ordinal))
							{
								XCentPList.RemoveKeyValue("com.apple.developer.associated-domains");
							}
						}
					}

					// remove development keys - generated when the cloudkit container is in development mode
					XCentPList.RemoveKeyValue("com.apple.developer.icloud-container-development-container-identifiers");
				}

				// set the icloud-container-environment according to the project settings
				if (XCentPList.HasKey("com.apple.developer.icloud-container-environment"))
				{
					List<string> ContainerEnvironmentGroup = XCentPList.GetArray("com.apple.developer.icloud-container-environment", "string");

					if (ContainerEnvironmentGroup.Count != 0)
					{
						ContainerEnvironmentGroup.Clear();

						// The new value is a string, not an array
						string NewContainerEnvironment = CodeSigningConfig.bForDistribution ? "Production" : "Development";
						XCentPList.SetValueForKey("com.apple.developer.icloud-container-environment", NewContainerEnvironment);
					}
				}
			}

			return XCentPList.SaveToString();
		}

		/// <summary>
		/// Constructs a MobileProvision from an xml blob extracted from the real ASN.1 file
		/// </summary>
		public MobileProvision(string EmbeddedPListText)
		{
			Data = new PListHelper(EmbeddedPListText);

			// Now extract things

			// Key: ApplicationIdentifierPrefix, Array<String>
			List<string> PrefixList = Data.GetArray("ApplicationIdentifierPrefix", "string");
			if (PrefixList.Count > 1)
			{
				_logger?.LogWarning("Found more than one entry for ApplicationIdentifierPrefix in the .mobileprovision, using the first one found");
			}

			if (PrefixList.Count > 0)
			{
				ApplicationIdentifierPrefix = PrefixList[0];
			}

			// Example date string from the XML: "2014-06-30T20:45:55Z";
			DateTimeStyles AppleDateStyle = DateTimeStyles.AllowWhiteSpaces | DateTimeStyles.AssumeUniversal | DateTimeStyles.AdjustToUniversal;

			string CreationDateString;
			if (Data.GetDate("CreationDate", out CreationDateString))
			{
				CreationDate = DateTime.Parse(CreationDateString, CultureInfo.InvariantCulture, AppleDateStyle);
			}

			string ExpirationDateString;
			if (Data.GetDate("ExpirationDate", out ExpirationDateString))
			{
				ExpirationDate = DateTime.Parse(ExpirationDateString, CultureInfo.InvariantCulture, AppleDateStyle);
			}

			// Key: DeveloperCertificates, Array<Data> (uuencoded)
			List<string> CertificateList = Data.GetArray("DeveloperCertificates", "data");
			foreach (string EncodedCert in CertificateList)
			{
				byte[] RawCert = Convert.FromBase64String(EncodedCert);
				DeveloperCertificates.Add(X509CertificateLoader.LoadCertificate(RawCert));
			}

			// Key: Name, String
			if (!Data.GetString("Name", out ProvisionName))
			{
				ProvisionName = "(unknown)";
			}

			// Key: ProvisionedDevices, Array<String>
			ProvisionedDeviceIDs = Data.GetArray("ProvisionedDevices", "string");

			// Key: application-identifier, Array<String>
			PListHelper? XCentPList = null;
			Data.ProcessValueForKey("Entitlements", "dict", delegate (XmlNode ValueNode)
			{
				XCentPList = PListHelper.CloneDictionaryRootedAt(ValueNode);
			});

			// Modify the application-identifier to be fully qualified if needed
#pragma warning disable CS8602
			if (!XCentPList.GetString("application-identifier", out ApplicationIdentifier))
#pragma warning restore CS8602
			{
				ApplicationIdentifier = "(unknown)";
			}

			// check for get-task-allow
			if (XCentPList != null)
			{
				bDebug = XCentPList.GetBool("get-task-allow");
			}

			if (!Data.GetString("UUID", out UUID))
			{
				UUID = "(unkown)";
			}

			List<string> Platforms = Data.GetArray("Platform", "string");
			if (Platforms.Contains("iOS"))
			{
				Platform = "IOS";
			}
			else if (Platforms.Contains("tvOS"))
			{
				Platform = "TVOS";
			}
			else
			{
				Platform = "";
			}
		}

		/// <summary>
		/// Does this provision contain the specified UDID?
		/// </summary>
		public bool ContainsUDID(string UDID)
		{
			bool bFound = false;
			if (ProvisionedDeviceIDs != null)
			{
				foreach (string TestUDID in ProvisionedDeviceIDs)
				{
					if (TestUDID.Equals(UDID, StringComparison.OrdinalIgnoreCase))
					{
						bFound = true;
						break;
					}
				}
			}
			return bFound;
		}
	}

	/// <summary>
	/// This class understands how to get the embedded plist in a .mobileprovision file.  It doesn't
	/// understand the full format and is not capable of writing a new one out or anything similar.
	/// </summary>
	public static class MobileProvisionParser
	{
		private static int StrStrByteArray(byte[] Haystack, int Offset, string Needle)
		{
			byte[] NeedleBytes = Encoding.UTF8.GetBytes(Needle);

			//@TODO: Is there anything better in .NET? That's going to be pretty slow on large files
			for (int i = Offset; i < Haystack.Length - NeedleBytes.Length; ++i)
			{
				bool bMatch = true;

				for (int j = 0; j < NeedleBytes.Length; ++j)
				{
					if (Haystack[i + j] != NeedleBytes[j])
					{
						bMatch = false;
						break;
					}
				}

				if (bMatch)
				{
					return i;
				}
			}

			return -1;
		}

		/// <summary>
		/// Parse the file
		/// </summary>
		public static MobileProvision? ParseFile(byte[] RawData)
		{
			//@TODO: This file is just an ASN.1 stream, should find or make a raw ASN1 parser and use
			// that instead of this (theoretically fragile) code (particularly the length extraction)

			string StartPattern = "<?xml";
			string EndPattern = "</plist>";

			// Search the start pattern
			int StartPos = StrStrByteArray(RawData, 0, StartPattern);
			if (StartPos != -1)
			{
				// Search the end pattern
				int EndPos = StrStrByteArray(RawData, StartPos, EndPattern);
				if (EndPos != -1)
				{
					// Offset the end position to take in account the end pattern
					EndPos += EndPattern.Length;

					// Convert the data to a string
					string PlistText = Encoding.UTF8.GetString(RawData, StartPos, EndPos - StartPos);

					// Return the constructed 'mobile provision'
					return new MobileProvision(PlistText);
				}
			}

			// Unable to find the start of the plist data
			Log.Logger.LogError("Failed to find embedded plist in .mobileprovision file");
			return null;
		}

		/// <summary>
		/// Parse the file
		/// </summary>
		public static MobileProvision? ParseFile(Stream InputStream)
		{
			// Read in the entire file
			int NumBytes = (int)InputStream.Length;
			byte[] RawData = new byte[NumBytes];
			InputStream.ReadExactly(RawData, 0, NumBytes);

			return ParseFile(RawData);
		}

		/// <summary>
		/// Parse the file
		/// </summary>
		public static MobileProvision ParseFile(string Filename)
		{
			FileStream InputStream = File.OpenRead(Filename);
			MobileProvision? Result = ParseFile(InputStream);
			InputStream.Close();
			if (Result == null)
			{
				throw new Exception("Failed to parse mobile provision file");
			}
			Result.FileName ??= Filename;

			return Result;
		}
	}

	/// <summary>
	/// The code signing config class
	/// </summary>
	public static class CodeSigningConfig
	{
		/// <summary>
		/// The display name in title bars, popups, etc ...
		/// </summary>
		public static string AppDisplayName = "Unreal iOS Configuration";

		/// <summary>
		/// The game directory
		/// </summary>
		public static string GameDirectory = "";        // "..\\..\\..\\..\\Engine\\Source\\UE4";

		/// <summary>
		/// The binaries directory
		/// </summary>
		public static string BinariesDirectory = "";    // "..\\..\\..\\..\\Engine\\Binaries\\IOS\\"

		/// <summary>
		/// Optional Prefix to append to .xcent and .mobileprovision files, for handling multiple certificates on same source game
		/// </summary>
		public static string SigningPrefix = "";

		/// <summary>
		/// PCStagingRootDir
		/// </summary>
		public static string PCStagingRootDir = "";

		/// <summary>
		/// The local staging directory for files needed by Xcode on the Mac (on PC)
		/// </summary>
		public static string PCXcodeStagingDir = "";

		/// <summary>
		/// The staging directory from UAT that is passed into IPP (for repackaging, etc...)
		/// </summary>
		public static string RepackageStagingDirectory = "";

		/// <summary>
		/// The delta manifest for deploying files for iterative deploy
		/// </summary>
		public static string DeltaManifest = "";

		/// <summary>
		/// The files to be retrieved from the device
		/// </summary>
		public static List<string> FilesForBackup = new List<string>();

		/// <summary>
		/// The project file that is passed into IPP from UAT
		/// </summary>
		public static string ProjectFile = "";

		/// <summary>
		/// The device to deploy or launch on
		/// </summary>
		public static string DeviceId = "";

		/// <summary>
		/// Determine whether to use RPC Util or not
		/// </summary>
		public static bool bUseRPCUtil = true;

		/// <summary>
		/// Optional override for the dev root on the target mac for remote builds.
		/// </summary>
		public static string? OverrideDevRoot = null;

		/// <summary>
		/// Optional value specifying that we are working with tvOS
		/// </summary>
		public static string OSString = "IOS";

		/// <summary>
		/// The local build directory (on PC)
		/// </summary>
		public static string BuildDirectory
		{
			get
			{
				string StandardGameBuildDir = GameDirectory + @"\Build\IOS";
				// if the normal Build dir exists, return it, otherwise, use the program Resources directory
				return Path.GetFullPath(Directory.Exists(StandardGameBuildDir) ?
					StandardGameBuildDir :
					GameDirectory + @"\Resources\IOS");
			}
		}

		/// <summary>
		/// The shared (Engine) build directory (on PC)
		/// </summary>
		public static string EngineBuildDirectory => Path.Combine(Unreal.EngineDirectory.FullName, "Build", CodeSigningConfig.OSString);

		/// <summary>
		/// The local build intermediate directory (on PC)
		/// </summary>
		public static string IntermediateDirectory
		{
			get
			{
				string IntermediateGameBuildDir = GameDirectory + @"\Intermediate\" + CodeSigningConfig.OSString;
				// if the normal Build dir exists, return it, otherwise, use the program Resources directory
				return Path.GetFullPath(Directory.Exists(IntermediateGameBuildDir) ?
					IntermediateGameBuildDir :
					BuildDirectory);
			}
		}

		/// <summary>
		/// The local directory cooked files are placed (on PC)
		/// </summary>
		public static string CookedDirectory => Path.GetFullPath(GameDirectory + @"\Saved\Cooked\" + CodeSigningConfig.OSString);

		/// <summary>
		/// The local directory config files are placed (on PC)
		/// </summary>
		public static string ConfigDirectory => Path.GetFullPath(GameDirectory + @"\Saved\Config");

		/// <summary>
		/// The engine config files are placed (on PC)
		/// </summary>
		public static string DefaultConfigDirectory => Path.Combine(Unreal.EngineDirectory.FullName, "Engine/Config");

		/// <summary>
		/// The engine directory
		/// </summary>
		public static string EngineDirectory => Unreal.EngineDirectory.FullName;

		/// <summary>
		/// The local directory that a payload (GameName.app) is assembled into before being copied to the Mac (on PC)
		/// </summary>
		public static string PayloadRootDirectory => Path.GetFullPath(PCStagingRootDir + @"\Payload");

		/// <summary>
		/// Program_Architecture
		/// </summary>
		public static string Program_Architecture = "";

		/// <summary>
		/// Program_GameName
		/// </summary>
		public static string Program_GameName = "";

		/// <summary>
		/// Returns the name of the file containing user overrides that will be applied when packaging on PC
		/// </summary>
		public static string GetPlistOverrideFilename()
		{
			return GetPlistOverrideFilename(false);
		}

		/// <summary>
		/// GetPlistOverrideFilename
		/// </summary>
		public static string GetPlistOverrideFilename(bool bWantDistributionOverlay)
		{
			string Prefix = "";
			if (bWantDistributionOverlay)
			{
				Prefix = "Distro_";
			}

			return Path.Combine(BuildDirectory, Prefix + Program_GameName + "Overrides.plist");
		}

		/// <summary>
		/// Whether or not to output extra information (like every file copy and date/time stamp)
		/// </summary>
		public static bool bVerbose = true;

		/// <summary>
		/// Whether or not to output extra information in code signing
		/// </summary>
		public static bool bCodeSignVerbose = false;

		/// <summary>
		/// Whether or not non-critical files will be packaged (critical == required for signing or .app validation, the app may still fail to
		/// run with only 'critical' files present).  Also affects the name and location of the IPA back on the PC
		/// </summary>
		public static bool bCreateStubSet = false;

		/// <summary>
		/// Is this a distribution packaging build?  Controls a number of aspects of packing (which signing prefix and provisioning profile to use, etc...)
		/// </summary>
		public static bool bForDistribution = false;

		/// <summary>
		/// Is this a c++ code based project?  So far used on repackagefromstage to properly choose the .ipa name
		/// </summary>
		public static bool bIsCodeBasedProject = false;

		/// <summary>
		/// Is this going to try automatic signing?
		/// </summary>
		public static bool bAutomaticSigning = false;

		/// <summary>
		/// Whether or not to strip symbols (they will always be stripped when packaging for distribution)
		/// </summary>
		public static bool bForceStripSymbols = false;

		/// <summary>
		/// Do a code signing update when repackaging?
		/// </summary>
		public static bool bPerformResignWhenRepackaging = false;

		/// <summary>
		/// Whether the cooked data will be cooked on the fly or was already cooked by the books.
		/// </summary>
		public static bool bCookOnTheFly = false;

		/// <summary>
		/// Whether the install should be performed on a provision
		/// </summary>
		public static bool bProvision = false;

		/// <summary>
		/// provision to be installed
		/// </summary>
		public static string Provision = "";

		/// <summary>
		/// provision to be installed
		/// </summary>
		public static string ProvisionUUID = "";

		/// <summary>
		/// provision to extract certificate for
		/// </summary>
		public static string? ProvisionFile;

		/// <summary>
		/// target name specificied by the command line, if not set GameName + optional Client will be used
		/// </summary>
		public static string TargetName = "";

		/// <summary>
		/// IOS Team ID to be used for automatic signing
		/// </summary>
		public static string TeamID = "";

		/// <summary>
		/// Whether the install should be performed on a certificate
		/// </summary>
		public static bool bCert = false;

		/// <summary>
		/// Certificate to be installed
		/// </summary>
		public static string Certificate = "";

		/// <summary>
		/// Certificate to be output
		/// </summary>
		public static string? OutputCertificate = null;

		/// <summary>
		/// An override server Mac name
		/// </summary>
		public static string? OverrideMacName = null;

		/// <summary>
		/// A name to use for the bundle indentifier and display name when resigning
		/// </summary>
		public static string? OverrideBundleName = null;

		/// <summary>
		/// Whether to use None or Best for the compression setting when repackaging IPAs (int version of Ionic.Zlib.CompressionLevel)
		/// By making this an int, we are able to delay load the Ionic assembly from a different path (which is required)
		/// </summary>
		[Obsolete("Deprecated in 5.8. The Ionic compression library is no longer used.")]
		public static int RecompressionSetting = 0;

		/// <summary>
		/// If the requirements blob is present in the existing executable when code signing, should it be carried forward
		/// (true) or should a dummy requirements blob be created with no actual requirements (false)
		/// </summary>
		public static bool bMaintainExistingRequirementsWhenCodeSigning = false;

		/// <summary>
		/// The code signing identity to use when signing via RPC
		/// </summary>
		public static string? CodeSigningIdentity;

		/// <summary>
		/// The minimum OS version
		/// </summary>
		public static string? MinOSVersion;

		/// <summary>
		/// Create an iterative IPA (i.e. a stub only with Icons and Splash images)
		/// </summary>
		public static bool bIterate = false;

		/// <summary>
		/// Returns a path to the place to back up documents from a device
		/// </summary>
		/// <returns></returns>
		public static string GetRootBackedUpDocumentsDirectory()
		{
			return Path.GetFullPath(Path.Combine(GameDirectory + @"\" + CodeSigningConfig.OSString + "_Backups"));
		}

		/// <summary>
		/// The init
		/// </summary>
		public static bool Initialize(FileReference? ProjectFile, bool bIsTVOS)
		{
			// if the path is a directory (relative to where the game was launched from), then get the absolute directory
			string FullGamePath;
			string OrigGamePath;

			if (ProjectFile == null)
			{
				FullGamePath = Unreal.EngineDirectory.FullName;
				OrigGamePath = "Engine";
				Program_GameName = "UnrealGame";
			}
			else
			{
				FullGamePath = OrigGamePath = CodeSigningConfig.ProjectFile = ProjectFile.FullName;
				Program_GameName = ProjectFile.GetFileNameWithoutAnyExtensions();
				CodeSigningConfig.ProjectFile = ProjectFile.FullName;
			}

			OSString = bIsTVOS ? "TVOS" : "IOS";

			if (Directory.Exists(FullGamePath))
			{
				GameDirectory = FullGamePath;
			}
			// is it a file? if so, just use the file's directory
			else if (File.Exists(FullGamePath))
			{
				GameDirectory = Path.GetDirectoryName(FullGamePath)!;
			}

			if (!Directory.Exists(GameDirectory))
			{
				Console.ForegroundColor = ConsoleColor.Red;
				Console.WriteLine();
				Console.WriteLine("Unable to find a game or program {0}. You may need to specify a path to the program", OrigGamePath);
				Console.ResetColor();
				return false;
			}

			// special case handling for anything inside Engine/Source, it will go to the Engine/Binaries directory, not the Game's binaries directory
			if (OrigGamePath.Replace("\\", "/", StringComparison.Ordinal).Contains("Engine/Source", StringComparison.Ordinal))
			{
				BinariesDirectory = Path.Combine(Unreal.EngineDirectory.FullName, "Binaries", OSString);
			}
			else if (!OrigGamePath.Replace("\\", "/", StringComparison.Ordinal).Contains(@"Binaries/" + OSString, StringComparison.Ordinal))
			{
				// no sense in adding Binaries\IOS when it's already there. This is a special case to handle packaging UnrealLaunchDaemon from the command line.
				BinariesDirectory = Path.Combine(GameDirectory, "Binaries", OSString) + "/";
			}
			else
			{
				BinariesDirectory = GameDirectory;
			}

			CodeSigningConfig.CodeSigningIdentity = CodeSigningConfig.bForDistribution ? "iPhone Distribution" : "iPhone Developer";

			// Windows doesn't allow environment vars to be set to blank so detect "none" and treat it as such
			if (CodeSigningConfig.SigningPrefix == "none")
			{
				CodeSigningConfig.SigningPrefix = CodeSigningConfig.bForDistribution ? "Distro_" : "";
			}

			return true;
		}
	}
}

