// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Text;
using System.Xml.Linq;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace UnrealBuildTool
{
	/// <summary>
	/// Public IOS functions exposed to UAT
	/// </summary>
	public static class IOSExports
	{
		/// <summary>
		/// 
		/// </summary>
		/// <param name="InProject"></param>
		/// <param name="Distribution"></param>
		/// <param name="MobileProvision"></param>
		/// <param name="SigningCertificate"></param>
		/// <param name="TeamUUID"></param>
		/// <param name="bAutomaticSigning"></param>
		public static void GetProvisioningData(FileReference InProject, bool Distribution, out string? MobileProvision, out string? SigningCertificate, out string? TeamUUID, out bool bAutomaticSigning)
		{
			IOSProjectSettings ProjectSettings = ((IOSPlatform)UEBuildPlatform.GetBuildPlatform(UnrealTargetPlatform.IOS)).ReadProjectSettings(InProject);
			if (ProjectSettings == null)
			{
				MobileProvision = null;
				SigningCertificate = null;
				TeamUUID = null;
				bAutomaticSigning = false;
				return;
			}
			if (ProjectSettings.bAutomaticSigning)
			{
				MobileProvision = null;
				SigningCertificate = null;
				TeamUUID = ProjectSettings.TeamID;
				bAutomaticSigning = true;
			}
			else
			{
				IOSProvisioningData Data = ((IOSPlatform)UEBuildPlatform.GetBuildPlatform(UnrealTargetPlatform.IOS)).ReadProvisioningData(ProjectSettings, Distribution);
				if (Data == null)
				{ // no provisioning, swith to automatic
					MobileProvision = null;
					SigningCertificate = null;
					TeamUUID = ProjectSettings.TeamID;
					bAutomaticSigning = true;
				}
				else
				{
					MobileProvision = Data.MobileProvision;
					SigningCertificate = Data.SigningCertificate;
					TeamUUID = Data.TeamUUID;
					bAutomaticSigning = false;
				}
			}
		}

		/// <summary>
		/// 
		/// </summary>
		/// <param name="Config"></param>
		/// <param name="ProjectFile"></param>
		/// <param name="InProjectName"></param>
		/// <param name="InProjectDirectory"></param>
		/// <param name="Executable"></param>
		/// <param name="InEngineDir"></param>
		/// <param name="bForDistribution"></param>
		/// <param name="CookFlavor"></param>
		/// <param name="bIsDataDeploy"></param>
		/// <param name="bCreateStubIPA"></param>
		/// <param name="BuildReceiptFileName"></param>
		/// <param name="Logger">Logger for output</param>
		/// <returns></returns>
		public static bool PrepForUATPackageOrDeploy(UnrealTargetConfiguration Config, FileReference ProjectFile, string InProjectName, DirectoryReference InProjectDirectory, FileReference Executable, DirectoryReference InEngineDir, bool bForDistribution, string CookFlavor, bool bIsDataDeploy, bool bCreateStubIPA, FileReference BuildReceiptFileName, ILogger Logger)
		{
			TargetReceipt Receipt = TargetReceipt.Read(BuildReceiptFileName);
			return new UEDeployIOS(Logger).PrepForUATPackageOrDeploy(Config, ProjectFile, InProjectName, InProjectDirectory.FullName, Executable, InEngineDir.FullName, bCreateStubIPA, Receipt);
		}

		/// <summary>
		/// 
		/// </summary>
		/// <param name="ProjectFile"></param>
		/// <param name="Config"></param>
		/// <param name="ProjectDirectory"></param>
		/// <param name="bIsUnrealGame"></param>
		/// <param name="GameName"></param>
		/// <param name="bIsClient"></param>
		/// <param name="ProjectName"></param>
		/// <param name="InEngineDir"></param>
		/// <param name="AppDirectory"></param>
		/// <param name="Receipt"></param>
		/// <param name="Logger">Logger for output</param>
		/// <returns></returns>
		public static bool GeneratePList(FileReference ProjectFile, UnrealTargetConfiguration Config, DirectoryReference ProjectDirectory, bool bIsUnrealGame, string GameName, bool bIsClient, string ProjectName, DirectoryReference InEngineDir, DirectoryReference AppDirectory, TargetReceipt Receipt, ILogger Logger)
		{
			return new UEDeployIOS(Logger).GeneratePList(ProjectFile, Config, ProjectDirectory.FullName, bIsUnrealGame, GameName, bIsClient, ProjectName, InEngineDir.FullName, AppDirectory.FullName, Receipt);
		}

		/// <summary>
		/// 
		/// </summary>
		/// <param name="PlatformType"></param>
		/// <param name="SourceFile"></param>
		/// <param name="TargetFile"></param>
		/// <param name="Logger"></param>
		public static void StripSymbols(UnrealTargetPlatform PlatformType, FileReference SourceFile, FileReference TargetFile, ILogger Logger)
		{
			IOSProjectSettings ProjectSettings = ((IOSPlatform)UEBuildPlatform.GetBuildPlatform(PlatformType)).ReadProjectSettings(null);
			IOSToolChain ToolChain = new IOSToolChain(null, ProjectSettings, ClangToolChainOptions.None, Logger);
			ToolChain.StripSymbols(SourceFile, TargetFile);
		}

		/// <summary>
		/// 
		/// </summary>
		/// <param name="ProjectFile"></param>
		/// <param name="Executable"></param>
		/// <param name="StageDirectory"></param>
		/// <param name="Platform"></param>
		/// <param name="Logger"></param>
		public static void GenerateAssetCatalog(FileReference ProjectFile, FileReference Executable, DirectoryReference StageDirectory, UnrealTargetPlatform Platform, ILogger Logger)
		{
			// Determine whether the user has modified icons that require a remote Mac to build.
			bool bUserImagesExist = false;
			DirectoryReference ResourcesDir = IOSToolChain.GenerateAssetCatalog(ProjectFile, Platform, ref bUserImagesExist);

			// Don't attempt to do anything remotely if the user is using the default UE images.
			if (!bUserImagesExist)
			{
				return;
			}

			// Also don't attempt to use a remote Mac if packaging for TVOS on PC.
			if (Platform == UnrealTargetPlatform.TVOS && BuildHostPlatform.Current.Platform != UnrealTargetPlatform.Mac)
			{
				return;
			}

			// Compile the asset catalog immediately
			if (BuildHostPlatform.Current.Platform != UnrealTargetPlatform.Mac)
			{
				FileReference OutputFile = FileReference.Combine(StageDirectory, "Assets.car");

				RemoteMac Remote = new RemoteMac(ProjectFile, Logger);
				Remote.RunAssetCatalogTool(Platform, ResourcesDir, OutputFile, Logger);
			}
			else
			{
				// Get the output file
				FileReference OutputFile = IOSToolChain.GetAssetCatalogFile(Platform, Executable);

				// Delete the Assets.car file to force the asset catalog to build every time, because
				// removals of files or copies of icons (for instance) with a timestamp earlier than
				// the last generated Assets.car will result in nothing built.
				if (FileReference.Exists(OutputFile))
				{
					FileReference.Delete(OutputFile);
				}

				// Run the process locally
				using (Process Process = new Process())
				{
					Process.StartInfo.FileName = "/usr/bin/xcrun";
					Process.StartInfo.Arguments = IOSToolChain.GetAssetCatalogArgs(Platform, ResourcesDir.FullName, OutputFile.Directory.FullName);
					;
					Process.StartInfo.UseShellExecute = false;
					Utils.RunLocalProcess(Process);
				}
			}
		}

		/// <summary>
		/// Set a secondary remote Mac to retrieve built data on a remote Mac.
		/// </summary>
		/// <returns></returns>

		public static void SetSecondaryRemoteMac(string ClientPlatform, FileReference ProjectFile, ILogger Logger)
		{
			RemoteMac Remote = new RemoteMac(ProjectFile, Logger, null, true, true);
			Remote.RetrieveFilesGeneratedOnPrimaryMac(ProjectFile, Logger, ClientPlatform);

			RemoteMac SecondaryRemote = new RemoteMac(ProjectFile, Logger, null, false);
			SecondaryRemote.UploadToSecondaryMac(ProjectFile, Logger);
		}

		/// <summary>
		/// Prepare the build and project on a Remote Mac to be able to debug an iOS or tvOS package built remotely
		/// </summary>
		/// <param name="ClientPlatform">TargetPlatform, iOS or tvOS</param>
		/// <param name="ProjectFile">Location of .uproject file</param>
		/// <param name="Logger">A logger</param>
		/// <returns></returns>
		public static void PrepareRemoteMacForDebugging(string ClientPlatform, FileReference ProjectFile, ILogger Logger)
		{
			RemoteMac Remote = new RemoteMac(ProjectFile, Logger);
			Remote.PrepareToDebug(ClientPlatform, Logger);
		}

		/// <summary>
		/// 
		/// </summary>
		/// <param name="Platform"></param>
		/// <param name="PlatformGameConfig"></param>
		/// <param name="AppName"></param>
		/// <param name="MobileProvisionFile"></param>
		/// <param name="bForDistribution"></param>
		/// <param name="IntermediateDir"></param>
		public static void WriteEntitlements(UnrealTargetPlatform Platform, ConfigHierarchy PlatformGameConfig,
		string AppName, FileReference? MobileProvisionFile, bool bForDistribution, string IntermediateDir)
		{
			// get some info from the mobileprovisioning file
			// the iCloud identifier and the bundle id may differ
			string iCloudContainerIdentifier = "";
			string iCloudContainerIdentifiersXML = "<array><string>iCloud.$(CFBundleIdentifier)</string></array>";
			string UbiquityContainerIdentifiersXML = "<array><string>iCloud.$(CFBundleIdentifier)</string></array>";
			string iCloudServicesXML = "<array><string>CloudKit</string><string>CloudDocuments</string></array>";
			string UbiquityKVStoreIdentifiersXML = "\t<string>$(TeamIdentifierPrefix)$(CFBundleIdentifier)</string>";

			string OutputFileName = Path.Combine(IntermediateDir, AppName + ".entitlements");

			if (MobileProvisionFile != null && File.Exists(MobileProvisionFile.FullName))
			{
				Console.WriteLine("Write entitlements from provisioning file {0}", MobileProvisionFile);

				MobileProvisionContents MobileProvisionContent = MobileProvisionContents.Read(MobileProvisionFile);

				iCloudContainerIdentifier = MobileProvisionContent.GetNodeValueByName("com.apple.developer.icloud-container-identifiers");
				iCloudContainerIdentifiersXML = MobileProvisionContent.GetNodeXMLValueByName("com.apple.developer.icloud-container-identifiers");

				string entitlementXML = MobileProvisionContent.GetNodeXMLValueByName("com.apple.developer.icloud-services");

				if (!entitlementXML.Contains('*', StringComparison.Ordinal) || Platform == UnrealTargetPlatform.TVOS)
				{
					// for iOS, replace the generic value (*) with the default
					iCloudServicesXML = entitlementXML;
				}

				entitlementXML = MobileProvisionContent.GetNodeXMLValueByName("com.apple.developer.ubiquity-container-identifiers");
				if (!entitlementXML.Contains('*', StringComparison.Ordinal) || !bForDistribution)
				{
					// for distribution, replace the generic value (*) with the default
					UbiquityContainerIdentifiersXML = entitlementXML;
				}

				entitlementXML = MobileProvisionContent.GetNodeXMLValueByName("com.apple.developer.ubiquity-kvstore-identifier");
				if (!entitlementXML.Contains('*', StringComparison.Ordinal) || !bForDistribution)
				{
					// for distribution, replace the generic value (*) with the default
					UbiquityKVStoreIdentifiersXML = entitlementXML;
				}
			}
			else
			{
				Console.WriteLine("Couldn't locate the mobile provisioning file {0}", MobileProvisionFile);
			}

			// write the entitlements file
			{
				bool bCloudKitSupported = false;
				PlatformGameConfig.GetBool("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bEnableCloudKitSupport", out bCloudKitSupported);
				Directory.CreateDirectory(Path.GetDirectoryName(OutputFileName)!);
				// we need to have something so Xcode will compile, so we just set the get-task-allow, since we know the value,
				// which is based on distribution or not (true means debuggable)
				StringBuilder Text = new StringBuilder();
				Text.AppendLine("<?xml version=\"1.0\" encoding=\"UTF-8\"?>");
				Text.AppendLine("<!DOCTYPE plist PUBLIC \"-//Apple//DTD PLIST 1.0//EN\" \"http://www.apple.com/DTDs/PropertyList-1.0.dtd\">");
				Text.AppendLine("<plist version=\"1.0\">");
				Text.AppendLine("<dict>");
				Text.AppendLine("\t<key>get-task-allow</key>");
				Text.AppendLine(String.Format("\t<{0}/>", bForDistribution ? "false" : "true"));
				if (bCloudKitSupported)
				{
					if (!String.IsNullOrEmpty(iCloudContainerIdentifiersXML))
					{
						Text.AppendLine("\t<key>com.apple.developer.icloud-container-identifiers</key>");
						Text.AppendLine(iCloudContainerIdentifiersXML);
					}

					if (!String.IsNullOrEmpty(iCloudServicesXML))
					{
						Text.AppendLine("\t<key>com.apple.developer.icloud-services</key>");
						Text.AppendLine(iCloudServicesXML);
					}

					if (!String.IsNullOrEmpty(UbiquityContainerIdentifiersXML))
					{
						Text.AppendLine("\t<key>com.apple.developer.ubiquity-container-identifiers</key>");
						Text.AppendLine(UbiquityContainerIdentifiersXML);
					}

					if (!String.IsNullOrEmpty(UbiquityKVStoreIdentifiersXML))
					{
						Text.AppendLine("\t<key>com.apple.developer.ubiquity-kvstore-identifier</key>");
						Text.AppendLine(UbiquityKVStoreIdentifiersXML);
					}

					Text.AppendLine("\t<key>com.apple.developer.icloud-container-environment</key>");
					Text.AppendLine(String.Format("\t<string>{0}</string>", bForDistribution ? "Production" : "Development"));
				}

				bool bRemoteNotificationsSupported = false;
				PlatformGameConfig.GetBool("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bEnableRemoteNotificationsSupport", out bRemoteNotificationsSupported);

				// for TVOS we need push notifications when building for distribution with CloudKit
				if (bCloudKitSupported && bForDistribution && Platform == UnrealTargetPlatform.TVOS)
				{
					bRemoteNotificationsSupported = true;
				}

				if (bRemoteNotificationsSupported)
				{
					Text.AppendLine("\t<key>aps-environment</key>");
					Text.AppendLine(String.Format("\t<string>{0}</string>", bForDistribution ? "production" : "development"));
				}

				// for Sign in with Apple
				bool bSignInWithAppleSupported = false;
				PlatformGameConfig.GetBool("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bEnableSignInWithAppleSupport", out bSignInWithAppleSupported);

				if (bSignInWithAppleSupported)
				{
					Text.AppendLine("\t<key>com.apple.developer.applesignin</key>");
					Text.AppendLine("\t<array><string>Default</string></array>");
				}

				// Add Multi-user support for tvOS
				bool bUserSwitching = false;
				PlatformGameConfig.GetBool("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bUserSwitching", out bUserSwitching);
				if (bUserSwitching && Platform == UnrealTargetPlatform.TVOS)
				{
					Text.AppendLine("\t<key>com.apple.developer.user-management</key>");
					Text.AppendLine("\t<array><string>runs-as-current-user</string></array>");
				}

				// As of iOS15, to support GameCenter, "com.apple.developer.game-center" entitlement must be set
				bool bEnableGameCenterSupport = false;
				PlatformGameConfig.GetBool("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bEnableGameCenterSupport", out bEnableGameCenterSupport);
				if (bEnableGameCenterSupport)
				{
					Text.AppendLine("\t<key>com.apple.developer.game-center</key>");
					Text.AppendLine("\t<true/>");
				}

				// End of entitlements
				Text.AppendLine("</dict>");
				Text.AppendLine("</plist>");

				if (File.Exists(OutputFileName))
				{
					// read existing file
					string ExisitingFileContents = File.ReadAllText(OutputFileName);
					bool bFileChanged = !ExisitingFileContents.Equals(Text.ToString(), StringComparison.Ordinal);
					// overwrite file if there are content changes
					if (bFileChanged)
					{
						File.WriteAllText(OutputFileName, Text.ToString());
					}
				}
				else
				{
					File.WriteAllText(OutputFileName, Text.ToString());
				}
			}

			// create a pList key named ICloudContainerIdentifier
			// to be used at run-time when intializing the CloudKit services
			if (!String.IsNullOrEmpty(iCloudContainerIdentifier))
			{
				string PListFile = IntermediateDir + "/" + AppName + "-Info.plist";
				if (File.Exists(PListFile))
				{
					string OldPListData = File.ReadAllText(PListFile);
					XDocument XDoc;
					try
					{
						XDoc = XDocument.Parse(OldPListData);
						if (XDoc.DocumentType != null)
						{
							XDoc.DocumentType.InternalSubset = null;
						}

						XElement? dictElement = XDoc.Root?.Element("dict");
						if (dictElement != null)
						{
							XElement containerIdKeyNew = new XElement("key", "ICloudContainerIdentifier");
							XElement containerIdValueNew = new XElement("string", iCloudContainerIdentifier);

							XElement? containerIdKey = dictElement.Elements("key").FirstOrDefault(x => x.Value == "ICloudContainerIdentifier");
							if (containerIdKey != null)
							{
								// if ICloudContainerIdentifier already exists in the pList file, update its value
								XElement? containerIdValue = containerIdKey.ElementsAfterSelf("string").FirstOrDefault();
								if (containerIdValue != null)
								{
									containerIdValue.Value = iCloudContainerIdentifier;
								}
								else
								{
									containerIdKey.AddAfterSelf(containerIdValueNew);
								}
							}
							else
							{
								// add ICloudContainerIdentifier to the pList
								dictElement.Add(containerIdKeyNew);
								dictElement.Add(containerIdValueNew);
							}

							XDoc.Save(PListFile);
						}
					}
					catch (Exception e)
					{
						throw new BuildException("plist is invalid {0}\n{1}", e, OldPListData);
					}
				}
			}
		}
	}
}
