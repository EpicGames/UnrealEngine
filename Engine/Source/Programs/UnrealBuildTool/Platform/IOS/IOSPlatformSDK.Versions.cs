// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool
{
	///////////////////////////////////////////////////////////////////////////////////////
	// If you are looking for any version numbers, see Engine/Config/IOS/IOS_SDK.json
	///////////////////////////////////////////////////////////////////////////////////////
}
