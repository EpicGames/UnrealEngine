// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace UnrealBuildTool
{
	/// <summary>
	/// Base class for platform-specific project generators
	/// </summary>
	class IOSProjectGenerator : AppleProjectGenerator
	{
		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="Arguments">Command line arguments passed to the project generator</param>
		/// <param name="Logger">Logger for output</param>
#pragma warning disable IDE0060 // Remove unused parameter - constructor is found by reflection in GenerateProjectFilesMode.ExecuteAsync
		public IOSProjectGenerator(CommandLineArguments Arguments, ILogger Logger)
			: base(Logger)
#pragma warning restore IDE0060
		{
		}

		/// <summary>
		/// Enumerate all the platforms that this generator supports
		/// </summary>
		public override IEnumerable<UnrealTargetPlatform> GetPlatforms()
		{
			yield return UnrealTargetPlatform.IOS;
		}
	}
}
