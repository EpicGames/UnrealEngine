// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.IO.Compression;
using System.Linq;
using System.Text;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	class IOSToolChain : AppleToolChain
	{
		// TODO: Should this really be static? If yes, does it need to be thread-safe?
		private static readonly List<FileItem> BundleDependencies = new List<FileItem>();

		protected IOSProjectSettings ProjectSettings;

		public IOSToolChain(ReadOnlyTargetRules? Target, IOSProjectSettings InProjectSettings, ClangToolChainOptions ToolchainOptions, ILogger InLogger)
			: this(Target, InProjectSettings, () => new IOSToolChainSettings(InProjectSettings, InLogger), ToolchainOptions, InLogger)
		{
		}

		protected IOSToolChain(ReadOnlyTargetRules? Target, IOSProjectSettings InProjectSettings, Func<AppleToolChainSettings> InCreateSettings, ClangToolChainOptions ToolchainOptions, ILogger InLogger)
			: base(Target, InCreateSettings, ToolchainOptions, InLogger)
		{
			ProjectSettings = InProjectSettings;
		}

		// ***********************************************************************
		// * NOTE:
		// *  Do NOT change the defaults to set your values, instead you should set the environment variables
		// *  properly in your system, as other tools make use of them to work properly!
		// *  The defaults are there simply for examples so you know what to put in your env vars...
		// ***********************************************************************

		// If you are looking for where to change the remote compile server name, look in RemoteToolChain.cs

		/// <summary>
		/// If this is set, then we do not do any post-compile steps -- except moving the executable into the proper spot on Mac.
		/// </summary>
		[XmlConfigFile]
		public static bool bUseDangerouslyFastMode = false;

		/// <summary>
		/// Which compiler\linker frontend to use
		/// </summary>
		private const string IOSCompiler = "clang++";

		/// <summary>
		/// Which library archiver to use
		/// </summary>
		private const string IOSArchiver = "libtool";

		protected override string XnuPlatformDefine => "XNU_PLATFORM_iPhoneOS";

		protected override ClangToolChainInfo GetToolChainInfo()
		{
			FileReference CompilerPath = FileReference.Combine(AppleToolChainSettings.ToolchainDir, IOSCompiler);
			FileReference ArchiverPath = FileReference.Combine(AppleToolChainSettings.ToolchainDir, IOSArchiver);
			return new AppleToolChainInfo(UnrealTargetPlatform.IOS, ApplePlatformSDK.DeveloperDir, CompilerPath, ArchiverPath, Logger);
		}

		public override void ModifyBuildProducts(ReadOnlyTargetRules Target, UEBuildBinary Binary, IEnumerable<string> Libraries, IEnumerable<UEBuildBundleResource> BundleResources, Dictionary<FileReference, BuildProductType> BuildProducts)
		{
			if (Target.IOSPlatform.bCreateStubIPA && Binary.Type != UEBuildBinaryType.StaticLibrary)
			{
				FileReference StubFile = FileReference.Combine(Binary.OutputFilePath.Directory, Binary.OutputFilePath.GetFileNameWithoutExtension() + ".stub");
				BuildProducts.Add(StubFile, BuildProductType.Package);
			}
			if (Target.IOSPlatform.bGeneratedSYM && ProjectSettings.bGenerateCrashReportSymbols && Binary.Type == UEBuildBinaryType.StaticLibrary)
			{
				FileReference DebugFile = FileReference.Combine(Binary.OutputFilePath.Directory, Binary.OutputFilePath.GetFileNameWithoutExtension() + ".udebugsymbols");
				BuildProducts.Add(DebugFile, BuildProductType.SymbolFile);
			}
		}

		/// <summary>
		/// Adds a build product and its associated debug file to a receipt.
		/// </summary>
		/// <param name="OutputFile">Build product to add</param>
		/// <param name="OutputType">Type of build product</param>
		public override bool ShouldAddDebugFileToReceipt(FileReference OutputFile, BuildProductType OutputType)
		{
			return OutputType == BuildProductType.Executable || OutputType == BuildProductType.DynamicLibrary;
		}

		public override FileReference GetDebugFile(FileReference OutputFile, string DebugExtension)
		{
			if (OutputFile.FullName.Contains(".framework", StringComparison.Ordinal))
			{
				// need to put the debug info outside of the framework
				return FileReference.Combine(OutputFile.Directory.ParentDirectory!, OutputFile.ChangeExtension(DebugExtension).GetFileName());
			}
			//  by default, just change the extension to the debug extension
			return OutputFile.ChangeExtension(DebugExtension);
		}

		public override ReadOnlyAppleTargetRules GetAppleTargetRules(ReadOnlyTargetRules InTarget)
		{
			return InTarget.IOSPlatform;
		}

		/// <inheritdoc/>
		protected override void GetCompileArguments_WarningsAndErrors(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			base.GetCompileArguments_WarningsAndErrors(CompileEnvironment, Arguments);

			// fix for Xcode 8.3 enabling nonportable include checks, but p4 has some invalid cases in it
			if (Settings.SDKVersionFloat >= 10.3)
			{
				Arguments.Add("-Wno-nonportable-include-path");
			}
		}

		/// <inheritdoc/>
		protected override void GetCompileArguments_Debugging(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			base.GetCompileArguments_Debugging(CompileEnvironment, Arguments);

			Arguments.Add("-fvisibility=hidden");
			Arguments.Add("-fvisibility-inlines-hidden");
		}

		/// <inheritdoc/>
		protected override void GetCompilerArguments_Sanitizers(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			base.GetCompilerArguments_Sanitizers(CompileEnvironment, Arguments);

			if (Options.HasFlag(ClangToolChainOptions.EnableAddressSanitizer))
			{
				Arguments.Add("-fno-omit-frame-pointer");
			}

			if (Options.HasFlag(ClangToolChainOptions.EnableUndefinedBehaviorSanitizer))
			{
				Arguments.Add("-fno-sanitize=bounds,enum,return,float-divide-by-zero");
			}
		}

		/// <inheritdoc/>
		protected override void GetCompileArguments_Global(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			bool bIsUsingPortableSDK = false;
			GetCompileArguments_Global(CompileEnvironment, Arguments, bIsUsingPortableSDK);
		}

		private void GetCompileArguments_Global(CppCompileEnvironment CompileEnvironment, List<string> Arguments, bool bIsUsingPortableSDK)
		{
			base.GetCompileArguments_Global(CompileEnvironment, Arguments);

			CppRootPaths RootPaths = CompileEnvironment.RootPaths;

			if (CompileEnvironment.Configuration == CppConfiguration.Shipping)
			{
				Arguments.Add("-ffunction-sections");
				Arguments.Add("-fdata-sections");
				Arguments.Add("-fno-unwind-tables");
				Arguments.Add("-fno-asynchronous-unwind-tables");

				if (!CompileEnvironment.bPGOProfile && !bMergeModules)
				{
					Arguments.Add("-fno-use-cxa-atexit");
				}
			}

			// What architecture(s) to build for
			Arguments.Add(FormatArchitectureArg(CompileEnvironment.Architectures));

			// Clang modules depend on SDK paths unavailable on remote workers
			if (bIsUsingPortableSDK)
			{
				AddPortableToolchainIncludePaths(CompileEnvironment, Arguments);
			}
			else
			{
				Arguments.Add($"-isysroot \"{NormalizeCommandLinePath(Settings.GetSDKPath(CompileEnvironment.Architecture), RootPaths)}\"");

				// Add additional frameworks so that their headers can be found
				foreach (UEBuildFramework Framework in CompileEnvironment.AdditionalFrameworks)
				{
					if (Framework.bLinkFramework)
					{
						DirectoryReference? FrameworkDirectory = Framework.GetFrameworkDirectory(CompileEnvironment.Platform, CompileEnvironment.Architecture, Logger);
						if (FrameworkDirectory != null)
						{
							// embedded frameworks have a framework inside of this directory, so we use this directory. regular frameworks need to go one up to point to the 
							// directory containing the framework. -F gives a path to look for the -framework
							if (FrameworkDirectory.FullName.EndsWith(".framework", StringComparison.Ordinal))
							{
								FrameworkDirectory = FrameworkDirectory.ParentDirectory;
							}
							Arguments.Add($"-F \"{NormalizeCommandLinePath(FrameworkDirectory!, RootPaths)}\"");
						}
					}
				}
			}
		}

		protected override void GetCompileArguments_PortableToolchain(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			bool bIsUsingPortableSDK = true;
			GetCompileArguments_Global(CompileEnvironment, Arguments, bIsUsingPortableSDK);
		}

		/// <inheritdoc/>
		protected override FileItem GetCompileArguments_FileType(CppCompileEnvironment CompileEnvironment, FileItem SourceFile, DirectoryReference OutputDir, List<string> Arguments, Action CompileAction, CPPOutput CompileResult)
		{
			FileItem TargetFile = base.GetCompileArguments_FileType(CompileEnvironment, SourceFile, OutputDir, Arguments, CompileAction, CompileResult);

			string Extension = Path.GetExtension(SourceFile.AbsolutePath).ToUpperInvariant();

			if (!Extension.Equals(".C", StringComparison.Ordinal) && !Extension.Equals(".SWIFT", StringComparison.Ordinal))
			{
				Arguments.Add(GetObjCExceptionsFlag(CompileEnvironment));
			}

			return TargetFile;
		}

		// Conditionally enable (default disabled) Objective-C exceptions
		static string GetObjCExceptionsFlag(CppCompileEnvironment CompileEnvironment)
		{
			string Result = "";

			if (CompileEnvironment.bEnableObjCExceptions)
			{
				Result += "-fobjc-exceptions";
			}
			else
			{
				Result += "-fno-objc-exceptions";
			}

			return Result;
		}

		public string GetAdditionalLinkerFlags(CppConfiguration InConfiguration)
		{
			if (InConfiguration != CppConfiguration.Shipping)
			{
				return ProjectSettings.AdditionalLinkerFlags;
			}
			else
			{
				return ProjectSettings.AdditionalShippingLinkerFlags;
			}
		}

		protected override void GetLinkArguments_Sanitizers(LinkEnvironment linkEnvironment, List<string> arguments)
		{
			base.GetLinkArguments_Sanitizers(linkEnvironment, arguments);

			if (Options.HasFlag(ClangToolChainOptions.EnableAddressSanitizer))
			{
				arguments.Add("-rpath \"@executable_path/Frameworks\"");
			}

			if (Options.HasFlag(ClangToolChainOptions.EnableUndefinedBehaviorSanitizer))
			{
				arguments.Add("-rpath \"@executable_path/libclang_rt.ubsan_ios_dynamic.dylib\"");
			}

			if (Options.HasFlag(ClangToolChainOptions.EnableThreadSanitizer))
			{
				arguments.Add("-rpath \"@executable_path/libclang_rt.tsan_ios_dynamic.dylib\"");
			}
		}

		protected override void GetLinkArguments_Global(LinkEnvironment LinkEnvironment, List<string> Arguments)
		{
			base.GetLinkArguments_Global(LinkEnvironment, Arguments);

			if (!LinkEnvironment.bIsBuildingLibrary && !LinkEnvironment.bIsBuildingDLL)
			{
				// We explicitly export our operators new/delete in binaries, so
				// dynamic linker can correctly redirect those from libc++ and
				// any other dynamic library we load.
				
				// _operator new(unsigned long)
				Arguments.Add("-Wl,-exported_symbol,__Znwm");
				// _operator new(unsigned long, std::nothrow_t const&)
				Arguments.Add("-Wl,-exported_symbol,__ZnwmRKSt9nothrow_t");
				// _operator delete(void*)
				Arguments.Add("-Wl,-exported_symbol,__ZdlPv");
				// _operator delete(void*, std::nothrow_t const&)
				Arguments.Add("-Wl,-exported_symbol,__ZdlPvRKSt9nothrow_t");
				// _operator new[](unsigned long)
				Arguments.Add("-Wl,-exported_symbol,__Znam");
				// _operator new[](unsigned long, std::nothrow_t const&)
				Arguments.Add("-Wl,-exported_symbol,__ZnamRKSt9nothrow_t");
				// _operator delete[](void*)
				Arguments.Add("-Wl,-exported_symbol,__ZdaPv");
				// _operator delete[](void*, std::nothrow_t const&)		
				Arguments.Add("-Wl,-exported_symbol,__ZdaPvRKSt9nothrow_t");
			}

			Arguments.Add("-Wl-no_pie");
			Arguments.Add("-stdlib=libc++");
			Arguments.Add("-ObjC");
			// Arguments.Add("-v");

			//Arguments.Add("-Wl,-O3");

			// need to tell where to load Framework dylibs
			Arguments.Add("-rpath @executable_path/Frameworks");

			Arguments.Add(GetAdditionalLinkerFlags(LinkEnvironment.Configuration));

			// link in the frameworks
			foreach (string Framework in LinkEnvironment.Frameworks)
			{
				if (Framework != "ARKit" || Settings.SDKVersionFloat >= 11.0f)
				{
					Arguments.Add($"-framework {Framework}");
				}
			}
			foreach (UEBuildFramework Framework in LinkEnvironment.AdditionalFrameworks)
			{
				DirectoryReference? FrameworkDirectory = Framework.GetFrameworkDirectory(LinkEnvironment.Platform, LinkEnvironment.Architecture, Logger);
				if (FrameworkDirectory != null)
				{
					// If this framework has a directory specified, we'll need to setup the path as well

					// embedded frameworks have a framework inside of this directory, so we use this directory. regular frameworks need to go one up to point to the 
					// directory containing the framework. -F gives a path to look for the -framework
					if (FrameworkDirectory.FullName.EndsWith(".framework", StringComparison.Ordinal))
					{
						FrameworkDirectory = FrameworkDirectory.ParentDirectory!;
					}
					Arguments.Add($"-F\"{NormalizeCommandLinePath(FrameworkDirectory, LinkEnvironment.RootPaths)}\"");
				}

				Arguments.Add($"-framework {Framework.Name}");
			}
			foreach (string Framework in LinkEnvironment.WeakFrameworks)
			{
				Arguments.Add($"-weak_framework {Framework}");
			}
		}

		public override FileItem LinkFiles(LinkEnvironment LinkEnvironment, bool bBuildImportLibraryOnly, IActionGraphBuilder Graph)
		{
			FileReference LinkerPath = LinkEnvironment.bIsBuildingLibrary ? Info.Archiver : Info.Clang;

			// Create an action that invokes the linker.
			Action LinkAction = Graph.CreateAction(ActionType.Link);
			string Arch = UnrealArchitectureConfig.ForPlatform(LinkEnvironment.Platform).ConvertToReadableArchitecture(LinkEnvironment.Architecture);
			LinkAction.CommandPath = LinkerPath;
			LinkAction.CommandDescription = $"{(bBuildImportLibraryOnly ? "LinkStub" : "Link")} [{Arch}]";
			LinkAction.CommandVersion = Info.ClangVersionString;
			LinkAction.bProducesImportLibrary = bBuildImportLibraryOnly || LinkEnvironment.bIsBuildingDLL;
			LinkAction.bCanExecuteInUBA = true;
			LinkAction.CacheBucket = GetCacheBucket(Target, LinkEnvironment);
			LinkAction.ArtifactMode = ArtifactMode.Enabled;
			LinkAction.RootPaths = LinkEnvironment.RootPaths;

			// RPC utility parameters are in terms of the Mac side
			LinkAction.WorkingDirectory = GetMacDevSrcRoot();

			// build this up over the rest of the function
			List<string> LinkArguments = new();
			if (LinkEnvironment.bIsBuildingLibrary)
			{
				GetArchiveArguments_Global(LinkEnvironment, LinkArguments);
			}
			else
			{
				GetLinkArguments_Global(LinkEnvironment, LinkArguments);
			}
			if (LinkEnvironment.bIsBuildingDLL)
			{
				// @todo roll this put into GetLinkArguments_Global
				LinkArguments.Add("-dynamiclib -Xlinker -export_dynamic -Xlinker -no_deduplicate");

				string InstallName = LinkEnvironment.InstallName ?? $"@executable_path/Frameworks/{LinkEnvironment.OutputFilePath.MakeRelativeTo(LinkEnvironment.OutputFilePath.Directory.ParentDirectory!)}";
				LinkArguments.Add($"-Xlinker -install_name -Xlinker {InstallName}");

				LinkArguments.Add("-Xlinker -rpath -Xlinker @executable_path/Frameworks");
				LinkArguments.Add("-Xlinker -rpath -Xlinker @loader_path/Frameworks");
			}

			if (!LinkEnvironment.bIsBuildingLibrary)
			{
				// Add the library paths to the argument list.
				foreach (DirectoryReference LibraryPath in LinkEnvironment.SystemLibraryPaths)
				{
					LinkArguments.Add($"-L\"{NormalizeCommandLinePath(LibraryPath, LinkEnvironment.RootPaths)}\"");
				}

				// Add the additional libraries to the argument list.
				foreach (string AdditionalLibrary in LinkEnvironment.SystemLibraries)
				{
					if (AdditionalLibrary.Contains(".framework/", StringComparison.Ordinal))
					{
						LinkArguments.Add($"-l\"{NormalizeCommandLinePath(new FileReference(AdditionalLibrary), LinkEnvironment.RootPaths)}\"");
					}
					else if (String.IsNullOrEmpty(Path.GetDirectoryName(AdditionalLibrary)) && String.IsNullOrEmpty(Path.GetExtension(AdditionalLibrary)))
					{
						LinkArguments.Add($"-l\"{AdditionalLibrary}\"");
					}
					else
					{
						FileReference LibraryPath = new(AdditionalLibrary);
						LinkArguments.Add($"-l\"{NormalizeCommandLinePath(LibraryPath, LinkEnvironment.RootPaths)}\"");
					}
				}

				foreach (FileReference LibraryPath in LinkEnvironment.Libraries)
				{
					// for absolute library paths, convert to remote filename
					// add it to the prerequisites to make sure it's built first (this should be the case of non-system libraries)
					LinkAction.PrerequisiteItems.Add(FileItem.GetItemByFileReference(LibraryPath));

					// and add to the commandline
					LinkArguments.Add($"\"{NormalizeCommandLinePath(LibraryPath, LinkEnvironment.RootPaths)}\"");
				}
			}

			// Handle additional framework assets that might need to be shadowed
			foreach (UEBuildFramework Framework in LinkEnvironment.AdditionalFrameworks)
			{
				if (Framework.ZipFile != null)
				{
					FileItem ExtractedTokenFile = ExtractFramework(Framework, LinkEnvironment.RootPaths, Graph);
					LinkAction.PrerequisiteItems.Add(ExtractedTokenFile);
				}
			}

			// Add the output file as a production of the link action.
			FileItem OutputFile = FileItem.GetItemByFileReference(LinkEnvironment.OutputFilePath);
			LinkAction.ProducedItems.Add(OutputFile);

			// Add arguments to generate a map file too
			if ((!LinkEnvironment.bIsBuildingLibrary || LinkEnvironment.bIsBuildingDLL) && LinkEnvironment.bCreateMapFile)
			{
				FileItem MapFile = FileItem.GetItemByFileReference(new FileReference(OutputFile.Location.FullName + ".map"));
				LinkArguments.Add($"-Wl,-map,\"{NormalizeCommandLinePath(MapFile, LinkEnvironment.RootPaths)}\"");
				LinkAction.ProducedItems.Add(MapFile);
			}

			// Add the input files to a response file, and pass the response file on the command-line.
			foreach (FileItem InputFile in LinkEnvironment.InputFiles)
			{
				LinkArguments.Add($"\"{NormalizeCommandLinePath(InputFile, LinkEnvironment.RootPaths)}\"");
				LinkAction.PrerequisiteItems.Add(InputFile);
			}

			// if we are making an LTO build, write the lto file next to build so dsymutil can find it
			if (LinkEnvironment.bAllowLTCG)
			{
				DirectoryReference LtoObjectDir;
				if (Target!.bShouldCompileAsDLL)
				{
					// go up a directory so we don't put this big file into the framework
					LtoObjectDir = DirectoryReference.Combine(OutputFile.Directory.Location.ParentDirectory!, $"{OutputFile.Name}.lto.o");
				}
				else
				{
					LtoObjectDir = DirectoryReference.Combine(OutputFile.Directory.Location, $"{OutputFile.Name}.lto.o");
				}
				LinkArguments.Add($"-flto -Xlinker -object_path_lto -Xlinker \"{NormalizeCommandLinePath(LtoObjectDir, LinkEnvironment.RootPaths)}\"");
			}

			// Add the output file to the command-line.
			LinkArguments.Add($"-o \"{NormalizeCommandLinePath(OutputFile, LinkEnvironment.RootPaths)}\"");

			// Add the additional arguments specified by the environment.
			LinkArguments.Add(LinkEnvironment.AdditionalArguments);

			FileReference ResponseFileName;
			if (bBuildImportLibraryOnly)
			{
				ResponseFileName = new(OutputFile.FullName + ResponseExt);
			}
			else
			{
				ResponseFileName = GetResponseFileName(LinkEnvironment, OutputFile);
			}

			LinkAction.ResponseFileContents = [.. LinkArguments];
			FileItem ResponseFileItem = Graph.CreateIntermediateTextFile(ResponseFileName, LinkAction.ResponseFileContents);
			string ResponseFileArgument = GetResponseFileArgument(ResponseFileItem, LinkEnvironment.RootPaths);

			LinkAction.PrerequisiteItems.Add(ResponseFileItem);

			LinkAction.CommandArguments = ResponseFileArgument;

			// Only execute linking on the local PC.
			LinkAction.bCanExecuteRemotely = false;

			LinkAction.StatusDescription = OutputFile.Location.GetFileName();

			if (LinkEnvironment.Configuration == CppConfiguration.Shipping && Path.GetExtension(OutputFile.AbsolutePath) != ".a")
			{
				// When building a shipping package, symbols are stripped from the exe as the last build step. This is a problem
				// when re-packaging and no source files change because the linker skips symbol generation and dsymutil will 
				// recreate a new .dsym file from a symboless exe file. It's just sad. To make things happy we need to delete 
				// the output file to force the linker to recreate it with symbols again.
				LinkAction.DeleteItems.Add(OutputFile);
			}

			return OutputFile;
		}

		static string GetAppBundleName(FileReference Executable)
		{
			// Get the app bundle name
			string AppBundleName = Executable.GetFileNameWithoutExtension();

			// Strip off any platform suffix
			int SuffixIdx = AppBundleName.IndexOf('-', StringComparison.Ordinal);
			if (SuffixIdx != -1)
			{
				AppBundleName = AppBundleName.Substring(0, SuffixIdx);
			}

			// Append the .app suffix
			return AppBundleName + ".app";
		}

		public static FileReference GetAssetCatalogFile(UnrealTargetPlatform Platform, FileReference Executable)
		{
			// Get the output file
			if (Platform == UnrealTargetPlatform.IOS)
			{
				return FileReference.Combine(Executable.Directory, "Payload", GetAppBundleName(Executable), "Assets.car");
			}
			else
			{
				return FileReference.Combine(Executable.Directory, "AssetCatalog", "Assets.car");
			}
		}

		public static string GetAssetCatalogArgs(UnrealTargetPlatform Platform, string InputDir, string OutputDir)
		{
			StringBuilder Arguments = new StringBuilder("actool");
			Arguments.Append(" --output-format human-readable-text");
			Arguments.Append(" --notices");
			Arguments.Append(" --warnings");
			Arguments.AppendFormat(" --output-partial-info-plist \"{0}/assetcatalog_generated_info.plist\"", InputDir);
			if (Platform == UnrealTargetPlatform.TVOS)
			{
				Arguments.Append(" --app-icon \"App Icon & Top Shelf Image\"");
				Arguments.Append(" --launch-image \"Launch Image\"");
				Arguments.Append(" --filter-for-device-model AppleTV5,3");
				Arguments.Append(" --target-device tv");
				Arguments.Append(" --minimum-deployment-target 15.0");
				Arguments.Append(" --platform appletvos");
			}
			else
			{
				Arguments.Append(" --app-icon AppIcon");
				Arguments.Append(" --product-type com.apple.product-type.application");
				Arguments.Append(" --target-device iphone");
				Arguments.Append(" --target-device ipad");
				Arguments.Append(" --minimum-deployment-target 15.0");
				Arguments.Append(" --platform iphoneos");
			}
			Arguments.Append(" --enable-on-demand-resources YES");
			Arguments.AppendFormat(" --compile \"{0}\"", OutputDir);
			Arguments.AppendFormat(" \"{0}/Assets.xcassets\"", InputDir);
			return Arguments.ToString();
		}

		private static bool IsCompiledAsFramework(string ExecutablePath)
		{
			// @todo ios: Get the receipt to here which has the property
			return ExecutablePath.Contains(".framework", StringComparison.Ordinal);
		}

		private static string GetdSYMPath(FileItem Executable)
		{
			string ExecutablePath = Executable.AbsolutePath;
			// for frameworks, we want to put the .dSYM outside of the framework, and the executable is inside the .framework
			if (IsCompiledAsFramework(ExecutablePath))
			{
				return Path.Combine(Path.GetDirectoryName(ExecutablePath)!, "..", Path.GetFileName(ExecutablePath) + ".dSYM");
			}

			// return standard dSYM location
			return Path.Combine(Path.GetDirectoryName(ExecutablePath)!, Path.GetFileName(ExecutablePath) + ".dSYM");
		}

		/// <summary>
		/// Generates debug info for a given executable
		/// </summary>
		/// <param name="Executable">FileItem describing the executable to generate debug info for</param>
		/// <param name="Graph">List of actions to be executed. Additional actions will be added to this list.</param>
		/// <param name="Logger">Logger for output</param>
		private List<FileItem> GenerateDebugInfo(FileItem Executable, IActionGraphBuilder Graph, ILogger Logger)
		{
			// Make a file item for the source and destination files
			string FullDestPathRoot = GetdSYMPath(Executable);

			FileItem OutputFile = FileItem.GetItemByPath(FullDestPathRoot);
			FileItem ZipOutputFile = FileItem.GetItemByPath(FullDestPathRoot + ".zip");

			// Make the compile action
			Action GenDebugAction = Graph.CreateAction(ActionType.GenerateDebugInfo);

			GenDebugAction.WorkingDirectory = GetMacDevSrcRoot();
			GenDebugAction.CommandPath = BuildHostPlatform.Current.Shell;
			string DsymutilPath = "/usr/bin/dsymutil";
			if (ProjectSettings.bGeneratedSYMBundle)
			{
				GenDebugAction.CommandArguments = String.Format("-c \"rm -rf \\\"{2}\\\"; \\\"{0}\\\" \\\"{1}\\\" -o \\\"{2}\\\"; cd \\\"{2}/..\\\"; zip -r -y -1 {3}.zip {3}\"",
					DsymutilPath,
					Executable.AbsolutePath,
					OutputFile.AbsolutePath,
					Path.GetFileName(FullDestPathRoot));
				GenDebugAction.ProducedItems.Add(ZipOutputFile);
				Logger.LogInformation("Zip file: {File}", ZipOutputFile.AbsolutePath);
			}
			else
			{
				GenDebugAction.CommandArguments = String.Format("-c \"rm -rf \\\"{2}\\\"; \\\"{0}\\\" \\\"{1}\\\" -f -o \\\"{2}\\\"\"",
						DsymutilPath,
						Executable.AbsolutePath,
						OutputFile.AbsolutePath);
			}
			GenDebugAction.PrerequisiteItems.Add(Executable);
			GenDebugAction.ProducedItems.Add(OutputFile);
			GenDebugAction.StatusDescription = GenDebugAction.CommandArguments;// string.Format("Generating debug info for {0}", Path.GetFileName(Executable.AbsolutePath));
			GenDebugAction.bCanExecuteRemotely = false;

			return GenDebugAction.ProducedItems.ToList(); // (ProjectSettings.bGeneratedSYMBundle ? ZipOutputFile : OutputFile);
		}

		/// <summary>
		/// Generates pseudo pdb info for a given executable
		/// </summary>
		/// <param name="Executable">FileItem describing the executable to generate debug info for</param>
		/// <param name="Graph">List of actions to be executed. Additional actions will be added to this list.</param>
		public static FileItem GeneratePseudoPDB(FileItem Executable, IActionGraphBuilder Graph)
		{
			// Make a file item for the source and destination files
			string FulldSYMPathRoot = GetdSYMPath(Executable);
			string FullDestPathRoot = Path.ChangeExtension(FulldSYMPathRoot, ".udebugsymbols");
			string PathToDWARF = Path.Combine(FulldSYMPathRoot, "Contents", "Resources", "DWARF", Path.GetFileName(Executable.AbsolutePath));

			FileItem dSYMFile = FileItem.GetItemByPath(FulldSYMPathRoot);

			FileItem DWARFFile = FileItem.GetItemByPath(PathToDWARF);

			FileItem OutputFile = FileItem.GetItemByPath(FullDestPathRoot);

			// Make the compile action
			Action GenDebugAction = Graph.CreateAction(ActionType.GenerateDebugInfo);
			GenDebugAction.WorkingDirectory = DirectoryReference.Combine(Unreal.EngineDirectory, "Binaries", "Mac");

			GenDebugAction.CommandPath = BuildHostPlatform.Current.Shell;
			GenDebugAction.CommandArguments = String.Format("-c \"rm -rf \\\"{1}\\\"; dwarfdump --uuid \\\"{3}\\\" | cut -d\\  -f2; chmod 777 ./DsymExporter; ./DsymExporter -UUID=$(dwarfdump --uuid \\\"{3}\\\" | cut -d\\  -f2) \\\"{0}\\\" \\\"{2}\\\"\"",
					DWARFFile.AbsolutePath,
					OutputFile.AbsolutePath,
					Path.GetDirectoryName(OutputFile.AbsolutePath),
					dSYMFile.AbsolutePath);
			GenDebugAction.PrerequisiteItems.Add(dSYMFile);
			GenDebugAction.ProducedItems.Add(OutputFile);
			GenDebugAction.StatusDescription = GenDebugAction.CommandArguments;// string.Format("Generating debug info for {0}", Path.GetFileName(Executable.AbsolutePath));
			GenDebugAction.bCanExecuteRemotely = false;

			return OutputFile;
		}

		public static void PackageStub(string BinaryPath, string GameName, string ExeName, bool bIsUsingAppleOnWindowsMode, bool bPerformDummySigning = false)
		{
			// create the ipa
			string IPAName = BinaryPath + "/" + ExeName + ".stub";
			// delete the old one
			if (File.Exists(IPAName))
			{
				File.Delete(IPAName);
			}

			// make the subdirectory if needed
			string DestSubdir = Path.GetDirectoryName(IPAName)!;
			if (!Directory.Exists(DestSubdir))
			{
				Directory.CreateDirectory(DestSubdir);
			}

			// set up the directories (xcode has .app named with the config names in them, like UnrealGame-IOS-Shipping, which the ExeName will be)
			string ZipWorkingDir = !bIsUsingAppleOnWindowsMode ? $"Payload/{ExeName}.app/" : $"Payload/{GameName}.app/";
			string ZipSourceDir = !bIsUsingAppleOnWindowsMode ? $"{BinaryPath}/{ExeName}.app" : $"{BinaryPath}/Payload/{GameName}.app";

			if (bPerformDummySigning)
			{
				// with modern, we don't codesign when building the source .app due to keychain issues over ssh and Xcode wants provision setup, even if using the - identity
				// so, now codesign right at the end with the dummy signature so that there is signature at all, which IPP on windows will replace
				int ExitCode;
				string Output = Utils.RunLocalProcessAndReturnStdOut("/bin/sh", $"-c 'codesign -f -s - \"{ZipSourceDir}\"", null, out ExitCode);
				if (ExitCode != 0)
				{
					throw new BuildException($"Failed to codesign with dummy identity, which should never happen [ExitCode = {ExitCode}, output = {Output}");
				}
				// also codesign the frameworks
				string FrameworksDir = Path.Combine(ZipSourceDir, "Frameworks");
				if (Directory.Exists(FrameworksDir))
				{
					foreach (string FrameworkDir in Directory.EnumerateDirectories(FrameworksDir))
					{
						Output = Utils.RunLocalProcessAndReturnStdOut("/bin/sh", $"-c 'codesign -f -s - \"{FrameworkDir}\"", null, out ExitCode);
						if (ExitCode != 0)
						{
							throw new BuildException($"Failed to codesign with dummy identity, which should never happen [ExitCode = {ExitCode}, output = {Output}");
						}
					}
				}
			}

			// Please note this archiving code does not support Apple extended file attributes (xattrs).
			// Currently, this code is only reachable on Mac platforms - bear in mind that it may not set Unix file attribute bits correctly
			// if it is made to run from Windows, and thus may produce app packages that are not capable of running without further modification.
			using ZipArchive Zip = ZipFile.Open(IPAName, ZipArchiveMode.Create);
			DirectoryReference SourceDir = new(ZipSourceDir);

			foreach (string FileSystemEntryPath in Directory.EnumerateFiles(ZipSourceDir, "*", SearchOption.AllDirectories))
			{
				string RelativeSourcePath = new FileReference(FileSystemEntryPath).MakeRelativeTo(SourceDir);
				string DestinationPath = Path.Combine(ZipWorkingDir, RelativeSourcePath);

				// The executable will be encrypted in the final distribution IPA and will compress very poorly, so keeping it
				// uncompressed gives a better indicator of IPA size for our distro builds
				bool bIsExecutable = Path.GetFileNameWithoutExtension(FileSystemEntryPath).Equals(GameName, StringComparison.OrdinalIgnoreCase);
				CompressionLevel CompressionLevel = bIsExecutable ? CompressionLevel.NoCompression : CompressionLevel.Optimal;

				Zip.CreateEntryFromFile(FileSystemEntryPath, DestinationPath, CompressionLevel);
			}
		}

		public static DirectoryReference GenerateAssetCatalog(FileReference? ProjectFile, UnrealTargetPlatform Platform, ref bool bUserImagesExist)
		{
			string EngineDir = Unreal.EngineDirectory.ToString();
			string BuildDir = (((ProjectFile != null) ? ProjectFile.Directory.ToString() : (String.IsNullOrEmpty(UnrealBuildTool.GetRemoteIniPath()) ? Unreal.EngineDirectory.ToString() : UnrealBuildTool.GetRemoteIniPath()))) + "/Build/" + (Platform == UnrealTargetPlatform.IOS ? "IOS" : "TVOS");
			string IntermediateDir = (((ProjectFile != null) ? ProjectFile.Directory.ToString() : Unreal.EngineDirectory.ToString())) + "/Intermediate/" + (Platform == UnrealTargetPlatform.IOS ? "IOS" : "TVOS");

			bUserImagesExist = false;

			string ResourcesDir = Path.Combine(IntermediateDir, "Resources");
			if (Platform == UnrealTargetPlatform.TVOS)
			{
				// copy the template asset catalog to the appropriate directory
				string Dir = Path.Combine(ResourcesDir, "Assets.xcassets");
				if (!Directory.Exists(Dir))
				{
					Directory.CreateDirectory(Dir);
				}
				// create the directories
				foreach (string directory in Directory.EnumerateDirectories(Path.Combine(EngineDir, "Build", "TVOS", "Resources", "Assets.xcassets"), "*", SearchOption.AllDirectories))
				{
					Dir = directory.Replace(Path.Combine(EngineDir, "Build", "TVOS"), IntermediateDir, StringComparison.Ordinal);
					if (!Directory.Exists(Dir))
					{
						Directory.CreateDirectory(Dir);
					}
				}
				// copy the default files
				foreach (string file in Directory.EnumerateFiles(Path.Combine(EngineDir, "Build", "TVOS", "Resources", "Assets.xcassets"), "*", SearchOption.AllDirectories))
				{
					Dir = file.Replace(Path.Combine(EngineDir, "Build", "TVOS"), IntermediateDir, StringComparison.Ordinal);
					File.Copy(file, Dir, true);
					FileInfo DestFileInfo = new FileInfo(Dir);
					DestFileInfo.Attributes &= ~FileAttributes.ReadOnly;
				}
				// copy the icons from the game directory if it has any
				string[][] Images = {
					new string []{ "Icon_Large_Back-1280x768.png", "App Icon & Top Shelf Image.brandassets/App Icon - Large.imagestack/Back.imagestacklayer/Content.imageset" },
					new string []{ "Icon_Large_Front-1280x768.png", "App Icon & Top Shelf Image.brandassets/App Icon - Large.imagestack/Front.imagestacklayer/Content.imageset" },
					new string []{ "Icon_Large_Middle-1280x768.png", "App Icon & Top Shelf Image.brandassets/App Icon - Large.imagestack/Middle.imagestacklayer/Content.imageset" },
					new string []{ "Icon_Small_Back-400x240.png", "App Icon & Top Shelf Image.brandassets/App Icon.imagestack/Back.imagestacklayer/Content.imageset" },
					new string []{ "Icon_Small_Back-800x480.png", "App Icon & Top Shelf Image.brandassets/App Icon.imagestack/Back.imagestacklayer/Content.imageset" },
					new string []{ "Icon_Small_Front-400x240.png", "App Icon & Top Shelf Image.brandassets/App Icon.imagestack/Front.imagestacklayer/Content.imageset" },
					new string []{ "Icon_Small_Front-800x480.png", "App Icon & Top Shelf Image.brandassets/App Icon.imagestack/Front.imagestacklayer/Content.imageset" },
					new string []{ "Icon_Small_Middle-400x240.png", "App Icon & Top Shelf Image.brandassets/App Icon.imagestack/Middle.imagestacklayer/Content.imageset" },
					new string []{ "Icon_Small_Middle-800x480.png", "App Icon & Top Shelf Image.brandassets/App Icon.imagestack/Middle.imagestacklayer/Content.imageset" },
					new string []{ "TopShelfWide-2320x720@2x.png", "App Icon & Top Shelf Image.brandassets/Top Shelf Image Wide.imageset" },
					new string []{ "TopShelfWide-2320x720.png", "App Icon & Top Shelf Image.brandassets/Top Shelf Image Wide.imageset" },
				};
				Dir = Path.Combine(IntermediateDir, "Resources", "Assets.xcassets");

				string BuildResourcesGraphicsDir = Path.Combine(BuildDir, "Resources", "Assets.xcassets");
				for (int Index = 0; Index < Images.Length; ++Index)
				{
					string SourceDir = Path.Combine((Directory.Exists(BuildResourcesGraphicsDir) ? (BuildDir) : (Path.Combine(EngineDir, "Build", "TVOS"))),
						"Resources",
						"Assets.xcassets");
					string Image = Path.Combine(SourceDir, Images[Index][1], Images[Index][0]);

					if (File.Exists(Image))
					{
						bUserImagesExist |= Image.StartsWith(BuildResourcesGraphicsDir, StringComparison.Ordinal);

						File.Copy(Image, Path.Combine(Dir, Images[Index][1], Images[Index][0]), true);
						FileInfo DestFileInfo = new FileInfo(Path.Combine(Dir, Images[Index][1], Images[Index][0]));
						DestFileInfo.Attributes &= ~FileAttributes.ReadOnly;
					}
				}
			}
			else
			{
				// copy the template asset catalog to the appropriate directory
				string Dir = Path.Combine(ResourcesDir, "Assets.xcassets");
				if (!Directory.Exists(Dir))
				{
					Directory.CreateDirectory(Dir);
				}
				// create the directories
				foreach (string directory in Directory.EnumerateDirectories(Path.Combine(EngineDir, "Build", "IOS", "Resources", "Assets.xcassets"), "*", SearchOption.AllDirectories))
				{
					Dir = directory.Replace(Path.Combine(EngineDir, "Build", "IOS"), IntermediateDir, StringComparison.Ordinal);
					if (!Directory.Exists(Dir))
					{
						Directory.CreateDirectory(Dir);
					}
				}
				// copy the default files
				foreach (string file in Directory.EnumerateFiles(Path.Combine(EngineDir, "Build", "IOS", "Resources", "Assets.xcassets"), "*", SearchOption.AllDirectories))
				{
					Dir = file.Replace(Path.Combine(EngineDir, "Build", "IOS"), IntermediateDir, StringComparison.Ordinal);
					File.Copy(file, Dir, true);
					FileInfo DestFileInfo = new FileInfo(Dir);
					DestFileInfo.Attributes &= ~FileAttributes.ReadOnly;
				}
				// copy the icons from the game directory if it has any
				string[][] Images = {
					new string []{ "IPhoneIcon20@2x.png", "Icon20@2x.png" },
					new string []{ "IPhoneIcon20@3x.png", "Icon20@3x.png" },
					new string []{ "IPhoneIcon29@2x.png", "Icon29@2x.png" },
					new string []{ "IPhoneIcon29@3x.png", "Icon29@3x.png" },
					new string []{ "IPhoneIcon40@2x.png", "Icon40@2x.png" },
					new string []{ "IPhoneIcon40@3x.png", "Icon40@3x.png" },
					new string []{ "IPhoneIcon60@2x.png", "Icon60@2x.png" },
					new string []{ "IPhoneIcon60@3x.png", "Icon60@3x.png" },
					new string []{ "IPadIcon20@2x.png", "Icon20@2x.png"},
					new string []{ "IPadIcon29@2x.png", "Icon29@2x.png"},
					new string []{ "IPadIcon40@2x.png", "Icon40@2x.png" },
					new string []{ "IPadIcon76@2x.png", "Icon76@2x.png"},
					new string []{ "IPadIcon83.5@2x.png", "Icon83.5@2x.png"},
					new string []{ "Icon1024.png", "Icon1024.png" },
				};
				Dir = Path.Combine(IntermediateDir, "Resources", "Assets.xcassets", "AppIcon.appiconset");

				string BuildResourcesGraphicsDir = Path.Combine(BuildDir, "Resources", "Graphics");
				for (int Index = 0; Index < Images.Length; ++Index)
				{
					string Image = Path.Combine((Directory.Exists(Path.Combine(BuildDir, "Resources", "Graphics")) ? (BuildDir) : (Path.Combine(EngineDir, "Build", "IOS"))), "Resources", "Graphics", Images[Index][1]);
					if (File.Exists(Image))
					{
						bUserImagesExist |= Image.StartsWith(BuildResourcesGraphicsDir, StringComparison.Ordinal);

						File.Copy(Image, Path.Combine(Dir, Images[Index][0]), true);
						FileInfo DestFileInfo = new FileInfo(Path.Combine(Dir, Images[Index][0]));
						DestFileInfo.Attributes &= ~FileAttributes.ReadOnly;
					}
				}

				StringBuilder ContentsJson = new StringBuilder();

				ContentsJson.AppendLine("{");
				ContentsJson.AppendLine("\"images\" : [");
				ContentsJson.AppendLine("{");
				ContentsJson.AppendLine("\"size\" : \"60x60\",");
				ContentsJson.AppendLine("\"idiom\" : \"iphone\",");
				ContentsJson.AppendLine("\"filename\" : \"IPhoneIcon60@2x.png\",");
				ContentsJson.AppendLine("\"scale\" : \"2x\"");
				ContentsJson.AppendLine("},");
				ContentsJson.AppendLine("{");
				ContentsJson.AppendLine("\"size\" : \"76x76\",");
				ContentsJson.AppendLine("\"idiom\" : \"ipad\",");
				ContentsJson.AppendLine("\"filename\" : \"IPadIcon76@2x.png\",");
				ContentsJson.AppendLine("\"scale\" : \"2x\"");
				ContentsJson.AppendLine("},");
				ContentsJson.AppendLine("{");
				ContentsJson.AppendLine("\"size\" : \"83.5x83.5\",");
				ContentsJson.AppendLine("\"idiom\" : \"ipad\",");
				ContentsJson.AppendLine("\"filename\" : \"IPadIcon83.5@2x.png\",");
				ContentsJson.AppendLine("\"scale\" : \"2x\"");
				ContentsJson.AppendLine("},");
				ContentsJson.AppendLine("{");
				ContentsJson.AppendLine("\"size\" : \"1024x1024\",");
				ContentsJson.AppendLine("\"idiom\" : \"ios-marketing\",");
				ContentsJson.AppendLine("\"filename\" : \"Icon1024.png\",");
				ContentsJson.AppendLine("\"scale\" : \"1x\"");
				ContentsJson.AppendLine("},");

				string[][] IconsInfo = {
					new string []{ "IPhoneIcon20@2x.png", "\"20x20\"",  "\"iphone\"", "\"2x\"" },
					new string []{ "IPhoneIcon20@3x.png", "\"20x20\"", "\"iphone\"", "\"3x\"" },
					new string []{ "IPhoneIcon29@2x.png", "\"29x29\"", "\"iphone\"", "\"2x\"" },
					new string []{ "IPhoneIcon29@3x.png", "\"29x29\"", "\"iphone\"", "\"3x\"" },
					new string []{ "IPhoneIcon40@2x.png", "\"40x40\"", "\"iphone\"", "\"2x\"" },
					new string []{ "IPhoneIcon40@3x.png", "\"40x40\"", "\"iphone\"", "\"3x\"" },
					new string []{ "IPhoneIcon60@3x.png", "\"60x60\"", "\"iphone\"", "\"3x\"" },
					new string []{ "IPadIcon20@2x.png", "\"20x20\"", "\"ipad\"", "\"2x\"" },
					new string []{ "IPadIcon29@2x.png", "\"29x29\"", "\"ipad\"", "\"2x\"" },
					new string []{ "IPadIcon40@2x.png", "\"40x40\"", "\"ipad\"", "\"2x\"" },
					};

				for (int Index = 0; Index < IconsInfo.Length; ++Index)
				{
					if (File.Exists(Path.Combine(Dir, IconsInfo[Index][0])))
					{
						ContentsJson.AppendLine("{");
						ContentsJson.AppendLine("\"size\" : " + IconsInfo[Index][1] + ",");
						ContentsJson.AppendLine("\"idiom\" : " + IconsInfo[Index][2] + ",");
						ContentsJson.AppendLine("\"filename\" : \"" + IconsInfo[Index][0] + "\",");
						ContentsJson.AppendLine("\"scale\" : " + IconsInfo[Index][3]);
						ContentsJson.AppendLine("},");
					}
				}

				ContentsJson.AppendLine("],");
				ContentsJson.AppendLine("\"info\" : {");
				ContentsJson.AppendLine("\"version\" : 1,");
				ContentsJson.AppendLine("\"author\" : \"xcode\"");
				ContentsJson.AppendLine("}");
				ContentsJson.AppendLine("}");

				string ContentsFile = Path.Combine(IntermediateDir, "Resources", "Assets.xcassets", "AppIcon.appiconset", "Contents.json");
				File.WriteAllText(ContentsFile, ContentsJson.ToString());

			}
			return new DirectoryReference(ResourcesDir);
		}

		public override ICollection<FileItem> PostBuild(ReadOnlyTargetRules Target, FileItem Executable, LinkEnvironment BinaryLinkEnvironment, IActionGraphBuilder Graph)
		{
			List<FileItem> OutputFiles = new List<FileItem>(base.PostBuild(Target, Executable, BinaryLinkEnvironment, Graph));

			if (BinaryLinkEnvironment.bIsBuildingLibrary)
			{
				return OutputFiles;
			}

			// For IOS/tvOS, generate the dSYM file if needed or requested
			if (Target!.IOSPlatform.bGeneratedSYM)
			{
				List<FileItem> Files = GenerateDebugInfo(Executable, Graph, Logger);
				foreach (FileItem item in Files)
				{
					OutputFiles.Add(item);
				}
				if (ProjectSettings.bGenerateCrashReportSymbols)
				{
					OutputFiles.Add(GeneratePseudoPDB(Executable, Graph));
				}
			}

			// The base class AppleToolchain makes the shared postbuild sync object, so we are done
			return OutputFiles;
		}

		public void StripSymbols(FileReference SourceFile, FileReference TargetFile)
		{
			StripSymbolsWithXcode(SourceFile, TargetFile, AppleToolChainSettings.ToolchainDir);
		}

		public override void SetupBundleDependencies(ReadOnlyTargetRules Target, IEnumerable<UEBuildBinary> Binaries, string GameName)
		{
			base.SetupBundleDependencies(Target, Binaries, GameName);

			foreach (UEBuildBinary Binary in Binaries)
			{
				BundleDependencies.Add(FileItem.GetItemByFileReference(Binary.OutputFilePath));
			}
		}
	};
}
