// Copyright Epic Games, Inc. All Rights Reserved.

using Microsoft.Extensions.Logging;

namespace UnrealBuildTool
{
	class IOSToolChainSettings : AppleToolChainSettings
	{
		public IOSToolChainSettings(IOSProjectSettings ProjectSettings, ILogger Logger)
			: this("iPhoneOS", "iPhoneSimulator", "ios", ProjectSettings, Logger)
		{
		}

		protected IOSToolChainSettings(string DevicePlatformName, string SimulatorPlatformName, string TargetOSName, IOSProjectSettings ProjectSettings, ILogger Logger)
			: base(DevicePlatformName, SimulatorPlatformName, TargetOSName, ProjectSettings.RuntimeVersion, true, Logger)
		{
		}
	}
}
