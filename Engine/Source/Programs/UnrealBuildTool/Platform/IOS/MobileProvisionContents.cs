// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Xml;
using EpicGames.Core;

namespace UnrealBuildTool
{
	/// <summary>
	/// Allows interrogating the contents of a .mobileprovision
	/// </summary>
	class MobileProvisionContents
	{
		/// <summary>
		/// The contents of the provision
		/// </summary>
		readonly XmlDocument Document;

		/// <summary>
		/// Map of key names to XML elements holding their values
		/// </summary>
		readonly Dictionary<string, XmlElement> NameToValue = new Dictionary<string, XmlElement>();

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="Document">XML file to create the mobile provision from. Call Read() to read from a signed file on disk.</param>
		public MobileProvisionContents(XmlDocument Document)
		{
			this.Document = Document;

			XmlNodeList? Nodes = Document.SelectNodes("/plist/dict/key");
			if (Nodes != null)
			{
				foreach (XmlElement? KeyElement in Nodes)
				{
					if (KeyElement == null)
					{
						continue;
					}

					XmlNode? ValueNode = KeyElement.NextSibling;
					while (ValueNode != null)
					{
						XmlElement? ValueElement = ValueNode as XmlElement;
						if (ValueElement != null)
						{
							NameToValue[KeyElement.InnerText] = ValueElement;
							break;
						}
					}
				}
			}
		}

		/// <summary>
		/// Gets the unique id for this mobileprovision
		/// </summary>
		/// <returns>UUID for the provision</returns>
		public string GetUniqueId()
		{
			XmlElement? UniqueIdElement;
			if (!NameToValue.TryGetValue("UUID", out UniqueIdElement))
			{
				throw new BuildException("Missing UUID in MobileProvision");
			}
			return UniqueIdElement.InnerText;
		}

		/// <summary>
		/// Reads a mobileprovision from a file on disk
		/// </summary>
		/// <param name="Location">Path to the file</param>
		/// <returns>New mobile provision instance</returns>
		public static MobileProvisionContents Read(FileReference Location)
		{
			XmlDocument Document = ReadXml(Location);
			return new MobileProvisionContents(Document);
		}

		/// <summary>
		/// Reads the plist file inside a mobileprovision
		/// </summary>
		/// <param name="Location">Path to the file</param>
		/// <returns>XML plist extracted from the mobile provision</returns>
		public static XmlDocument ReadXml(FileReference Location)
		{
			// Provision data is stored as PKCS7-signed file in ASN.1 BER format
			using (BinaryReader Reader = new BinaryReader(File.Open(Location.FullName, FileMode.Open, FileAccess.Read)))
			{
				long Length = Reader.BaseStream.Length;
				while (Reader.BaseStream.Position < Length)
				{
					Asn.FieldInfo Field = Asn.ReadField(Reader);
					if (Field.Tag == Asn.FieldTag.OBJECT_IDENTIFIER)
					{
						int[] Identifier = Asn.ReadObjectIdentifier(Reader, Field.Length);
						if (Enumerable.SequenceEqual(Identifier, Asn.ObjectIdentifier.Pkcs7_Data))
						{
							while (Reader.BaseStream.Position < Length)
							{
								Asn.FieldInfo NextField = Asn.ReadField(Reader);
								if (NextField.Tag == Asn.FieldTag.OCTET_STRING)
								{
									byte[] Data = Reader.ReadBytes(NextField.Length);

									XmlDocument Document = new XmlDocument();
									Document.XmlResolver = null;
									Document.Load(new MemoryStream(Data));
									return Document;
								}
								else
								{
									Asn.SkipValue(Reader, NextField);
								}
							}
						}
					}
					else
					{
						Asn.SkipValue(Reader, Field);
					}
				}
				throw new BuildException("No PKCS7-Data section found in {0}", Location);
			}
		}

		// return the outerXML of the node's value
		public string GetNodeXMLValueByName(string InValue)
		{
			XmlNodeList elemList = Document.GetElementsByTagName("key");
			for (int i = 0; i < elemList.Count; i++)
			{
				XmlNode? Node = elemList[i];
				if (Node != null && Node.InnerXml.Equals(InValue, StringComparison.Ordinal))
				{
					XmlNode? valueNode = Node.NextSibling;

					if (valueNode != null)
					{
						return valueNode.OuterXml;
					}
				}
			}
			return "";
		}

		// return the innerXML of the node's value
		public string GetNodeValueByName(string InValue)
		{
			XmlNodeList elemList = Document.GetElementsByTagName("key");
			for (int i = 0; i < elemList.Count; i++)
			{
				XmlNode? Node = elemList[i];
				if (Node != null && Node.InnerXml.Equals(InValue, StringComparison.Ordinal))
				{
					XmlNode? valueNode = Node.NextSibling;
					if (valueNode != null)
					{
						if (valueNode.Name.Equals("array", StringComparison.Ordinal))
						{
							XmlNode? firstChildNode = valueNode.FirstChild;
							if (firstChildNode != null)
							{
								return firstChildNode.InnerXml;
							}
						}
						else
						{
							return valueNode.InnerXml;
						}
					}
				}
			}
			return "";
		}
	}
}
