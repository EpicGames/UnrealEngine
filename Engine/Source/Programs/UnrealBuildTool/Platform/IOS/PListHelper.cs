// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections;
using System.Collections.Generic;
using System.IO;
using System.Text;
using System.Text.RegularExpressions;
using System.Xml;

namespace UnrealBuildTool
{
	/// <summary>
	/// A helper class to handler modifying text based PList files 
	/// </summary>
	public class PListHelper
	{
		/// <summary>
		/// Helper for info plist
		/// </summary>
		public XmlDocument Doc;

		bool bReadOnly = false;

		/// <summary>
		/// Set read only
		/// </summary>
		public void SetReadOnly(bool bNowReadOnly)
		{
			bReadOnly = bNowReadOnly;
		}

		/// <summary>
		/// constructor
		/// </summary>
		public PListHelper(string Source)
		{
			Doc = new XmlDocument();
			Doc.XmlResolver = null;
			Doc.LoadXml(Source);
		}

		/// <summary>
		/// Create plist from file
		/// </summary>
		public static PListHelper CreateFromFile(string Filename)
		{
			byte[] RawPList = File.ReadAllBytes(Filename);
			return new PListHelper(Encoding.UTF8.GetString(RawPList));
		}

		/// <summary>
		/// save the plist to file
		/// </summary>
		public void SaveToFile(string Filename)
		{
			File.WriteAllText(Filename, SaveToString(), Encoding.UTF8);
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public PListHelper()
		{
			string EmptyFileText =
				"<?xml version=\"1.0\" encoding=\"UTF-8\"?>\n" +
				"<!DOCTYPE plist PUBLIC \"-//Apple//DTD PLIST 1.0//EN\" \"http://www.apple.com/DTDs/PropertyList-1.0.dtd\">\n" +
				"<plist version=\"1.0\">\n" +
				"<dict>\n" +
				"</dict>\n" +
				"</plist>\n";

			Doc = new XmlDocument();
			Doc.XmlResolver = null;
			Doc.LoadXml(EmptyFileText);
		}

		/// <summary>
		/// Convert value to plist format
		/// </summary>
		public XmlElement? ConvertValueToPListFormat(object Value)
		{
			XmlElement ValueElement;

			if (Value is string str)
			{
				ValueElement = Doc.CreateElement("string");
				ValueElement.InnerText = str;
			}
			else if (Value is Dictionary<string, object> dict)
			{
				ValueElement = Doc.CreateElement("dict");
				foreach (KeyValuePair<string, object> KVP in dict)
				{
					AddKeyValuePair(ValueElement, KVP.Key, KVP.Value);
				}
			}
			else if (Value is PListHelper)
			{
				PListHelper? PList = Value as PListHelper;

				ValueElement = Doc.CreateElement("dict");

				XmlNode? docElement = PList?.Doc?.DocumentElement;
				XmlNode? SourceDictionaryNode = docElement?.SelectSingleNode("/plist/dict");
				if (SourceDictionaryNode == null)
				{
					throw new InvalidDataException("The PListHelper object contains a document without a dictionary.");
				}

				foreach (XmlNode TheirChild in SourceDictionaryNode)
				{
					ValueElement.AppendChild(Doc.ImportNode(TheirChild, true));
				}
			}
			else if (Value is Array)
			{
				if (Value is byte[])
				{
					ValueElement = Doc.CreateElement("data");
					ValueElement.InnerText = (Value is byte[] byteArray) ? Convert.ToBase64String(byteArray) : String.Empty;
				}
				else
				{
					ValueElement = Doc.CreateElement("array");
					if (Value is Array array)
					{
						foreach (object? A in array)
						{
							XmlElement? childNode = ConvertValueToPListFormat(A);
							if (childNode != null)
							{
								ValueElement.AppendChild(childNode);
							}
						}
					}
				}
			}
			else if (Value is IList)
			{
				ValueElement = Doc.CreateElement("array");
				if (Value is IList list && ValueElement != null)
				{
					foreach (object? A in list)
					{
						XmlElement? child = ConvertValueToPListFormat(A);
						if (child != null)
						{
							ValueElement.AppendChild(child);
						}
					}
				}
			}
			else if (Value is bool vbool)
			{
				ValueElement = Doc.CreateElement(vbool ? "true" : "false");
			}
			else if (Value is double vdouble)
			{
				ValueElement = Doc.CreateElement("real");
				ValueElement.InnerText = vdouble.ToString();
			}
			else if (Value is long vlong)
			{
				ValueElement = Doc.CreateElement("integer");
				ValueElement.InnerText = vlong.ToString();
			}
			else if (Value is int vint)
			{
				ValueElement = Doc.CreateElement("integer");
				ValueElement.InnerText = vint.ToString();
			}
			else
			{
				throw new InvalidDataException(String.Format("Object '{0}' is in an unknown type that cannot be converted to PList format", Value));
			}

			return ValueElement;
		}

		/// <summary>
		/// Add key value pair
		/// </summary>
		public void AddKeyValuePair(XmlNode DictRoot, string KeyName, object Value)
		{
			if (bReadOnly)
			{
				throw new AccessViolationException("PList has been set to read only and may not be modified");
			}

			XmlElement KeyElement = Doc.CreateElement("key");
			KeyElement.InnerText = KeyName;

			DictRoot.AppendChild(KeyElement);
			XmlElement? newChild = ConvertValueToPListFormat(Value);
			if (newChild != null)
			{
				DictRoot.AppendChild(newChild);
			}
		}

		/// <summary>
		/// Add key value pair
		/// </summary>
		public void AddKeyValuePair(string KeyName, object? Value)
		{
			XmlNode? dictRoot = null;

			if (Doc.DocumentElement != null)
			{
				dictRoot = Doc.DocumentElement.SelectSingleNode("/plist/dict");
			}

			if (dictRoot != null)
			{
				XmlNode? childNode;
				if (Value != null)
				{
					childNode = ConvertValueToPListFormat(Value);
				}
				else
				{
					childNode = Doc.CreateElement("null");
				}

				if (childNode != null)
				{
					dictRoot.AppendChild(childNode);
				}

				if (Value != null)
				{
					AddKeyValuePair(dictRoot, KeyName, Value);
				}
			}
		}

		/// <summary>
		/// Clones a dictionary from an existing .plist into a new one.  Root should point to the dict key in the source plist.
		/// </summary>
		public static PListHelper CloneDictionaryRootedAt(XmlNode Root)
		{
			// Create a new empty dictionary
			PListHelper Result = new PListHelper();

			// Copy all of the entries in the source dictionary into the new one
			XmlNode? NewDictRoot = Result.Doc.DocumentElement?.SelectSingleNode("/plist/dict");
			if (NewDictRoot != null)
			{
				foreach (XmlNode TheirChild in Root)
				{
					XmlNode? importedNode = Result.Doc.ImportNode(TheirChild, true);
					if (importedNode != null)
					{
						NewDictRoot.AppendChild(importedNode);
					}
				}
			}

			return Result;
		}

		/// <summary>
		/// Get String
		/// </summary>
		public bool GetString(string Key, out string Value)
		{
			string PathToValue = String.Format("/plist/dict/key[.='{0}']/following-sibling::string[1]", Key);

			XmlNode? ValueNode = Doc.DocumentElement?.SelectSingleNode(PathToValue);
			if (ValueNode == null)
			{
				Value = "";
				return false;
			}

			Value = ValueNode.InnerText ?? "";
			return true;
		}

		/// <summary>
		/// Get date
		/// </summary>
		public bool GetDate(string Key, out string Value)
		{
			string PathToValue = String.Format("/plist/dict/key[.='{0}']/following-sibling::date[1]", Key);

			XmlNode? ValueNode = Doc.DocumentElement?.SelectSingleNode(PathToValue);
			if (ValueNode == null)
			{
				Value = "";
				return false;
			}

			Value = ValueNode.InnerText;
			return true;
		}

		/// <summary>
		/// Get bool
		/// </summary>
		public bool GetBool(string Key)
		{
			string PathToValue = String.Format("/plist/dict/key[.='{0}']/following-sibling::node()", Key);

			XmlNode? ValueNode = Doc.DocumentElement?.SelectSingleNode(PathToValue);
			if (ValueNode == null)
			{
				return false;
			}

			return ValueNode.Name == "true";
		}

		/// <summary>
		/// Process one node event
		/// </summary>
		public delegate void ProcessOneNodeEvent(XmlNode ValueNode);

		/// <summary>
		/// Process the value for the key
		/// </summary>
		public void ProcessValueForKey(string Key, string ExpectedValueType, ProcessOneNodeEvent ValueHandler)
		{
			string PathToValue = String.Format("/plist/dict/key[.='{0}']/following-sibling::{1}[1]", Key, ExpectedValueType);

			XmlNode? ValueNode = Doc.DocumentElement?.SelectSingleNode(PathToValue);
			if (ValueNode != null)
			{
				ValueHandler(ValueNode);
			}
		}

		/// <summary>
		/// Merge two plists together.  Whenever both have the same key, the value in the dominant source list wins.
		/// </summary>
		public void MergePlistIn(string dominantPlist)
		{
			if (bReadOnly)
			{
				throw new AccessViolationException("PList has been set to read only and may not be modified");
			}

			XmlDocument dominant = new XmlDocument();
			dominant.XmlResolver = null;
			dominant.LoadXml(dominantPlist);

			XmlNode? dictionaryNode = Doc.DocumentElement?.SelectSingleNode("/plist/dict");
			if (dictionaryNode == null)
			{
				throw new InvalidOperationException("Invalid PList format: missing dictionary node");
			}

			// Merge any key-value pairs in the strong .plist into the weak .plist
			XmlNodeList? strongKeys = dominant.DocumentElement?.SelectNodes("/plist/dict/key");
			if (strongKeys != null)
			{
				foreach (XmlNode strongKeyNode in strongKeys)
				{
					string strongKey = strongKeyNode.InnerText;

					XmlNode? weakNode = Doc.DocumentElement?.SelectSingleNode($"/plist/dict/key[.='{strongKey}']");
					if (weakNode == null)
					{
						// Doesn't exist in dominant plist, inject key-value pair
						XmlNode? valueNode = strongKeyNode.NextSibling;
						if (valueNode != null)
						{
							dictionaryNode.AppendChild(Doc.ImportNode(strongKeyNode, true));
							dictionaryNode.AppendChild(Doc.ImportNode(valueNode, true));
						}
					}
					else
					{
						// Remove the existing value node from the weak file
						XmlNode? existingValueNode = weakNode.NextSibling;
						if (existingValueNode != null && existingValueNode.Name == "string")
						{
							weakNode.ParentNode?.RemoveChild(existingValueNode);

							// Insert a clone of the dominant value node
							XmlNode? dominantValueNode = strongKeyNode.NextSibling;
							if (dominantValueNode != null && dominantValueNode.Name == "string")
							{
								weakNode.ParentNode?.InsertAfter(Doc.ImportNode(dominantValueNode, true), weakNode);
							}
						}
					}
				}
			}
		}

		/// <summary>
		/// Returns each of the entries in the value tag of type array for a given key
		/// If the key is missing, an empty array is returned.
		/// Only entries of a given type within the array are returned.
		/// </summary>
		public List<string> GetArray(string Key, string EntryType)
		{
			List<string> Result = new List<string>();

			ProcessValueForKey(Key, "array",
				delegate(XmlNode ValueNode)
				{
					foreach (XmlNode ChildNode in ValueNode.ChildNodes)
					{
						if (EntryType == ChildNode.Name)
						{
							string Value = ChildNode.InnerText;
							Result.Add(Value);
						}
					}
				});

			return Result;
		}

		/// <summary>
		/// Returns true if the key exists (and has a value) and false otherwise
		/// </summary>
		public bool HasKey(string KeyName)
		{
			string PathToKey = String.Format("/plist/dict/key[.='{0}']", KeyName);

			XmlNode? KeyNode = Doc.DocumentElement?.SelectSingleNode(PathToKey);
			return (KeyNode != null);
		}

		/// <summary>
		/// Remove the key value
		/// </summary>
		public void RemoveKeyValue(string KeyName)
		{
			if (bReadOnly)
			{
				throw new AccessViolationException("PList has been set to read only and may not be modified");
			}

			string PathToKey = String.Format("/plist/dict/key[.='{0}']", KeyName);
			XmlNode? KeyNode = Doc.DocumentElement?.SelectSingleNode(PathToKey);
			if (KeyNode != null && KeyNode.ParentNode != null)
			{
				XmlNode? ValueNode = KeyNode.NextSibling;
				//remove value
				if (ValueNode != null)
				{
					ValueNode.RemoveAll();
					ValueNode.ParentNode?.RemoveChild(ValueNode);
				}

				//remove key
				KeyNode.RemoveAll();
				KeyNode.ParentNode.RemoveChild(KeyNode);
			}
		}

		/// <summary>
		/// Set the value for the key
		/// </summary>
		public void SetValueForKey(string KeyName, object Value)
		{
			if (bReadOnly)
			{
				throw new AccessViolationException("PList has been set to read only and may not be modified");
			}

			XmlNode? DictionaryNode = Doc.DocumentElement?.SelectSingleNode("/plist/dict");

			string PathToKey = String.Format("/plist/dict/key[.='{0}']", KeyName);
			XmlNode? KeyNode = Doc.DocumentElement?.SelectSingleNode(PathToKey);

			XmlNode? ValueNode = null;
			if (KeyNode != null)
			{
				ValueNode = KeyNode.NextSibling;
			}

			if (ValueNode == null)
			{
				KeyNode = Doc.CreateNode(XmlNodeType.Element, "key", null);
				KeyNode.InnerText = KeyName;

				ValueNode = ConvertValueToPListFormat(Value);

				if (DictionaryNode != null && KeyNode != null && ValueNode != null)
				{
					DictionaryNode.AppendChild(KeyNode);
					DictionaryNode.AppendChild(ValueNode);
				}
			}
			else
			{
				// Remove the existing value and create a new one
				ValueNode.ParentNode?.RemoveChild(ValueNode);
				ValueNode = ConvertValueToPListFormat(Value);

				// Insert the value after the key
				KeyNode?.ParentNode?.InsertAfter(ValueNode!, KeyNode);

			}
		}

		/// <summary>
		/// Set the string
		/// </summary>
		public void SetString(string Key, string Value)
		{
			SetValueForKey(Key, Value);
		}

		/// <summary>
		/// Save the string
		/// </summary>
		public string SaveToString()
		{
			// Convert the XML back to text in the same style as the original .plist
			StringBuilder TextOut = new StringBuilder();

			// Work around the fact it outputs the wrong encoding by default (and set some other settings to get something similar to the input file)
			TextOut.Append("<?xml version=\"1.0\" encoding=\"UTF-8\"?>\n");
			XmlWriterSettings Settings = new XmlWriterSettings();
			Settings.Indent = true;
			Settings.IndentChars = "\t";
			Settings.NewLineChars = "\n";
			Settings.NewLineHandling = NewLineHandling.Replace;
			Settings.OmitXmlDeclaration = true;
			Settings.Encoding = new UTF8Encoding(false);

			// Work around the fact that it embeds an empty declaration list to the document type which codesign dislikes...
			// Replacing InternalSubset with null if it's empty.  The property is readonly, so we have to reconstruct it entirely
			Doc.ReplaceChild(Doc.CreateDocumentType(
					Doc.DocumentType!.Name,
					Doc.DocumentType!.PublicId,
					Doc.DocumentType!.SystemId,
					String.IsNullOrEmpty(Doc.DocumentType!.InternalSubset) ? null : Doc.DocumentType!.InternalSubset),
				Doc.DocumentType!);

			XmlWriter Writer = XmlWriter.Create(TextOut, Settings);

			Doc.Save(Writer);

			// Remove the space from any standalone XML elements because the iOS parser does not handle them
			return Regex.Replace(TextOut.ToString(), @"<(?<tag>\S+) />", "<${tag}/>");
		}
	}
}