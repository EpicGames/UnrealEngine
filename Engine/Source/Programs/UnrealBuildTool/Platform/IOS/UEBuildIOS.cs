// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	partial struct UnrealArch
	{
		// @todo add x64 simulators to run on old macs?
		/// <summary>
		/// IOS Simulator
		/// </summary>
		public static UnrealArch IOSSimulator { get; } = FindOrAddByName("iossimulator", bIsX64: false);

		/// <summary>
		/// TVOS Simulator
		/// </summary>
		public static UnrealArch TVOSSimulator { get; } = FindOrAddByName("tvossimulator", bIsX64: false);

		private static readonly IReadOnlyDictionary<UnrealArch, string> AppleToolchainArchitectures = new Dictionary<UnrealArch, string>()
		{
			{ UnrealArch.Arm64,         "arm64" },
			{ UnrealArch.X64,           "x86_64" },
			{ UnrealArch.IOSSimulator,  "arm64" },
			{ UnrealArch.TVOSSimulator, "arm64" },
		};

		/// <summary>
		/// Apple-specific low level name for the generic platforms
		/// </summary>
		public string AppleName
		{
			get
			{
				if (AppleToolchainArchitectures.ContainsKey(this))
				{
					return AppleToolchainArchitectures[this];
				}

				throw new BuildException($"Unknown architecture {ToString()} passed to UnrealArch.AppleName");
			}
		}
	}

	/// <summary>
	/// IOS-specific target settings
	/// </summary>
	public class IOSTargetRules : AppleTargetRules
	{
		/// <summary>
		///
		/// </summary>
		public bool bShipForBitcode = false;

		/// <summary>
		/// If true, then a stub IPA will be generated when compiling is done (minimal files needed for a valid IPA).
		/// </summary>
		[CommandLine("-CreateStub", Value = "true")]
		public bool bCreateStubIPA = false;

		/// <summary>
		/// Whether to generate a native Xcode project as a wrapper for the framework.
		/// </summary>
		public bool bGenerateFrameworkWrapperProject = false;

		/// <summary>
		/// Don't generate crashlytics data
		/// </summary>
		[CommandLine("-alwaysgeneratedsym", Value = "true")]
		[CommandLine("-EnableDSYM", Value = "true")]
		[XmlConfigFile(Category = "BuildConfiguration", Name = "bUseDSYMFiles")]
		public bool bGeneratedSYM = false;

		/// <summary>
		/// Don't generate crashlytics data
		/// </summary>
		[CommandLine("-skipcrashlytics")]
		public bool bSkipCrashlytics = false;

		/// <summary>
		/// Mark the build for distribution
		/// </summary>
		[CommandLine("-distribution")]
		public bool bForDistribution = false;

		/// <summary>
		/// Manual override for the provision to use. Should be a full path.
		/// </summary>
		[CommandLine("-ImportProvision=")]
		public string? ImportProvision = null;

		/// <summary>
		/// Imports the given certificate (inc private key) into a temporary keychain before signing.
		/// </summary>
		[CommandLine("-ImportCertificate=")]
		public string? ImportCertificate = null;

		/// <summary>
		/// Password for the imported certificate
		/// </summary>
		[CommandLine("-ImportCertificatePassword=")]
		public string? ImportCertificatePassword = null;

		/// <summary>
		/// Enables runtime ray tracing support.
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bEnableRayTracing")]
		public bool bEnableRayTracing = false;

		/// <summary>
		/// Whether the Metal Mobile SM6 renderer is enabled. Also implies ray tracing support.
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bSupportsMetalMobileSM6")]
		public bool bSupportsMetalMobileSM6 = false;

		/// <summary>
		/// Cached project settings for the target (set in ResetTarget)
		/// </summary>
		public IOSProjectSettings? ProjectSettings = null;

		/// <summary>
		/// Constructor
		/// </summary>
		public IOSTargetRules()
		{
			string? AddressSanitizer = Environment.GetEnvironmentVariable("ENABLE_ADDRESS_SANITIZER");
			if (AddressSanitizer != null && AddressSanitizer == "YES")
			{
				bEnableAddressSanitizer = true;
			}

			string? ThreadSanitizer = Environment.GetEnvironmentVariable("ENABLE_THREAD_SANITIZER");
			if (ThreadSanitizer != null && ThreadSanitizer == "YES")
			{
				bEnableThreadSanitizer = true;
			}

			string? UndefSanitizerMode = Environment.GetEnvironmentVariable("ENABLE_UNDEFINED_BEHAVIOR_SANITIZER");
			if (UndefSanitizerMode != null && UndefSanitizerMode == "YES")
			{
				bEnableUndefinedBehaviorSanitizer = true;
			}
		}
	}

	/// <summary>
	/// Read-only wrapper for IOS-specific target settings
	/// </summary>
	public class ReadOnlyIOSTargetRules : ReadOnlyAppleTargetRules
	{
		/// <summary>
		/// The private mutable settings object
		/// </summary>
		private readonly IOSTargetRules Inner;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="Inner">The settings object to wrap</param>
		public ReadOnlyIOSTargetRules(IOSTargetRules Inner)
			: base(Inner)
		{
			this.Inner = Inner;
		}

		/// <summary>
		/// Accessors for fields on the inner TargetRules instance
		/// </summary>
		#region Read-only accessor properties 
#pragma warning disable CS1591
		public bool bGenerateFrameworkWrapperProject => Inner.bGenerateFrameworkWrapperProject;

		public bool bGeneratedSYM => Inner.bGeneratedSYM;

		public bool bCreateStubIPA => Inner.bCreateStubIPA;

		public bool bSkipCrashlytics => Inner.bSkipCrashlytics;

		public bool bForDistribution => Inner.bForDistribution;

		public string? ImportProvision => Inner.ImportProvision;

		public string? ImportCertificate => Inner.ImportCertificate;

		public string? ImportCertificatePassword => Inner.ImportCertificatePassword;

		public float RuntimeVersion => Single.Parse(Inner.ProjectSettings!.RuntimeVersion, System.Globalization.CultureInfo.InvariantCulture);

		public bool bEnableRayTracing => Inner.bEnableRayTracing;
		public bool bSupportsMetalMobileSM6 => Inner.bSupportsMetalMobileSM6;
#pragma warning restore CS1591
		#endregion
	}

	/// <summary>
	/// Stores project-specific IOS settings. Instances of this object are cached by IOSPlatform.
	/// </summary>
	public class IOSProjectSettings
	{
		/// <summary>
		/// The cached project file location
		/// </summary>
		public readonly FileReference? ProjectFile;

		/// <summary>
		/// Whether to build the iOS project as a framework.
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bBuildAsFramework")]
		[CommandLine("-build-as-framework")]
		public readonly bool bBuildAsFramework = false;

		/// <summary>
		/// Whether to generate a native Xcode project as a wrapper for the framework.
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bGenerateFrameworkWrapperProject")]
		public readonly bool bGenerateFrameworkWrapperProject = false;

		/// <summary>
		/// Whether to generate a dSYM file or not.
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bGeneratedSYMFile")]
		[CommandLine("-generatedsymfile")]
		public readonly bool bGeneratedSYMFile = false;

		/// <summary>
		/// Whether to generate a dSYM bundle (as opposed to single file dSYM)
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bGeneratedSYMBundle")]
		[CommandLine("-generatedsymbundle")]
		public readonly bool bGeneratedSYMBundle = false;

		/// <summary>
		/// Whether to generate a dSYM file or not.
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bGenerateCrashReportSymbols")]
		public readonly bool bGenerateCrashReportSymbols = false;

		/// <summary>
		/// The minimum supported version
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/IOSRuntimeSettings.IOSRuntimeSettings", "MinimumiOSVersion")]
		private readonly string? MinimumIOSVersion = null;

		/// <summary>
		/// Whether to support iPhone
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bSupportsIPhone")]
		private readonly bool bSupportsIPhone = true;

		/// <summary>
		/// Whether to support iPad
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bSupportsIPad")]
		private readonly bool bSupportsIPad = true;

		/// <summary>
		/// additional linker flags for shipping
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/IOSRuntimeSettings.IOSRuntimeSettings", "AdditionalShippingLinkerFlags")]
		public readonly string AdditionalShippingLinkerFlags = "";

		/// <summary>
		/// additional linker flags for non-shipping
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/IOSRuntimeSettings.IOSRuntimeSettings", "AdditionalLinkerFlags")]
		public readonly string AdditionalLinkerFlags = "";

		/// <summary>
		/// mobile provision to use for code signing
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/IOSRuntimeSettings.IOSRuntimeSettings", "MobileProvision")]
		public readonly string MobileProvision = "";

		/// <summary>
		/// signing certificate to use for code signing
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/IOSRuntimeSettings.IOSRuntimeSettings", "SigningCertificate")]
		public readonly string SigningCertificate = "";

		/// <summary>
		/// true if notifications are enabled
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bEnableStoreKitSupport")]
		public readonly bool bEnableStoreKitSupport = true;

		/// <summary>
		/// true if notifications are enabled
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bEnableRemoteNotificationsSupport")]
		public readonly bool bNotificationsEnabled = false;

		/// <summary>
		/// true if notifications are enabled
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bEnableBackgroundFetch")]
		public readonly bool bBackgroundFetchEnabled = false;

		/// <summary>
		/// true if iTunes file sharing support is enabled
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bSupportsITunesFileSharing")]
		public readonly bool bFileSharingEnabled = false;

		/// <summary>
		/// true if the iOS app should adopt the UIScene-based lifecycle (TN3187). Compile-time only.
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bUseSceneBasedLifecycle")]
		public readonly bool bUseSceneBasedLifecycle = false;

		/// <summary>
		/// The bundle identifier
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/IOSRuntimeSettings.IOSRuntimeSettings", "BundleIdentifier")]
		public readonly string BundleIdentifier = "";

		/// <summary>
		/// true if using Xcode managed provisioning, else false
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bAutomaticSigning")]
		public readonly bool bAutomaticSigning = false;

		/// <summary>
		/// The IOS Team ID
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/IOSRuntimeSettings.IOSRuntimeSettings", "IOSTeamID")]
		public readonly string TeamID = "";

		/// <summary>
		/// true to change FORCEINLINE to a regular INLINE.
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bDisableForceInline")]
		public readonly bool bDisableForceInline = false;

		/// <summary>
		/// true if IDFA are enabled
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bEnableAdvertisingIdentifier")]
		public readonly bool bEnableAdvertisingIdentifier = false;

		/// <summary>
		/// true when building for distribution
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Game, "/Script/UnrealEd.ProjectPackagingSettings", "ForDistribution")]
		public readonly bool bForDistribution = false;

		/// <summary>
		/// override for the app's display name if different from the project name
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Game, "/Script/UnrealEd.ProjectPackagingSettings", "BundleName")]
		public readonly string BundleName = "";

		/// <summary>
		/// longer display name than BundleName if needed
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Game, "/Script/UnrealEd.ProjectPackagingSettings", "BundleDisplayName")]
		public readonly string BundleDisplayName = "";

		/// <summary>
		/// Which version of the iOS to allow at run time
		/// </summary>
		public virtual string RuntimeVersion
		{
			get
			{
				switch (MinimumIOSVersion)
				{
					case "IOS_15":
						return "15.0";
					case "IOS_16":
						return "16.0";
					case "IOS_17":
						return "17.0";
					case "IOS_18":
						return "18.0";
					case "IOS_26":
						return "26.0";
					case "IOS_Minimum":
					default:
						if (String.IsNullOrEmpty(MinimumIOSVersion) || MinimumIOSVersion.StartsWith("IOS_", StringComparison.Ordinal))
						{
							return UEBuildPlatformSDK.GetSDKForPlatform("IOS")!.GetSoftwareInfo()!.Min!;
						}
						else
						{
							// Assume the user is overriding the defaults, ie: "17.4.1"
							return MinimumIOSVersion;
						}
				}
			}
		}

		/// <summary>
		/// Enables iOS 16 dynamic linker workaround
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bEnableIOS16DynLinkerBugWAR")]
		public readonly bool bEnableIOS16DynLinkerBugWAR = false;

		/// <summary>
		/// which devices the game is allowed to run on
		/// </summary>
		public virtual string RuntimeDevices
		{
			get
			{
				if (bSupportsIPad && !bSupportsIPhone)
				{
					return "2";
				}
				else if (!bSupportsIPad && bSupportsIPhone)
				{
					return "1";
				}
				else
				{
					return "1,2";
				}
			}
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="ProjectFile">The project file to read settings for</param>
		/// <param name="Bundle">Bundle identifier needed when project file is empty</param>
		public IOSProjectSettings(FileReference? ProjectFile, string? Bundle)
			: this(ProjectFile, UnrealTargetPlatform.IOS, Bundle)
		{
		}

		/// <summary>
		/// Protected constructor. Used by TVOSProjectSettings.
		/// </summary>
		/// <param name="ProjectFile">The project file to read settings for</param>
		/// <param name="Platform">The platform to read settings for</param>
		/// <param name="Bundle">Bundle identifier needed when project file is empty</param>
		protected IOSProjectSettings(FileReference? ProjectFile, UnrealTargetPlatform Platform, string? Bundle)
		{
			this.ProjectFile = ProjectFile;
			ConfigCache.ReadSettings(DirectoryReference.FromFile(ProjectFile), Platform, this);
			if ((ProjectFile == null || String.IsNullOrEmpty(ProjectFile.FullName)) && !String.IsNullOrEmpty(Bundle))
			{
				BundleIdentifier = Bundle;
			}

			string ProjectName = (ProjectFile != null) ? ProjectFile.GetFileNameWithoutAnyExtensions() : "UnrealGame";
			BundleIdentifier = BundleIdentifier.Replace("[PROJECT_NAME]", ProjectName, System.StringComparison.Ordinal).Replace("_", "", StringComparison.Ordinal);
		}
	}

	/// <summary>
	/// IOS provisioning data
	/// </summary>
	class IOSProvisioningData
	{
		public string? SigningCertificate;
		public FileReference? MobileProvisionFile;
		public string? MobileProvisionUUID;
		public string? MobileProvisionName;
		public string? TeamUUID;
		public string? BundleIdentifier;
		public bool bHaveCertificate = false;

		public string? MobileProvision => MobileProvisionFile?.GetFileName();

		public IOSProvisioningData(IOSProjectSettings ProjectSettings, bool bForDistribution, ILogger Logger)
			: this(ProjectSettings, false, bForDistribution, Logger)
		{
		}

		protected IOSProvisioningData(IOSProjectSettings ProjectSettings, bool bIsTVOS, bool bForDistribution, ILogger Logger)
		{
			SigningCertificate = ProjectSettings.SigningCertificate;
			string? MobileProvision = ProjectSettings.MobileProvision;

			FileReference? ProjectFile = ProjectSettings.ProjectFile;
			CodeSigningConfig.Initialize(ProjectFile, bIsTVOS);

			if (!String.IsNullOrEmpty(SigningCertificate))
			{
				// TODO: Verify if there are side-effects that require these methods to run.
				_ = AppleCodeSign.FindCertificates();
				_ = AppleCodeSign.FindProvisions(ProjectSettings.BundleIdentifier, bForDistribution, out _);
			}
			else
			{
				SigningCertificate = bForDistribution ? "iPhone Distribution" : "iPhone Developer";
				bHaveCertificate = true;
			}

			if (!String.IsNullOrEmpty(MobileProvision))
			{
				FileReference PossibleMobileProvisionFile = FileReference.Combine(AppleExports.GetProvisionDirectory(), MobileProvision);
				if (FileReference.Exists(PossibleMobileProvisionFile))
				{
					MobileProvisionFile = PossibleMobileProvisionFile;
				}
			}

			if (MobileProvisionFile == null || !bHaveCertificate)
			{

				SigningCertificate = "";
				MobileProvisionFile = null;
				Logger.LogInformation("Provision not specified or not found for {Project}, searching for compatible match...", ((ProjectFile != null) ? ProjectFile.GetFileNameWithoutAnyExtensions() : "UnrealGame"));

				_ = AppleCodeSign.FindCertAndProvision(ProjectSettings.BundleIdentifier, out MobileProvisionFile, out SigningCertificate);

				if (MobileProvisionFile != null)
				{
					Logger.LogInformation("Provision found for {Project}, Provision: {Provision}, Certificate: {Certificate}", ((ProjectFile != null) ? ProjectFile.GetFileNameWithoutAnyExtensions() : "UnrealGame"), MobileProvisionFile, SigningCertificate);
				}
			}

			// add to the dictionary
			SigningCertificate = SigningCertificate.Replace("\"", "", StringComparison.Ordinal);

			// read the provision to get the UUID
			if (MobileProvisionFile == null)
			{
				Logger.LogInformation("No matching provision file was discovered for {ProjectFile}. Please ensure you have a compatible provision installed.", ProjectFile);
			}
			else if (!FileReference.Exists(MobileProvisionFile))
			{
				Logger.LogInformation("Selected mobile provision for {ProjectFile} ({MobileProvisionFile}) was not found. Please ensure you have a compatible provision installed.", ProjectFile, MobileProvisionFile);
			}
			else
			{
				byte[] AllBytes = FileReference.ReadAllBytes(MobileProvisionFile);

				uint StartIndex = (uint)AllBytes.Length;
				uint EndIndex = (uint)AllBytes.Length;

				for (uint i = 0; i + 4 < AllBytes.Length; i++)
				{
					if (AllBytes[i] == '<' && AllBytes[i + 1] == '?' && AllBytes[i + 2] == 'x' && AllBytes[i + 3] == 'm' && AllBytes[i + 4] == 'l')
					{
						StartIndex = i;
						break;
					}
				}

				if (StartIndex < AllBytes.Length)
				{
					for (uint i = StartIndex; i + 7 < AllBytes.Length; i++)
					{
						if (AllBytes[i] == '<' && AllBytes[i + 1] == '/' && AllBytes[i + 2] == 'p' && AllBytes[i + 3] == 'l' && AllBytes[i + 4] == 'i' && AllBytes[i + 5] == 's' && AllBytes[i + 6] == 't' && AllBytes[i + 7] == '>')
						{
							EndIndex = i + 7;
							break;
						}
					}
				}

				if (StartIndex < AllBytes.Length && EndIndex < AllBytes.Length)
				{
					byte[] TextBytes = new byte[EndIndex - StartIndex];
					Buffer.BlockCopy(AllBytes, (int)StartIndex, TextBytes, 0, (int)(EndIndex - StartIndex));

					string AllText = Encoding.UTF8.GetString(TextBytes);
					int idx = AllText.IndexOf("<key>UUID</key>", StringComparison.Ordinal);
					if (idx > 0)
					{
						idx = AllText.IndexOf("<string>", idx, StringComparison.Ordinal);
						if (idx > 0)
						{
							idx += "<string>".Length;
							MobileProvisionUUID = AllText.Substring(idx, AllText.IndexOf("</string>", idx, StringComparison.Ordinal) - idx);
						}
					}
					idx = AllText.IndexOf("<key>com.apple.developer.team-identifier</key>", StringComparison.Ordinal);
					if (idx > 0)
					{
						idx = AllText.IndexOf("<string>", idx, StringComparison.Ordinal);
						if (idx > 0)
						{
							idx += "<string>".Length;
							TeamUUID = AllText.Substring(idx, AllText.IndexOf("</string>", idx, StringComparison.Ordinal) - idx);
						}
					}
					idx = AllText.IndexOf("<key>application-identifier</key>", StringComparison.Ordinal);
					if (idx > 0)
					{
						idx = AllText.IndexOf("<string>", idx, StringComparison.Ordinal);
						if (idx > 0)
						{
							idx += "<string>".Length;
							string FullID = AllText.Substring(idx, AllText.IndexOf("</string>", idx, StringComparison.Ordinal) - idx);
							BundleIdentifier = FullID.Substring(FullID.IndexOf('.', StringComparison.Ordinal) + 1);
						}
					}
					idx = AllText.IndexOf("<key>Name</key>", StringComparison.Ordinal);
					if (idx > 0)
					{
						idx = AllText.IndexOf("<string>", idx, StringComparison.Ordinal);
						if (idx > 0)
						{
							idx += "<string>".Length;
							MobileProvisionName = AllText.Substring(idx, AllText.IndexOf("</string>", idx, StringComparison.Ordinal) - idx);
						}
					}
				}

				if (String.IsNullOrEmpty(MobileProvisionUUID) || String.IsNullOrEmpty(TeamUUID))
				{
					SigningCertificate = null;
					Logger.LogInformation("Failed to parse the mobile provisioning profile.");
				}
			}
		}
	}

	class IOSArchitectureConfig : AppleArchitectureConfig
	{
		public IOSArchitectureConfig()
			: base(new[] { UnrealArch.Arm64, UnrealArch.IOSSimulator })
		{

		}

		public override UnrealArchitectures ActiveArchitectures(FileReference? ProjectFile, string? TargetName)
		{
			// always use arm64 unless overridden on command line
			return new UnrealArchitectures(UnrealArch.Arm64);
		}
	}

	class IOSPlatform : AppleBuildPlatform
	{
		readonly List<IOSProjectSettings> CachedProjectSettings = new List<IOSProjectSettings>();
		readonly List<IOSProjectSettings> CachedProjectSettingsByBundle = new List<IOSProjectSettings>();
		readonly Dictionary<string, IOSProvisioningData> ProvisionCache = new Dictionary<string, IOSProvisioningData>();

		public IOSPlatform(UEBuildPlatformSDK InSDK, ILogger Logger)
			: this(InSDK, UnrealTargetPlatform.IOS, Logger)
		{
		}

		protected IOSPlatform(UEBuildPlatformSDK InSDK, UnrealTargetPlatform TargetPlatform, ILogger Logger)
			: base(TargetPlatform, InSDK, new IOSArchitectureConfig(), Logger)
		{
		}

		public override List<FileReference> FinalizeBinaryPaths(FileReference BinaryName, FileReference? ProjectFile, ReadOnlyTargetRules Target)
		{
			List<FileReference> BinaryPaths = new List<FileReference>();
			if (Target.bShouldCompileAsDLL)
			{
				BinaryPaths.Add(FileReference.Combine(BinaryName.Directory, Target.Configuration.ToString(), Target.Name + ".framework", Target.Name));
			}
			else
			{
				BinaryPaths.Add(BinaryName);
			}
			return BinaryPaths;
		}

		public override void ResetTarget(TargetRules Target)
		{
			base.ResetTarget(Target);
			
			Target.bDeployAfterCompile = true;

			Target.IOSPlatform.ProjectSettings = ((IOSPlatform)GetBuildPlatform(Target.Platform)).ReadProjectSettings(Target.ProjectFile);

			if (AppleExports.CreatingAppOnWindows(Target.ProjectFile))
			{
				// In apple on windows mode, if we are stripping the executable, or if the project requested it, or if it's a buildmachine, generate the dsym
				if (Target.IOSPlatform.bStripSymbols || Target.IOSPlatform.ProjectSettings.bGeneratedSYMFile || Unreal.IsBuildMachine())
				{
					Target.IOSPlatform.bGeneratedSYM = true;
				}
			}

			// Set bShouldCompileAsDLL when building as a framework
			Target.bShouldCompileAsDLL = Target.IOSPlatform.ProjectSettings.bBuildAsFramework;
		}

		public override void ValidateTarget(TargetRules Target)
		{
			base.ValidateTarget(Target);

			if (Target.bShouldCompileAsDLL)
			{
				int PreviousDefinition = Target.GlobalDefinitions.FindIndex(s => s.Contains("BUILD_EMBEDDED_APP", StringComparison.Ordinal));
				if (PreviousDefinition >= 0)
				{
					Target.GlobalDefinitions.RemoveAt(PreviousDefinition);
				}

				Target.GlobalDefinitions.Add("BUILD_EMBEDDED_APP=1");

				if (Target.Platform == UnrealTargetPlatform.IOS)
				{
					Target.ExportPublicHeader = "Headers/PreIOSEmbeddedView.h";
				}
			}

			Target.bCheckSystemHeadersForModification = false;

			if (Target.StaticAllocator == StaticAllocatorType.None)
			{
				// Prefer MB2 as MB3 requires extended virtual address space entitlement
				Target.StaticAllocator = StaticAllocatorType.Binned2;
			}
		}

		public override void ValidateModule(UEBuildModule Module, ReadOnlyTargetRules Target, ILogger Logger)
		{
			base.ValidateModule(Module, Target, Logger);

			// @todo temporarily disabling due to VisionOS rquiring newer Xcode than this will allow - we may remove this entirely
#if false
			if (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Mac && !Target.IOSPlatform.bSkipClangValidation)
			{
				ApplePlatformSDK SDK = (ApplePlatformSDK?)GetSDK() ?? new ApplePlatformSDK(Logger);
				foreach (FileReference LibLoc in Module.PublicLibraries)
				{
					switch (LibLoc.GetExtension())
					{
						case ".a":
							{
								// When static lib, grep it
								string Args = "-c \"strings ";
								Args += LibLoc.FullName;
								Args += " | grep -m1 -i \\(clang\"";
								string StdOutResult = Utils.RunLocalProcessAndReturnStdOut("bash", Args);
								if (String.IsNullOrEmpty(StdOutResult))
								{
									continue;
								}

								// This Regex will extract a 2-4 segment version code from string containing a 2-5 segment code 
								// ie: if given string: "Apple clang version 14.0.0 (clang-1400.0.17.3.1)"
								//     it'll extract: "1400.0.17.3"  (note the dropped 5th segment)
								Match M = Regex.Match(StdOutResult, @"(\(clang-(?<ver>\d+.\d+(.(\d+))?(.(\d+))?)(.(\d+))?\))");
								if (M.Success)
								{
									string LibString = M.Groups["ver"].ToString();
									Version? LibVersion = new Version(LibString);
									if (LibVersion != null && LibVersion > SDK.MinimumStaticLibClangVersion)
									{
										throw new BuildException("iOS Static Library:'{0}' is built with a version of clang newer than UE supports ({1} > {2}). \nPlease rebuild {3} with the minimum supported version of Xcode/clang.", LibLoc.GetFileName(), LibString, SDK.MinimumStaticLibClangVersion, LibLoc);
									}
								}
							}
							break;

						default:
							// For now, we don't validate any other types of libs (dylib, Framework, etc)
							break;
					}
				}
			}
#endif
		}

		public override AppleTargetRules GetAppleTargetRules(TargetRules Target)
		{
			return Target.IOSPlatform;
		}

		public override ReadOnlyAppleTargetRules GetAppleTargetRules(ReadOnlyTargetRules Target)
		{
			return Target.IOSPlatform;
		}

		/// <summary>
		/// Determines if the given name is a build product for a target.
		/// </summary>
		/// <param name="FileName">The name to check</param>
		/// <param name="NamePrefixes">Target or application names that may appear at the start of the build product name (eg. "UnrealEditor", "ShooterGameEditor")</param>
		/// <param name="NameSuffixes">Suffixes which may appear at the end of the build product name</param>
		/// <returns>True if the string matches the name of a build product, false otherwise</returns>
		public override bool IsBuildProduct(string FileName, string[] NamePrefixes, string[] NameSuffixes)
		{
			return base.IsBuildProduct(FileName, NamePrefixes, NameSuffixes)
				|| IsBuildProductName(FileName, NamePrefixes, NameSuffixes, ".stub")
				|| IsBuildProductName(FileName, NamePrefixes, NameSuffixes, ".dSYM.zip")
				|| IsBuildProductName(FileName, NamePrefixes, NameSuffixes, ".o");
		}

		public IOSProjectSettings ReadProjectSettings(FileReference? ProjectFile, string? Bundle = "")
		{
			IOSProjectSettings? ProjectSettings = null;

			// Use separate lists to prevent an overridden Bundle id polluting the standard project file. 
			bool bCacheByBundle = !String.IsNullOrEmpty(Bundle);
			if (bCacheByBundle)
			{
				lock (CachedProjectSettingsByBundle)
				{
					ProjectSettings = CachedProjectSettingsByBundle.FirstOrDefault(x => x.ProjectFile == ProjectFile && x.BundleIdentifier == Bundle);
					if (ProjectSettings == null)
					{
						ProjectSettings = CreateProjectSettings(ProjectFile, Bundle);
						CachedProjectSettingsByBundle.Add(ProjectSettings);
					}
				}
			}
			else
			{
				lock (CachedProjectSettings)
				{
					ProjectSettings = CachedProjectSettings.FirstOrDefault(x => x.ProjectFile == ProjectFile);
					if (ProjectSettings == null)
					{
						ProjectSettings = CreateProjectSettings(ProjectFile, Bundle);
						CachedProjectSettings.Add(ProjectSettings);
					}
				}
			}

			return ProjectSettings;
		}

		/// <summary>
		/// Checks if bindless is enabled for any supported iOS shader platform.
		/// Returns true if framework should be included for iOS.
		/// </summary>
		private static bool ShouldIncludeMetalShaderConverterForIOS(ReadOnlyTargetRules Target)
		{
			// Read Engine config hierarchy for iOS platform
			ConfigHierarchy Ini = ConfigCache.ReadHierarchy(
				ConfigHierarchyType.Engine,
				DirectoryReference.FromFile(Target.ProjectFile),
				UnrealTargetPlatform.IOS);

			// Read platform support settings from IOSRuntimeSettings
			Ini.GetBool("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bSupportsMetal", out bool bSupportsMetal);
			Ini.GetBool("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bSupportsMetalMobileSM5", out bool bSupportsMetalMobileSM5);
			Ini.GetBool("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bSupportsMetalMobileSM6", out bool bSupportsMetalMobileSM6);

			// Check each shader platform: bindless must be enabled AND platform must be supported
			// METAL_ES3_1_IOS
			if (bSupportsMetal)
			{
				if (Ini.TryGetValue("ShaderPlatformConfig METAL_ES3_1_IOS", "BindlessConfiguration", out string? ES31Config) &&
					!string.IsNullOrEmpty(ES31Config) && !ES31Config.Equals("Disabled", StringComparison.OrdinalIgnoreCase))
				{
					return true;
				}
			}

			// METAL_SM5_IOS
			if (bSupportsMetalMobileSM5)
			{
				if (Ini.TryGetValue("ShaderPlatformConfig METAL_SM5_IOS", "BindlessConfiguration", out string? SM5Config) &&
					!string.IsNullOrEmpty(SM5Config) && !SM5Config.Equals("Disabled", StringComparison.OrdinalIgnoreCase))
				{
					return true;
				}
			}

			// METAL_SM6_IOS
			if (bSupportsMetalMobileSM6)
			{
				if (Ini.TryGetValue("ShaderPlatformConfig METAL_SM6_IOS", "BindlessConfiguration", out string? SM6Config) &&
					!string.IsNullOrEmpty(SM6Config) && !SM6Config.Equals("Disabled", StringComparison.OrdinalIgnoreCase))
				{
					return true;
				}
			}

			return false;
		}

		protected virtual IOSProjectSettings CreateProjectSettings(FileReference? ProjectFile, string? Bundle)
		{
			return new IOSProjectSettings(ProjectFile, Bundle);
		}

		public IOSProvisioningData ReadProvisioningData(FileReference? ProjectFile, bool bForDistribution = false, string? Bundle = "")
		{
			IOSProjectSettings ProjectSettings = ReadProjectSettings(ProjectFile, Bundle);
			return ReadProvisioningData(ProjectSettings, bForDistribution);
		}

		public IOSProvisioningData ReadProvisioningData(IOSProjectSettings ProjectSettings, bool bForDistribution = false)
		{
			string ProvisionKey = ProjectSettings.BundleIdentifier + " " + bForDistribution.ToString();

			IOSProvisioningData? ProvisioningData;

			lock (ProvisionCache)
			{
				if (!ProvisionCache.TryGetValue(ProvisionKey, out ProvisioningData))
				{
					ProvisioningData = CreateProvisioningData(ProjectSettings, bForDistribution);
					ProvisionCache.Add(ProvisionKey, ProvisioningData);
				}
			}
			return ProvisioningData;
		}

		protected virtual IOSProvisioningData CreateProvisioningData(IOSProjectSettings ProjectSettings, bool bForDistribution)
		{
			return new IOSProvisioningData(ProjectSettings, bForDistribution, Logger);
		}

		public override string[] GetDebugInfoExtensions(ReadOnlyTargetRules InTarget, UEBuildBinaryType InBinaryType)
		{
			if (InTarget.IOSPlatform.bGeneratedSYM)
			{
				IOSProjectSettings ProjectSettings = ReadProjectSettings(InTarget.ProjectFile);

				// which format?
				if (ProjectSettings.bGeneratedSYMBundle)
				{
					return new string[] { ".dSYM.zip" };
				}
				else
				{
					return new string[] { ".dSYM" };
				}
			}

			return Array.Empty<string>();
		}

		public override bool CanUseXGE()
		{
			return BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Mac;
		}

		public static bool HasCustomIcons(DirectoryReference ProjectDirectoryName, ILogger Logger)
		{
			string IconDir = Path.Combine(ProjectDirectoryName.FullName, "Build", "IOS", "Resources", "Graphics");
			if (Directory.Exists(IconDir))
			{
				foreach (string f in Directory.EnumerateFiles(IconDir))
				{
					if (f.Contains("Icon", StringComparison.Ordinal) && Path.GetExtension(f).Contains(".png", StringComparison.Ordinal))
					{
						Logger.LogInformation("Requiring custom build because project {Project} has custom icons", Path.GetFileName(ProjectDirectoryName.FullName));
						return true;
					}
				}
			}
			return false;
		}

		/// <summary>
		/// Check for the default configuration
		/// return true if the project uses the default build config
		/// </summary>
		public override bool HasDefaultBuildConfig(UnrealTargetPlatform Platform, DirectoryReference ProjectDirectoryName)
		{
			string[] BoolKeys = new string[] {
				"bShipForBitcode", "bGeneratedSYMFile",
				"bGeneratedSYMBundle", "bEnableRemoteNotificationsSupport", "bEnableCloudKitSupport",
				"bGenerateCrashReportSymbols", "bEnableBackgroundFetch"
			};
			string[] StringKeys = new string[] {
				"MinimumiOSVersion",
				"AdditionalLinkerFlags",
				"AdditionalShippingLinkerFlags"
			};

			// check for custom icons
			if (HasCustomIcons(ProjectDirectoryName, Logger))
			{
				return false;
			}

			// look up iOS specific settings
			if (!DoProjectSettingsMatchDefault(Platform, ProjectDirectoryName, "/Script/IOSRuntimeSettings.IOSRuntimeSettings",
					BoolKeys, null, StringKeys))
			{
				return false;
			}

			// check the base settings
			return base.HasDefaultBuildConfig(Platform, ProjectDirectoryName);
		}

		/// <summary>
		/// Check for the build requirement due to platform requirements
		/// return true if the project requires a build
		/// </summary>
		public override bool RequiresBuild(UnrealTargetPlatform Platform, DirectoryReference ProjectDirectoryName)
		{
			// check for custom icons
			return HasCustomIcons(ProjectDirectoryName, Logger);
		}

		public override bool ShouldCompileMonolithicBinary(UnrealTargetPlatform InPlatform)
		{
			// This platform currently always compiles monolithic
			return true;
		}

		/// <summary>
		/// Modify the rules for a newly created module, where the target is a different host platform.
		/// This is not required - but allows for hiding details of a particular platform.
		/// </summary>
		/// <param name="ModuleName">The name of the module</param>
		/// <param name="Rules">The module rules</param>
		/// <param name="Target">The target being build</param>
		public override void ModifyModuleRulesForOtherPlatform(string ModuleName, ModuleRules Rules, ReadOnlyTargetRules Target)
		{
			bool bIsPlatformAvailableForTarget = UEBuildPlatform.IsPlatformAvailableForTarget(Platform, Target, bIgnoreSDKCheck: true);
			bool bIsPlatformAvailableForTargetWithSDK = UEBuildPlatform.IsPlatformAvailableForTarget(Platform, Target);

			// don't do any target platform stuff if SDK is not available
			if (!bIsPlatformAvailableForTarget)
			{
				return;
			}
			
			// no iOS tools on Windows Arm64
			bool bIsWindowsArm64 = (Target.Platform == UnrealTargetPlatform.Win64 && Target.Architecture == UnrealArch.Arm64);
			if (bIsWindowsArm64)
			{
				return;
			}		

			AddPlatformStaticShaders(Target, Rules, []);

			if (Target.Platform == UnrealTargetPlatform.Win64 || Target.Platform == UnrealTargetPlatform.Mac)
			{
				bool bBuildShaderFormats = Target.bForceBuildShaderFormats;
				if (!Target.bBuildRequiresCookedData && Target.Type != TargetType.Program)
				{
					if (ModuleName == "TargetPlatform")
					{
						bBuildShaderFormats = true;
					}
				}

				if (!Target.bBuildRequiresCookedData && Target.bBuildDeveloperTools)
				{
					if (ModuleName == "Engine")
					{
						Rules.DynamicallyLoadedModuleNames.Add("IOSTargetPlatformSettings");
						Rules.DynamicallyLoadedModuleNames.Add("TVOSTargetPlatformSettings");

						if (bIsPlatformAvailableForTargetWithSDK)
						{
							Rules.DynamicallyLoadedModuleNames.Add("IOSTargetPlatform");
							Rules.DynamicallyLoadedModuleNames.Add("IOSTargetPlatformControls");
							Rules.DynamicallyLoadedModuleNames.Add("TVOSTargetPlatform");
							Rules.DynamicallyLoadedModuleNames.Add("TVOSTargetPlatformControls");
						}
					}
					else if (ModuleName == "TargetPlatform")
					{
						if (bIsPlatformAvailableForTargetWithSDK)
						{
							Rules.DynamicallyLoadedModuleNames.Add("TextureFormatASTC");
							Rules.DynamicallyLoadedModuleNames.Add("TextureFormatETC2");
							if (Target.bCompileAgainstEngine)
							{
								Rules.DynamicallyLoadedModuleNames.Add("AudioFormatADPCM");
							}
						}
					}
				}

				// allow standalone tools to use targetplatform modules, without needing Engine
				if (ModuleName == "TargetPlatform")
				{
					if (Target.bForceBuildTargetPlatforms)
					{
						Rules.DynamicallyLoadedModuleNames.Add("IOSTargetPlatformSettings");
						Rules.DynamicallyLoadedModuleNames.Add("TVOSTargetPlatformSettings");

						if (bIsPlatformAvailableForTargetWithSDK)
						{
							Rules.DynamicallyLoadedModuleNames.Add("IOSTargetPlatform");
							Rules.DynamicallyLoadedModuleNames.Add("IOSTargetPlatformControls");
							Rules.DynamicallyLoadedModuleNames.Add("TVOSTargetPlatform");
							Rules.DynamicallyLoadedModuleNames.Add("TVOSTargetPlatformControls");
						}
					}

					if (bBuildShaderFormats)
					{
						Rules.DynamicallyLoadedModuleNames.Add("MetalShaderFormat");
					}
				}

				if (ModuleName == "UnrealEd" && bIsPlatformAvailableForTargetWithSDK)
				{
					Rules.DynamicallyLoadedModuleNames.Add("IOSPlatformEditor");
				}
			}
		}

		public override void ModifyModuleRulesForActivePlatform(string ModuleName, ModuleRules Rules, ReadOnlyTargetRules Target)
		{
			if (ModuleName == "Launch")
			{
				Rules.PrivateDependencyModuleNames.AddRange(new string[] {
					"AudioMixerAudioUnit",
					"IOSAudio",
					"LaunchDaemonMessages",
				});

				Rules.DynamicallyLoadedModuleNames.AddRange(new string[] {
					"IOSLocalNotification",
					"IOSRuntimeSettings",
				});

				// needed for Metal layer
				Rules.PublicFrameworks.Add("QuartzCore");

				IOSProjectSettings ProjectSettings = ((IOSPlatform)UEBuildPlatform.GetBuildPlatform(Target.Platform)).ReadProjectSettings(Target.ProjectFile);
				if (ProjectSettings.bEnableIOS16DynLinkerBugWAR &&
					Target.Platform == UnrealTargetPlatform.IOS)
				{
					Rules.PublicDependencyModuleNames.Add("Interpose");
				}
			}
		}

		/// <summary>
		/// Whether this platform should create debug information or not
		/// </summary>
		/// <param name="Target">The target being built</param>
		/// <returns>bool    true if debug info should be generated, false if not</returns>
		public override bool ShouldCreateDebugInfo(ReadOnlyTargetRules Target)
		{
			return true;
		}

		/// <summary>
		/// Setup the target environment for building
		/// </summary>
		/// <param name="Target">Settings for the target being compiled</param>
		/// <param name="CompileEnvironment">The compile environment for this target</param>
		/// <param name="LinkEnvironment">The link environment for this target</param>
		public override void SetUpEnvironment(ReadOnlyTargetRules Target, CppCompileEnvironment CompileEnvironment, LinkEnvironment LinkEnvironment)
		{
			base.SetUpEnvironment(Target, CompileEnvironment, LinkEnvironment);

			IOSProjectSettings ProjectSettings = ((IOSPlatform)UEBuildPlatform.GetBuildPlatform(Target.Platform)).ReadProjectSettings(Target.ProjectFile);
			if (!ProjectFileGenerator.bGenerateProjectFiles)
			{
				Logger.LogInformation("Compiling against OS Version {RuntimeVersion} [minimum allowed at runtime]", ProjectSettings.RuntimeVersion);
			}

			CompileEnvironment.Definitions.Add("PLATFORM_IOS=1");
			CompileEnvironment.Definitions.Add("UE_PLATFORM_IOS=1");
			// set a define that is _only_ on iPhones, and not AppleTV, etc
			if (Target.Platform == UnrealTargetPlatform.IOS)
			{
				CompileEnvironment.Definitions.Add("UE_PLATFORM_IOS_ONLY=1");
			}

			CompileEnvironment.Definitions.Add("WITH_EDITOR=0");
			CompileEnvironment.Definitions.Add("USE_NULL_RHI=0");

			if (ProjectSettings.bNotificationsEnabled)
			{
				CompileEnvironment.Definitions.Add("NOTIFICATIONS_ENABLED=1");
			}
			else
			{
				CompileEnvironment.Definitions.Add("NOTIFICATIONS_ENABLED=0");
			}
			if (ProjectSettings.bBackgroundFetchEnabled)
			{
				CompileEnvironment.Definitions.Add("BACKGROUNDFETCH_ENABLED=1");
			}
			else
			{
				CompileEnvironment.Definitions.Add("BACKGROUNDFETCH_ENABLED=0");
			}
			if (ProjectSettings.bFileSharingEnabled)
			{
				CompileEnvironment.Definitions.Add("FILESHARING_ENABLED=1");
			}
			else
			{
				CompileEnvironment.Definitions.Add("FILESHARING_ENABLED=0");
			}

			CompileEnvironment.Definitions.Add("UE_DISABLE_FORCE_INLINE=" + (ProjectSettings.bDisableForceInline ? "1" : "0"));

			if (Target.Architecture == UnrealArch.IOSSimulator || Target.Architecture == UnrealArch.TVOSSimulator)
			{
				CompileEnvironment.Definitions.Add("WITH_IOS_SIMULATOR=1");
			}
			else
			{
				CompileEnvironment.Definitions.Add("WITH_IOS_SIMULATOR=0");
			}

			if (ProjectSettings.bEnableAdvertisingIdentifier)
			{
				CompileEnvironment.Definitions.Add("ENABLE_ADVERTISING_IDENTIFIER=1");
			}

			if (Target.bPGOOptimize || Target.bPGOProfile)
			{
				Logger.LogInformation("PGO {PgoType} build", Target.bPGOOptimize ? "optimize" : "profile");

				CompileEnvironment.PGODirectory = DirectoryReference.Combine(Target.ProjectFile?.Directory ?? Unreal.WritableEngineDirectory, "Platforms", "IOS", "Build", "PGO").FullName;
				CompileEnvironment.PGOFilenamePrefix = String.Format("{0}-IOS.profdata", Target.Name);

				LinkEnvironment.PGODirectory = CompileEnvironment.PGODirectory;
				LinkEnvironment.PGOFilenamePrefix = CompileEnvironment.PGOFilenamePrefix;

				Logger.LogInformation("PGO Dir: {PgoDir}", CompileEnvironment.PGODirectory);
				Logger.LogInformation("PGO Prefix: {PgoPrefix}", CompileEnvironment.PGOFilenamePrefix);
			}

			// if the project has an Oodle compression Dll, enable the decompressor on IOS
			if (Target.ProjectFile != null)
			{
				DirectoryReference ProjectDir = Target.ProjectFile.Directory;
				string OodleDllPath = DirectoryReference.Combine(ProjectDir, "Binaries/ThirdParty/Oodle/Mac/libUnrealPakPlugin.dylib").FullName;
				if (File.Exists(OodleDllPath))
				{
					Logger.LogDebug("        Registering custom oodle compressor for {Platform}", UnrealTargetPlatform.IOS.ToString());
					CompileEnvironment.Definitions.Add("REGISTER_OODLE_CUSTOM_COMPRESSOR=1");
				}
			}

			// convert runtime version into standardized integer
			float TargetFloat = Target.IOSPlatform.RuntimeVersion;
			int IntPart = (int)TargetFloat;
			int FracPart = (int)((TargetFloat - IntPart) * 10);
			int TargetNum = IntPart * 10000 + FracPart * 100;
			CompileEnvironment.Definitions.Add("MINIMUM_UE_COMPILED_IOS_VERSION=" + TargetNum);

			LinkEnvironment.AdditionalFrameworks.Add(new UEBuildFramework("GameKit"));
			LinkEnvironment.AdditionalFrameworks.Add(new UEBuildFramework("DeviceCheck"));

			if (ProjectSettings.bEnableStoreKitSupport)
			{
				CompileEnvironment.Definitions.Add("UE_WITH_STORE_KIT=1");
				LinkEnvironment.AdditionalFrameworks.Add(new UEBuildFramework("StoreKit"));
			}
			else
			{
				CompileEnvironment.Definitions.Add("UE_WITH_STORE_KIT=0");
			}

			CompileEnvironment.Definitions.Add("UE_IOS_SCENE_LIFECYCLE=" + (ProjectSettings.bUseSceneBasedLifecycle ? "1" : "0"));

			if (Target.IOSPlatform.bEnableRayTracing && Target.IOSPlatform.bSupportsMetalMobileSM6 && Target.Type != TargetType.Server)
			{
				CompileEnvironment.Definitions.Add("RHI_RAYTRACING=1");
			}

			if (ShouldIncludeMetalShaderConverterForIOS(Target))
			{
				CompileEnvironment.Definitions.Add("PLATFORM_SUPPORTS_BINDLESS_RENDERING=1");
			}
		}

		/// <summary>
		/// Setup the binaries for this specific platform.
		/// </summary>
		/// <param name="Target">The target being built</param>
		/// <param name="ExtraModuleNames"></param>
		public override void AddExtraModules(ReadOnlyTargetRules Target, List<string> ExtraModuleNames)
		{
			if (Target.Type != TargetType.Program)
			{
				ExtraModuleNames.Add("IOSPlatformFeatures");
			}
		}

		/// <summary>
		/// Creates a toolchain instance for the given platform.
		/// </summary>
		/// <param name="Target">The target being built</param>
		/// <returns>New toolchain instance.</returns>
		public override UEToolChain CreateToolChain(ReadOnlyTargetRules Target)
		{
			ClangToolChainOptions Options = GetToolChainOptionsForSanitizers(Target);
			IOSProjectSettings ProjectSettings = ReadProjectSettings(Target.ProjectFile);
			return new IOSToolChain(Target, ProjectSettings, Options, Logger);
		}

		/// <inheritdoc/>
		public override void Deploy(TargetReceipt Receipt)
		{
			if (!Receipt.HasValueForAdditionalProperty("CompileAsDll", "true"))
			{
				new UEDeployIOS(Logger).PrepTargetForDeployment(Receipt);
			}
		}
	}

	class IOSPlatformFactory : UEBuildPlatformFactory
	{
		public override UnrealTargetPlatform TargetPlatform => UnrealTargetPlatform.IOS;

		/// <summary>
		/// Register the platform with the UEBuildPlatform class
		/// </summary>
		public override void RegisterBuildPlatforms(ILogger Logger)
		{
			ApplePlatformSDK SDK = new ApplePlatformSDK(Logger);

			// Register this build platform for IOS
			UEBuildPlatform.RegisterBuildPlatform(new IOSPlatform(SDK, Logger), Logger);
			UEBuildPlatform.RegisterPlatformWithGroup(UnrealTargetPlatform.IOS, UnrealPlatformGroup.Apple);
			UEBuildPlatform.RegisterPlatformWithGroup(UnrealTargetPlatform.IOS, UnrealPlatformGroup.IOS);
		}
	}
}
