// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using System.Threading;
using System.Xml.Linq;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	class UEDeployIOS : UEBuildDeploy
	{
		public UEDeployIOS(ILogger InLogger)
			: base(InLogger)
		{
		}

		protected UnrealPluginLanguage? UPL = null;
		public delegate bool FilenameFilter(string InFilename);

		public bool ForDistribution
		{
			get => bForDistribution;
			set => bForDistribution = value;
		}
		bool bForDistribution = false;

		public class VersionUtilities
		{
			public static string? BuildDirectory
			{
				get;
				set;
			}
			public static string? GameName
			{
				get;
				set;
			}

			public static bool bCustomLaunchscreenStoryboard = false;

			static string RunningVersionFilename => Path.Combine(BuildDirectory!, GameName + ".PackageVersionCounter");

			/// <summary>
			/// Reads the GameName.PackageVersionCounter from disk and bumps the minor version number in it
			/// </summary>
			/// <returns></returns>
			public static string ReadRunningVersion()
			{
				string CurrentVersion = "0.0";
				if (File.Exists(RunningVersionFilename))
				{
					CurrentVersion = File.ReadAllText(RunningVersionFilename);
				}

				return CurrentVersion;
			}

			/// <summary>
			/// Pulls apart a version string of one of the two following formats:
			///	  "7301.15 11-01 10:28"   (Major.Minor Date Time)
			///	  "7486.0"  (Major.Minor)
			/// </summary>
			/// <param name="CFBundleVersion"></param>
			/// <param name="VersionMajor"></param>
			/// <param name="VersionMinor"></param>
			/// <param name="TimeStamp"></param>
			public static void PullApartVersion(string CFBundleVersion, out int VersionMajor, out int VersionMinor, out string TimeStamp)
			{
				// Expecting source to be like "7301.15 11-01 10:28" or "7486.0"
				string[] Parts = CFBundleVersion.Split(new char[] { ' ' });

				// Parse the version string
				string[] VersionParts = Parts[0].Split(new char[] { '.' });

				if (!Int32.TryParse(VersionParts[0], out VersionMajor))
				{
					VersionMajor = 0;
				}

				if ((VersionParts.Length < 2) || (!Int32.TryParse(VersionParts[1], out VersionMinor)))
				{
					VersionMinor = 0;
				}

				TimeStamp = "";
				if (Parts.Length > 1)
				{
					TimeStamp = String.Join(" ", Parts, 1, Parts.Length - 1);
				}
			}

			public static string ConstructVersion(int MajorVersion, int MinorVersion)
			{
				return String.Format("{0}.{1}", MajorVersion, MinorVersion);
			}

			/// <summary>
			/// Parses the version string (expected to be of the form major.minor or major)
			/// Also parses the major.minor from the running version file and increments it's minor by 1.
			///
			/// If the running version major matches and the running version minor is newer, then the bundle version is updated.
			///
			/// In either case, the running version is set to the current bundle version number and written back out.
			/// </summary>
			/// <returns>The (possibly updated) bundle version</returns>
			public static string CalculateUpdatedMinorVersionString(string CFBundleVersion)
			{
				// Read the running version and bump it
				int RunningMajorVersion;
				int RunningMinorVersion;

				string RunningVersion = ReadRunningVersion();
				PullApartVersion(RunningVersion, out RunningMajorVersion, out RunningMinorVersion, out _);
				RunningMinorVersion++;

				// Read the passed in version and bump it
				int MajorVersion;
				int MinorVersion;
				PullApartVersion(CFBundleVersion, out MajorVersion, out MinorVersion, out _);
				MinorVersion++;

				// Combine them if the stub time is older
				if ((RunningMajorVersion == MajorVersion) && (RunningMinorVersion > MinorVersion))
				{
					// A subsequent cook on the same sync, the only time that we stomp on the stub version
					MinorVersion = RunningMinorVersion;
				}

				// Combine them together
				string ResultVersionString = ConstructVersion(MajorVersion, MinorVersion);

				// Update the running version file
				Directory.CreateDirectory(Path.GetDirectoryName(RunningVersionFilename)!);
				File.WriteAllText(RunningVersionFilename, ResultVersionString);

				return ResultVersionString;
			}

			/// <summary>
			/// Updates the minor version in the CFBundleVersion key of the specified PList if this is a new package.
			/// Also updates the key EpicAppVersion with the bundle version and the current date/time (no year)
			/// </summary>
			public static string UpdateBundleVersion(string OldPList)
			{
				string CFBundleVersion;
				if (!Unreal.IsBuildMachine())
				{
					int Index = OldPList.IndexOf("CFBundleVersion", StringComparison.Ordinal);
					if (Index != -1)
					{
						int Start = OldPList.IndexOf("<string>", Index, StringComparison.Ordinal) + ("<string>").Length;
						CFBundleVersion = OldPList.Substring(Start, OldPList.IndexOf("</string>", Index, StringComparison.Ordinal) - Start);
						CFBundleVersion = CalculateUpdatedMinorVersionString(CFBundleVersion);
					}
					else
					{
						CFBundleVersion = "0.0";
					}
				}
				else
				{
					// get the changelist
					CFBundleVersion = ReadOnlyBuildVersion.Current.Changelist.ToString();

				}

				return CFBundleVersion;
			}
		}

		protected virtual string GetTargetPlatformName()
		{
			return "IOS";
		}

		public static string EncodeBundleName(string PlistValue, string ProjectName)
		{
			string result = PlistValue.Replace("[PROJECT_NAME]", ProjectName, StringComparison.Ordinal).Replace("_", "", StringComparison.Ordinal);
			result = result.Replace("&", "&amp;", StringComparison.Ordinal);
			result = result.Replace("\"", "&quot;", StringComparison.Ordinal);
			result = result.Replace("\'", "&apos;", StringComparison.Ordinal);
			result = result.Replace("<", "&lt;", StringComparison.Ordinal);
			result = result.Replace(">", "&gt;", StringComparison.Ordinal);

			return result;
		}

		public static string GetMinimumOSVersion(string MinVersion, ILogger Logger)
		{
			string MinVersionToReturn;
			switch (MinVersion)
			{
				case "":
				case "IOS_15":
				case "IOS_Minimum":
					MinVersionToReturn = "15.0";
					break;
				case "IOS_16":
					MinVersionToReturn = "16.0";
					break;
				case "IOS_17":
					MinVersionToReturn = "17.0";
					break;
				case "IOS_18":
					MinVersionToReturn = "18.0";
					break;
				case "IOS_26":
					MinVersionToReturn = "26.0";
					break;
				default:
					MinVersionToReturn = "15.0";
					Logger.LogInformation("MinimumiOSVersion {MinVersion} specified in ini file is no longer supported, defaulting to {MinVersionToReturn}", MinVersion, MinVersionToReturn);
					break;
			}
			return MinVersionToReturn;
		}

		public static void WritePlistFile(FileReference PlistFile, DirectoryReference? ProjectLocation, UnrealPluginLanguage? UPL, UnrealTargetConfiguration Config, string GameName, bool bIsUnrealGame, string ProjectName)
		{
			ConfigHierarchy Ini = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, ProjectLocation, UnrealTargetPlatform.IOS);
			// required capabilities
			List<string> RequiredCaps = new() { "arm64", "metal" };

			// orientations
			string InterfaceOrientation = "";
			Ini.GetString("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "PreferredLandscapeOrientation", out string PreferredLandscapeOrientation);
			Ini.GetBool("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bSupportsLandscapeLeftOrientation", out bool bSupportsLandscapeLeft);
			Ini.GetBool("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bSupportsLandscapeRightOrientation", out bool bSupportsLandscapeRight);

			if (bSupportsLandscapeLeft && bSupportsLandscapeRight)
			{
				// if both landscape orientations are present, set the UIInterfaceOrientation key
				// in the orientation list, the preferred orientation should be first
				if (PreferredLandscapeOrientation == "LandscapeLeft")
				{
					InterfaceOrientation = "\t<key>UIInterfaceOrientation</key>\n\t<string>UIInterfaceOrientationLandscapeLeft</string>\n";
				}
				else
				{
					// by default, landscape right is the preferred orientation - Apple's UI guidlines
					InterfaceOrientation = "\t<key>UIInterfaceOrientation</key>\n\t<string>UIInterfaceOrientationLandscapeRight</string>\n";
				}
			}

			// ITunes file sharing
			Ini.GetBool("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bSupportsITunesFileSharing", out bool bSupportsITunesFileSharing);
			Ini.GetBool("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bSupportsFilesApp", out bool bSupportsFilesApp);

			// device family — controls install eligibility on iPhone vs iPad
			Ini.GetBool("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bSupportsIPhone", out bool bSupportsIPhone);
			Ini.GetBool("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bSupportsIPad", out bool bSupportsIPad);

			// disable https requirement
			Ini.GetBool("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bDisableHTTPS", out bool bDisableHTTPS);

			bool bUseZenStore = false;
			if (Config != UnrealTargetConfiguration.Shipping)
			{
				ConfigHierarchy PlatformGameConfig = ConfigCache.ReadHierarchy(ConfigHierarchyType.Game, ProjectLocation, UnrealTargetPlatform.IOS);
				PlatformGameConfig.GetBool("/Script/UnrealEd.ProjectPackagingSettings", "bUseZenStore", out bUseZenStore);
			}

			// bundle display name
			Ini.GetString("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "BundleDisplayName", out string BundleDisplayName);

			// Get Google Support details
			Ini.GetBool("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bEnableGoogleSupport", out bool bEnableGoogleSupport);

			// Write the Google iOS URL Scheme if we need it.
			Ini.GetString("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "GoogleReversedClientId", out string GoogleReversedClientId);
			bEnableGoogleSupport = bEnableGoogleSupport && !String.IsNullOrWhiteSpace(GoogleReversedClientId);

			// Add remote-notifications as background mode
			Ini.GetBool("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bEnableRemoteNotificationsSupport", out bool bRemoteNotificationsSupported);

			// Add audio as background mode
			Ini.GetBool("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bSupportsBackgroundAudio", out bool bBackgroundAudioSupported);

			// Add background fetch as background mode
			Ini.GetBool("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bEnableBackgroundFetch", out bool bBackgroundFetch);

			// UIScene lifecycle adoption. Must match the UE_IOS_SCENE_LIFECYCLE compile define.
			Ini.GetBool("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bUseSceneBasedLifecycle", out bool bUseSceneBasedLifecycle);

			// Get any Location Services permission descriptions added
			Ini.GetString("/Script/LocationServicesIOSEditor.LocationServicesIOSSettings", "LocationAlwaysUsageDescription", out string LocationAlwaysUsageDescription);
			Ini.GetString("/Script/LocationServicesIOSEditor.LocationServicesIOSSettings", "LocationWhenInUseDescription", out string LocationWhenInUseDescription);

			// extra plist data
			Ini.GetString("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "AdditionalPlistData", out string ExtraData);

			Ini.GetBool("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bCustomLaunchscreenStoryboard", out VersionUtilities.bCustomLaunchscreenStoryboard);

			// generate the plist file
			StringBuilder Text = new StringBuilder();
			Text.AppendLine("<?xml version=\"1.0\" encoding=\"UTF-8\"?>");
			Text.AppendLine("<!DOCTYPE plist PUBLIC \"-//Apple//DTD PLIST 1.0//EN\" \"http://www.apple.com/DTDs/PropertyList-1.0.dtd\">");
			Text.AppendLine("<plist version=\"1.0\">");
			Text.AppendLine("<dict>");
			Text.AppendLine("\t<key>CFBundleURLTypes</key>");
			Text.AppendLine("\t<array>");
			Text.AppendLine("\t\t<dict>");

			Text.AppendLine("\t\t\t<key>CFBundleURLName</key>");
			Text.AppendLine("\t\t\t<string>com.Epic.Unreal</string>");
			Text.AppendLine("\t\t\t<key>CFBundleURLSchemes</key>");
			Text.AppendLine("\t\t\t<array>");
			Text.AppendLine(String.Format("\t\t\t\t<string>{0}</string>", bIsUnrealGame ? "UnrealGame" : GameName));
			if (bEnableGoogleSupport)
			{
				Text.AppendLine(String.Format("\t\t\t\t<string>{0}</string>", GoogleReversedClientId));
			}
			Text.AppendLine("\t\t\t</array>");
			Text.AppendLine("\t\t</dict>");
			Text.AppendLine("\t</array>");
			Text.AppendLine("\t<key>UIStatusBarHidden</key>");
			Text.AppendLine("\t<true/>");
			Text.AppendLine("\t<key>UIFileSharingEnabled</key>");
			Text.AppendLine(String.Format("\t<{0}/>", bSupportsITunesFileSharing ? "true" : "false"));
			if (bSupportsFilesApp)
			{
				Text.AppendLine("\t<key>LSSupportsOpeningDocumentsInPlace</key>");
				Text.AppendLine("\t<true/>");
			}

			Text.AppendLine("\t<key>CFBundleDisplayName</key>");
			Text.AppendLine(String.Format("\t<string>{0}</string>", EncodeBundleName(BundleDisplayName, ProjectName)));

			Text.AppendLine("\t<key>UIRequiresFullScreen</key>");
			Text.AppendLine("\t<true/>");

			Text.AppendLine("\t<key>UIViewControllerBasedStatusBarAppearance</key>");
			Text.AppendLine("\t<false/>");
			if (!String.IsNullOrEmpty(InterfaceOrientation))
			{
				Text.AppendLine(InterfaceOrientation);
			}

			Text.AppendLine("\t<key>UIRequiredDeviceCapabilities</key>");
			Text.AppendLine("\t<array>");
			foreach (string Cap in RequiredCaps)
			{
				Text.AppendLine($"\t\t<string>{Cap}</string>\n");
			}
			Text.AppendLine("\t</array>");

			Text.AppendLine("\t<key>UILaunchStoryboardName</key>");
			Text.AppendLine("\t<string>LaunchScreen</string>");

			Text.AppendLine("\t<key>UIDeviceFamily</key>");
			Text.AppendLine("\t<array>");
			if (bSupportsIPhone)
			{
				Text.AppendLine("\t\t<integer>1</integer>");
			}
			if (bSupportsIPad)
			{
				Text.AppendLine("\t\t<integer>2</integer>");
			}
			Text.AppendLine("\t</array>");

			// Support high refresh rates (iPhone only)
			// https://developer.apple.com/documentation/quartzcore/optimizing_promotion_refresh_rates_for_iphone_13_pro_and_ipad_pro
			Ini.GetBool("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bSupportHighRefreshRates", out bool bSupportHighRefreshRates);
			if (bSupportHighRefreshRates)
			{
				Text.AppendLine("\t<key>CADisableMinimumFrameDurationOnPhone</key><true/>");
			}

			// set exempt encryption
			Ini.GetBool("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bUsesNonExemptEncryption", out bool bUsesNonExemptEncryption);
			Ini.GetString("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "ITSEncryptionExportComplianceCode", out string ITSEncryptionExportComplianceCode);
			Text.AppendLine("\t<key>ITSAppUsesNonExemptEncryption</key>");
			Text.AppendLine(String.Format("\t<{0}/>", bUsesNonExemptEncryption ? "true" : "false"));
			if (bUsesNonExemptEncryption && !String.IsNullOrWhiteSpace(ITSEncryptionExportComplianceCode))
			{
				Text.AppendLine("\t<key>ITSEncryptionExportComplianceCode</key>");
				Text.AppendLine(String.Format("\t<string>{0}</string>", ITSEncryptionExportComplianceCode));
			}
			
			// add location services descriptions if used
			if (!String.IsNullOrWhiteSpace(LocationAlwaysUsageDescription))
			{
				Text.AppendLine("\t<key>NSLocationAlwaysAndWhenInUseUsageDescription</key>");
				Text.AppendLine(String.Format("\t<string>{0}</string>", LocationAlwaysUsageDescription));
			}
			if (!String.IsNullOrWhiteSpace(LocationWhenInUseDescription))
			{
				Text.AppendLine("\t<key>NSLocationWhenInUseUsageDescription</key>");
				Text.AppendLine(String.Format("\t<string>{0}</string>", LocationWhenInUseDescription));
			}
			// disable HTTPS requirement
			if (bDisableHTTPS || bUseZenStore)
			{
				Text.AppendLine("\t<key>NSAppTransportSecurity</key>");
				Text.AppendLine("\t\t<dict>");
				Text.AppendLine("\t\t\t<key>NSAllowsArbitraryLoads</key><true/>");
				Text.AppendLine("\t\t\t<key>NSAllowsArbitraryLoadsInWebContent</key><true/>");
				Text.AppendLine("\t\t\t<key>NSAllowsLocalNetworking</key><true/>");
				
				if (bUseZenStore)
				{
					Text.AppendLine("\t\t\t<key>NSExceptionDomains</key>");
					Text.AppendLine("\t\t\t<dict>");
					Text.AppendLine("\t\t\t\t<key>169.245.0.0/16</key>");
					Text.AppendLine("\t\t\t\t<dict>");
					Text.AppendLine("\t\t\t\t\t<key>NSIncludesSubdomains</key><true/>");
					Text.AppendLine("\t\t\t\t\t<key>NSExceptionAllowsInsecureHTTPLoads</key><true/>");
					Text.AppendLine("\t\t\t\t</dict>");
					Text.AppendLine("\t\t\t</dict>");
				}
				Text.AppendLine("\t\t</dict>");
			}

			// add a TVOS setting since they share this file
			Text.AppendLine("\t<key>TVTopShelfImage</key>");
			Text.AppendLine("\t<dict>");
			Text.AppendLine("\t\t<key>TVTopShelfPrimaryImageWide</key>");
			Text.AppendLine("\t\t<string>Top Shelf Image Wide</string>");
			Text.AppendLine("\t</dict>");

			if (!String.IsNullOrEmpty(ExtraData))
			{
				ExtraData = ExtraData.Replace("\\n", "\n", StringComparison.Ordinal);
				foreach (string Line in ExtraData.Split("\r\n".ToCharArray()))
				{
					if (!String.IsNullOrWhiteSpace(Line))
					{
						Text.AppendLine("\t" + Line);
					}
				}
			}

			// UIScene lifecycle manifest. UIKit only routes scene callbacks when this key is present.
			// Single-window only - UIApplicationSupportsMultipleScenes stays false.
			if (bUseSceneBasedLifecycle)
			{
				Text.AppendLine("\t<key>UIApplicationSceneManifest</key>");
				Text.AppendLine("\t<dict>");
				Text.AppendLine("\t\t<key>UIApplicationSupportsMultipleScenes</key>");
				Text.AppendLine("\t\t<false/>");
				Text.AppendLine("\t\t<key>UISceneConfigurations</key>");
				Text.AppendLine("\t\t<dict>");
				Text.AppendLine("\t\t\t<key>UIWindowSceneSessionRoleApplication</key>");
				Text.AppendLine("\t\t\t<array>");
				Text.AppendLine("\t\t\t\t<dict>");
				Text.AppendLine("\t\t\t\t\t<key>UISceneConfigurationName</key>");
				Text.AppendLine("\t\t\t\t\t<string>Default Configuration</string>");
				Text.AppendLine("\t\t\t\t\t<key>UISceneDelegateClassName</key>");
				Text.AppendLine("\t\t\t\t\t<string>IOSSceneDelegate</string>");
				Text.AppendLine("\t\t\t\t</dict>");
				Text.AppendLine("\t\t\t</array>");
				Text.AppendLine("\t\t</dict>");
				Text.AppendLine("\t</dict>");
			}

			// Add remote-notifications as background mode
			if (bRemoteNotificationsSupported || bBackgroundFetch || bBackgroundAudioSupported)
			{
				Text.AppendLine("\t<key>UIBackgroundModes</key>");
				Text.AppendLine("\t<array>");
				if (bBackgroundAudioSupported)
				{
					Text.AppendLine("\t\t<string>audio</string>");
				}
				if (bRemoteNotificationsSupported)
				{
					Text.AppendLine("\t\t<string>remote-notification</string>");
				}
				if (bBackgroundFetch)
				{
					Text.AppendLine("\t\t<string>fetch</string>");
				}
				Text.AppendLine("\t</array>");
			}
			Text.AppendLine("</dict>");
			Text.AppendLine("</plist>");

			DirectoryReference.CreateDirectory(PlistFile.Directory);

			if (UPL != null)
			{
				// Allow UPL to modify the plist here
				XDocument XDoc;
				try
				{
					XDoc = XDocument.Parse(Text.ToString());
				}
				catch (Exception e)
				{
					throw new BuildException("plist is invalid {0}\n{1}", e, Text.ToString());
				}

				XDoc.DocumentType!.InternalSubset = "";
				UPL.ProcessPluginNode("None", "iosPListUpdates", "", ref XDoc);
				string result = XDoc.Declaration?.ToString() + "\n" + XDoc.ToString().Replace(
					"<!DOCTYPE plist PUBLIC \"-//Apple//DTD PLIST 1.0//EN\" \"http://www.apple.com/DTDs/PropertyList-1.0.dtd\"[]>",
					"<!DOCTYPE plist PUBLIC \"-//Apple//DTD PLIST 1.0//EN\" \"http://www.apple.com/DTDs/PropertyList-1.0.dtd\">",
					StringComparison.Ordinal);
				File.WriteAllText(PlistFile.FullName, result);

				Text = new StringBuilder(result);
			}

			Text = Text.Replace("[PROJECT_NAME]", "$(UE_PROJECT_NAME)");

			File.WriteAllText(PlistFile.FullName, Text.ToString());
		}

		private static readonly Lock GenerateIOSPListLock = new();

		public static bool GenerateIOSPList(FileReference? ProjectFile, UnrealTargetConfiguration Config, string ProjectDirectory, bool bIsUnrealGame, string GameName, bool bIsClient, string ProjectName, string InEngineDir, string AppDirectory, UnrealPluginLanguage? UPL, string? BundleID, bool bBuildAsFramework, ILogger Logger)
		{
			lock (GenerateIOSPListLock) // Needs to be protected because multiple configs are built in parallel
			{
				if (AppleExports.CreatingAppOnWindows(ProjectFile))
				{
					return AppleOnWindowsAppMaker.GenerateLegacyIOSPList(ProjectFile, ProjectDirectory, bIsUnrealGame, GameName, bIsClient, ProjectName, InEngineDir, AppDirectory, UPL, BundleID, bBuildAsFramework, Logger);
				}
				
				// get the settings from the ini file
				// plist replacements
				DirectoryReference? DirRef = bIsUnrealGame ? (!String.IsNullOrEmpty(UnrealBuildTool.GetRemoteIniPath()) ? new DirectoryReference(UnrealBuildTool.GetRemoteIniPath()!) : null) : new DirectoryReference(ProjectDirectory);

				// generate the Info.plist for future use
				string BuildDirectory = ProjectDirectory + "/Build/IOS";
				string IntermediateDirectory = ProjectDirectory + "/Intermediate/IOS";
				string PListFile = IntermediateDirectory + "/" + GameName + "-Info.plist";
				;
				ProjectName = !String.IsNullOrEmpty(ProjectName) ? ProjectName : GameName;
				VersionUtilities.BuildDirectory = BuildDirectory;
				VersionUtilities.GameName = GameName;

				WritePlistFile(new FileReference(PListFile), DirRef, UPL, Config, GameName, bIsUnrealGame, ProjectName);

				if (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Mac && !bBuildAsFramework)
				{
					// need to make sure touching this plist is in the same lock as FinalizeAppWithXcode()
					string MutexName = GlobalSingleInstanceMutex.GetUniqueMutexForPath("UnrealBuildTool_XcodeBuild", Unreal.RootDirectory);
					using (SingleInstanceMutex.Acquire(MutexName))
					{
						FileReference FinalPlistFile;
						FinalPlistFile = new FileReference($"{ProjectDirectory}/Build/IOS/UBTGenerated/Info.Template.plist");

						DirectoryReference.CreateDirectory(FinalPlistFile.Directory);
						// @todo: writeifdifferent is better
						FileReference.Delete(FinalPlistFile);
						File.Copy(PListFile, FinalPlistFile.FullName);
					}
				}

				return true;
			}
		}

		public static VersionNumber? GetSdkVersion(TargetReceipt Receipt)
		{
			VersionNumber? SdkVersion = null;
			if (Receipt != null)
			{
				ReceiptProperty? SdkVersionProperty = Receipt.AdditionalProperties.FirstOrDefault(x => x.Name == "SDK");
				if (SdkVersionProperty != null)
				{
					_ = VersionNumber.TryParse(SdkVersionProperty.Value, out SdkVersion);
				}
			}
			return SdkVersion;
		}

		public static bool GetCompileAsDll(TargetReceipt? Receipt)
		{
			if (Receipt != null)
			{
				ReceiptProperty? CompileAsDllProperty = Receipt.AdditionalProperties.FirstOrDefault(x => x.Name == "CompileAsDll");
				if (CompileAsDllProperty != null && CompileAsDllProperty.Value == "true")
				{
					return true;
				}
			}
			return false;
		}

		public bool GeneratePList(FileReference ProjectFile, UnrealTargetConfiguration Config, string ProjectDirectory, bool bIsUnrealGame, string GameName, bool bIsClient, string ProjectName, string InEngineDir, string AppDirectory, TargetReceipt Receipt)
		{
			List<string> UPLScripts = CollectPluginDataPaths(Receipt.AdditionalProperties, Logger);
			bool bBuildAsFramework = GetCompileAsDll(Receipt);
			return GeneratePList(ProjectFile, Config, ProjectDirectory, bIsUnrealGame, GameName, bIsClient, ProjectName, InEngineDir, AppDirectory, UPLScripts, "", bBuildAsFramework);
		}

		public virtual bool GeneratePList(FileReference? ProjectFile, UnrealTargetConfiguration Config, string ProjectDirectory, bool bIsUnrealGame, string GameName, bool bIsClient, string ProjectName, string InEngineDir, string AppDirectory, List<string> UPLScripts, string? BundleID, bool bBuildAsFramework)
		{
			// remember name with -IOS-Shipping, etc
			// string ExeName = GameName;

			// strip out the markup
			GameName = GameName.Split("-".ToCharArray())[0];

			List<(string UEArch, string NativeArch)> ProjectArches = new() { ("None", "None") };

			string UPLBuildDir = Path.Combine(ProjectDirectory, "Intermediate", "IOS", "UPLBuild");

			if(AppleExports.CreatingAppOnWindows(ProjectFile))
			{
				// get the receipt
				if (bIsUnrealGame)
				{
					//               ReceiptFilename = TargetReceipt.GetDefaultPath(Unreal.EngineDirectory, "UnrealGame", UnrealTargetPlatform.IOS, Config, "");
					UPLBuildDir = Path.Combine(Unreal.EngineDirectory.ToString(), "Intermediate", "IOS-Deploy", "UnrealGame", Config.ToString(), "Payload", "UnrealGame.app");
				}
				else
				{
					//                ReceiptFilename = TargetReceipt.GetDefaultPath(new DirectoryReference(ProjectDirectory), GameName, UnrealTargetPlatform.IOS, Config, "");
					UPLBuildDir = AppDirectory;//Path.Combine(ProjectDirectory, "Binaries", "IOS", "Payload", ProjectName + ".app");
				}
			}
			else
			{
				Directory.CreateDirectory(UPLBuildDir);
			}

			string RelativeEnginePath = Unreal.EngineDirectory.MakeRelativeTo(DirectoryReference.GetCurrentDirectory());

			UnrealPluginLanguage UPL = new UnrealPluginLanguage(ProjectFile, UPLScripts, ProjectArches, null, UnrealTargetPlatform.IOS, Logger);

			// Passing in true for distribution is not ideal here but given the way that ios packaging happens and this call chain it seems unavoidable for now, maybe there is a way to correctly pass it in that I can't find?
			UPL.Init(ProjectArches.Select(x => x.NativeArch), true, RelativeEnginePath, UPLBuildDir, ProjectDirectory, Config.ToString(), false);

			return GenerateIOSPList(ProjectFile, Config, ProjectDirectory, bIsUnrealGame, GameName, bIsClient, ProjectName, InEngineDir, AppDirectory, UPL, BundleID, bBuildAsFramework, Logger);
		}

		public bool PrepForUATPackageOrDeploy(UnrealTargetConfiguration Config, FileReference ProjectFile, string InProjectName, string InProjectDirectory, FileReference Executable, string InEngineDir, bool bCreateStubIPA, TargetReceipt Receipt)
		{
			List<string> UPLScripts = CollectPluginDataPaths(Receipt.AdditionalProperties, Logger);
			bool bBuildAsFramework = GetCompileAsDll(Receipt);
			return PrepForUATPackageOrDeploy(Config, ProjectFile, InProjectName, InProjectDirectory, Executable, InEngineDir, bCreateStubIPA, UPLScripts, "", bBuildAsFramework);
		}

		public bool PrepForUATPackageOrDeploy(UnrealTargetConfiguration Config, FileReference? ProjectFile, string InProjectName, string InProjectDirectory, FileReference Executable, string InEngineDir, bool bCreateStubIPA, List<string> UPLScripts, string? BundleID, bool bBuildAsFramework)
		{
			if (BuildHostPlatform.Current.Platform != UnrealTargetPlatform.Mac)
			{
				throw new BuildException("UEDeployIOS.PrepForUATPackageOrDeploy only supports running on the Mac");
			}

			// If we are building as a framework, we don't need to do all of this.
			if (bBuildAsFramework)
			{
				return false;
			}

			string SubDir = GetTargetPlatformName();

			bool bIsUnrealGame = Executable.FullName.Contains("UnrealGame", StringComparison.Ordinal);
			DirectoryReference BinaryPath = Executable.Directory!;
			string GameExeName = Executable.GetFileName();
			string GameName = bIsUnrealGame ? "UnrealGame" : GameExeName.Split("-".ToCharArray())[0];
			string PayloadDirectory = BinaryPath + "/Payload";
			string AppDirectory = PayloadDirectory + "/" + GameName + ".app";

			if (AppleExports.CreatingAppOnWindows(ProjectFile))
			{
				return AppleOnWindowsAppMaker.PrepareForUAT(this, Config, ProjectFile, InProjectName, InProjectDirectory, InEngineDir, bCreateStubIPA, UPLScripts, BundleID, bBuildAsFramework, BinaryPath, PayloadDirectory, AppDirectory, SubDir, bIsUnrealGame, GameExeName, GameName, Logger, GetTargetPlatformName());
			}

			Logger.LogInformation("Generating plist (only step needed when deploying using Xcode)");
			AppDirectory = BinaryPath + "/" + GameExeName + ".app";
			GeneratePList(ProjectFile, Config, InProjectDirectory, bIsUnrealGame, GameExeName, false, InProjectName, InEngineDir, AppDirectory, UPLScripts, BundleID, bBuildAsFramework);

			return false;
		}

		public override bool PrepTargetForDeployment(TargetReceipt Receipt)
		{
			List<string> UPLScripts = CollectPluginDataPaths(Receipt.AdditionalProperties, Logger);
			bool bBuildAsFramework = GetCompileAsDll(Receipt);
			return PrepTargetForDeployment(Receipt.ProjectFile, Receipt.TargetName, Receipt.BuildProducts.First(x => x.Type == BuildProductType.Executable).Path, Receipt.Configuration, UPLScripts, false, "", bBuildAsFramework);
		}

		public bool PrepTargetForDeployment(FileReference? ProjectFile, string TargetName, FileReference Executable, UnrealTargetConfiguration Configuration, List<string> UPLScripts, bool bCreateStubIPA, string? BundleID, bool bBuildAsFramework)
		{
			string GameName = TargetName;
			string ProjectDirectory = (DirectoryReference.FromFile(ProjectFile) ?? Unreal.EngineDirectory).FullName;
			bool bIsUnrealGame = GameName.Contains("UnrealGame", StringComparison.Ordinal);

			if (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Mac && Environment.GetEnvironmentVariable("UBT_NO_POST_DEPLOY") != "true")
			{
				return PrepForUATPackageOrDeploy(Configuration, ProjectFile, GameName, ProjectDirectory, Executable, "../../Engine", bCreateStubIPA, UPLScripts, BundleID, bBuildAsFramework);
			}
			else
			{
				// @todo tvos merge: This used to copy the bundle back - where did that code go? It needs to be fixed up for TVOS directories
				GeneratePList(ProjectFile, Configuration, ProjectDirectory, bIsUnrealGame, GameName, false, (ProjectFile == null) ? "" : Path.GetFileNameWithoutExtension(ProjectFile.FullName), "../../Engine", "", UPLScripts, BundleID, bBuildAsFramework);
			}
			return true;
		}

		public static List<string> CollectPluginDataPaths(List<ReceiptProperty> ReceiptProperties, ILogger Logger)
		{
			List<string> PluginExtras = new List<string>();
			if (ReceiptProperties == null)
			{
				Logger.LogInformation("Receipt is NULL");
				//Logger.LogInformation("Receipt is NULL");
				return PluginExtras;
			}

			// collect plugin extra data paths from target receipt
			IEnumerable<ReceiptProperty> Results = ReceiptProperties.Where(x => x.Name == "IOSPlugin");
			foreach (ReceiptProperty Property in Results)
			{
				// Keep only unique paths
				string PluginPath = Property.Value;
				if (PluginExtras.FirstOrDefault(x => x == PluginPath) == null)
				{
					PluginExtras.Add(PluginPath);
					Logger.LogInformation("IOSPlugin: {PluginPath}", PluginPath);
				}
			}
			return PluginExtras;
		}
	}
}
