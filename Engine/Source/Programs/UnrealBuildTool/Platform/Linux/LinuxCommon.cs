// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics;
using System.IO;
using Microsoft.Extensions.Logging;

namespace UnrealBuildTool
{
	class LinuxCommon
	{
		public static string? Which(string name, ILogger Logger)
		{
			Process proc = new Process();
			proc.StartInfo.FileName = "/bin/sh";
			proc.StartInfo.Arguments = System.String.Format("-c 'which {0}'", name);
			proc.StartInfo.UseShellExecute = false;
			proc.StartInfo.CreateNoWindow = true;
			proc.StartInfo.RedirectStandardOutput = true;
			proc.StartInfo.RedirectStandardError = true;

			proc.Start();
			proc.WaitForExit();

			string? path = proc.StandardOutput.ReadLine();
			Logger.LogDebug("which {Name} result: ({ExitCode}) {Path}", name, proc.ExitCode, path);

			if (proc.ExitCode == 0 && System.String.IsNullOrEmpty(proc.StandardError.ReadToEnd()))
			{
				return path;
			}
			return null;
		}

		public static string? WhichClang(ILogger Logger)
		{
			string? InternalSDKPath = UEBuildPlatform.GetSDK(UnrealTargetPlatform.Linux)?.GetInternalSDKPath();
			if (!System.String.IsNullOrEmpty(InternalSDKPath))
			{
				return Path.Combine(InternalSDKPath, "bin", "clang++");
			}

			return Which("clang++", Logger);
		}
	}
}

