// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace UnrealBuildTool
{
	/// <summary>
	/// Public Linux functions exposed to UAT
	/// </summary>
	public static class LinuxExports
	{
		/// <summary>
		/// 
		/// </summary>
		/// <param name="SourceFile"></param>
		/// <param name="TargetFile"></param>
		/// <param name="Logger">Logger for output</param>
		public static void StripSymbols(FileReference SourceFile, FileReference TargetFile, ILogger Logger)
		{
			LinuxPlatformSDK? LinuxSDK = UEBuildPlatformSDK.GetSDKForPlatform("Linux") as LinuxPlatformSDK;

			if (LinuxSDK == null)
			{
				LinuxSDK = new LinuxPlatformSDK(Logger);
				UEBuildPlatformSDK.RegisterSDKForPlatform(LinuxSDK, "Linux", true);
			}

			LinuxToolChain ToolChain = new LinuxToolChain(LinuxPlatform.DefaultHostArchitecture, LinuxSDK, ClangToolChainOptions.None, Logger);
			ToolChain.StripSymbols(SourceFile, TargetFile, Logger);
		}
	}
}
