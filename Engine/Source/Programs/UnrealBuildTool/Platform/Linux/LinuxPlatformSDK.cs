// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Text.RegularExpressions;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

///////////////////////////////////////////////////////////////////
// If you are looking for supported version numbers, look in the
// LinuxPlatformSDK.Versions.cs file next to this file
///////////////////////////////////////////////////////////////////

namespace UnrealBuildTool
{
	partial class LinuxPlatformSDK : UEBuildPlatformSDK
	{
		public LinuxPlatformSDK(ILogger Logger)
			: base(Logger)
		{
		}

		protected override string? GetInstalledSDKVersion()
		{
			// @todo turnkey: ForceUseSystemCompiler() returns true, we should probably run system clang -V or similar to get version

			DirectoryReference? SDKDir = GetSDKLocation();
			if (SDKDir == null)
			{
				return null;
			}

			FileReference VersionFile = FileReference.Combine(SDKDir, SDKVersionFileName());

			if (!FileReference.Exists(VersionFile))
			{
				// ErrorMessage = "Cannot use an old toolchain (missing " + PlatformSDK.SDKVersionFileName() + " file, assuming version earlier than v11)";
				return null;
			}

			StreamReader SDKVersionFile = new StreamReader(VersionFile.FullName);
			string? SDKVersionString = SDKVersionFile.ReadLine();
			SDKVersionFile.Close();

			return SDKVersionString;
		}

		public override bool TryConvertVersionToInt(string? StringValue, out ulong OutValue, string? Hint)
		{
			if (StringValue != null)
			{
				// if it doesnt start with a v (for an SDK version), assume its a valid version
				// which will be used for devices. If a messed up toolchain ends up not having a v<num>
				// 1 will be an invalid range number to pass which will fail the SDK range check
				if (StringValue[0] != 'v')
				{
					OutValue = 1;
					return true;
				}

				// Example: v11_clang-5.0.0-centos7
				string FullVersionPattern = @"^v([0-9]+)_.*$";
				Match Result = Regex.Match(StringValue, FullVersionPattern);
				if (Result.Success)
				{
					return UInt64.TryParse(Result.Groups[1].Value, out OutValue);
				}
			}

			OutValue = 0;
			return false;
		}

		protected override bool PlatformSupportsAutoSDKs()
		{
			return true;
		}

		protected override bool DoesHookRequireAdmin(SDKHookType Hook)
		{
			return false;
		}

		/// <summary>
		/// Platform name (embeds architecture for now)
		/// </summary>
		private const string TargetPlatformName = "Linux_x64";

		/// <summary>
		/// Force using system compiler and error out if not possible
		/// </summary>
		private int bForceUseSystemCompiler = -1;

		/// <summary>
		/// Whether to compile with the verbose flag
		/// </summary>
		public bool bVerboseCompiler = false;

		/// <summary>
		/// Whether to link with the verbose flag
		/// </summary>
		public bool bVerboseLinker = false;

		/// <summary>
		/// Whether platform supports switching SDKs during runtime
		/// </summary>
		/// <returns>true if supports</returns>

		/// <summary>
		/// Returns a path to the internal SDK
		/// </summary>
		/// <returns>Valid path to the internal SDK, null otherwise</returns>
		public override string? GetInternalSDKPath()
		{
			string? SDKRoot = Environment.GetEnvironmentVariable(SDKRootEnvVar);
			if (!String.IsNullOrEmpty(SDKRoot))
			{
				string AutoSDKPath = Path.Combine(SDKRoot, "Host" + BuildHostPlatform.Current.Platform, GetAutoSDKPlatformName(), GetAutoSDKDirectoryForMainVersion(), LinuxPlatform.DefaultHostArchitecture.LinuxName);
				if (DirectoryReference.Exists(new DirectoryReference(AutoSDKPath)))
				{
					return AutoSDKPath;
				}
			}

			string InTreeSDKPath = Path.Combine(LinuxPlatformSDK.GetInTreeSDKRoot().FullName, GetMainVersion(), LinuxPlatform.DefaultHostArchitecture.LinuxName);
			if (DirectoryReference.Exists(new DirectoryReference(InTreeSDKPath)))
			{
				return InTreeSDKPath;
			}

			return null;
		}

		protected override bool PreferAutoSDK()
		{
			// having LINUX_ROOT set (for legacy reasons or for convenience of cross-compiling certain third party libs) should not make UBT skip AutoSDKs
			return true;
		}

		public static string HaveLinuxDependenciesFile()
		{
			// This file must have no extension so that GitDeps considers it a binary dependency - it will only be pulled by the Setup script if Linux is enabled.
			return "HaveLinuxDependencies";
		}

		public static string SDKVersionFileName()
		{
			return "ToolchainVersion.txt";
		}

		/// <summary>
		/// Returns the in-tree root for the Linux Toolchain for this host platform.
		/// </summary>
		private static DirectoryReference GetInTreeSDKRoot()
		{
			return DirectoryReference.Combine(Unreal.RootDirectory, "Engine/Extras/ThirdPartyNotUE/SDKs", "Host" + BuildHostPlatform.Current.Platform, TargetPlatformName);
		}

		/// <summary>
		/// Whether a host can use its system sdk for this platform
		/// </summary>
		public virtual bool ForceUseSystemCompiler()
		{
			// by default tools chains don't parse arguments, but we want to be able to check the -bForceUseSystemCompiler flag.
			if (bForceUseSystemCompiler == -1)
			{
				bForceUseSystemCompiler = 0;
				string[] CmdLine = Environment.GetCommandLineArgs();

				foreach (string CmdLineArg in CmdLine)
				{
					if (CmdLineArg.Equals("-ForceUseSystemCompiler", StringComparison.OrdinalIgnoreCase))
					{
						bForceUseSystemCompiler = 1;
						break;
					}
				}
			}

			return bForceUseSystemCompiler == 1;
		}

		/// <summary>
		/// Returns the root SDK path for all architectures
		/// WARNING: Do not cache this value - it may be changed after sourcing OutputEnvVars.txt
		/// </summary>
		/// <returns>Valid SDK string</returns>
		public virtual DirectoryReference? GetSDKLocation()
		{
			// if new multi-arch toolchain is used, prefer it
			DirectoryReference? MultiArchRoot = DirectoryReference.FromString(Environment.GetEnvironmentVariable("LINUX_MULTIARCH_ROOT"));

			if (MultiArchRoot == null)
			{
				// check if in-tree SDK is available
				DirectoryReference InTreeSDKVersionRoot = GetInTreeSDKRoot();
				if (InTreeSDKVersionRoot != null)
				{
					DirectoryReference InTreeSDKVersionPath = DirectoryReference.Combine(InTreeSDKVersionRoot, GetMainVersion());
					if (DirectoryReference.Exists(InTreeSDKVersionPath))
					{
						MultiArchRoot = InTreeSDKVersionPath;
					}
				}
			}
			return MultiArchRoot;
		}

		/// <summary>
		/// Returns the SDK path for a specific architecture
		/// WARNING: Do not cache this value - it may be changed after sourcing OutputEnvVars.txt
		/// </summary>
		/// <returns>Valid SDK DirectoryReference</returns>
		public virtual DirectoryReference? GetBaseLinuxPathForArchitecture(UnrealArch Architecture)
		{
			// if new multi-arch toolchain is used, prefer it
			DirectoryReference? MultiArchRoot = GetSDKLocation();
			DirectoryReference? BaseLinuxPath;

			if (MultiArchRoot != null)
			{
				BaseLinuxPath = DirectoryReference.Combine(MultiArchRoot, Architecture.LinuxName);
			}
			else
			{
				// use cross linux toolchain if LINUX_ROOT is specified
				BaseLinuxPath = DirectoryReference.FromString(Environment.GetEnvironmentVariable("LINUX_ROOT"));
			}
			return BaseLinuxPath;
		}

		/// <summary>
		/// Whether the path contains a valid clang version
		/// </summary>
		private static bool IsValidClangPath(DirectoryReference BaseLinuxPath)
		{
			FileReference ClangPath = FileReference.Combine(BaseLinuxPath, @"bin", (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Win64) ? "clang++.exe" : "clang++");
			return FileReference.Exists(ClangPath);
		}

		/// <summary>
		/// Whether the required external SDKs are installed for this platform
		/// </summary>
		protected override SDKStatus HasRequiredManualSDKInternal()
		{
			// FIXME: UBT should loop across all the architectures and compile for all the selected ones.

			// do not cache this value - it may be changed after sourcing OutputEnvVars.txt
			DirectoryReference? BaseLinuxPath = GetBaseLinuxPathForArchitecture(LinuxPlatform.DefaultHostArchitecture);

			if (ForceUseSystemCompiler())
			{
				if (!String.IsNullOrEmpty(LinuxCommon.WhichClang(Logger)))
				{
					return SDKStatus.Valid;
				}
			}
			else if (BaseLinuxPath != null)
			{
				// paths to our toolchains if BaseLinuxPath is specified
				if (IsValidClangPath(BaseLinuxPath))
				{
					return SDKStatus.Valid;
				}
			}

			return SDKStatus.Invalid;
		}
	}
}
