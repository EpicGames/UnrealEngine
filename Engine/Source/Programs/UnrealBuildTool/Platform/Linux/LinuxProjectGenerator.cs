// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text.RegularExpressions;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace UnrealBuildTool
{
	/// <summary>
	/// Base class for platform-specific project generators
	/// </summary>
	class LinuxProjectGenerator : PlatformProjectGenerator
	{
		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="Arguments">Command line arguments passed to the project generator</param>
		/// <param name="Logger">Logger for output</param>
#pragma warning disable IDE0060 // Remove unused parameter - constructor is found by reflection in GenerateProjectFilesMode.ExecuteAsync
		public LinuxProjectGenerator(CommandLineArguments Arguments, ILogger Logger)
			: base(Logger)
#pragma warning restore IDE0060
		{
		}

		/// <summary>
		/// Enumerate all the platforms that this generator supports
		/// </summary>
		public override IEnumerable<UnrealTargetPlatform> GetPlatforms()
		{
			yield return UnrealTargetPlatform.Linux;
			yield return UnrealTargetPlatform.LinuxArm64;
		}

		/// <inheritdoc/>
		public override bool HasVisualStudioSupport(VSSettings InVSSettings)
		{
			return false;
		}

		/// <inheritdoc/>
		public override IList<string> GetSystemIncludePaths(UEBuildTarget InTarget)
		{
			List<string> Result = [];
			string? UseLibcxxEnvVarOverride = Environment.GetEnvironmentVariable("UE_LINUX_USE_LIBCXX");
			bool ShouldUseLibcxx = String.IsNullOrEmpty(UseLibcxxEnvVarOverride) || UseLibcxxEnvVarOverride == "1";

			UnrealArch TargetArchitecture = InTarget.Architectures.SingleArchitecture;

			UEBuildPlatformSDK BuildPlatformSdk = UEBuildPlatform.GetSDK(InTarget.Platform)!;
			LinuxPlatformSDK? LinuxPlatformSdk = BuildPlatformSdk as LinuxPlatformSDK;
			string? InternalSdkPath = BuildPlatformSdk.GetInternalSDKPath();
			string? BaseLinuxPath = InternalSdkPath ?? LinuxPlatformSdk?.GetBaseLinuxPathForArchitecture(TargetArchitecture)?.FullName;

			if (ShouldUseLibcxx && BaseLinuxPath is not null)
			{
				// libc++ include directories
				Result.Add(Path.Combine(BaseLinuxPath, "include"));
				Result.Add(Path.Combine(BaseLinuxPath, "include", "c++", "v1"));
			}

			if (InternalSdkPath != null)
			{
				string PlatformSdkVersionString = BuildPlatformSdk.GetInstalledVersion()!;
				string Version = GetLinuxToolchainVersionFromFullString(PlatformSdkVersionString);
				string ClangIncludeDirectory = Path.Combine(InternalSdkPath, "lib/clang/" + Version + "/include/");

				Result.Add(Path.Combine(InternalSdkPath, "include"));
				Result.Add(Path.Combine(InternalSdkPath, "include/c++/v1"));
				Result.Add(Path.Combine(InternalSdkPath, "usr/include"));
				Result.Add(ClangIncludeDirectory);
				if (!Directory.Exists(ClangIncludeDirectory))
				{
					Logger.LogWarning("Clang include directory doesn't exist on disk. VersionString={VersionString}, ClangDir={ClangDir}",
						PlatformSdkVersionString, ClangIncludeDirectory);
				}
			}

			return [.. Result.Distinct()];
		}

		/// <summary>
		/// Get clang toolchain version from full version string
		/// v17_clang-10.0.1-centos7 -> 10.0.1
		/// v17_clang-16.0.1-centos7 -> 16
		/// </summary>
		/// <param name="FullVersion">Full clang toolchain version string. Example: "v17_clang-10.0.1-centos7"</param>
		/// <returns>Clang toolchain version. Example: 10.0.1 or 16</returns>
		/// <remarks>Starting with clang 16.x the directory naming changed to include major version only</remarks>
		private static string GetLinuxToolchainVersionFromFullString(string FullVersion)
		{
			string FullVersionPattern = @"^v[0-9]+_.*-(([0-9]+)\.[0-9]+\.[0-9]+)-.*$";
			Regex Regex = new Regex(FullVersionPattern);
			Match Match = Regex.Match(FullVersion);
			if (!Match.Success)
			{
				throw new ArgumentException("Wrong full version string", FullVersion);
			}

			Group MajorVersionGroup = Match.Groups[2];
			CaptureCollection MajorVersionCaptures = MajorVersionGroup.Captures;
			if (MajorVersionCaptures.Count != 1)
			{
				throw new ArgumentException("Multiple regex captures in major version string", FullVersion);
			}

			if (Int32.TryParse(MajorVersionCaptures[0].Value, out int MajorVersion))
			{
				if (MajorVersion >= 16)
				{
					return MajorVersionCaptures[0].Value;
				}
			}

			Group FullNumberVersionGroup = Match.Groups[1];
			CaptureCollection FullNumberVersionCaptures = FullNumberVersionGroup.Captures;
			return FullNumberVersionCaptures[0].Value;
		}
	}
}
