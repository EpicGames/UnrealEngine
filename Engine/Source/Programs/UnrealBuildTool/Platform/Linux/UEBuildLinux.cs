// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	partial struct UnrealArch
	{
		private static readonly IReadOnlyDictionary<UnrealArch, string> LinuxToolchainArchitectures = new Dictionary<UnrealArch, string>()
		{
			{ UnrealArch.Arm64,         "aarch64-unknown-linux-gnueabi" },
			{ UnrealArch.X64,           "x86_64-unknown-linux-gnu" },
		};

		/// <summary>
		/// Returns the low-architecture specific string for the generic architectures
		/// </summary>
		public string LinuxName
		{
			get
			{
				if (LinuxToolchainArchitectures.ContainsKey(this))
				{
					return LinuxToolchainArchitectures[this];
				}

				throw new BuildException($"Unknown architecture {ToString()} passed to UnrealArch.LinuxName");
			}
		}
	}
	
	/// <summary>
	/// Available runtime ray tracing modes on Linux platform
	/// </summary>
	public enum LinuxRayTracingRuntimeMode
	{
		/// <summary>
		/// Raytracing is disabled
		/// </summary>
		Disabled,

		/// <summary>
		/// Only inline raytracing is enabled
		/// </summary>
		Inline,

		/// <summary>
		/// Full raytracing with RTPSOs and hit shaders are available
		/// </summary>
		Full,
	}

	/// <summary>
	/// Linux-specific target settings
	/// </summary>
	public class LinuxTargetRules
	{
		/// <summary>
		/// Constructor
		/// </summary>
		public LinuxTargetRules()
		{
			XmlConfig.ApplyTo(this);
		}

		/// <summary>
		/// Enables address sanitizer (ASan)
		/// </summary>
		[CommandLine("-EnableASan")]
		[XmlConfigFile(Category = "BuildConfiguration", Name = "bEnableAddressSanitizer")]
		public bool bEnableAddressSanitizer = false;

		/// <summary>
		/// Enables LibFuzzer
		/// </summary>
		[CommandLine("-EnableLibFuzzer")]
		[XmlConfigFile(Category = "BuildConfiguration", Name = "bEnableLibFuzzer")]
		public bool bEnableLibFuzzer = false;

		/// <summary>
		/// Enables thread sanitizer (TSan)
		/// </summary>
		[CommandLine("-EnableTSan")]
		[XmlConfigFile(Category = "BuildConfiguration", Name = "bEnableThreadSanitizer")]
		public bool bEnableThreadSanitizer = false;

		/// <summary>
		/// Enables undefined behavior sanitizer (UBSan)
		/// </summary>
		[CommandLine("-EnableUBSan")]
		[XmlConfigFile(Category = "BuildConfiguration", Name = "bEnableUndefinedBehaviorSanitizer")]
		public bool bEnableUndefinedBehaviorSanitizer = false;

		/// <summary>
		/// Enables memory sanitizer (MSan)
		/// </summary>
		[CommandLine("-EnableMSan")]
		[XmlConfigFile(Category = "BuildConfiguration", Name = "bEnableMemorySanitizer")]
		public bool bEnableMemorySanitizer = false;

		/// <summary>
		/// Enables AutoRTFM sanitizer (AutoRTFMSan)
		/// </summary>
		[CommandLine("-EnableAutoRTFMSan")]
		[CommandLine("-EnableRTFMSan")]
		[XmlConfigFile(Category = "BuildConfiguration", Name = "bEnableAutoRTFMSanitizer")]
		public bool bEnableAutoRTFMSanitizer = false;

		/// <summary>
		/// Whether or not to preserve the portable symbol file produced by dump_syms
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/LinuxPlatform.LinuxTargetSettings")]
		public bool bPreservePSYM = false;

		/// <summary>
		/// Turns on tuning of debug info for LLDB
		/// </summary>
		[CommandLine("-EnableLLDB")]
		[XmlConfigFile(Category = "BuildConfiguration", Name = "bTuneDebugInfoForLLDB")]
		public bool bTuneDebugInfoForLLDB = false;

		/// <summary>
		/// Whether to globally disable calling dump_syms
		/// </summary>
		[CommandLine("-NoDumpSyms")]
		[XmlConfigFile(Category = "BuildConfiguration", Name = "bDisableDumpSyms")]
		public bool bDisableDumpSyms = false;

		/// <summary>
		/// Whether to globally skip symbol stripping
		/// </summary>
		[CommandLine("-NoStripSymbols")]
		[XmlConfigFile(Category = "BuildConfiguration", Name = "bDisableStripSymbols")]
		public bool bDisableStripSymbols = false;

		/// <summary>
		/// If specified along with -PGOProfile, use sample-based PGO instead of instrumented.
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		[CommandLine("-LinuxSampleBasedPGO")]
		public bool bLinuxSampleBasedPGO = false;

		/// <summary>
		/// Enables runtime ray tracing support.
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/LinuxTargetPlatform.LinuxTargetSettings", "RayTracingMode")]
		public LinuxRayTracingRuntimeMode RayTracingMode = LinuxRayTracingRuntimeMode.Disabled;
	}

	/// <summary>
	/// Read-only wrapper for Linux-specific target settings
	/// </summary>
	public class ReadOnlyLinuxTargetRules
	{
		private readonly LinuxTargetRules Inner;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="Inner">The settings object to wrap</param>
		public ReadOnlyLinuxTargetRules(LinuxTargetRules Inner)
		{
			this.Inner = Inner;
		}

		/// <summary>
		/// Accessors for fields on the inner TargetRules instance
		/// </summary>
		#region Read-only accessor properties 
#pragma warning disable CS1591

		public bool bPreservePSYM => Inner.bPreservePSYM;

		public bool bEnableAddressSanitizer => Inner.bEnableAddressSanitizer;

		public bool bEnableLibFuzzer => Inner.bEnableLibFuzzer;

		public bool bEnableThreadSanitizer => Inner.bEnableThreadSanitizer;

		public bool bEnableUndefinedBehaviorSanitizer => Inner.bEnableUndefinedBehaviorSanitizer;

		public bool bEnableMemorySanitizer => Inner.bEnableMemorySanitizer;

		public bool bEnableAutoRTFMSanitizer => Inner.bEnableAutoRTFMSanitizer;

		public bool bTuneDebugInfoForLLDB => Inner.bTuneDebugInfoForLLDB;

		public bool bDisableDumpSyms => Inner.bDisableDumpSyms;
		
		public LinuxRayTracingRuntimeMode RayTracingMode => Inner.RayTracingMode;

		public bool bDisableStripSymbols => Inner.bDisableStripSymbols;

		public bool bLinuxSampleBasedPGO => Inner.bLinuxSampleBasedPGO;
#pragma warning restore CS1591
		#endregion
	}

	// Usable by both Linux and LinuxArm64 (platform passed to constructor)
	class LinuxArchitectureConfig : UnrealArchitectureConfig
	{
		public LinuxArchitectureConfig(UnrealTargetPlatform Platform)
			: base(Platform == UnrealTargetPlatform.Linux ? UnrealArch.X64 : UnrealArch.Arm64)
		{
		}
	}

	class LinuxPlatform : UEBuildPlatform
	{
		/// <summary>
		/// Linux host architecture (compiler target triplet)
		/// @todo Remove this and get the actual Host architecture?
		/// </summary>
		public static readonly UnrealArch DefaultHostArchitecture = UnrealArch.X64;

		/// <summary>
		/// SDK in use by the platform
		/// </summary>
		protected LinuxPlatformSDK SDK;

		/// <summary>
		/// Constructor
		/// </summary>
		public LinuxPlatform(LinuxPlatformSDK InSDK, ILogger Logger)
			: this(UnrealTargetPlatform.Linux, InSDK, Logger)
		{
			SDK = InSDK;
		}

		public LinuxPlatform(UnrealTargetPlatform UnrealTarget, LinuxPlatformSDK InSDK, ILogger Logger)
			: base(UnrealTarget, InSDK, new LinuxArchitectureConfig(UnrealTarget), Logger)
		{
			SDK = InSDK;
		}

		public override void ResetTarget(TargetRules Target)
		{
			ValidateTarget(Target);
		}

		public static bool IsLTOEnabled(ReadOnlyTargetRules Target)
		{
			// Force LTO on if using PGO, or if specified in the target rules.
			return Target.bAllowLTCG || Target.bPGOOptimize || Target.bPGOProfile;
		}

		public override void ValidateTarget(TargetRules Target)
		{
			base.ValidateTarget(Target);

			if (!String.IsNullOrWhiteSpace(Environment.GetEnvironmentVariable("CLANG_STATIC_ANALYZER_MODE")))
			{
				Target.StaticAnalyzer = StaticAnalyzer.Default;
				Target.StaticAnalyzerOutputType = (Environment.GetEnvironmentVariable("CLANG_ANALYZER_OUTPUT")?.Contains("html", StringComparison.OrdinalIgnoreCase) == true) ? StaticAnalyzerOutputType.Html : StaticAnalyzerOutputType.Text;
				Target.StaticAnalyzerMode = String.Equals(Environment.GetEnvironmentVariable("CLANG_STATIC_ANALYZER_MODE"), "shallow", StringComparison.OrdinalIgnoreCase) ? StaticAnalyzerMode.Shallow : StaticAnalyzerMode.Deep;
			}
			else if (Target.StaticAnalyzer == StaticAnalyzer.Clang)
			{
				Target.StaticAnalyzer = StaticAnalyzer.Default;
			}

			// Disable linking and ignore build outputs if we're using a static analyzer
			if (Target.StaticAnalyzer == StaticAnalyzer.Default)
			{
				Target.bDisableLinking = true;
				Target.bIgnoreBuildOutputs = true;

				// Clang static analysis requires non unity builds
				Target.bUseUnityBuild = false;
				if (Target.bStaticAnalyzerIncludeGenerated)
				{
					Target.bAlwaysUseUnityForGeneratedFiles = false;
				}
			}

			// Editor target types get overwritten in UEBuildTarget.cs so lets avoid adding this here. ResetTarget is called with
			// default settings for TargetRules meanings Type == Game once then Type == Editor a 2nd time when building the Editor.
			// BuildVersion string is not set at this point so we can avoid setting a Sanitizer suffix if this is the first ResetTarget
			// with an unset TargetType. This avoids creating UnrealEditor-ASan.target while the binary is UnrealEditor.
			// These need to be promoted to higher level concepts vs this hacky solution
			if (!Target.IsNameOverriden() && !String.IsNullOrEmpty(Target.BuildVersion) && Target.Type != TargetType.Editor)
			{
				string? SanitizerSuffix = null;

				if (Target.LinuxPlatform.bEnableAddressSanitizer)
				{
					SanitizerSuffix = "ASan";
				}
				else if (Target.LinuxPlatform.bEnableThreadSanitizer)
				{
					SanitizerSuffix = "TSan";
				}
				else if (Target.LinuxPlatform.bEnableUndefinedBehaviorSanitizer)
				{
					SanitizerSuffix = "UBSan";
				}
				else if (Target.LinuxPlatform.bEnableMemorySanitizer)
				{
					SanitizerSuffix = "MSan";
				}
				if (Target.LinuxPlatform.bEnableAutoRTFMSanitizer)
				{
					SanitizerSuffix += "AutoRTFMSan";
				}
				if (Target.LinuxPlatform.bEnableLibFuzzer)
				{
					SanitizerSuffix += "LibFuzzer";
				}

				if (!String.IsNullOrEmpty(SanitizerSuffix))
				{
					Target.Name = Target.Name + "-" + SanitizerSuffix;
				}
			}

			if (Target.GlobalDefinitions.Contains("USE_NULL_RHI=1"))
			{
				Target.bCompileCEF3 = false;
			}

			// check if OS update invalidated our build
			Target.bCheckSystemHeadersForModification = (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Linux);

			Target.bCompileISPC = true;

			if (Target.bIWYU)
			{
				IWYUToolChain.ValidateTarget(Target);
			}

			// Disable chaining PCHs for the moment because it is crashing clang
			Target.bChainPCHs = false;

			if (Target.StaticAllocator == StaticAllocatorType.None)
			{
				// default to FMallocBinned2 as it uses less Virtual Memory Areas
				Target.StaticAllocator = StaticAllocatorType.Binned2;
			}
			if (Target.LinuxPlatform.bEnableAddressSanitizer || Target.LinuxPlatform.bEnableThreadSanitizer || Target.LinuxPlatform.bEnableMemorySanitizer)
			{
				Target.StaticAllocator = StaticAllocatorType.Ansi;
			}
		}

		public override bool CanUseXGE()
		{
			// [RCL] 2018-05-02: disabling XGE even during a native build because the support is not ready and you can have mysterious build failures when ib_console is installed.
			// [RCL] 2018-07-10: enabling XGE for Windows to see if the crash from 2016 still persists. Please disable if you see spurious build errors that don't repro without XGE
			// [bschaefer] 2018-08-24: disabling XGE due to a bug where XGE seems to be lower casing folders names that are headers ie. misc/Header.h vs Misc/Header.h
			// [bschaefer] 2018-10-04: enabling XGE as an update in xgConsole seems to have fixed it for me
			// [bschaefer] 2018-12-17: disable XGE again, as the same issue before seems to still be happening but intermittently
			// [bschaefer] 2019-6-13: enable XGE, as the bug from before is now fixed
			return BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Win64;
		}

		/// <summary>
		/// Determines if the given name is a build product for a target.
		/// </summary>
		/// <param name="FileName">The name to check</param>
		/// <param name="NamePrefixes">Target or application names that may appear at the start of the build product name (eg. "UnrealEditor", "ShooterGameEditor")</param>
		/// <param name="NameSuffixes">Suffixes which may appear at the end of the build product name</param>
		/// <returns>True if the string matches the name of a build product, false otherwise</returns>
		public override bool IsBuildProduct(string FileName, string[] NamePrefixes, string[] NameSuffixes)
		{
			if (FileName.StartsWith("lib", StringComparison.Ordinal))
			{
				return IsBuildProductName(FileName, 3, FileName.Length - 3, NamePrefixes, NameSuffixes, ".a")
					|| IsBuildProductName(FileName, 3, FileName.Length - 3, NamePrefixes, NameSuffixes, ".so")
					|| IsBuildProductName(FileName, 3, FileName.Length - 3, NamePrefixes, NameSuffixes, ".sym")
					|| IsBuildProductName(FileName, 3, FileName.Length - 3, NamePrefixes, NameSuffixes, ".debug");
			}
			else
			{
				return IsBuildProductName(FileName, NamePrefixes, NameSuffixes, "")
					|| IsBuildProductName(FileName, NamePrefixes, NameSuffixes, ".so")
					|| IsBuildProductName(FileName, NamePrefixes, NameSuffixes, ".a")
					|| IsBuildProductName(FileName, NamePrefixes, NameSuffixes, ".sym")
					|| IsBuildProductName(FileName, NamePrefixes, NameSuffixes, ".debug");
			}
		}

		/// <summary>
		/// Get the extension to use for the given binary type
		/// </summary>
		/// <param name="InBinaryType"> The binary type being built</param>
		/// <returns>string    The binary extension (i.e. 'exe' or 'dll')</returns>
		public override string GetBinaryExtension(UEBuildBinaryType InBinaryType)
		{
			switch (InBinaryType)
			{
				case UEBuildBinaryType.DynamicLinkLibrary:
					return ".so";
				case UEBuildBinaryType.Executable:
					return "";
				case UEBuildBinaryType.StaticLibrary:
					return ".a";
			}
			return base.GetBinaryExtension(InBinaryType);
		}

		/// <summary>
		/// Get the extensions to use for debug info for the given binary type
		/// </summary>
		/// <param name="InTarget">Rules for the target being built</param>
		/// <param name="InBinaryType"> The binary type being built</param>
		/// <returns>string[]    The debug info extensions (i.e. 'pdb')</returns>
		public override string[] GetDebugInfoExtensions(ReadOnlyTargetRules InTarget, UEBuildBinaryType InBinaryType)
		{
			switch (InBinaryType)
			{
				case UEBuildBinaryType.DynamicLinkLibrary:
				case UEBuildBinaryType.Executable:
					if (InTarget.LinuxPlatform.bPreservePSYM)
					{
						return new string[] { ".psym", ".sym", ".debug" };
					}
					else
					{
						return new string[] { ".sym", ".debug" };
					}
			}
			return Array.Empty<string>();
		}

		/// <summary>
		/// Modify the rules for a newly created module, where the target is a different host platform.
		/// This is not required - but allows for hiding details of a particular platform.
		/// </summary>
		/// <param name="ModuleName">The name of the module</param>
		/// <param name="Rules">The module rules</param>
		/// <param name="Target">The target being build</param>
		public override void ModifyModuleRulesForOtherPlatform(string ModuleName, ModuleRules Rules, ReadOnlyTargetRules Target)
		{
			// don't do any target platform stuff if SDK is not available
			if (!UEBuildPlatform.IsPlatformAvailableForTarget(Platform, Target, bIgnoreSDKCheck: true))
			{
				return;
			}

			if (Target.Platform == UnrealTargetPlatform.Win64)
			{
				if (!Target.bBuildRequiresCookedData)
				{
					if (ModuleName == "Engine")
					{
						if (Target.bBuildDeveloperTools)
						{
							Rules.DynamicallyLoadedModuleNames.Add("LinuxTargetPlatformSettings");
							Rules.DynamicallyLoadedModuleNames.Add("LinuxTargetPlatformControls");
							Rules.DynamicallyLoadedModuleNames.Add("LinuxTargetPlatform");
							Rules.DynamicallyLoadedModuleNames.Add("LinuxArm64TargetPlatformSettings");
							Rules.DynamicallyLoadedModuleNames.Add("LinuxArm64TargetPlatformControls");
							Rules.DynamicallyLoadedModuleNames.Add("LinuxArm64TargetPlatform");
						}
					}
				}

				// allow standalone tools to use targetplatform modules, without needing Engine
				if (Target.bForceBuildTargetPlatforms && ModuleName == "TargetPlatform")
				{
					Rules.DynamicallyLoadedModuleNames.Add("LinuxTargetPlatformSettings");
					Rules.DynamicallyLoadedModuleNames.Add("LinuxTargetPlatformControls");
					Rules.DynamicallyLoadedModuleNames.Add("LinuxTargetPlatform");
					Rules.DynamicallyLoadedModuleNames.Add("LinuxArm64TargetPlatformSettings");
					Rules.DynamicallyLoadedModuleNames.Add("LinuxArm64TargetPlatformControls");
					Rules.DynamicallyLoadedModuleNames.Add("LinuxArm64TargetPlatform");
				}
			}
		}

		/// <summary>
		/// Modify the rules for a newly created module, in a target that's being built for this platform.
		/// This is not required - but allows for hiding details of a particular platform.
		/// </summary>
		/// <param name="ModuleName">The name of the module</param>
		/// <param name="Rules">The module rules</param>
		/// <param name="Target">The target being build</param>
		public override void ModifyModuleRulesForActivePlatform(string ModuleName, ModuleRules Rules, ReadOnlyTargetRules Target)
		{
			bool bBuildShaderFormats = Target.bForceBuildShaderFormats;

			if (!Target.bBuildRequiresCookedData)
			{
				if (ModuleName == "TargetPlatform")
				{
					bBuildShaderFormats = true;
				}
			}

			// allow standalone tools to use target platform modules, without needing Engine
			if (ModuleName == "TargetPlatform")
			{
				if (Target.bForceBuildTargetPlatforms)
				{
					Rules.DynamicallyLoadedModuleNames.Add("LinuxTargetPlatformSettings");
					Rules.DynamicallyLoadedModuleNames.Add("LinuxTargetPlatformControls");
					Rules.DynamicallyLoadedModuleNames.Add("LinuxTargetPlatform");
					Rules.DynamicallyLoadedModuleNames.Add("LinuxArm64TargetPlatformSettings");
					Rules.DynamicallyLoadedModuleNames.Add("LinuxArm64TargetPlatformControls");
					Rules.DynamicallyLoadedModuleNames.Add("LinuxArm64TargetPlatform");
				}

				if (bBuildShaderFormats)
				{
					Rules.DynamicallyLoadedModuleNames.Add("ShaderFormatOpenGL");
					Rules.DynamicallyLoadedModuleNames.Add("VulkanShaderFormat");
					Rules.DynamicallyLoadedModuleNames.Add("ShaderFormatVectorVM");
				}
			}
		}

		public virtual void SetUpSpecificEnvironment(ReadOnlyTargetRules Target, CppCompileEnvironment CompileEnvironment, LinkEnvironment LinkEnvironment)
		{
			CompileEnvironment.Definitions.Add("PLATFORM_LINUX=1");
			CompileEnvironment.Definitions.Add("PLATFORM_UNIX=1");
			CompileEnvironment.Definitions.Add("UE_PLATFORM_LINUX=1");
			CompileEnvironment.Definitions.Add("UE_PLATFORM_UNIX=1");

			CompileEnvironment.Definitions.Add("LINUX=1"); // For libOGG

			// this define does not set jemalloc as default, just indicates its support
			CompileEnvironment.Definitions.Add("PLATFORM_SUPPORTS_JEMALLOC=1");

			// LinuxArm64 uses only Linux header files
			CompileEnvironment.Definitions.Add("OVERRIDE_PLATFORM_HEADER_NAME=Linux");

			CompileEnvironment.Definitions.Add("PLATFORM_LINUXARM64=" +
				(Target.Platform == UnrealTargetPlatform.LinuxArm64 ? "1" : "0"));
		}

		/// <inheritdoc/>
		public override void SetUpEnvironment(ReadOnlyTargetRules Target, CppCompileEnvironment CompileEnvironment, LinkEnvironment LinkEnvironment)
		{
			// During the native builds, check the system includes as well (check toolchain when cross-compiling?)
			DirectoryReference? BaseLinuxPath = SDK.GetBaseLinuxPathForArchitecture(Target.Architecture);
			if (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Linux && BaseLinuxPath == null)
			{
				CompileEnvironment.SystemIncludePaths.Add(new DirectoryReference("/usr/include"));
			}

			if (CompileEnvironment.bPGOOptimize != LinkEnvironment.bPGOOptimize)
			{
				throw new BuildException("Inconsistency between PGOOptimize settings in Compile ({0}) and Link ({1}) environments",
					CompileEnvironment.bPGOOptimize,
					LinkEnvironment.bPGOOptimize
				);
			}

			if (CompileEnvironment.bPGOProfile != LinkEnvironment.bPGOProfile)
			{
				throw new BuildException("Inconsistency between PGOProfile settings in Compile ({0}) and Link ({1}) environments",
					CompileEnvironment.bPGOProfile,
					LinkEnvironment.bPGOProfile
				);
			}

			if (CompileEnvironment.bPGOOptimize)
			{
				DirectoryReference BaseDir = Unreal.WritableEngineDirectory;
				if (Target.ProjectFile != null)
				{
					BaseDir = DirectoryReference.FromFile(Target.ProjectFile);
					// projects put PGO data in Platform/Linux/Build/PGO, even if Linux platform isn't a Platform Extension
					CompileEnvironment.PGODirectory = Path.Combine(BaseDir.FullName, "Platforms", Target.Platform.ToString(), "Build", "PGO");
				}
				else
				{
					// project-less build put PGO data in Engine/Build/Linux/PGO, because Linux platform isn't a Platform Extension
					CompileEnvironment.PGODirectory = Path.Combine(BaseDir.FullName, "Build", Target.Platform.ToString(), "PGO");
				}
				CompileEnvironment.PGODirectory = CompileEnvironment.PGODirectory.Replace('\\', '/') + "/";
				CompileEnvironment.PGOFilenamePrefix = String.Format("{0}-{1}-{2}.profdata", Target.Name, Target.Platform, Target.Configuration);

				// Check if the profdata file exists and disable if not.
				// If the file exists but has zero length, this is a "soft" disabling. E.g. PGO data has become stale and we want to temporarily compile without PGO - do not complain about it.
				string PGOFilePath = Path.Combine(CompileEnvironment.PGODirectory, CompileEnvironment.PGOFilenamePrefix);
				FileInfo Info = new FileInfo(PGOFilePath);
				if (!Info.Exists || Info.Length == 0)
				{
					if (!Info.Exists)
					{
						Logger.LogWarning("Warning: PGO file '{PGOFilePath}' does not exist, disabling optimization", PGOFilePath);
					}
					else
					{
						Logger.LogInformation("PGO file '{PGOFilePath}' exists but has 0 length. Assuming that PGO data is temporarily missing, disabling optimization without a warning.", PGOFilePath);
					}
					CompileEnvironment.bPGOOptimize = false;
					LinkEnvironment.bPGOOptimize = false;

					CompileEnvironment.PGODirectory = "";
					CompileEnvironment.PGOFilenamePrefix = "";
				}
				else
				{
					LinkEnvironment.PGODirectory = CompileEnvironment.PGODirectory;
					LinkEnvironment.PGOFilenamePrefix = CompileEnvironment.PGOFilenamePrefix;
				}
			}

			LinkEnvironment.bCodeCoverage = CompileEnvironment.bCodeCoverage;

			// For consistency with other platforms, also enable LTO whenever doing profile-guided optimizations.
			// Obviously both PGI (instrumented) and PGO (optimized) binaries need to have that
			if (CompileEnvironment.bPGOProfile || CompileEnvironment.bPGOOptimize)
			{
				CompileEnvironment.bAllowLTCG = true;
				LinkEnvironment.bAllowLTCG = true;
			}

			if (CompileEnvironment.bAllowLTCG != LinkEnvironment.bAllowLTCG)
			{
				throw new BuildException("Inconsistency between LTCG settings in Compile ({0}) and Link ({1}) environments",
					CompileEnvironment.bAllowLTCG,
					LinkEnvironment.bAllowLTCG
				);
			}

			CompileEnvironment.Definitions.Add("INT64_T_TYPES_NOT_LONG_LONG=1");
			
			if (Target.LinuxPlatform.RayTracingMode != LinuxRayTracingRuntimeMode.Disabled && Target.Type != TargetType.Server)
			{
				CompileEnvironment.Definitions.Add("RHI_RAYTRACING=1");
			}

			// link with Linux libraries.
			LinkEnvironment.SystemLibraries.Add("pthread");

			if (Target.bUseAutoRTFMCompiler)
			{
				// #jira SOL-9041: Sections prefixes cause relocation errors and thus we avoid using them.
				string DisableSectionsGuidedSectionPrefix = "-bbsections-guided-section-prefix=0";
				string DisableProfileGuidedSectionPrefix = "-profile-guided-section-prefix=0";

				CompileEnvironment.AdditionalArguments += $" -Xclang -mllvm -Xclang {DisableSectionsGuidedSectionPrefix} -Xclang -mllvm -Xclang {DisableProfileGuidedSectionPrefix}";
				LinkEnvironment.AdditionalArguments += $" -Wl,-mllvm -Wl,{DisableSectionsGuidedSectionPrefix} -Wl,-mllvm -Wl,{DisableProfileGuidedSectionPrefix}";
			}

			// let this class or a sub class do settings specific to that class
			SetUpSpecificEnvironment(Target, CompileEnvironment, LinkEnvironment);
		}

		/// <summary>
		/// Whether this platform should create debug information or not
		/// </summary>
		/// <param name="Target">The target being built</param>
		/// <returns>bool    true if debug info should be generated, false if not</returns>
		public override bool ShouldCreateDebugInfo(ReadOnlyTargetRules Target)
		{
			switch (Target.Configuration)
			{
				case UnrealTargetConfiguration.Development:
				case UnrealTargetConfiguration.Shipping:
				case UnrealTargetConfiguration.Test:
				case UnrealTargetConfiguration.Debug:
				default:
					return true;
			}
		}

		/// <summary>
		/// Creates a toolchain instance for the given platform.
		/// </summary>
		/// <param name="Target">The target being built</param>
		/// <returns>New toolchain instance.</returns>
		public override UEToolChain CreateToolChain(ReadOnlyTargetRules Target)
		{
			ClangToolChainOptions Options = ClangToolChainOptions.None;

			if (Target.LinuxPlatform.bEnableAddressSanitizer)
			{
				Options |= ClangToolChainOptions.EnableAddressSanitizer;

				if (Target.LinkType != TargetLinkType.Monolithic)
				{
					Options |= ClangToolChainOptions.EnableSharedSanitizer;
				}
			}
			if (Target.LinuxPlatform.bEnableThreadSanitizer)
			{
				Options |= ClangToolChainOptions.EnableThreadSanitizer;

				if (Target.LinkType != TargetLinkType.Monolithic)
				{
					throw new BuildException("Thread Sanitizer (TSan) unsupported for non-monolithic builds");
				}
			}
			if (Target.LinuxPlatform.bEnableUndefinedBehaviorSanitizer)
			{
				Options |= ClangToolChainOptions.EnableUndefinedBehaviorSanitizer;

				if (Target.LinkType != TargetLinkType.Monolithic)
				{
					Options |= ClangToolChainOptions.EnableSharedSanitizer;
				}
			}
			if (Target.LinuxPlatform.bEnableMemorySanitizer)
			{
				Options |= ClangToolChainOptions.EnableMemorySanitizer;

				if (Target.LinkType != TargetLinkType.Monolithic)
				{
					throw new BuildException("Memory Sanitizer (MSan) unsupported for non-monolithic builds");
				}
			}
			if (Target.LinuxPlatform.bEnableAutoRTFMSanitizer)
			{
				Options |= ClangToolChainOptions.EnableAutoRTFMSanitizer;
			}
			if (Target.LinuxPlatform.bDisableDumpSyms)
			{
				Options |= ClangToolChainOptions.DisableDumpSyms;
			}
			if (Target.LinuxPlatform.bDisableStripSymbols)
			{
				Options |= ClangToolChainOptions.DisableStripSymbols;
			}
			if (Target.LinuxPlatform.bEnableLibFuzzer)
			{
				Options |= ClangToolChainOptions.EnableLibFuzzer;

				if (Target.LinkType != TargetLinkType.Monolithic)
				{
					throw new BuildException("LibFuzzer is unsupported for non-monolithic builds.");
				}
			}

			if (IsLTOEnabled(Target))
			{
				Options |= ClangToolChainOptions.EnableLinkTimeOptimization;

				// Prefer Thin LTO for PGO builds, even if it's not explicitly specified
				if (Target.bPreferThinLTO || Target.bPGOOptimize || Target.bPGOProfile)
				{
					Options |= ClangToolChainOptions.EnableThinLTO;
				}
			}

			if (Target.LinuxPlatform.bLinuxSampleBasedPGO)
			{
				Options |= ClangToolChainOptions.SampleBasedPGO;
			}

			if (Target.bUseAutoRTFMCompiler)
			{
				Options |= ClangToolChainOptions.UseAutoRTFMCompiler;
			}

			if (Target.bCompressDebugFile)
			{
				Options |= ClangToolChainOptions.CompressDebugFile;
			}

			if (Target.LinuxPlatform.bTuneDebugInfoForLLDB)
			{
				Options |= ClangToolChainOptions.TuneDebugInfoForLLDB;
			}

			if (Target.LinuxPlatform.bPreservePSYM)
			{
				Options |= ClangToolChainOptions.PreservePSYM;
			}

			// Disable color logging if we are on a build machine
			if (Unreal.IsBuildMachine())
			{
				Log.ColorConsoleOutput = false;
			}

			LinuxToolChain toolChain = new LinuxToolChain(Target.Architecture, SDK, Options, Logger);

			if (Target.bIWYU)
			{
				return new IWYUToolChain(Target, toolChain, Logger);
			}

			return toolChain;
		}

		/// <inheritdoc/>
		public override void Deploy(TargetReceipt Receipt)
		{
			new UEDeployLinux(Logger).PrepTargetForDeployment(Receipt);
		}
	}

	class UEDeployLinux : UEBuildDeploy
	{
		public UEDeployLinux(ILogger InLogger)
			: base(InLogger)
		{
		}

		public override bool PrepTargetForDeployment(TargetReceipt Receipt)
		{
			return base.PrepTargetForDeployment(Receipt);
		}
	}

	class LinuxPlatformFactory : UEBuildPlatformFactory
	{
		public override UnrealTargetPlatform TargetPlatform => UnrealTargetPlatform.Linux;

		/// <summary>
		/// Register the platform with the UEBuildPlatform class
		/// </summary>
		public override void RegisterBuildPlatforms(ILogger Logger)
		{
			LinuxPlatformSDK SDK = new LinuxPlatformSDK(Logger);
			LinuxPlatformSDK SDKArm64 = new LinuxPlatformSDK(Logger);

			// Register this build platform for Linux x86-64 and Arm64
			UEBuildPlatform.RegisterBuildPlatform(new LinuxPlatform(UnrealTargetPlatform.Linux, SDK, Logger), Logger);
			UEBuildPlatform.RegisterPlatformWithGroup(UnrealTargetPlatform.Linux, UnrealPlatformGroup.Linux);
			UEBuildPlatform.RegisterPlatformWithGroup(UnrealTargetPlatform.Linux, UnrealPlatformGroup.Unix);
			UEBuildPlatform.RegisterPlatformWithGroup(UnrealTargetPlatform.Linux, UnrealPlatformGroup.Desktop);
			UEBuildPlatform.RegisterPlatformWithGroup(UnrealTargetPlatform.Linux, UnrealPlatformGroup.PosixOS);

			UEBuildPlatform.RegisterBuildPlatform(new LinuxPlatform(UnrealTargetPlatform.LinuxArm64, SDKArm64, Logger), Logger);
			UEBuildPlatform.RegisterPlatformWithGroup(UnrealTargetPlatform.LinuxArm64, UnrealPlatformGroup.Linux);
			UEBuildPlatform.RegisterPlatformWithGroup(UnrealTargetPlatform.LinuxArm64, UnrealPlatformGroup.Unix);
			UEBuildPlatform.RegisterPlatformWithGroup(UnrealTargetPlatform.LinuxArm64, UnrealPlatformGroup.Desktop);
			UEBuildPlatform.RegisterPlatformWithGroup(UnrealTargetPlatform.LinuxArm64, UnrealPlatformGroup.PosixOS);
		}
	}
}
