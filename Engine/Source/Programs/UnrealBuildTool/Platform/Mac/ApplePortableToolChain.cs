// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	internal class ApplePortableToolChainInfo
	{
		public bool IsValid { get; }
		public FileReference? ClangPath { get; }
		public DirectoryReference? OSSHeadersPath { get; }
		public DirectoryReference? ClangBasePath { get; }
		public DirectoryReference? LibCxxIncludePath { get; }
		public DirectoryReference? ClangBuiltinIncludePath { get; }
		public string? SDKVersion { get; }

		public ApplePortableToolChainInfo(ILogger Logger)
		{
			IsValid = false;
			
			DirectoryReference? HostAutoSdkDir = TryGetAutoSDKDir();
			if (HostAutoSdkDir == null)
			{
				Logger.LogDebug("Auto SDK HostMac directory not found, portable toolchain unavailable");
				return;
			}

			DirectoryReference AppleToolchainDir = DirectoryReference.Combine(HostAutoSdkDir, "Apple");
			if (!DirectoryReference.Exists(AppleToolchainDir))
			{
				Logger.LogDebug("Apple directory not found at {Path}", AppleToolchainDir);
				return;
			}

			string? ClangSubdir = GetConfigValue("ClangSubdir");
			string? OssHeadersSubdir = GetConfigValue("OSSHeadersSubdir");

			if (String.IsNullOrEmpty(ClangSubdir) || String.IsNullOrEmpty(OssHeadersSubdir))
			{
				Logger.LogDebug("AppleOpenSource_SDK.json config missing required paths");
				return;
			}

			SDKVersion = ResolveVersion(AppleToolchainDir, Logger);
			if (SDKVersion == null)
			{
				Logger.LogDebug("No valid Apple portable toolchain version found in {Path}", AppleToolchainDir);
				return;
			}

			DirectoryReference VersionDir = DirectoryReference.Combine(AppleToolchainDir, SDKVersion);
			FileReference Clang = FileReference.Combine(VersionDir, ClangSubdir);
			DirectoryReference OssHeaders = DirectoryReference.Combine(VersionDir, OssHeadersSubdir);

			if (!FileReference.Exists(Clang))
			{
				Logger.LogDebug("Portable toolchain clang not found at {Path}", Clang);
				return;
			}

			DirectoryReference OssIncludePath = DirectoryReference.Combine(OssHeaders, "usr", "include");
			if (!DirectoryReference.Exists(OssIncludePath))
			{
				Logger.LogDebug("Apple OSS headers not found at {Path}", OssIncludePath);
				return;
			}

			DirectoryReference ClangBase = Clang.Directory.ParentDirectory!;

			DirectoryReference LibCxxInclude = DirectoryReference.Combine(ClangBase, "include", "c++", "v1");
			if (!DirectoryReference.Exists(LibCxxInclude))
			{
				Logger.LogWarning("libc++ headers not found at {Path}", LibCxxInclude);
				return;
			}

			DirectoryReference? ClangBuiltinInclude = null;
			DirectoryReference ClangLibDir = DirectoryReference.Combine(ClangBase, "lib", "clang");

			if (DirectoryReference.Exists(ClangLibDir))
			{
				foreach (DirectoryReference VersionSubDir in DirectoryReference.EnumerateDirectories(ClangLibDir))
				{
					DirectoryReference IncludePath = DirectoryReference.Combine(VersionSubDir, "include");
					if (DirectoryReference.Exists(IncludePath))
					{
						ClangBuiltinInclude = IncludePath;
						break;
					}
				}
			}

			if (ClangBuiltinInclude == null)
			{
				Logger.LogWarning("Clang builtin headers not found in {Path}", ClangLibDir);
				return;
			}

			Logger.LogInformation("Found Apple portable toolchain: {Version}", SDKVersion);

			ClangPath = Clang;
			OSSHeadersPath = OssHeaders;
			ClangBasePath = ClangBase;
			LibCxxIncludePath = LibCxxInclude;
			ClangBuiltinIncludePath = ClangBuiltinInclude;
			IsValid = true;
		}

		public static string? ResolveVersion(DirectoryReference AppleToolchainDir, ILogger Logger)
		{
			string? MainVersion = GetConfigValue("MainVersion");
			if (!String.IsNullOrEmpty(MainVersion))
			{
				DirectoryReference VersionDir = DirectoryReference.Combine(AppleToolchainDir, MainVersion);
				if (DirectoryReference.Exists(VersionDir))
				{
					return MainVersion;
				}
				Logger.LogDebug("MainVersion '{Version}' specified but directory not found at {Path}", MainVersion, VersionDir);
			}

			// Try to select based on Xcode version mapping
			string? MappedVersion = ResolveVersionFromXcodeMapping(AppleToolchainDir, Logger);
			if (MappedVersion != null)
			{
				return MappedVersion;
			}

			Logger.LogDebug("No suitable portable toolchain was found.");

			return null;
		}

		/// <summary>
		/// Resolves toolchain version based on XcodeToToolchainVersion mapping.
		/// </summary>
		private static string? ResolveVersionFromXcodeMapping(DirectoryReference AppleToolchainDir, ILogger Logger)
		{
			// Get the installed Xcode version
			string? XcodeVersionString = ApplePlatformSDK.InstalledXcodeVersion.Value;
			if (String.IsNullOrEmpty(XcodeVersionString))
			{
				return null;
			}

			if (!Version.TryParse(XcodeVersionString, out Version? XcodeVersion))
			{
				Logger.LogDebug("Failed to parse Xcode version: {Version}", XcodeVersionString);
				return null;
			}

			// Get the mapping array from config
			string[]? Mappings = GetConfigArray("XcodeToToolchainVersion");
			if (Mappings == null || Mappings.Length == 0)
			{
				return null;
			}

			// Find the matching toolchain prefix (iterate in reverse for highest match)
			string? ToolchainPrefix = null;
			for (int MappingIndex = Mappings.Length - 1; MappingIndex >= 0; MappingIndex--)
			{
				string[] Parts = Mappings[MappingIndex].Split('-');
				if (Parts.Length != 2)
				{
					continue;
				}

				if (Version.TryParse(Parts[0], out Version? MappingVersion) && XcodeVersion >= MappingVersion)
				{
					ToolchainPrefix = Parts[1];
					Logger.LogInformation("Xcode {XcodeVersion} mapped to toolchain prefix {Prefix}", XcodeVersion, ToolchainPrefix);
					break;
				}
			}

			if (ToolchainPrefix == null)
			{
				return null;
			}

			// Find directory starting with this prefix
			if (!DirectoryReference.Exists(AppleToolchainDir))
			{
				return null;
			}

			foreach (DirectoryReference Dir in DirectoryReference.EnumerateDirectories(AppleToolchainDir))
			{
				string DirName = Dir.GetDirectoryName();
				if (DirName.StartsWith(ToolchainPrefix + "_", StringComparison.OrdinalIgnoreCase))
				{
					return DirName;
				}
			}

			Logger.LogDebug("No directory found with prefix {Prefix}", ToolchainPrefix);
			return null;
		}

		private static DirectoryReference? TryGetAutoSDKDir()
		{
			string? SdksRoot = Environment.GetEnvironmentVariable("UE_SDKS_ROOT");
			if (String.IsNullOrEmpty(SdksRoot))
			{
				return null;
			}

			DirectoryReference HostDir = DirectoryReference.Combine(new DirectoryReference(SdksRoot), "HostMac");
			return DirectoryReference.Exists(HostDir) ? HostDir : null;
		}

		private static JsonObject? CachedConfig;
		private static bool ConfigLoadAttempted;

		private static string? GetConfigValue(string Key)
		{
			EnsureConfigLoaded();

			if (CachedConfig != null && CachedConfig.TryGetStringField(Key, out string? Value))
			{
				return Value;
			}

			return null;
		}

		private static string[]? GetConfigArray(string Key)
		{
			EnsureConfigLoaded();

			if (CachedConfig != null && CachedConfig.TryGetStringArrayField(Key, out string[]? Values))
			{
				return Values;
			}

			return null;
		}

		private static void EnsureConfigLoaded()
		{
			if (!ConfigLoadAttempted)
			{
				ConfigLoadAttempted = true;
				FileReference ConfigFile = FileReference.Combine(
					Unreal.EngineDirectory,
					"Config",
					"Apple",
					"AppleOpenSource_SDK.json");

				if (FileReference.Exists(ConfigFile))
				{
					JsonObject.TryRead(ConfigFile, out CachedConfig);
				}
			}
		}
	}
}
