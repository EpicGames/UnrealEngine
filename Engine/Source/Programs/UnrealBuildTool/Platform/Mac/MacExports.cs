// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace UnrealBuildTool
{
	/// <summary>
	/// Public Mac functions exposed to UAT
	/// </summary>
	public static class MacExports
	{
		/// <summary>
		/// Describes the architecture of the host. Note - this ignores translation.
		/// IsRunningUnderRosetta can be used to detect that we're running under translation
		/// </summary>
		public static UnrealArch HostArchitecture => IsRunningOnAppleArchitecture ? UnrealArch.Arm64 : UnrealArch.X64;

		/// <summary>
		/// Cached result for AppleArch check
		/// </summary>
		private static bool? IsRunningOnAppleArchitectureVar;

		/// <summary>
		/// Cached result for Rosetta check
		/// </summary>
		private static bool? IsRunningUnderRosettaVar;

		/// <summary>
		/// Returns true if we're running under Rosetta 
		/// </summary>
		/// <returns></returns>
		public static bool IsRunningUnderRosetta
		{
			get
			{
				if (!IsRunningUnderRosettaVar.HasValue)
				{
					string TranslatedOutput = Utils.RunLocalProcessAndReturnStdOut("/usr/sbin/sysctl", "sysctl", null);
					IsRunningUnderRosettaVar = TranslatedOutput.Contains("sysctl.proc_translated: 1", StringComparison.Ordinal);
				}

				return IsRunningUnderRosettaVar.Value;
			}
		}

		/// <summary>
		/// Returns true if we're running on Apple architecture (either natively which dotnet will do, or under Rosetta)
		/// </summary>
		/// <returns></returns>
		public static bool IsRunningOnAppleArchitecture
		{
			get
			{
				if (!IsRunningOnAppleArchitectureVar.HasValue)
				{
					// On an m1 mac this appears to be where the brand is.
					string BrandOutput = Utils.RunLocalProcessAndReturnStdOut("/usr/sbin/sysctl", "-n machdep.cpu.brand_string", null);
					IsRunningOnAppleArchitectureVar = BrandOutput.Contains("Apple", StringComparison.Ordinal) || IsRunningUnderRosetta;
				}

				return IsRunningOnAppleArchitectureVar.Value;
			}
		}

		/// <summary>
		/// Strips symbols from a file
		/// </summary>
		/// <param name="SourceFile">The input file</param>
		/// <param name="TargetFile">The output file</param>
		/// <param name="Logger"></param>
		public static void StripSymbols(FileReference SourceFile, FileReference TargetFile, ILogger Logger)
		{
			MacToolChain ToolChain = new MacToolChain(null, ClangToolChainOptions.None, Logger);
			ToolChain.StripSymbols(SourceFile, TargetFile);
		}
	}
}
