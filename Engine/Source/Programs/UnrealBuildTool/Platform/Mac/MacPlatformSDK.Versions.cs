// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool
{
	//////////////////////////////////////////////////////////////////////////////////
	// If you are looking for version numbers, see Engine/Config/Mac/Mac_SDK.json
	//////////////////////////////////////////////////////////////////////////////////
}