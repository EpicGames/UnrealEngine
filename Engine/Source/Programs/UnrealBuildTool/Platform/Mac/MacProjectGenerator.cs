// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace UnrealBuildTool
{
	/// <summary>
	/// Base class for platform-specific project generators
	/// </summary>
	class MacProjectGenerator : AppleProjectGenerator
	{
		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="Arguments">Command line arguments passed to the project generator</param>
		/// <param name="Logger">Logger for output</param>
#pragma warning disable IDE0060 // Remove unused parameter - constructor is found by reflection in GenerateProjectFilesMode.ExecuteAsync
		public MacProjectGenerator(CommandLineArguments Arguments, ILogger Logger)
			: base(Logger)
#pragma warning restore IDE0060
		{
		}

		/// <summary>
		/// Register the platform with the UEPlatformProjectGenerator class
		/// </summary>
		public override IEnumerable<UnrealTargetPlatform> GetPlatforms()
		{
			yield return UnrealTargetPlatform.Mac;
		}
	}
}
