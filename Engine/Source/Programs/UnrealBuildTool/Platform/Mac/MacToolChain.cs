// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text.RegularExpressions;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	class MacToolChainSettings : AppleToolChainSettings
	{
		public override string GetTargetVersionForTargetType(TargetType Type) => ((ApplePlatformSDK)UEBuildPlatformSDK.GetSDKForPlatform("Mac")!).GetBuildTargetVersion(Type);

		/// <summary>
		/// Which version of the Mac OS X to allow at run time
		/// </summary>
		public static string MinMacBuildVersion(TargetType TargetType)
		{
			return ((ApplePlatformSDK)UEBuildPlatformSDK.GetSDKForPlatform("Mac")!).GetBuildTargetVersion(TargetType);
		}

		/// <summary>
		/// Minimum version of Mac OS X to actually run on, running on earlier versions will display the system minimum version error dialog and exit.
		/// </summary>
		public static string MinMacDeploymentVersion(TargetType TargetType)
		{
			return ((ApplePlatformSDK)UEBuildPlatformSDK.GetSDKForPlatform("Mac")!).GetDeploymentTargetVersion(TargetType);
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="bVerbose">Whether to output verbose logging</param>
		/// <param name="Logger">Logger for output</param>
		public MacToolChainSettings(bool bVerbose, ILogger Logger)
			// passing 0.0 for target version because we have per-target versions when making the Tuple
			: base("MacOSX", null, "macos", "0.0", bVerbose, Logger)
		{
		}
	}

	/// <summary>
	/// Mac toolchain wrapper
	/// </summary>
	class MacToolChain : AppleToolChain
	{
		public MacToolChain(ReadOnlyTargetRules? Target, ClangToolChainOptions InOptions, ILogger InLogger)
			: base(Target, () => new MacToolChainSettings(true, InLogger), InOptions, InLogger)
		{
		}

		/// <summary>
		/// Which compiler\linker frontend to use
		/// </summary>
		private const string MacCompiler = "clang++";

		/// <summary>
		/// Which archiver to use
		/// </summary>
		private const string MacArchiver = "libtool";

		protected override string XnuPlatformDefine => "XNU_PLATFORM_MacOSX";

		protected override ClangToolChainInfo GetToolChainInfo()
		{
			FileReference CompilerPath = FileReference.Combine(AppleToolChainSettings.ToolchainDir, MacCompiler);
			FileReference ArchiverPath = FileReference.Combine(AppleToolChainSettings.ToolchainDir, MacArchiver);
			return new AppleToolChainInfo(UnrealTargetPlatform.Mac, ApplePlatformSDK.DeveloperDir, CompilerPath, ArchiverPath, Logger);
		}

		public static DirectoryReference FindProductDirectory(FileReference? ProjectFile, DirectoryReference BinaryDir, string? NameIfProgram)
		{
			// a project file is always used if there is one
			if (ProjectFile != null)
			{
				return ProjectFile.Directory;
			}
			// look to see if this is a program, and hunt down where the Programs directory is
			if (NameIfProgram != null)
			{
				return FindProgramDirectoryFromSource(BinaryDir, NameIfProgram, true) ?? Unreal.EngineDirectory;
			}
			return Unreal.EngineDirectory;
		}

		public static DirectoryReference? FindProgramDirectoryFromSource(DirectoryReference StartingDir, string ProgramName, bool bCreateIfNotFound)
		{
			DirectoryReference? ProgramFinder = StartingDir;
			while (ProgramFinder != null &&
!String.Equals(ProgramFinder.GetDirectoryName(), "Source", StringComparison.OrdinalIgnoreCase) &&
!String.Equals(ProgramFinder.GetDirectoryName(), "Intermediate", StringComparison.OrdinalIgnoreCase) &&
!String.Equals(ProgramFinder.GetDirectoryName(), "Binaries", StringComparison.OrdinalIgnoreCase))
			{
				ProgramFinder = ProgramFinder.ParentDirectory;
			}

			// we are now at Source or similar directory, go up one more, then into Programs, and finally the "project" directory
			if (ProgramFinder != null)
			{
				DirectoryReference ProgramsDir = DirectoryReference.Combine(ProgramFinder, "../Programs");
				// if it doesn't exist, something went wrong, and we can't use this. throw an exception to catch it, for now
				if (!DirectoryReference.Exists(ProgramsDir))
				{
					throw new BuildException($"Unable to find Programs directory for {ProgramName}, when starting in {StartingDir}");
				}

				ProgramFinder = DirectoryReference.Combine(ProgramsDir, ProgramName);
				// if it exists, we have a ProductDir we can use for plists, icons, etc
				if (!DirectoryReference.Exists(ProgramFinder))
				{
					if (bCreateIfNotFound)
					{
						DirectoryReference.CreateDirectory(ProgramFinder);
						return ProgramFinder;
					}
				}
				else
				{
					return ProgramFinder;
				}
			}

			return null;
		}

		public override ReadOnlyAppleTargetRules GetAppleTargetRules(ReadOnlyTargetRules InTarget)
		{
			return InTarget.MacPlatform;
		}

		/// <inheritdoc/>
		protected override void GetCompileArguments_WarningsAndErrors(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			base.GetCompileArguments_WarningsAndErrors(CompileEnvironment, Arguments);

			//Arguments.Add("-Wsign-compare"); // fed up of not seeing the signed/unsigned warnings we get on Windows - lets enable them here too.
		}

		/// <inheritdoc/>
		protected override void GetCompileArguments_Debugging(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			base.GetCompileArguments_Debugging(CompileEnvironment, Arguments);

			// TODO: Mac always enables exceptions, is this correct?
			if (!CompileEnvironment.bEnableExceptions)
			{
				Arguments.Remove("-fno-exceptions");
				Arguments.Remove("-DPLATFORM_EXCEPTIONS_DISABLED=1");
			}
			Arguments.Add("-fexceptions");
			Arguments.Add("-DPLATFORM_EXCEPTIONS_DISABLED=0");

			if (CompileEnvironment.bHideSymbolsByDefault)
			{
				Arguments.Add("-fvisibility-ms-compat");
				Arguments.Add("-fvisibility-inlines-hidden");
			}
		}

		/// <inheritdoc/>
		protected override void GetCompileArguments_AdditionalArgs(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			if (!String.IsNullOrWhiteSpace(CompileEnvironment.AdditionalArguments))
			{
				string EscapedAdditionalArgs = String.Empty;
				foreach (string AdditionalArg in CompileEnvironment.AdditionalArguments.Split(new char[] { ' ' }, StringSplitOptions.RemoveEmptyEntries))
				{
					Match DefinitionMatch = Regex.Match(AdditionalArg, "-D\"?(?<Name>.*)=(?<Value>.*)\"?");
					if (DefinitionMatch.Success)
					{
						EscapedAdditionalArgs += String.Format(" -D{0}=\"{1}\"", DefinitionMatch.Groups["Name"].Value, DefinitionMatch.Groups["Value"].Value);
					}
					else
					{
						EscapedAdditionalArgs += " " + AdditionalArg;
					}
				}

				if (!String.IsNullOrWhiteSpace(EscapedAdditionalArgs))
				{
					Arguments.Add(EscapedAdditionalArgs);
				}
			}
		}

		/// <inheritdoc/>
		protected override void GetCompileArguments_Global(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			bool bIsUsingPortableSDK = false;
			GetCompileArguments_Global(CompileEnvironment, Arguments, bIsUsingPortableSDK);
		}

		private void GetCompileArguments_Global(CppCompileEnvironment CompileEnvironment, List<string> Arguments, bool bIsUsingPortableSDK)
		{
			base.GetCompileArguments_Global(CompileEnvironment, Arguments);

			CppRootPaths RootPaths = CompileEnvironment.RootPaths;

			Arguments.Add("-fasm-blocks");

			// Disable FMA contraction on arm builds in order to preserve consistency with x64
			// derived data builds.
			// This is also added to the x64 builds in case someone ever adds -mfma.
			Arguments.Add("-ffp-contract=off");

			if (CompileEnvironment.bEnableOSX109Support)
			{
				Arguments.Add("-faligned-new"); // aligned operator new is supported only on macOS 10.14 and above
			}

			// Pass through architecture and OS info
			Arguments.Add("" + FormatArchitectureArg(CompileEnvironment.Architectures));

			// Clang modules depend on SDK paths unavailable on remote workers
			if (bIsUsingPortableSDK)
			{
				AddPortableToolchainIncludePaths(CompileEnvironment, Arguments);
			}
			else
			{
				Arguments.Add($"-isysroot \"{NormalizeCommandLinePath(Settings.GetSDKPath(CompileEnvironment.Architecture), RootPaths)}\"");
				
				List<string> FrameworksSearchPaths = new List<string>();
				foreach (UEBuildFramework Framework in CompileEnvironment.AdditionalFrameworks)
				{
					FileReference FrameworkPath = new FileReference(Path.GetFullPath(Framework.Name));
					if (!FrameworksSearchPaths.Contains(FrameworkPath.Directory.FullName))
					{
						Arguments.Add($"-F \"{NormalizeCommandLinePath(FrameworkPath.Directory, RootPaths)}\"");
						FrameworksSearchPaths.Add(FrameworkPath.Directory.FullName);
					}
				}
			}
		}

		protected override void GetCompileArguments_PortableToolchain(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			bool bIsUsingPortableSDK = true;
			GetCompileArguments_Global(CompileEnvironment, Arguments, bIsUsingPortableSDK);
		}

		void AddFrameworkToLinkCommand(LinkEnvironment linkEnvironment, List<string> LinkArguments, string FrameworkName, string Arg = "-framework")
		{
			if (FrameworkName.EndsWith(".framework", StringComparison.Ordinal))
			{
				FileReference frameworkPath = new FileReference(FrameworkName);
				LinkArguments.Add($"-F \"{NormalizeCommandLinePath(frameworkPath.Directory, linkEnvironment.RootPaths)}\"");
				FrameworkName = Path.GetFileNameWithoutExtension(FrameworkName);
			}
			LinkArguments.Add($"{Arg} \"{FrameworkName}\"");
		}

		protected override void GetLinkArguments_Sanitizers(LinkEnvironment linkEnvironment, List<string> arguments)
		{
			base.GetLinkArguments_Sanitizers(linkEnvironment, arguments);

			if (Options.HasFlag(ClangToolChainOptions.EnableAddressSanitizer) ||
				Options.HasFlag(ClangToolChainOptions.EnableThreadSanitizer) ||
				Options.HasFlag(ClangToolChainOptions.EnableUndefinedBehaviorSanitizer) ||
				Options.HasFlag(ClangToolChainOptions.EnableLibFuzzer))
			{
				arguments.Add("-g");
			}
		}

		protected override void GetLinkArguments_Global(LinkEnvironment LinkEnvironment, List<string> Arguments)
		{
			base.GetLinkArguments_Global(LinkEnvironment, Arguments);

			// Temporary workaround for linker warning with Xcode 14:
			//		'ld: warning: could not create compact unwind for _inflate_fast: registers 27 not saved contiguously in frame'
			if (CompilerVersionLessThan(14, 0, 0))
			{
				Arguments.Add("-Wl,-fatal_warnings");
			}

			if (LinkEnvironment.bIsBuildingDLL)
			{
				Arguments.Add("-dynamiclib");
			}

			if (LinkEnvironment.Configuration == CppConfiguration.Debug)
			{
				// Apple's Clang is not supposed to run the de-duplication pass when linking in debug configs. Xcode adds this flag automatically, we need it as well, otherwise linking would take very long
				Arguments.Add("-Wl,-no_deduplicate");
			}

			// Needed to make sure install_name_tool will be able to update paths in Mach-O headers
			Arguments.Add("-headerpad_max_install_names");
		}

		public static string LoadEngineDisplayVersion(bool bIgnorePatchVersion = false)
		{
			BuildVersion? Version;
			if (BuildVersion.TryRead(BuildVersion.GetDefaultFileName(), out Version))
			{
				return $"{Version.MajorVersion}.{Version.MinorVersion}.{(bIgnorePatchVersion ? 0 : Version.PatchVersion)}";
			}
			else
			{
				return "4.0.0";
			}
		}

		private string LoadEngineAPIVersion()
		{
			if (BuildVersion.TryRead(BuildVersion.GetDefaultFileName(), out BuildVersion? Version))
			{
				if ((Target?.bPrecompile == true || Version.IsPromotedBuild) && Version.EffectiveCompatibleChangelist > 0)
				{
					int CL = Version.EffectiveCompatibleChangelist;
					return $"{CL / (100 * 100)}.{(CL / 100) % 100}.{CL % 100}";
				}
			}
			return LoadEngineDisplayVersion(true);
		}

		private string GetGameNameFromExecutablePath(string ExePath)
		{
			string ExeName = Path.GetFileName(ExePath);
			string[] ExeNameParts = ExeName.Split('-');
			string GameName = ExeNameParts[0];

			if (GameName == "EpicGamesBootstrapLauncher")
			{
				GameName = "EpicGamesLauncher";
			}
			else if (GameName == "UE5" && ProjectFile != null)
			{
				GameName = ProjectFile.GetFileNameWithoutAnyExtensions();
			}

			return GameName;
		}

		private void AddLibraryPathToRPaths(string Library, string ExeAbsolutePath, ref List<string> RPaths, List<string> LinkArguments, bool bIsBuildingAppBundle, ILogger Logger)
		{
			string LibraryFullPath = Path.GetFullPath(Library);
			string LibraryDir = Path.GetDirectoryName(LibraryFullPath)!;
			string ExeDir = Path.GetDirectoryName(ExeAbsolutePath)!;

			// Only dylibs and frameworks, and only those that are outside of Engine/Binaries/Mac and Engine/Source/ThirdParty, and outside of the folder where the executable is need an additional RPATH entry
			if ((Library.EndsWith("dylib", StringComparison.Ordinal) || Library.EndsWith(".framework", StringComparison.Ordinal))
				&& !LibraryFullPath.Contains("/Engine/Source/ThirdParty/", StringComparison.Ordinal) && LibraryDir != ExeDir && !RPaths.Contains(LibraryDir))
			{
				// macOS gatekeeper erroneously complains about not seeing the CEF3 framework in the codesigned Launcher because it's only present in one of the folders specified in RPATHs.
				// To work around this we will only add a single RPATH entry for it, for the framework stored in .app/Contents/UE/ subfolder of the packaged app bundle
				bool bCanUseMultipleRPATHs = !ExeAbsolutePath.Contains("EpicGamesLauncher-Mac-Shipping", StringComparison.Ordinal) || !Library.Contains("CEF3", StringComparison.Ordinal);

				// First, add a path relative to the executable.
				string FinalExeDir = ExeDir;
				if (bIsBuildingAppBundle)
				{
					FinalExeDir = ExeAbsolutePath + ".app/Contents/MacOS";
				}
				string RelativePath = Utils.MakePathRelativeTo(LibraryDir, FinalExeDir).Replace("\\", "/", StringComparison.Ordinal);

				if (bCanUseMultipleRPATHs)
				{
					LinkArguments.Add("-rpath \"@loader_path/" + RelativePath + "\"");

					// We are referencing plugins that need to be relative to the engine not the dylib we are building
					// To be safe leave the previous relative loader_path in place and add a relative engine executable path
					if (!bIsBuildingAppBundle && LibraryDir.Contains("/Engine/Plugins/", StringComparison.Ordinal) && ExeAbsolutePath.EndsWith("dylib", StringComparison.Ordinal))
					{
						int EngineDirStrIndex = LibraryDir.IndexOf("Engine", StringComparison.Ordinal);
						if (EngineDirStrIndex >= 0)
						{
							LinkArguments.Add("-rpath \"@executable_path/../../../../../../" + LibraryDir.Substring(EngineDirStrIndex) + "\"");
						}
					}
				}

				// If building an app bundle, we also need an RPATH for use in packaged game and a separate one for staged builds
				if (bIsBuildingAppBundle)
				{
					string EngineDir = Unreal.RootDirectory.ToString();
					string? ProjectDir = ProjectFile?.Directory.FullName;

					// In packaged games dylibs are stored in Contents/UE subfolders, for example in GameName.app/Contents/UE/Engine/Binaries/ThirdParty/PhysX/Mac
					string BundleUEDir = Path.GetFullPath(ExeDir + "/../../Contents/UE");
					string? BundleLibraryDir = null;
					if (LibraryDir.StartsWith(EngineDir + "/", StringComparison.Ordinal))
					{
						BundleLibraryDir = LibraryDir.Replace(EngineDir, BundleUEDir, StringComparison.Ordinal);
					}
					else if (ProjectDir != null && LibraryDir.StartsWith(ProjectDir + "/", StringComparison.Ordinal))
					{
						string GameName = GetGameNameFromExecutablePath(ExeAbsolutePath);
						BundleLibraryDir = LibraryDir.Replace(ProjectDir, BundleUEDir + "/" + GameName, StringComparison.Ordinal);
					}

					if (BundleLibraryDir != null)
					{
						string BundleRelativeDir = Utils.MakePathRelativeTo(BundleLibraryDir, ExeDir).Replace("\\", "/", StringComparison.Ordinal);
						LinkArguments.Add("-rpath \"@loader_path/" + BundleRelativeDir + "\"");
					}
					else
					{
						Logger.LogWarning("Unexpected third party dylib location when generating RPATH entries: {LibraryFullPath}. Skipping.", LibraryFullPath);
					}

					// For staged code-based games we need additional entry if the game is not stored directly in the engine's root directory
					if (bCanUseMultipleRPATHs)
					{
						string StagedUEDir = Path.GetFullPath(ExeDir + "/../../../../../..");
						string StagedLibraryDir = LibraryDir.Replace(EngineDir, StagedUEDir, StringComparison.Ordinal);
						string StagedRelativeDir = Utils.MakePathRelativeTo(StagedLibraryDir, ExeDir).Replace("\\", "/", StringComparison.Ordinal);
						if (StagedRelativeDir != RelativePath)
						{
							LinkArguments.Add("-rpath \"@loader_path/" + StagedRelativeDir + "\"");
						}
					}
				}

				RPaths.Add(LibraryDir);
			}
		}

		public override FileItem[] LinkImportLibrary(LinkEnvironment LinkEnvironment, IActionGraphBuilder Graph)
		{
			// we actually create Actions for every dylib, but we never return a FileItem. Instead depend on a Link action
			// linking to the stub dylib to make thie action take place. If we returned FileItems here, UBT would link a 
			// stub lib for every dylub even if not needed by anything
			if (LinkEnvironment.bIsBuildingDLL)
			{
				LinkAllFiles(LinkEnvironment, true, Graph);
			}

			return Array.Empty<FileItem>();
		}

		public override FileItem? LinkFiles(LinkEnvironment LinkEnvironment, bool bBuildImportLibraryOnly, IActionGraphBuilder Graph)
		{
			return null;
		}

		public override FileItem[] LinkAllFiles(LinkEnvironment LinkEnvironment, bool bBuildImportLibraryOnly, IActionGraphBuilder Graph)
		{
			List<FileItem> OutputFiles = new();
			// To solve the problem with cross dependencies, for now we create a broken dylib that does not link with other engine dylibs.
			// This is fixed in later step, FixDylibDependencies. For this and to know what libraries to copy whilst creating an app bundle,
			// we gather the list of engine dylibs.
			List<string> EngineAndGameLibraries = new List<string>();

			bool bIsBuildingLibrary = LinkEnvironment.bIsBuildingLibrary || bBuildImportLibraryOnly;
			bool bIsBuildingAppBundle = !LinkEnvironment.bIsBuildingDLL && !LinkEnvironment.bIsBuildingLibrary && !LinkEnvironment.bIsBuildingConsoleApplication;
			FileReference LinkerPath = bIsBuildingLibrary ? Info.Archiver : Info.Clang;

			if (LinkEnvironment.Architectures.bIsMultiArch)
			{
				List<FileItem> PerArchOutputFiles = new();
				foreach (UnrealArch Arch in LinkEnvironment.Architectures.Architectures)
				{
					LinkEnvironment ArchEnvironment = new LinkEnvironment(LinkEnvironment, Arch);
					if (!bBuildImportLibraryOnly)
					{
						ArchEnvironment.OutputFilePaths = LinkEnvironment.OutputFilePaths.Select(x => new FileReference($"{LinkEnvironment.IntermediateDirectory}/{Path.GetFileName(x.FullName)}_{Arch}")).ToList();
					}

					PerArchOutputFiles.Add(LinkArchitectureFiles(ArchEnvironment, LinkEnvironment.OutputFilePath, bBuildImportLibraryOnly, Graph));
				}

				if (bBuildImportLibraryOnly)
				{
					return [.. PerArchOutputFiles];
				}

				FileItem OutputFileItem = FileItem.GetItemByFileReference(LinkEnvironment.OutputFilePath);
				OutputFiles.Add(OutputFileItem);

				Action LipoAction = Graph.CreateAction(ActionType.Link);
				LipoAction.PrerequisiteItems.UnionWith(PerArchOutputFiles);
				LipoAction.ProducedItems.Add(OutputFileItem);
				LipoAction.WorkingDirectory = GetMacDevSrcRoot();
				LipoAction.CommandPath = FileReference.Combine(Info.Clang.Directory, "lipo");
				LipoAction.CommandDescription = (bBuildImportLibraryOnly ? "LipoStub" : "Lipo");
				LipoAction.CommandVersion = Info.ClangVersionString;
				LipoAction.CommandArguments = String.Join(" ", PerArchOutputFiles.Select(x => $"\"{x}\"")) + $" -create -output \"{OutputFileItem.AbsolutePath}\"";
				LipoAction.StatusDescription = Path.GetFileName(OutputFileItem.AbsolutePath);
				LipoAction.bCanExecuteRemotely = false;
			}
			else
			{
				OutputFiles.Add(LinkArchitectureFiles(LinkEnvironment, LinkEnvironment.OutputFilePath, bBuildImportLibraryOnly, Graph));
			}

			if (!DirectoryReference.Exists(LinkEnvironment.IntermediateDirectory!))
			{
				DirectoryReference.CreateDirectory(LinkEnvironment.IntermediateDirectory!);
			}

			return OutputFiles.ToArray();
		}

		private FileItem MakeStubItem(LinkEnvironment LinkEnvironment, string DylibPath)
		{
			FileReference DylibPathRef = new FileReference(DylibPath);
			bool bIsEngineDylib = DylibPathRef.IsUnderDirectory(Unreal.EngineDirectory);

			string IntermediateDirectory = (ProjectFile == null || LinkEnvironment.OutputFilePaths.All(x => x.IsUnderDirectory(Unreal.EngineDirectory)) || bIsEngineDylib ? Unreal.EngineDirectory : ProjectFile.Directory) + "/Intermediate/Build/Mac";
			string DylibFilename = Path.GetFileName(DylibPath);

			string AppName = String.Empty;
			int firstDash = DylibFilename.IndexOf('-', StringComparison.Ordinal);
			if (firstDash != -1)
			{
				AppName = DylibFilename.Substring(0, firstDash);
			}
			else if (DylibFilename.StartsWith("lib", StringComparison.Ordinal)) // This is expected to exist. if there is no dash involved this is the target lib itself and name is part of the name
			{
				AppName = Path.GetFileNameWithoutExtension(DylibFilename).Substring(3);
			}
			return FileItem.GetItemByPath(Path.Combine(IntermediateDirectory, LinkEnvironment.Architecture.ToString(), AppName, LinkEnvironment.Configuration.ToString(), "Stubs", DylibFilename));
		}

		private FileItem LinkArchitectureFiles(LinkEnvironment LinkEnvironment, FileReference MultiArchOutputFile, bool bBuildImportLibraryOnly, IActionGraphBuilder Graph)
		{
			List<FileItem> InputFiles = LinkEnvironment.InputFiles;

			// with precompiled builds, we end up trying to link the arm64 and x64 versions of the engine .o files. Remove the wrong one
			// until we can separate out the .o files by architecture
			string ArchToRemove = LinkEnvironment.Architecture == UnrealArch.Arm64 ? "x64" : "arm64";
			InputFiles.RemoveAll(x => x.Location.ContainsName(ArchToRemove, 0));

			// Create an action that invokes the linker.
			Action LinkAction = Graph.CreateAction(ActionType.Link);
			LinkAction.RootPaths = LinkEnvironment.RootPaths;

			FileReference LinkerPath = LinkEnvironment.bIsBuildingLibrary ? Info.Archiver : Info.Clang;

			LinkAction.WorkingDirectory = GetMacDevSrcRoot();
			LinkAction.CommandPath = LinkerPath;
			string Arch = UnrealArchitectureConfig.ForPlatform(UnrealTargetPlatform.Mac).ConvertToReadableArchitecture(LinkEnvironment.Architecture);
			LinkAction.CommandDescription = $"{(bBuildImportLibraryOnly ? "LinkStub" : "Link")} [{Arch}]";
			LinkAction.CommandVersion = Info.ClangVersionString;
			LinkAction.bProducesImportLibrary = bBuildImportLibraryOnly || LinkEnvironment.bIsBuildingDLL;
			LinkAction.bCanExecuteInUBA = true;
			LinkAction.CacheBucket = GetCacheBucket(Target, LinkEnvironment);
			LinkAction.ArtifactMode = ArtifactMode.Enabled;

			List<string> LinkArguments = new();
			if (LinkEnvironment.bIsBuildingLibrary)
			{
				GetArchiveArguments_Global(LinkEnvironment, LinkArguments);
			}
			else
			{
				GetLinkArguments_Global(LinkEnvironment, LinkArguments);
			}

			if (LinkEnvironment.bIsBuildingDLL)
			{
				LinkArguments.Add("-current_version " + LoadEngineAPIVersion());
				LinkArguments.Add("-compatibility_version " + LoadEngineDisplayVersion(true));
			}

			// Tell the action that we're building an import library here and it should conditionally be
			// ignored as a prerequisite for other actions
			LinkAction.bProducesImportLibrary = bBuildImportLibraryOnly || LinkEnvironment.bIsBuildingDLL;

			// Add the output file as a production of the link action.
			FileItem OutputFile;
			if (bBuildImportLibraryOnly)
			{
				OutputFile = MakeStubItem(LinkEnvironment, LinkEnvironment.OutputFilePath.FullName);
			}
			else
			{
				OutputFile = FileItem.GetItemByFileReference(LinkEnvironment.OutputFilePath);
			}

			string DylibsPath = "@rpath";

			string AbsolutePath = MultiArchOutputFile.FullName.Replace("\\", "/", StringComparison.Ordinal);
			if (!LinkEnvironment.bIsBuildingLibrary)
			{
				LinkArguments.Add("-rpath @loader_path/ -rpath @executable_path/");
			}

			bool bIsBuildingAppBundle = !LinkEnvironment.bIsBuildingDLL && !LinkEnvironment.bIsBuildingLibrary && !LinkEnvironment.bIsBuildingConsoleApplication;
			if (bIsBuildingAppBundle)
			{
				LinkArguments.Add("-rpath @executable_path/../../../");
			}

			List<string> RPaths = new List<string>();

			// static libs and stub dylibs don't need to look other libs
			if (bBuildImportLibraryOnly)
			{
				LinkArguments.Add("-undefined dynamic_lookup");
				LinkArguments.Add("-Wl,-no_fixup_chains");
			}
			else if (!LinkEnvironment.bIsBuildingLibrary)
			{
				IEnumerable<string> AdditionalLibraries = Enumerable.Concat(LinkEnvironment.SystemLibraries, LinkEnvironment.Libraries.Select(x => x.FullName));

				// @todo set this to false if we have any rpath issues, or maybe set it to false when making Monolithic
				// we will remove the old path entirely after a lot of wide testing
				bool bCanUseSimpleRPaths = LinkEnvironment.LinkType == TargetLinkType.Monolithic;
				if (bCanUseSimpleRPaths)
				{
					string StagedBundlePrefix = bIsBuildingAppBundle ? "../UE/"  : "";
					string UnstagedBundlePrefix = bIsBuildingAppBundle ? "../../../" : "";

					HashSet<DirectoryReference> UsedDestDirs = new();
					HashSet<FileReference> UsedSourceLibs = new();
					void ProcessLib(FileReference SourceLib, FileReference DestLib, FileReference ReferencingFile)
					{
						if (String.Equals(SourceLib.GetExtension(), ".dylib", StringComparison.OrdinalIgnoreCase))
						{
							DirectoryReference DestDir = DestLib.Directory;
							if (UsedSourceLibs.Contains(SourceLib) || UsedDestDirs.Contains(DestDir))
							{
								return;
							}

							UsedDestDirs.Add(DestDir);
							UsedSourceLibs.Add(SourceLib);

							string RPathEntry;
							// handle staged rpaths
							if (ProjectFile != null && DestDir.IsUnderDirectory(ProjectFile.Directory))
							{
								RPathEntry = StagedBundlePrefix + ProjectFile.GetFileNameWithoutAnyExtensions() + "/" + DestDir.MakeRelativeTo(ProjectFile.Directory);
							}
							else
							{
								RPathEntry = StagedBundlePrefix + DestDir.MakeRelativeTo(Unreal.EngineDirectory.ParentDirectory!);
							}
							LinkArguments.Add($"-rpath \"@executable_path/{RPathEntry}\"");

							// handle unstaged paths
							RPathEntry = UnstagedBundlePrefix + DestDir.MakeRelativeTo(ReferencingFile.Directory);
							LinkArguments.Add($"-rpath \"@loader_path/{RPathEntry}\"");
						}
					}

					// first look in the RuntimeDependency list to look for remapped locations, anything not remapped we will deal with 
					// after this loop
					foreach (ModuleRules.RuntimeDependency SrcDep in LinkEnvironment.RuntimeDependencies)
					{
						FileReference DestPath = new(SrcDep.Path);
						FileReference SourcePath = SrcDep.SourcePath == null ? DestPath : new FileReference(SrcDep.SourcePath);
						ProcessLib(SourcePath, DestPath, MultiArchOutputFile);
					}
					foreach (string AdditionalLibrary in AdditionalLibraries)
					{
						FileReference LibPath = new(AdditionalLibrary);
						ProcessLib(LibPath, LibPath, MultiArchOutputFile);
					}
				}

				// Add the additional libraries to the argument list.
				foreach (string AdditionalLibrary in AdditionalLibraries)
				{
					if (!String.IsNullOrEmpty(Path.GetDirectoryName(AdditionalLibrary)) &&
							 (Path.GetDirectoryName(AdditionalLibrary)!.Contains("Binaries/Mac", StringComparison.Ordinal) ||
							 Path.GetDirectoryName(AdditionalLibrary)!.Contains("Binaries\\Mac", StringComparison.Ordinal)))
					{
						// It's an engine or game dylib. Save it for later
						FileItem LinkedLib;
						if (LinkEnvironment.bIsCrossReferenced)
						{
							LinkedLib = MakeStubItem(LinkEnvironment, AdditionalLibrary);
						}
						else
						{
							LinkedLib = FileItem.GetItemByPath(AdditionalLibrary);
						}

						LinkAction.PrerequisiteItems.Add(LinkedLib);
						LinkArguments.Add($"\"{NormalizeCommandLinePath(LinkedLib, LinkEnvironment.RootPaths)}\"");
					}
					else if (AdditionalLibrary.Contains(".framework/", StringComparison.Ordinal))
					{
						LinkArguments.Add($"\"{NormalizeCommandLinePath(FileItem.GetItemByPath(AdditionalLibrary), LinkEnvironment.RootPaths)}\"");
					}
					else if (String.IsNullOrEmpty(Path.GetDirectoryName(AdditionalLibrary)) && String.IsNullOrEmpty(Path.GetExtension(AdditionalLibrary)))
					{
						LinkArguments.Add(String.Format("-l\"{0}\"", AdditionalLibrary));
					}
					else
					{
						LinkArguments.Add($"\"{NormalizeCommandLinePath(FileItem.GetItemByPath(AdditionalLibrary), LinkEnvironment.RootPaths)}\"");
					}

					if (!bCanUseSimpleRPaths)
					{
						AddLibraryPathToRPaths(AdditionalLibrary, AbsolutePath, ref RPaths, LinkArguments, bIsBuildingAppBundle, Logger);
					}
				}

				foreach (string AdditionalLibrary in LinkEnvironment.DelayLoadDLLs)
				{
					LinkArguments.Add($"-weak_library \"{NormalizeCommandLinePath(FileItem.GetItemByPath(AdditionalLibrary), LinkEnvironment.RootPaths)}\"");

					if (!bCanUseSimpleRPaths)
					{
						AddLibraryPathToRPaths(AdditionalLibrary, AbsolutePath, ref RPaths, LinkArguments, bIsBuildingAppBundle, Logger);
					}
				}

				// Add frameworks
				Dictionary<string, bool> AllFrameworks = new Dictionary<string, bool>();
				foreach (string Framework in LinkEnvironment.Frameworks)
				{
					if (!AllFrameworks.ContainsKey(Framework))
					{
						AllFrameworks.Add(Framework, false);
					}
				}
				foreach (UEBuildFramework Framework in LinkEnvironment.AdditionalFrameworks)
				{
					if (Framework.ZipFile != null)
					{
						FileItem ExtractedTokenFile = ExtractFramework(Framework, LinkEnvironment.RootPaths, Graph);
						LinkAction.PrerequisiteItems.Add(ExtractedTokenFile);
					}

					if (Framework.bLinkFramework && !AllFrameworks.ContainsKey(Framework.Name))
					{
						AllFrameworks.Add(Framework.Name, false);
					}
				}
				foreach (string Framework in LinkEnvironment.WeakFrameworks)
				{
					if (!AllFrameworks.ContainsKey(Framework))
					{
						AllFrameworks.Add(Framework, true);
					}
				}

				foreach (KeyValuePair<string, bool> Framework in AllFrameworks)
				{
					AddFrameworkToLinkCommand(LinkEnvironment, LinkArguments, Framework.Key, Framework.Value ? "-weak_framework" : "-framework");
					AddLibraryPathToRPaths(Framework.Key, AbsolutePath, ref RPaths, LinkArguments, bIsBuildingAppBundle, Logger);
				}

				// Write the MAP file to the output directory.
				if (LinkEnvironment.bCreateMapFile)
				{
					string MapFileBaseName = OutputFile.AbsolutePath;

					int AppIdx = MapFileBaseName.IndexOf(".app/Contents/MacOS", StringComparison.Ordinal);
					if (AppIdx != -1)
					{
						MapFileBaseName = MapFileBaseName.Substring(0, AppIdx);
					}

					FileReference MapFilePath = new FileReference(MapFileBaseName + ".map");
					FileItem MapFile = FileItem.GetItemByFileReference(MapFilePath);
					LinkArguments.Add(String.Format("-Wl,-map,\"{0}\"", MapFilePath));
					LinkAction.ProducedItems.Add(MapFile);
				}
			}

			if (LinkEnvironment.bIsBuildingDLL)
			{
				// Add the output file to the command-line.
				string? InstallName = LinkEnvironment.InstallName;
				InstallName ??= String.Format("{0}/{1}", DylibsPath, Path.GetFileName(OutputFile.AbsolutePath).Replace($".dylib_{LinkEnvironment.Architecture}", ".dylib", StringComparison.Ordinal));
				LinkArguments.Add(String.Format("-install_name \"{0}\"", InstallName));
			}

			List<string> InputFileNames = new List<string>();
			foreach (FileItem InputFile in InputFiles)
			{
				InputFileNames.Add(String.Format("\"{0}\"", NormalizeCommandLinePath(InputFile, LinkEnvironment.RootPaths)));
				LinkAction.PrerequisiteItems.Add(InputFile);
			}

			foreach (string Filename in InputFileNames)
			{
				LinkArguments.Add(Filename);
			}

			// Add the output file to the command-line.
			LinkArguments.Add(String.Format("-o \"{0}\"", NormalizeCommandLinePath(OutputFile, LinkEnvironment.RootPaths)));

			// Add the additional arguments specified by the environment.
			LinkArguments.Add(LinkEnvironment.AdditionalArguments);

			FileReference ResponseFileName;
			if (bBuildImportLibraryOnly)
			{
				ResponseFileName = new(OutputFile.FullName + ResponseExt);
			}
			else
			{
				ResponseFileName = GetResponseFileName(LinkEnvironment, OutputFile);
			}

			LinkAction.ResponseFileContents = [.. LinkArguments];
			FileItem ResponseFileItem = Graph.CreateIntermediateTextFile(ResponseFileName, LinkAction.ResponseFileContents);
			string ResponseFileArgument = GetResponseFileArgument(ResponseFileItem, LinkEnvironment.RootPaths);

			LinkAction.PrerequisiteItems.Add(ResponseFileItem);

			LinkAction.CommandArguments = ResponseFileArgument;

			// Only execute linking on the local Mac.
			LinkAction.bCanExecuteRemotely = false;

			LinkAction.StatusDescription = Path.GetFileName(OutputFile.AbsolutePath);

			LinkAction.ProducedItems.Add(OutputFile);

			// Delete all items we produce
			LinkAction.DeleteItems.UnionWith(LinkAction.ProducedItems);

			return OutputFile;
		}

		/// <summary>
		/// Generates debug info for a given executable
		/// </summary>
		/// <param name="MachOBinary">FileItem describing the executable or dylib to generate debug info for</param>
		/// <param name="LinkEnvironment"></param>
		/// <param name="Graph">List of actions to be executed. Additional actions will be added to this list.</param>
		public static FileItem GenerateDebugInfo(FileItem MachOBinary, LinkEnvironment LinkEnvironment, IActionGraphBuilder Graph)
		{
			string BinaryPath = MachOBinary.AbsolutePath;
			if (BinaryPath.Contains(".app", StringComparison.Ordinal))
			{
				while (BinaryPath.Contains(".app", StringComparison.Ordinal))
				{
					BinaryPath = Path.GetDirectoryName(BinaryPath)!;
				}
				BinaryPath = Path.Combine(BinaryPath, Path.GetFileName(Path.ChangeExtension(MachOBinary.AbsolutePath, ".dSYM")));
			}
			else
			{
				BinaryPath = Path.ChangeExtension(BinaryPath, ".dSYM");
			}

			FileItem OutputFile = FileItem.GetItemByPath(BinaryPath);

			// Delete on the local machine
			if (Directory.Exists(OutputFile.AbsolutePath))
			{
				Directory.Delete(OutputFile.AbsolutePath, true);
			}

			// Make the compile action
			Action GenDebugAction = Graph.CreateAction(ActionType.GenerateDebugInfo);
			GenDebugAction.WorkingDirectory = GetMacDevSrcRoot();
			GenDebugAction.CommandPath = BuildHostPlatform.Current.Shell;

			// Deletes ay existing file on the building machine. Also, waits 30 seconds, if needed, for the input file to be created in an attempt to work around
			// a problem where dsymutil would exit with an error saying the input file did not exist.
			// Note that the source and dest are switched from a copy command
			string DsymutilPath = "/usr/bin/dsymutil";
			string UniversalDsymutilScriptPath = FileReference.Combine(Unreal.EngineDirectory, "Build/BatchFiles/Mac/GenerateUniversalDSYM.sh").FullName;

			string ArgumentString = "-c \"";
			ArgumentString += String.Format("for i in {{1..30}}; ");
			ArgumentString += String.Format("do if [ -f \\\"{0}\\\" ] ; ", MachOBinary.AbsolutePath);
			ArgumentString += String.Format("then ");
			ArgumentString += String.Format("break; ");
			ArgumentString += String.Format("else ");
			ArgumentString += String.Format("sleep 1; ");
			ArgumentString += String.Format("fi; ");
			ArgumentString += String.Format("done; ");

			ArgumentString += String.Format("if [ ! -f \\\"{1}\\\" ] || [ \\\"{0}\\\" -nt \\\"{1}\\\" ] ; ", MachOBinary.AbsolutePath, OutputFile.AbsolutePath);
			ArgumentString += String.Format("then ");
			ArgumentString += String.Format("rm -rf \\\"{0}\\\"; ", OutputFile.AbsolutePath);
			// use the new script for monolthic (ie large) targets
			if (LinkEnvironment.LinkType == TargetLinkType.Monolithic)
			{
				ArgumentString += String.Format(" \\\"{0}\\\" \\\"{1}\\\" \\\"{2}\\\"; ",
					UniversalDsymutilScriptPath,
					MachOBinary.AbsolutePath,
					OutputFile.AbsolutePath);
			}
			else
			{
				ArgumentString += String.Format(" \\\"{0}\\\" -f \\\"{1}\\\" -o \\\"{2}\\\"; ",
					DsymutilPath,
					MachOBinary.AbsolutePath,
					OutputFile.AbsolutePath);
			}
			ArgumentString += String.Format("fi; ");
			ArgumentString += "\"";

			GenDebugAction.CommandArguments = ArgumentString;

			GenDebugAction.PrerequisiteItems.Add(MachOBinary);
			GenDebugAction.ProducedItems.Add(OutputFile);
			GenDebugAction.CommandDescription = "";
			GenDebugAction.StatusDescription = "Generating " + Path.GetFileName(BinaryPath);
			GenDebugAction.bCanExecuteRemotely = false;

			return OutputFile;
		}

		private static readonly Dictionary<ReadOnlyTargetRules, DirectoryReference> BundleContentsDirectories = new();

		public override void ModifyBuildProducts(ReadOnlyTargetRules Target, UEBuildBinary Binary, IEnumerable<string> Libraries, IEnumerable<UEBuildBundleResource> BundleResources, Dictionary<FileReference, BuildProductType> BuildProducts)
		{
			if (Target.bUsePDBFiles == true)
			{
				KeyValuePair<FileReference, BuildProductType>[] BuildProductsArray = BuildProducts.ToArray();

				foreach (KeyValuePair<FileReference, BuildProductType> BuildProductPair in BuildProductsArray)
				{
					string[] DebugExtensions = Array.Empty<string>();
					switch (BuildProductPair.Value)
					{
						case BuildProductType.Executable:
							DebugExtensions = UEBuildPlatform.GetBuildPlatform(Target.Platform).GetDebugInfoExtensions(Target, UEBuildBinaryType.Executable);
							break;
						case BuildProductType.DynamicLibrary:
							DebugExtensions = UEBuildPlatform.GetBuildPlatform(Target.Platform).GetDebugInfoExtensions(Target, UEBuildBinaryType.DynamicLinkLibrary);
							break;
					}
					string? DSYMExtension = Array.Find(DebugExtensions, element => element == ".dSYM");
					if (!String.IsNullOrEmpty(DSYMExtension))
					{
						string BinaryPath = BuildProductPair.Key.FullName;
						if (BinaryPath.Contains(".app", StringComparison.Ordinal))
						{
							while (BinaryPath.Contains(".app", StringComparison.Ordinal))
							{
								BinaryPath = Path.GetDirectoryName(BinaryPath)!;
							}
							BinaryPath = Path.Combine(BinaryPath, BuildProductPair.Key.GetFileName());
							BinaryPath = Path.ChangeExtension(BinaryPath, DSYMExtension);
							FileReference Ref = new FileReference(BinaryPath);
							BuildProducts[Ref] = BuildProductType.SymbolFile;
						}
					}
					else if (BuildProductPair.Value == BuildProductType.SymbolFile && BuildProductPair.Key.FullName.Contains(".app", StringComparison.Ordinal))
					{
						BuildProducts.Remove(BuildProductPair.Key);
					}
					if (BuildProductPair.Value == BuildProductType.DynamicLibrary && Target.bCreateMapFile)
					{
						BuildProducts.Add(new FileReference(BuildProductPair.Key.FullName + ".map"), BuildProductType.MapFile);
					}
				}
			}

			if (Target.bIsBuildingConsoleApplication)
			{
				return;
			}

			DirectoryReference? BundleContentsDirectory;
			lock (BundleContentsDirectories)
			{
				if (!BundleContentsDirectories.ContainsKey(Target) && Binary.Type == UEBuildBinaryType.Executable)
				{
					// For Mac binary executables, we may build it outside of app, but expect it to be inside of .app after modern Xcode does its thing
					// We still keep the executable outside of app as RequiredResource, so that Horde will copy in-between agents
					FileReference FinalBinaryPath = Binary.OutputFilePath;
					if (!FinalBinaryPath.Directory.FullName.EndsWith(".app/Contents/MacOS", StringComparison.OrdinalIgnoreCase))
					{
						FinalBinaryPath = FileReference.Combine(FinalBinaryPath.Directory, FinalBinaryPath.GetFileName() + ".app", "Contents", "MacOS", FinalBinaryPath.GetFileName());
						BuildProducts[Binary.OutputFilePath] = BuildProductType.RequiredResource;
						BuildProducts.Add(FinalBinaryPath, BuildProductType.Executable);
					}
					BundleContentsDirectories.Add(Target, FinalBinaryPath.Directory.ParentDirectory!);
				}
				BundleContentsDirectory = BundleContentsDirectories.GetValueOrDefault(Target);
			}

			if (BundleContentsDirectory == null)
			{
				return;
			}

			// We need to know what third party dylibs would be copied to the bundle
			if (Binary.Type != UEBuildBinaryType.StaticLibrary)
			{
				foreach (UEBuildBundleResource Resource in BundleResources)
				{
					if (Directory.Exists(Resource.ResourcePath))
					{
						foreach (string ResourceFile in Directory.GetFiles(Resource.ResourcePath, "*", SearchOption.AllDirectories))
						{
							BuildProducts.Add(
								FileReference.Combine(BundleContentsDirectory!, Resource.BundleContentsSubdir!,
									ResourceFile.Substring(Path.GetDirectoryName(Resource.ResourcePath)!.Length + 1)),
								BuildProductType.RequiredResource);
						}
					}
					else
					{
						BuildProducts.Add(
							FileReference.Combine(BundleContentsDirectory, Resource.BundleContentsSubdir!,
								Path.GetFileName(Resource.ResourcePath)!), BuildProductType.RequiredResource);
					}
				}
			}

			if (Binary.Type == UEBuildBinaryType.Executable)
			{
				// And we also need all the resources
				BuildProducts.Add(FileReference.Combine(BundleContentsDirectory!, "Info.plist"), BuildProductType.RequiredResource);
				BuildProducts.Add(FileReference.Combine(BundleContentsDirectory!, "PkgInfo"), BuildProductType.RequiredResource);

				// when we codesign, we need to copy the signature around on build machines, etc. this will put the CodeResources (that were created by post-build signing)
				// is in the .target receipt file
				BuildProducts.Add(FileReference.Combine(BundleContentsDirectory!, "_CodeSignature", "CodeResources"),
					BuildProductType.RequiredResource);
			}
		}

		private readonly ConcurrentBag<FileItem> DebugInfoFiles = new ConcurrentBag<FileItem>();

		public override void FinalizeOutput(ReadOnlyTargetRules Target, TargetMakefileBuilder MakefileBuilder, DirectoryReference ExeDir)
		{
			base.FinalizeOutput(Target, MakefileBuilder, ExeDir);

			TargetMakefile Makefile = MakefileBuilder.Makefile;

			// Re-add any .dSYM files that may have been stripped out.
			List<string> OutputFiles = Makefile.OutputItems.Select(Item => Path.ChangeExtension(Item.FullName, ".dSYM")).Distinct().ToList();
			foreach (FileItem DebugItem in DebugInfoFiles)
			{
				if (OutputFiles.Any(Item => String.Equals(Item, DebugItem.FullName, StringComparison.OrdinalIgnoreCase)))
				{
					Makefile.OutputItems.Add(DebugItem);
				}
			}
		}

		public override ICollection<FileItem> PostBuild(ReadOnlyTargetRules Target, FileItem Executable, LinkEnvironment BinaryLinkEnvironment, IActionGraphBuilder Graph)
		{
			ICollection<FileItem> OutputFiles = base.PostBuild(Target, Executable, BinaryLinkEnvironment, Graph);

			if (BinaryLinkEnvironment.bIsBuildingLibrary)
			{
				return OutputFiles;
			}

			// For Mac, generate the dSYM file if the config file is set to do so
			if (BinaryLinkEnvironment.bUsePDBFiles == true)
			{
				DebugInfoFiles.Add(GenerateDebugInfo(Executable, BinaryLinkEnvironment, Graph));
			}

			if ((BinaryLinkEnvironment.bIsBuildingDLL && (Options & ClangToolChainOptions.OutputDylib) == 0) ||
				(BinaryLinkEnvironment.bIsBuildingConsoleApplication && Executable.Name.EndsWith("-Cmd", StringComparison.Ordinal)))
			{
				return OutputFiles;
			}

			return OutputFiles;
		}

		public void StripSymbols(FileReference SourceFile, FileReference TargetFile)
		{
			StripSymbolsWithXcode(SourceFile, TargetFile, AppleToolChainSettings.ToolchainDir);
		}
	};
}
