// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// Mac-specific target settings
	/// </summary>
	public class MacTargetRules : AppleTargetRules
	{
		/// <summary>
		/// Enables LibFuzzer.
		/// </summary>
		[CommandLine("-EnableLibFuzzer")]
		[XmlConfigFile(Category = "BuildConfiguration", Name = "bEnableLibFuzzer")]
		public bool bEnableLibFuzzer = false;

		/// <summary>
		/// Enables the generation of .dsym files. This can be disabled to enable faster iteration times during development.
		/// </summary>
		[CommandLine("-EnableDSYM", Value = "true")]
		[XmlConfigFile(Category = "BuildConfiguration", Name = "bUseDSYMFiles")]
		public bool bUseDSYMFiles = false;

		/// <summary>
		/// Enables runtime ray tracing support.
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/MacTargetPlatform.MacTargetSettings", "bEnableRayTracing")]
		public bool bEnableRayTracing = false;

		/// <summary>
		/// When generating a project file for an editor target, forces the exe name to match the target name instead of being overridden to 'UnrealEditor'. 
		/// </summary>
		public bool bForceTargetNameAsExeName { get; set; } = false;

		/// <summary>
		/// Constructor
		/// </summary>
		public MacTargetRules()
		{
			string? AddressSanitizer = Environment.GetEnvironmentVariable("ENABLE_ADDRESS_SANITIZER");
			if (AddressSanitizer != null && AddressSanitizer == "YES")
			{
				bEnableAddressSanitizer = true;
			}

			string? ThreadSanitizer = Environment.GetEnvironmentVariable("ENABLE_THREAD_SANITIZER");
			if (ThreadSanitizer != null && ThreadSanitizer == "YES")
			{
				bEnableThreadSanitizer = true;
			}

			string? UndefSanitizerMode = Environment.GetEnvironmentVariable("ENABLE_UNDEFINED_BEHAVIOR_SANITIZER");
			if (UndefSanitizerMode != null && UndefSanitizerMode == "YES")
			{
				bEnableUndefinedBehaviorSanitizer = true;
			}
		}
	}

	/// <summary>
	/// Read-only wrapper for Mac-specific target settings
	/// </summary>
	public class ReadOnlyMacTargetRules : ReadOnlyAppleTargetRules
	{
		/// <summary>
		/// The private mutable settings object
		/// </summary>
		private readonly MacTargetRules Inner;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="Inner">The settings object to wrap</param>
		public ReadOnlyMacTargetRules(MacTargetRules Inner)
			: base(Inner)
		{
			this.Inner = Inner;
		}

		/// <summary>
		/// Accessors for fields on the inner TargetRules instance
		/// </summary>
		#region Read-only accessor properties 
#pragma warning disable CS1591

		public bool bEnableLibFuzzer => Inner.bEnableLibFuzzer;

		public bool bEnableRayTracing => Inner.bEnableRayTracing;

		public bool bForceTargetNameAsExeName => Inner.bForceTargetNameAsExeName;

#pragma warning restore CS1591
		#endregion
	}

	class MacArchitectureConfig : AppleArchitectureConfig
	{
		public MacArchitectureConfig()
			: base(new[] { UnrealArch.X64, UnrealArch.Arm64 })
		{
		}

		public override UnrealArch GetHostArchitecture()
		{
			return MacExports.IsRunningOnAppleArchitecture ? UnrealArch.Arm64 : UnrealArch.X64;
		}

		public override string ConvertToReadableArchitecture(UnrealArch Architecture)
		{
			if (Architecture == UnrealArch.X64)
			{
				return "Intel";
			}
			if (Architecture == UnrealArch.Arm64)
			{
				return "Apple";
			}
			return base.ConvertToReadableArchitecture(Architecture);
		}

		public override UnrealArchitectures ActiveArchitectures(FileReference? ProjectFile, string? TargetName)
		{
			return GetProjectArchitectures(ProjectFile, TargetName, false, false);
		}

		public override UnrealArchitectures DistributionArchitectures(FileReference? ProjectFile, string? TargetName)
		{
			return GetProjectArchitectures(ProjectFile, TargetName, false, true);
		}

		public override UnrealArchitectures ProjectSupportedArchitectures(FileReference? ProjectFile, string? TargetName = null)
		{
			return GetProjectArchitectures(ProjectFile, TargetName, true, false);
		}

		private static readonly Dictionary<string, UnrealArchitectures> ProjectArchitectureCache = new();
		private static UnrealArchitectures GetProjectArchitectures(FileReference? ProjectFile, string? TargetName, bool bGetAllSupported, bool bIsDistributionMode)
		{
			string Key = $"{ProjectFile}{TargetName}{bGetAllSupported}{bIsDistributionMode}";
			lock (ProjectArchitectureCache)
			{
				UnrealArchitectures? CachedArches;
				if (ProjectArchitectureCache.TryGetValue(Key, out CachedArches))
				{
					return CachedArches;
				}
			}

			bool bIsEditor = false;
			bool bIsBuildMachine = Unreal.IsBuildMachine();

			// get project ini from ProjetFile, or if null, then try to get it from the target rules
			if (TargetName != null)
			{
				RulesAssembly RulesAsm;
				if (ProjectFile == null)
				{
					RulesAsm = RulesCompiler.CreateEngineRulesAssembly(Unreal.IsEngineInstalled(), false, false, Log.Logger);
				}
				else
				{
					RulesAsm = RulesCompiler.CreateProjectRulesAssembly(ProjectFile, Unreal.IsEngineInstalled(), false, false, Log.Logger);
				}

				try
				{
					// CreateTargetRules here needs to have an UnrealArchitectures object, because otherwise with 'null', it will call
					// back to this function to get the ActiveArchitectures! in this case the arch is unimportant
					UnrealArchitectures DummyArchitectures = new(UnrealArch.X64);
					TargetRules? Rules = RulesAsm.CreateTargetRules(TargetName, UnrealTargetPlatform.Mac, UnrealTargetConfiguration.Development, DummyArchitectures, ProjectFile, null, Log.Logger, ValidationOptions: TargetRulesValidationOptions.ValidateNothing);
					bIsEditor = Rules.Type == TargetType.Editor;

					// the projectfile passed in may be a game's uproject file that we are compiling a program in the context of, 
					// but we still want the settings for the program
					if (Rules.Type == TargetType.Program)
					{
						ProjectFile = Rules.ProjectFile;
					}
				}
				catch (Exception)
				{
					// do nothing if it fails, assume no project
				}
			}

			ConfigHierarchy EngineIni = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, ProjectFile?.Directory, UnrealTargetPlatform.Mac);

			// get values from project ini
			string SupportKey = bIsEditor ? "EditorTargetArchitecture" : "TargetArchitecture";
			string DefaultKey = bIsEditor ? "EditorDefaultArchitecture" : "DefaultArchitecture";
			string SupportedArchitecture;
			string DefaultArchitecture;
			bool bBuildAllSupportedOnBuildMachine;
			EngineIni.GetString("/Script/MacTargetPlatform.MacTargetSettings", SupportKey, out SupportedArchitecture);
			EngineIni.GetString("/Script/MacTargetPlatform.MacTargetSettings", DefaultKey, out DefaultArchitecture);
			EngineIni.GetBool("/Script/MacTargetPlatform.MacTargetSettings", "bBuildAllSupportedOnBuildMachine", out bBuildAllSupportedOnBuildMachine);

			bool bSupportsArm64 = SupportedArchitecture.Contains("universal", StringComparison.OrdinalIgnoreCase) || SupportedArchitecture.Contains("apple", StringComparison.OrdinalIgnoreCase);
			bool bSupportsX86 = SupportedArchitecture.Contains("universal", StringComparison.OrdinalIgnoreCase) || SupportedArchitecture.Contains("intel", StringComparison.OrdinalIgnoreCase);

			// make sure we found a good value
			if (!bSupportsArm64 && !bSupportsX86)
			{
				throw new BuildException($"Unknown {SupportKey} value found ('{SupportedArchitecture}') in .ini");
			}

			// choose a supported architecture(s) based on desired type
			List<UnrealArch> Architectures = new();

			// return all supported if getting supported, compiling for distribution, or we want active, and "universal" is selected
			if (bGetAllSupported || bIsDistributionMode || DefaultArchitecture.Contains("universal", StringComparison.OrdinalIgnoreCase) ||
				(bIsBuildMachine && bBuildAllSupportedOnBuildMachine))
			{
				if (bSupportsArm64)
				{
					Architectures.Add(UnrealArch.Arm64);
				}
				if (bSupportsX86)
				{
					Architectures.Add(UnrealArch.X64);
				}
			}
			else if (DefaultArchitecture.Contains("host", StringComparison.OrdinalIgnoreCase))
			{
				// if we don't support Arm, then always use X64, otherwise use whatever the host arch is
				Architectures.Add(bSupportsArm64 ? UnrealArch.Host.Value : UnrealArch.X64);
			}
			else if (DefaultArchitecture.Contains("apple", StringComparison.OrdinalIgnoreCase))
			{
				if (!bSupportsArm64)
				{
					throw new BuildException($"{DefaultKey} is set to {DefaultArchitecture}, but AppleSilicon is not a supported architecture");
				}
				Architectures.Add(UnrealArch.Arm64);
			}
			else if (DefaultArchitecture.Contains("intel", StringComparison.OrdinalIgnoreCase))
			{
				if (!bSupportsX86)
				{
					throw new BuildException($"{DefaultKey} is set to {DefaultArchitecture}, but Intel is not a supported architecture");
				}
				Architectures.Add(UnrealArch.X64);
			}
			else
			{
				throw new BuildException($"Unknown {DefaultKey} value found ('{DefaultArchitecture}') in .ini");
			}

			UnrealArchitectures Result = new UnrealArchitectures(Architectures);
			lock (ProjectArchitectureCache)
			{
				// Another thread could have beaten us to the add
				ProjectArchitectureCache.TryAdd(Key, Result);
			}
			return Result;
		}
	}

	class MacPlatform : AppleBuildPlatform
	{
		public MacPlatform(UEBuildPlatformSDK InSDK, ILogger InLogger)
			: base(UnrealTargetPlatform.Mac, InSDK, new MacArchitectureConfig(), InLogger)
		{
		}

		public override void ValidateTarget(TargetRules Target)
		{
			base.ValidateTarget(Target);

				if (Target.bStaticAnalyzerIncludeGenerated)
				{
					Target.bAlwaysUseUnityForGeneratedFiles = false;
				}

			if (BuildHostPlatform.Current.Platform != UnrealTargetPlatform.Mac)
			{
				// @todo: Temporarily disable precompiled header files when building remotely due to errors
				Target.bUsePCHFiles = false;
			}

			// Mac-Arm todo - Remove this all when we feel confident no more x86-only plugins will come around
			bool bCompilingForArm = Target.Architectures.Contains(UnrealArch.Arm64);
			if (bCompilingForArm && Target.Name != "UnrealHeaderTool")
			{
				Target.DisablePlugins.AddRange(Array.Empty<string>());
			}

			if (Target.StaticAllocator == StaticAllocatorType.None)
			{
				Target.StaticAllocator = StaticAllocatorType.Binned3;
			}

			Target.GlobalDefinitions.Add("GL_SILENCE_DEPRECATION=1");

			Target.bUsePDBFiles = Target.DebugInfo != DebugInfoMode.None && ShouldCreateDebugInfo(new ReadOnlyTargetRules(Target));
			Target.bUsePDBFiles &= Target.MacPlatform.bUseDSYMFiles;

			// we always deploy - the build machines need to be able to copy the files back, which needs the full bundle
			Target.bDeployAfterCompile = true;

			Target.bCheckSystemHeadersForModification = BuildHostPlatform.Current.Platform != UnrealTargetPlatform.Mac;
		}

		public override AppleTargetRules GetAppleTargetRules(TargetRules Target)
		{
			return Target.MacPlatform;
		}

		public override ReadOnlyAppleTargetRules GetAppleTargetRules(ReadOnlyTargetRules Target)
		{
			return Target.MacPlatform;
		}

		/// <summary>
		/// Determines if the given name is a build product for a target.
		/// </summary>
		/// <param name="FileName">The name to check</param>
		/// <param name="NamePrefixes">Target or application names that may appear at the start of the build product name (eg. "UnrealEditor", "ShooterGameEditor")</param>
		/// <param name="NameSuffixes">Suffixes which may appear at the end of the build product name</param>
		/// <returns>True if the string matches the name of a build product, false otherwise</returns>
		public override bool IsBuildProduct(string FileName, string[] NamePrefixes, string[] NameSuffixes)
		{
			return base.IsBuildProduct(FileName, NamePrefixes, NameSuffixes)
				|| IsBuildProductName(FileName, NamePrefixes, NameSuffixes, ".a")
				|| IsBuildProductName(FileName, NamePrefixes, NameSuffixes, ".app");
		}

		/// <summary>
		/// Get the extensions to use for debug info for the given binary type
		/// </summary>
		/// <param name="Target">Rules for the target being built</param>
		/// <param name="InBinaryType"> The binary type being built</param>
		/// <returns>string[]    The debug info extensions (i.e. 'pdb')</returns>
		public override string[] GetDebugInfoExtensions(ReadOnlyTargetRules Target, UEBuildBinaryType InBinaryType)
		{
			switch (InBinaryType)
			{
				case UEBuildBinaryType.DynamicLinkLibrary:
				case UEBuildBinaryType.Executable:
					return Target.bUsePDBFiles ? new string[] { ".dSYM" } : Array.Empty<string>();
				case UEBuildBinaryType.StaticLibrary:
				default:
					return Array.Empty<string>();
			}
		}

		/// <summary>
		/// Modify the rules for a newly created module, where the target is a different host platform.
		/// This is not required - but allows for hiding details of a particular platform.
		/// </summary>
		/// <param name="ModuleName">The name of the module</param>
		/// <param name="Rules">The module rules</param>
		/// <param name="Target">The target being build</param>
		public override void ModifyModuleRulesForOtherPlatform(string ModuleName, ModuleRules Rules, ReadOnlyTargetRules Target)
		{
			// don't do any target platform stuff if SDK is not available
			bool bIsPlatformAvailableForTarget = UEBuildPlatform.IsPlatformAvailableForTarget(Platform, Target, bIgnoreSDKCheck: true);
			bool bIsPlatformAvailableForTargetWithSDK = UEBuildPlatform.IsPlatformAvailableForTarget(Platform, Target);

			if (!bIsPlatformAvailableForTarget)
			{
				return;
			}

			if (Target.Platform == UnrealTargetPlatform.Win64 && Target.Type == TargetType.Editor)
			{
				// because remote IOS building needs the new XcodeProject Settings to show up in the editor, we bring in the Mac bits that expose it
				if (ModuleName == "Engine")
				{
					Rules.DynamicallyLoadedModuleNames.Add("MacTargetPlatformSettings");
					if (bIsPlatformAvailableForTargetWithSDK)
					{
						Rules.DynamicallyLoadedModuleNames.AddAll("MacTargetPlatform", "MacTargetPlatformControls", "MacPlatformEditor");
					}
				}
			}
		}

		public override DirectoryReference? GetBundleDirectory(ReadOnlyTargetRules Rules, List<FileReference> OutputFiles)
		{
			if (Rules.bIsBuildingConsoleApplication)
			{
				return null;
			}
			else
			{
				return OutputFiles[0].Directory.ParentDirectory!.ParentDirectory;
			}
		}

		/// <summary>
		/// For platforms that need to output multiple files per binary (ie Android "fat" binaries)
		/// this will emit multiple paths. By default, it simply makes an array from the input
		/// </summary>
		public override List<FileReference> FinalizeBinaryPaths(FileReference BinaryName, FileReference? ProjectFile, ReadOnlyTargetRules Target)
		{
			// Xcode now builds binary outside of .app, instead Xcode will be responsible of generating .app
			List<FileReference> BinaryPaths =
			[
				BinaryName
			];
			
			return BinaryPaths;
		}

		/// <summary>
		/// Modify the rules for a newly created module, in a target that's being built for this platform.
		/// This is not required - but allows for hiding details of a particular platform.
		/// </summary>
		/// <param name="ModuleName">The name of the module</param>
		/// <param name="Rules">The module rules</param>
		/// <param name="Target">The target being build</param>
		public override void ModifyModuleRulesForActivePlatform(string ModuleName, ModuleRules Rules, ReadOnlyTargetRules Target)
		{
			bool bBuildShaderFormats = Target.bForceBuildShaderFormats;

			if (!Target.bBuildRequiresCookedData)
			{
				if (ModuleName == "TargetPlatform")
				{
					bBuildShaderFormats = true;
				}
			}

			// allow standalone tools to use target platform modules, without needing Engine
			if (ModuleName == "TargetPlatform")
			{
				if (Target.bForceBuildTargetPlatforms)
				{
					Rules.DynamicallyLoadedModuleNames.Add("MacTargetPlatform");
					Rules.DynamicallyLoadedModuleNames.Add("MacTargetPlatformSettings");
					Rules.DynamicallyLoadedModuleNames.Add("MacTargetPlatformControls");
				}

				if (bBuildShaderFormats)
				{
					// Rules.DynamicallyLoadedModuleNames.Add("ShaderFormatD3D");
					Rules.DynamicallyLoadedModuleNames.Add("ShaderFormatOpenGL");
					Rules.DynamicallyLoadedModuleNames.Add("MetalShaderFormat");
					Rules.DynamicallyLoadedModuleNames.Add("ShaderFormatVectorVM");

					Rules.DynamicallyLoadedModuleNames.Remove("VulkanRHI");
					Rules.DynamicallyLoadedModuleNames.Add("VulkanShaderFormat");
				}
			}
		}

		/// <summary>
		/// Setup the target environment for building
		/// </summary>
		/// <param name="Target">Settings for the target being compiled</param>
		/// <param name="CompileEnvironment">The compile environment for this target</param>
		/// <param name="LinkEnvironment">The link environment for this target</param>
		public override void SetUpEnvironment(ReadOnlyTargetRules Target, CppCompileEnvironment CompileEnvironment, LinkEnvironment LinkEnvironment)
		{
			base.SetUpEnvironment(Target, CompileEnvironment, LinkEnvironment);

			CompileEnvironment.Definitions.Add("PLATFORM_MAC=1");
			CompileEnvironment.Definitions.Add("UE_PLATFORM_MAC=1");

			if (Target.MacPlatform.bEnableRayTracing && Target.Type != TargetType.Server)
			{
				CompileEnvironment.Definitions.Add("RHI_RAYTRACING=1");
			}
		}

		/// <summary>
		/// Whether this platform should create debug information or not
		/// </summary>
		/// <param name="Target">The target being built</param>
		/// <returns>true if debug info should be generated, false if not</returns>
		public override bool ShouldCreateDebugInfo(ReadOnlyTargetRules Target)
		{
			// Always generate debug symbols on the build machines.
			bool IsBuildMachine = Unreal.IsBuildMachine();

			switch (Target.Configuration)
			{
				case UnrealTargetConfiguration.Development:
				case UnrealTargetConfiguration.Shipping:
				case UnrealTargetConfiguration.Test:
					return !Target.bOmitPCDebugInfoInDevelopment || IsBuildMachine;
				case UnrealTargetConfiguration.DebugGame:
				case UnrealTargetConfiguration.Debug:
				default:
					return true;
			}
		}

		/// <summary>
		/// Creates a toolchain instance for the given platform.
		/// </summary>
		/// <param name="Target">The target being built</param>
		/// <returns>New toolchain instance.</returns>
		public override UEToolChain CreateToolChain(ReadOnlyTargetRules Target)
		{
			ClangToolChainOptions Options = GetToolChainOptionsForSanitizers(Target);
			if (Target.bShouldCompileAsDLL)
			{
				Options |= ClangToolChainOptions.OutputDylib;
			}

			return new MacToolChain(Target, Options, Logger);
		}

		/// <inheritdoc/>
		public override void Deploy(TargetReceipt Receipt)
		{
			new UEDeployMac(Logger).PrepTargetForDeployment(Receipt);
		}
	}

	class MacPlatformFactory : UEBuildPlatformFactory
	{
		public override UnrealTargetPlatform TargetPlatform => UnrealTargetPlatform.Mac;

		/// <summary>
		/// Register the platform with the UEBuildPlatform class
		/// </summary>
		public override void RegisterBuildPlatforms(ILogger Logger)
		{
			ApplePlatformSDK SDK = new ApplePlatformSDK(Logger);

			// Register this build platform for Mac
			UEBuildPlatform.RegisterBuildPlatform(new MacPlatform(SDK, Logger), Logger);
			UEBuildPlatform.RegisterPlatformWithGroup(UnrealTargetPlatform.Mac, UnrealPlatformGroup.Apple);
			UEBuildPlatform.RegisterPlatformWithGroup(UnrealTargetPlatform.Mac, UnrealPlatformGroup.Desktop);
			UEBuildPlatform.RegisterPlatformWithGroup(UnrealTargetPlatform.Mac, UnrealPlatformGroup.PosixOS);
		}
	}
}
