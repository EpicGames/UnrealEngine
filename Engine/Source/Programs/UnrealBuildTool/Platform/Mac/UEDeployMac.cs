// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace UnrealBuildTool
{
	class UEDeployMac : UEBuildDeploy
	{
		public UEDeployMac(ILogger InLogger)
			: base(InLogger)
		{
		}

		public override bool PrepTargetForDeployment(TargetReceipt Receipt)
		{
			Logger.LogInformation("Deploying now!");
			return base.PrepTargetForDeployment(Receipt);
		}

		public static bool GeneratePList(string ProjectDirectory, bool bIsUnrealGame, string GameName, string ProjectName, string InEngineDir, string ExeName, TargetType TargetType)
		{
			string IntermediateDirectory = (bIsUnrealGame ? InEngineDir : ProjectDirectory) + "/Intermediate/Mac";
			string DestPListFile = IntermediateDirectory + "/" + ExeName + "-Info.plist";
			string SrcPListFile = (bIsUnrealGame ? (InEngineDir + "Source/Programs/") : (ProjectDirectory + "/Source/")) + GameName + "/Resources/Mac/Info.plist";
			if (!File.Exists(SrcPListFile))
			{
				SrcPListFile = InEngineDir + "/Source/Runtime/Launch/Resources/Mac/Info.plist";
			}

			string PListData;
			if (File.Exists(SrcPListFile))
			{
				PListData = File.ReadAllText(SrcPListFile);
			}
			else
			{
				return false;
			}

			// bundle identifier
			// plist replacements
			DirectoryReference? DirRef = bIsUnrealGame ? (!System.String.IsNullOrEmpty(UnrealBuildTool.GetRemoteIniPath()) ? new DirectoryReference(UnrealBuildTool.GetRemoteIniPath()!) : null) : new DirectoryReference(ProjectDirectory);
			ConfigHierarchy Ini = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, DirRef, UnrealTargetPlatform.IOS);

			string BundleIdentifier;
			Ini.GetString("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "BundleIdentifier", out BundleIdentifier);

			string BundleVersion = MacToolChain.LoadEngineDisplayVersion();
			// duplicating some logic in MacToolchain for the BundleID
			string[] ExeNameParts = ExeName.Split('-');
			bool bBuildingEditor = ExeNameParts[0].EndsWith("Editor", StringComparison.Ordinal);
			string FinalBundleID = bBuildingEditor ? $"com.epicgames.{ExeNameParts[0]}" : BundleIdentifier.Replace("[PROJECT_NAME]", ProjectName, StringComparison.Ordinal);
			FinalBundleID = FinalBundleID.Replace("_", "", StringComparison.Ordinal);

			PListData = PListData.Replace("${EXECUTABLE_NAME}", ExeName, StringComparison.Ordinal).
				Replace("${APP_NAME}", FinalBundleID, StringComparison.Ordinal).
				Replace("${ICON_NAME}", GameName, StringComparison.Ordinal).
				Replace("${MACOSX_DEPLOYMENT_TARGET}", MacToolChainSettings.MinMacDeploymentVersion(TargetType), StringComparison.Ordinal).
				Replace("${BUNDLE_VERSION}", BundleVersion, StringComparison.Ordinal);

			if (!Directory.Exists(IntermediateDirectory))
			{
				Directory.CreateDirectory(IntermediateDirectory);
			}
			File.WriteAllText(DestPListFile, PListData);

			return true;
		}
	}
}
