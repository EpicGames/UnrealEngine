// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using Microsoft.Extensions.Logging;
using System;
using System.IO;
using System.Linq;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// Helper functionality for easier F5 debugging for blueprint-only projects
	/// </summary>
	public static class GDKContentOnlyProjectSupport
	{
		private class Config
		{
			/// <summary>
			/// Path to the .uproject file to use when F5 debugging a content-only project via the UE5 engine project
			/// </summary>
			[XmlConfigFile(Category = "GDKPlatform")]
			public string ContentOnlyDebugProject = "";

			public Config()
			{
				XmlConfig.ApplyTo(this);
			}
		}

		/// <summary>
		/// Get the configured content-only debug project, if any
		/// </summary>
		public static string ContentOnlyDebugProject
		{
			get
			{
				Config Config = new();
				return Config.ContentOnlyDebugProject;
			}
		}

		/// <summary>
		/// Helper function to get the content-only debug project for the given target, if applicable and supported
		/// </summary>
		private static string? GetProjectFileFromTargetInternal(ReadOnlyTargetRules Target, ILogger Logger)
		{
			string ProjectFileName = ContentOnlyDebugProject;

			// early out if this isn't required
			if (Target.ProjectFile != null || Target.Type == TargetType.Program || String.IsNullOrEmpty(ProjectFileName) || !Environment.CommandLine.Contains("-frommsbuild", StringComparison.OrdinalIgnoreCase))
			{
				return null;
			}

			// find & verify the uproject name
			if (String.IsNullOrEmpty(Path.GetExtension(ProjectFileName)))
			{
				ProjectFileName += ".uproject";
			}
			if (String.IsNullOrEmpty(Path.GetDirectoryName(ProjectFileName)))
			{
				FileReference? FoundProject = NativeProjects.EnumerateProjectFiles(Logger).FirstOrDefault(X => X.GetFileName().Equals(ProjectFileName, StringComparison.OrdinalIgnoreCase));
				if (FoundProject != null)
				{
					ProjectFileName = FoundProject.FullName;
				}
			}
			else if (!Path.IsPathRooted(ProjectFileName))
			{
				ProjectFileName = Path.GetFullPath(ProjectFileName, Unreal.RootDirectory.FullName);
			}
			if (!File.Exists(ProjectFileName))
			{
				throw new BuildException($"ContentOnlyDebugProject not found: {ProjectFileName}");
			}

			return ProjectFileName;
		}

		/// <summary>
		/// Stores the path to the content-only debug project in the target receipt
		/// </summary>
		public static bool ModifyTargetReceipt(ReadOnlyTargetRules Target, TargetReceipt Receipt, ILogger Logger)
		{
			string? ProjectFileName = GetProjectFileFromTargetInternal(Target, Logger);
			if (ProjectFileName == null)
			{
				return false;
			}

			// add the receipt property
			Receipt.AdditionalProperties.Add(new ReceiptProperty("ContentOnlyDebugProject", ProjectFileName));
			Logger.LogInformation("*** GDK F5 debugging for content-only projects is enabled via ContentOnlyDebugProject. Using project {ProjectFileName} ***", ProjectFileName);
			return true;
		}

		/// <summary>
		/// Retrieves the path to the project from the given target receipt
		/// </summary>
		public static FileReference? GetProjectFileFromTargetReceipt(TargetReceipt Receipt, ILogger Logger)
		{
			FileReference? ProjectFile = Receipt.ProjectFile;

			// No project file was specified - this might be the UE5 project & someone wants to F5 debug it.
			// We need to know the project so the MicrosoftGame.config can be generated with the correct package data
			if (ProjectFile == null && Receipt.TargetType != TargetType.Program && Environment.CommandLine.Contains("-frommsbuild", StringComparison.OrdinalIgnoreCase))
			{
				string? ProjectFilePath = Receipt.AdditionalProperties.FirstOrDefault(X => X.Name == "ContentOnlyDebugProject")?.Value;
				ProjectFile = GetVerifiedProjectFile(ProjectFilePath, Logger, true);
				if (ProjectFile == null)
				{
					return null;
				}
			}

			return ProjectFile;
		}

		private static FileReference? GetVerifiedProjectFile(string? ProjectFilePath, ILogger Logger, bool bUseLog)
		{
			FileReference? ProjectFile = null;

			if (ProjectFilePath == null)
			{
				if (bUseLog)
				{
					// ideally this would be a build failure for F5 debugging - unfortunately this same code is run via UnrealVS batch build and we can't distingish it
					Logger.LogError("{BuildConfigXmlFileName}: warning : *** No project specified for the engine solution - this is required for F5 debugging so that the correct MicrosoftGame.config can be generated. Missing <ContentOnlyDebugProject> *** \nPlease add this to your BuildConfiguration.xml:\n\n<GDKPlatform>\n\t<ContentOnlyDebugProject>Path/To/Project.uproject</ContentOnlyDebugProject>\n</GDKPlatform>\n", GetBuildConfigXmlFileName());
				}
			}
			else
			{
				if (!Path.IsPathRooted(ProjectFilePath))
				{
					ProjectFilePath = Path.Combine(Unreal.RootDirectory.FullName, ProjectFilePath);
				}

				ProjectFile = new FileReference(ProjectFilePath);
				if (!FileReference.Exists(ProjectFile))
				{
					throw new BuildException($"*** Cannot find ContentOnlyDebugProject {ProjectFile} ***");
				}

				// see if this project has a temporary target.cs file
				DirectoryReference ProjectDir = ProjectFile.Directory;
				DirectoryReference TempSourceDir = DirectoryReference.Combine(ProjectDir, "Intermediate", "Source");
				if (DirectoryReference.Exists(TempSourceDir) && DirectoryReference.EnumerateFiles(TempSourceDir, "*.target.cs", SearchOption.TopDirectoryOnly).Any())
				{
					throw new BuildException($"{GetBuildConfigXmlFileName()}: error : *** Content-only project {ProjectFile.GetFileName()} has a temporary target.cs. You need to include temp targets in your solution and debug {ProjectFile.GetFileNameWithoutAnyExtensions()} directly.\nPlease add this to your BuildConfiguration.xml and regenerate project files: ***\n<ProjectFileGenerator>\n\t<bIncludeTempTargets>true</bIncludeTempTargets>\n</ProjectFileGenerator>\n");
				}
			}

			return ProjectFile;
		}

		private static string GetBuildConfigXmlFileName()
		{
			// find a BuildConfiguration.xml, favoring the global appdata one
			DirectoryReference? AppDataFolder = DirectoryReference.GetSpecialFolder(Environment.SpecialFolder.ApplicationData);
			XmlConfig.InputFile? BuildConfig = XmlConfig.InputFiles.FirstOrDefault(X => AppDataFolder != null && X.Location.IsUnderDirectory(AppDataFolder)) ?? XmlConfig.InputFiles.FirstOrDefault();

			// ideally we have a path to a usable BuildConfiguration.xml, but any 'file name' will do; it will get shown in VS's Error List window.
			return BuildConfig?.Location.FullName ?? "BuildConfiguration.xml";
		}

		/// <summary>
		/// Retrieves the path to the project from the given target
		/// </summary>
		public static FileReference? GetProjectFileFromTarget(ReadOnlyTargetRules Target, ILogger Logger, bool bUseLog = true)
		{
			FileReference? ProjectFile = Target.ProjectFile;

			// No project file was specified - this might be the UE5 project & someone wants to F5 debug it.
			// We need to know the project so the MicrosoftGame.config can be generated with the correct package data
			if (ProjectFile == null && Target.Type != TargetType.Program && Environment.CommandLine.Contains("-frommsbuild", StringComparison.OrdinalIgnoreCase))
			{
				string? ProjectFilePath = GetProjectFileFromTargetInternal(Target, Logger);
				ProjectFile = GetVerifiedProjectFile(ProjectFilePath, Logger, bUseLog);
				if (ProjectFile == null)
				{
					return null;
				}

				if (bUseLog)
				{
					Logger.LogInformation("*** GDK F5 debugging for content-only projects is enabled via ContentOnlyDebugProject. Using project {ProjectFileName} ***", ProjectFilePath);
				}
			}

			return ProjectFile;
		}
	}
}
