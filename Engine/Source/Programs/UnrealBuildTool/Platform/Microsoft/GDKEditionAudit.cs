// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using Microsoft.Extensions.Logging;
using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;

namespace UnrealBuildTool
{
	/// <summary>
	/// Helper functionality for optionally auditing the GDK edition on any used lib and dll files
	/// </summary>
	public static class GDKEditionAudit
	{
		private class Config
		{
			/// <summary>
			/// Whether to verify that any libs the application links to are using the same GDK edition
			/// </summary>
			[XmlConfigFile(Category = "GDKPlatform")]
			public bool bVerifyLibGDKEditions = false;

			/// <summary>
			/// Whether to verify that any DLLs the application links to are using the same GDK edition
			/// </summary>
			[XmlConfigFile(Category = "GDKPlatform")]
			public bool bVerifyDLLGDKEditions = false;

			public Config()
			{
				XmlConfig.ApplyTo(this);

				bVerifyLibGDKEditions |= Environment.GetCommandLineArgs().Contains("-VerifyLibGDKEditions", StringComparer.InvariantCultureIgnoreCase);
				bVerifyDLLGDKEditions |= Environment.GetCommandLineArgs().Contains("-VerifyDLLGDKEditions", StringComparer.InvariantCultureIgnoreCase);
			}
		};

		/// <summary>
		/// Submission validator xml report can suggest multiple GDK editions for a single executable - the additional ones come from the libs that are linked to.
		/// This function checks all the libs we are planning to link against and warns if they were built against a different GDK
		/// </summary>
		internal static void VerifyLinkEnvironment(VCEnvironment EnvVars, LinkEnvironment LinkEnvironment, bool bBuildImportLibraryOnly, ILogger Logger)
		{
			Config Config = new();
			if (LinkEnvironment.bIsBuildingLibrary || bBuildImportLibraryOnly || !Config.bVerifyLibGDKEditions)
			{
				return;
			}

			Logger.LogInformation("Verifying GDK editions from linked libs...");

			// locate dumpbin
			string DumpBinPath = DirectoryReference.Combine(EnvVars.CompilerDir, "bin", "Hostx64", "x64", "dumpbin.exe").ToString();
			if (!File.Exists(DumpBinPath))
			{
				Logger.LogWarning("Cannot verify the GDK edition of any libs because dumpbin cannot be found at {DumpBinPath}", DumpBinPath);
				return;
			}

			// collect as many libraries as we can
			List<FileReference> AllLibraries = new();
			foreach (FileReference Library in LinkEnvironment.Libraries)
			{
				if (FileReference.Exists(Library))
				{
					AllLibraries.Add(Library);
				}
			}
			foreach (string SystemLibrary in LinkEnvironment.SystemLibraries)
			{
				FileReference Library = new(SystemLibrary);
				if (FileReference.Exists(Library))
				{
					AllLibraries.Add(Library);
				}
				else
				{
					foreach (DirectoryReference SystemLibraryDir in LinkEnvironment.SystemLibraryPaths)
					{
						Library = FileReference.Combine(SystemLibraryDir, SystemLibrary);
						if (FileReference.Exists(Library))
						{
							AllLibraries.Add(Library);
							break;
						}
					}
				}
			}

			// check the GDK edition for all libraries and log out any libraries that have a different GDK edition (if they have one)
			string? CurrentGDKEdition = UEBuildPlatformSDK.GetSDKForPlatform(LinkEnvironment.Platform.ToString())!.GetInstalledVersion();
			foreach (FileReference Library in AllLibraries)
			{
				// skip libs that should be excluded
				if (LinkEnvironment.ExcludedLibraries.Contains(Library.GetFileNameWithoutExtension()))
				{
					continue;
				}

				// run DUMPBIN /SYMBOLS <lib> and parse the output for a _XBLD_GRDK_EDITION_* declaration
				string StdOutString = Utils.RunLocalProcessAndReturnStdOut(DumpBinPath, String.Format("/symbols \"{0}\"", Library.FullName), null, out int ExitCode);
				if (ExitCode != 0)
				{
					Logger.LogError("{Error}", StdOutString);
					continue;
				}
				string[] Results = StdOutString.Split(new char[] { '\r', '\n' }, StringSplitOptions.RemoveEmptyEntries);
				string? GDKEdition = System.Linq.Enumerable.FirstOrDefault(Results, s => s.Contains("_XBLD_GRDK_EDITION_", StringComparison.Ordinal));
				if (GDKEdition != null)
				{
					GDKEdition = GDKEdition.Substring(GDKEdition.Length - 6, 6); //take last 6 characters   _XBLD_GRDK_EDITION_YYMMQQ -> YYMMQQ
					if (GDKEdition != CurrentGDKEdition)
					{
						Logger.LogWarning("Library {LibraryFullName} was built against GDK edition {GDKEdition}. The current GDK edition is {CurrentGDKEdition}.", Library.FullName, GDKEdition, CurrentGDKEdition);
					}
				}
			}
		}

		/// <summary>
		/// Optionally verify that any DLLs the application links to are using the same GDK edition
		/// </summary>
		internal static void VerifyRuntimeDependencies(VCEnvironment EnvVars, UnrealTargetPlatform Platform, IEnumerable<FileReference> RuntimeFileDependencies, ILogger Logger)
		{
			Config Config = new();
			if (!Config.bVerifyDLLGDKEditions)
			{
				return;
			}
			Logger.LogInformation("Verifying GDK editions from DLL dependencies...");

			// locate dumpbin
			string DumpBinPath = DirectoryReference.Combine(EnvVars.CompilerDir, "bin", "Hostx64", "x64", "dumpbin.exe").ToString();
			if (!File.Exists(DumpBinPath))
			{
				Logger.LogWarning("Cannot verify the GDK edition of any DLLs because dumpbin cannot be found at {DumpBinPath}", DumpBinPath);
				return;
			}

			// iterate all DLLs
			string? CurrentGDKEdition = UEBuildPlatformSDK.GetSDKForPlatform(Platform.ToString())!.GetInstalledVersion();
			foreach (FileReference FileDependency in RuntimeFileDependencies)
			{
				// skip anything that's not a DLL
				if (!FileDependency.HasExtension("dll"))
				{
					continue;
				}

				try
				{
					// run DUMPBIN /SECTION:.xbld /RAWDATA:1,10000 to get the .xbld section contents. The 10000 hopefully means the entire block will be printed one one line so it's easier to parse
					string StdOutString = Utils.RunLocalProcessAndReturnStdOut(DumpBinPath, String.Format($"/NOLOGO /SECTION:.xbld /RAWDATA:1,10000 \"{FileDependency.FullName}\""), null, out int ExitCode);
					if (ExitCode != 0)
					{
						Logger.LogError("{Error}", StdOutString);
						continue;
					}
					string[] Results = StdOutString.Split(new char[] { '\r', '\n' }, StringSplitOptions.RemoveEmptyEntries);

					// parse the result.
					int StartIndex = Results.FindIndex(s => s.StartsWith("RAW DATA #", StringComparison.Ordinal));                                                        // Step 1: find the start of the raw data
					if (StartIndex != -1 && StartIndex < Results.Length)
					{
						string[] RawData = Results[StartIndex + 1].Split(new char[] { ':' }, StringSplitOptions.RemoveEmptyEntries);                                        // Step 2: raw data is formatted <base address>:<hex data>  - separate it
						if (RawData.Length == 2)
						{
							byte[] SectionBytes = RawData[1].Split(new char[] { ' ' }, StringSplitOptions.RemoveEmptyEntries).Select(s => Convert.ToByte(s, 16)).ToArray(); // Step 3: get each byte of the hex data
							string[] XBLDStrings = System.Text.Encoding.ASCII.GetString(SectionBytes).Split(new char[] { '\0' }, StringSplitOptions.RemoveEmptyEntries);       // Step 4: convert the bytes into string table. The bytes contain a series of null-terminated strings

							// find the string we're interested in
							List<string> FoundGDKEditions = new();
							foreach (string XBLDString in XBLDStrings)
							{
								if (XBLDString.StartsWith("_xbld_edition_sdktype=", StringComparison.Ordinal))
								{
									string GDKEdition = XBLDString.Substring(XBLDString.Length - 6, 6); // check last 6 characters - YYMMQQ
									if (GDKEdition != CurrentGDKEdition && !FoundGDKEditions.Contains(GDKEdition))
									{
										FoundGDKEditions.Add(GDKEdition);
										Logger.LogWarning("Runtime dependency {FileDependency} was built against GDK edition {GDKEdition}. The current GDK edition is {CurrentGDKEdition}", FileDependency, GDKEdition, CurrentGDKEdition);
									}
								}
							}
						}
					}
				}
				catch (Exception)
				{
				}
			}
		}
	}
}
