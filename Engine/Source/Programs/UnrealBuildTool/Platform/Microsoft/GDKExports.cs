// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Linq;
using EpicGames.Core;

namespace UnrealBuildTool
{
	/// <summary>
	/// Public GDK functions exposed to UAT
	/// </summary>
	public static class GDKExports
	{
		/// <summary>
		/// Returns the top-level root directory of the GameCore SDK
		/// </summary>
		/// <returns></returns>
		public static string GetGSDKRoot()
		{
			return Utils.CleanDirectorySeparators(Environment.GetEnvironmentVariable("GameDK") ?? "");
		}

		/// <summary>
		/// Returns the root directory for the current version of the GameCore SDK
		/// </summary>
		/// <returns></returns>
		public static string GetCurrentGSDKDir()
		{
			string GameDKCoreLatest = Utils.CleanDirectorySeparators(Environment.GetEnvironmentVariable("GameDKCoreLatest") ?? "");
			if (bFavorModernFolderStructure && !String.IsNullOrEmpty(GameDKCoreLatest))
			{
				return GameDKCoreLatest;
			}

			string GameDKLatest = Utils.CleanDirectorySeparators(Environment.GetEnvironmentVariable("GameDKLatest") ?? "");
			if (!String.IsNullOrEmpty(GameDKLatest))
			{
				return GameDKLatest;
			}

			return GameDKCoreLatest;
		}

		/// <summary>
		/// Returns whether the current GRDK version is a valid supported version
		/// </summary>
		/// <returns></returns>
		public static bool IsGRDKEditionValid()
		{
			return GDKVersionValid.Value;
		}

		/// <summary>
		/// Returns whether the current GDK uses the legacy folder structure (pre-October 2025)
		/// </summary>
		/// <returns></returns>
		public static bool IsLegacyFolderStructure()
		{
			return GDKLegacyFolderStructure.Value;
		}

		/// <summary>
		/// Returns the SDK binaries directory
		/// </summary>
		public static string GetSDKBinariesFolder()
		{
			string GDKBinDirectory = Path.Combine(GetGSDKRoot(), "bin\\");
			return GDKBinDirectory;
		}

		/// <summary>
		/// Returns the directory where GDK Extension SDKs are located. Only returns valid paths for SDKs using the legacy folder structure (pre-October 2025)
		/// </summary>
		/// <param name="ExtensionName"></param>
		/// <param name="bIsForRedist"></param>
		/// <returns></returns>
		public static string GetExtensionDirectory(string ExtensionName, bool bIsForRedist = false)
		{
			if (IsGRDKEditionValid())
			{
				string BaseGDKExtensionPath = Path.Combine(GetCurrentGSDKDir(), "GRDK", "ExtensionLibraries");

				if (GetGDKVersionNumber() >= 241000)
				{
					if (bIsForRedist)
					{
						return Path.Combine(BaseGDKExtensionPath, ExtensionName, "Redist", "x64");
					}
					else
					{
						return Path.Combine(BaseGDKExtensionPath, ExtensionName);
					}
				}
				else
				{
					return Path.Combine(BaseGDKExtensionPath, ExtensionName, bIsForRedist ? "Redist" : "DesignTime", "CommonConfiguration", "Neutral");
				}
			}
			else
			{
				return "";
			}
		}

		/// <summary>
		/// Returns the installed version of the GDK
		/// </summary>
		/// <returns></returns>
		public static string? GetSDKVersion()
		{
			DirectoryReference? CurrentGDKDir = DirectoryReference.FromString(GetCurrentGSDKDir());
			return CurrentGDKDir?.GetDirectoryName();
		}

		/// <summary>
		/// Returns the installed version number of the GDK
		/// </summary>
		/// <returns></returns>
		public static int? GetGDKVersionNumber()
		{
			string? GDKVersion = GetSDKVersion();
			if (GDKVersion != null && Int32.TryParse(GDKVersion, out int GDKVersionInt))
			{
				return GDKVersionInt;
			}

			return null;
		}

		/// <summary>
		/// Returns whether the GDK supports the given architecture
		/// </summary>
		public static bool IsArchitectureSupported(UnrealArch Arch)
		{
			// X64 always supported
			if (Arch == UnrealArch.X64 || Arch == UnrealArch.Arm64ec)
			{
				return true;
			}

			// Native ARM64 runtime is only supported in the April 2026 GDK onwards, with the modern folder structure
			if (Arch == UnrealArch.Arm64 && GetGDKVersionNumber() >= 260400 && !IsLegacyFolderStructure())
			{
				return true;
			}

			return false;
		}

		/// <summary>
		/// (Experimental and unsupported) Modifies the given target rules so that it will produce a build that is compatible with PC GDK. Expected for Win64 TargetInfo only.
		///
		/// If using bUseCustomConfig=true, it is necessary to add the following entries to {project dir}/Config/Windows/Custom/MSGameStore/WindowsEngine.ini
		/// Otherwise, when bUseCustomConfig=false, add these to {project dir}/Config/Windows/WindowsEngine.ini
		/// 
		/// <code>
		/// ; (required) this is required to make BuildCookRun generate, deploy and launch msixvc packages
		/// [/Script/WindowsTargetPlatform.WindowsTargetSettings]
		/// CustomDeployment=MSGameStore
		/// 
		/// ; (recommended) add this if you are expecting to do intelligent delivery and use on-demand chunks etc
		/// [StreamingInstall]
		/// DefaultProviderName=GDKPackageChunkInstall
		/// 
		/// ; (recommended) add this if you are planning on using Xbox cross-platform save games
		/// [PlatformFeatures]
		/// SaveGameSystemModule=GDKSaveGameSystem
		/// 
		/// ; (optional) set this if you are using EOSSDK
		/// [/Script/MSGamingSupport.MSGamingSettings]
		/// +AllowedMissingSymbolFiles="EOSSDK-Win64-Shipping.DLL"	
		///		
		/// ; (optional) set this if you use bWithOSS=true
		/// [OnlineSubsystem]
		/// DefaultPlatformService=GDK
		/// 
		/// ; (optional) only need the following sections if you use bWithPlayFab=true
		/// [/Script/Engine.Engine]
		/// !NetDriverDefinitions=ClearArray
		/// +NetDriverDefinitions=(DefName="GameNetDriver",DriverClassName="/Script/PlayFabParty.PlayFabPartyNetDriver",DriverClassNameFallback="/Script/PlayFabParty.PlayFabPartyNetDriver")
		/// +NetDriverDefinitions=(DefName="BeaconNetDriver",DriverClassName="/Script/PlayFabParty.PlayFabPartyNetDriver",DriverClassNameFallback="/Script/PlayFabParty.PlayFabPartyNetDriver")
		/// +NetDriverDefinitions=(DefName="DemoNetDriver",DriverClassName="/Script/Engine.DemoNetDriver",DriverClassNameFallback="/Script/Engine.DemoNetDriver")
		/// [PlayFab]
		/// AppId={Your Hex PlayFab Title ID String Here}
		/// </code>
		///
		///
		/// If using bUseCustomConfig=true, it is necessary to add the following entries to {project dir}/Config/Windows/Custom/MSGameStore/WindowsGame.ini:
		/// Othewreise, when bUseCustomConfig=false, add these to {project dir}/Config/Windows/WindowsGame.ini
		/// <code>
		/// ; (required) this is necessary to stage the game for Win64 because GDK is a restricted platform folder
		/// [Staging]
		/// +RemapDirectories=(From="Engine/Platforms/GDK", To="Engine")
		/// +RemapDirectories=(From="{project dir}/Platforms/GDK", To="{project dir}")
		/// </code>
		///
		/// </summary>
		/// <param name="TargetRules"></param>
		/// <param name="Target"></param>
		/// <param name="bUseCustomConfig">Whether you have put all MSGameStore config settings in Config/Custom/MSGameStore/*.ini - typically done with a custom MSG build target MyGameMSGTarget : MyGameTarget </param>
		/// <param name="bWithOSS">Whether to include the OnlineSubsystemGDK plugin. Requires additional config settings</param>
		/// <param name="bWithPlayFab">Whether to include the PlayFab plugin. Requires additional config settings</param>
		/// <param name="bWithXCurl">Whether to use XCurl instead of libCurl</param>
		public static void MakeMSGamingRuntimeTarget(TargetRules TargetRules, TargetInfo Target, bool bUseCustomConfig = true, bool bWithOSS = true, bool bWithPlayFab = false, bool bWithXCurl = false)
		{
			if (Target.Platform == UnrealTargetPlatform.Win64)
			{
				// Add override directory for Microsoft Game Store config files. This allows you to keep the MSGameStore specific configuration separate (such as DefaultPlatformService for OSS etc)
				if (bUseCustomConfig)
				{
					TargetRules.BuildEnvironment = TargetBuildEnvironment.Unique;
					TargetRules.CustomConfig = "MSGameStore";
				}

				// Add override for the toolchain. The main purpose of this is to enable automatic MicrosoftGame.config generation when F5 debugging in Visual Studio
				TargetRules.ToolChainName = "MSGamingToolChain";

				// Configure the platform for GDK
				TargetRules.WindowsPlatform.WindowsSdkVersion = "10.0.19041.0";   // GRDK requires a minimum of this windows sdk version
				TargetRules.WindowsPlatform.bUseBundledDbgHelp = false;           // not necessary as Win 10 minimum version is at least 1903
				//TargetRules.WindowsPlatform.TargetWindowsVersion = 0x0A00;      // GDK only runs on Win 10 (@todo: Not enabling this at the moment due to compile issues with WindowsMemoryTrace.cpp & VirtualAlloc2 bugs

				TargetRules.GlobalDefinitions.Add("WINDOWS_USE_WER=1");           // this will mean that exceptions are caught by Windows Error Reporting and crash callstacks uploaded to Partner Center
				TargetRules.GlobalDefinitions.Add("_GAMING_DESKTOP");             // this isn't checked in many places, so it's probably not necessary

				TargetRules.EnablePlugins.Add("MSGamingRuntime");                 // this includes all of the required GDK runtime modules

				if (bWithPlayFab)
				{
					TargetRules.EnablePlugins.Add("PlayFabParty");
				}
				if (bWithOSS)
				{
					TargetRules.EnablePlugins.Add("OnlineSubsystemGDK");
				}
				if (bWithXCurl)
				{
					TargetRules.WindowsPlatform.bUseXCurl = true;				
				}
			}
		}

		const bool bFavorModernFolderStructure = true; // choose whether to use the modern folder structure if we find it

		private static readonly Lazy<bool> GDKLegacyFolderStructure = new(() =>
		{
			string? GameDKCoreLatest = Environment.GetEnvironmentVariable("GameDKCoreLatest");
			string? GameDKLatest = Environment.GetEnvironmentVariable("GameDKLatest");
			if (!String.IsNullOrEmpty(GameDKCoreLatest) || !String.IsNullOrEmpty(GameDKLatest))
			{
				bool bHasModernFolderStructure = false;
				if (!String.IsNullOrEmpty(GameDKCoreLatest))
				{
					string ModernGRDKFolder = Path.Combine(GameDKCoreLatest, "windows");
					bHasModernFolderStructure = (ModernGRDKFolder != null && Directory.Exists(ModernGRDKFolder));
				}

				bool bHasLegacyFolderStructure = false;
				if (!String.IsNullOrEmpty(GameDKLatest))
				{
					string LegacyGDKFolder = Path.Combine(GameDKLatest, "GRDK");
					bHasLegacyFolderStructure = (LegacyGDKFolder != null && Directory.Exists(LegacyGDKFolder));
				}

				if (bHasModernFolderStructure && bFavorModernFolderStructure)
				{
					return false;
				}

				return bHasLegacyFolderStructure;
			}

			return true;
		});

		private static readonly Lazy<bool> GDKVersionValid = new(() =>
		{
			DirectoryReference? CurrentGDKDir = DirectoryReference.FromString(GetCurrentGSDKDir());
			if (CurrentGDKDir != null)
			{
				if (Int32.TryParse( CurrentGDKDir!.GetDirectoryName(), out int GDKEdition))
				{
					foreach ( UnrealTargetPlatform Platform in UnrealTargetPlatform.GetValidPlatforms().Where( X => X.IsInGroup(UnrealPlatformGroup.Microsoft)))
					{
						UEBuildPlatformSDK? PlatformSDK = UEBuildPlatformSDK.GetSDKForPlatform(Platform.ToString());
						if (PlatformSDK != null)
						{
							string? MinVersion = PlatformSDK.GetVersionFromConfig("MinGDKVersion");
							string? MaxVersion = PlatformSDK.GetVersionFromConfig("MaxGDKVersion");
							if (Int32.TryParse(MinVersion, out int MinGDKEdition) && Int32.TryParse(MaxVersion, out int MaxGDKEdition))
							{
								if (GDKEdition >= MinGDKEdition && GDKEdition <= MaxGDKEdition)
								{
									return true;
								}
								break;
							}
						}
					}
				}
			}

			return false;
		});
	}
}
