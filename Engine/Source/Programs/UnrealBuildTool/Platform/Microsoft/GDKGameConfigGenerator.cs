// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using Microsoft.Extensions.Logging;
using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Threading.Tasks;
using System.Xml.Linq;
using UnrealBuildBase;

namespace UnrealBuildTool.GDK
{
	/// <summary>
	/// GDK-specific appx manifest generator (MicrosoftGame.config)
	/// </summary>
	public abstract class GDKGameConfigGenerator : AppXManifestGeneratorBase
	{

		/// Platform-specific target device family
		protected virtual string? TargetDeviceFamily => null;

		/// Platform-specific architecture
		protected virtual string DefaultArchitecture => "neutral";

		/// DLC plugin file to use, if any
		protected FileReference? DLCFile;

		/// DLC package name (when generating DLC alongside the main package)
		protected string? DLCPackage;

		/// Cached properties for the current DLC
		protected Dictionary<string, string>? DLCConfig;

		/// Helper to check if we are dealing with DLC
		protected bool IsDLC => (DLCFile != null) || (DLCPackage != null);

		/// Whether we have a bootstrap exe (Windows only) - affects dependencies
		protected bool NoBootstrapExe = false;

		/// Curent GDKEdition
		protected static int GDKEdition => GDKExports.GetGDKVersionNumber() ?? 0;

		/// <summary>
		/// Create a manifest generator for the given platform variant.
		/// </summary>
		protected GDKGameConfigGenerator(UnrealTargetPlatform InPlatform, ILogger InLogger)
			: base(InPlatform, InLogger)
		{
		}

		/// <summary>
		/// Create a manifest for the given platform and return the list of modified files
		/// </summary>
		public static List<string>? Create(UnrealTargetPlatform Platform, ILogger Logger, DirectoryReference InOutputPath, string? InTargetName, FileReference? InProjectFile, IEnumerable<AppXManifestExecutable> InExecutables, bool bDeleteIntermediates = false, FileReference? DLCFile = null, string? DLCPackage = null, string CustomConfig = "", bool NoBootstrapExe = true)
		{
			GDKGameConfigGenerator Generator = GDKGameConfigGeneratorFactory.CreateGenerator(Platform, Logger);
			Generator.DLCFile = DLCFile;
			Generator.DLCPackage = DLCPackage;
			Generator.CustomConfig = CustomConfig;
			Generator.NoBootstrapExe = NoBootstrapExe;

			List<string>? ModifiedFiles = Generator.CreateManifest("MicrosoftGame.config", InOutputPath, InTargetName, InProjectFile, InExecutables);

			return ModifiedFiles;
		}

		/// <summary>
		/// Get the Game element
		/// </summary>
		protected virtual XElement GetGame(IEnumerable<AppXManifestExecutable> InExecutables, out string IdentityName)
		{
			int ManifestVersion = GetManifestVersion();

			XElement Game = new(XName.Get("Game"),
				new XAttribute("configVersion", ManifestVersion.ToString()),
				GetIdentity(out IdentityName),
				GetShellVisuals(),
				GetResources()
				);

			if (!IsDLC)
			{
				Game.Add(
					GetExecutableList(InExecutables),
					GetProtocolList(InExecutables)
				);

				string? TitleId = GetConfigString("TitleId", null, null);
				if (TitleId != null && TitleId.Length > 0)
				{
					Game.Add(new XElement(XName.Get("TitleId"), TitleId));
				}

				string? MSAAppId = GetConfigString("MSAAppId", null, null);
				if (MSAAppId != null && MSAAppId.Length > 0)
				{
					Game.Add(new XElement(XName.Get("MSAAppId"), MSAAppId));
				}

				string? StoreId = GetConfigString("StoreId", null, null);
				if (StoreId != null && StoreId.Length > 0)
				{
					Game.Add(new XElement(XName.Get("StoreId"), StoreId));
				}

				// The latest guidance from Microsoft says this flag should always be included in the manifest even if it's false, and most titles should keep it false.
				bool bRequiresXboxLive = GetConfigBool("bRequireXboxLive", null);
				Game.Add(new XElement(XName.Get("RequiresXboxLive"), bRequiresXboxLive));

				bool bUseSimplifiedUserModel = GetConfigBool("bUseSimplifiedUserModel", null);
				if (bUseSimplifiedUserModel)
				{
					Game.Add(new XElement(XName.Get("AdvancedUserModel"), "false"));
				}
			}
			else
			{
				if (DLCConfig!.TryGetValue("StoreId", out string? StoreId) && !String.IsNullOrEmpty(StoreId))
				{
					Game.Add(new XElement(XName.Get("StoreId"), StoreId));
				}

				string? OwnerStoreId = GetConfigString("StoreId", null, null);
				if (OwnerStoreId != null && OwnerStoreId.Length > 0)
				{
					Game.Add(new XElement(XName.Get("AllowedProducts"),
						new XElement(XName.Get("AllowedProduct"), OwnerStoreId)));
				}

				Game.Add(new XElement(XName.Get("TargetDeviceFamilyForDLC"), TargetDeviceFamily));
			}

			return Game;
		}

		/// <summary>
		/// Get the executable list element
		/// </summary>
		protected XElement GetExecutableList(IEnumerable<AppXManifestExecutable> InExecutables)
		{
			XElement ExecutableList = new(XName.Get("ExecutableList"));

			if (!InExecutables.Any())
			{
				return ExecutableList;
			}

			// Need exactly one exe not marked as dev only in manifest.  Most optimized exe gets non-dev.  Dependent on UnrealTargetConfiguration being ordered by optimization level.
			UnrealTargetConfiguration MaxTargetConfiguration = InExecutables.MaxBy( X => X.Configuration ).Configuration;
			foreach (AppXManifestExecutable Executable in InExecutables)
			{
				XElement ExecutableElement = GetExecutable(Executable, Executable.Configuration != MaxTargetConfiguration);
				ExecutableList.Add(ExecutableElement);
			}

			return ExecutableList;
		}

		private string GetAppId(UnrealTargetConfiguration InTargetConfig, UnrealArch? InArchitecture)
		{
			bool bIsDevBuild = (InTargetConfig != UnrealTargetConfiguration.Shipping);
			bool bSkipAppIdSuffixForShippingConfig = GetConfigBool("bSkipAppIdSuffixForShippingConfig", null, false);
			bool bIncludeConfigSuffix = bIsDevBuild || !bSkipAppIdSuffixForShippingConfig;

			string? PackageBaseName = ValidateProjectBaseName(ReadIniString("ProjectName", IniSection_GeneralProjectSettings, "UEGame"));
			string AppId = "App" + PackageBaseName + (bIncludeConfigSuffix ? InTargetConfig.ToString() : String.Empty);
			if (InArchitecture != null && !InArchitecture!.Value.bIsX64)
			{
				AppId += "ARM64";
			}

			return AppId;
		}

		/// <summary>
		/// Get the executable element for the given configuration
		/// </summary>
		protected XElement GetExecutable(AppXManifestExecutable Executable, bool IsDevOnly)
		{
			bool bIsDevBuild = (Executable.Configuration != UnrealTargetConfiguration.Shipping);

			string RelativeExePath;
			if (Path.IsPathRooted(Executable.ExecutablePath))
			{
				RelativeExePath = Utils.MakePathRelativeTo(Executable.ExecutablePath, Path.Combine(Executable.ExecutablePath, "../../../.."));
			}
			else
			{
				RelativeExePath = Executable.ExecutablePath;
			}

			string AppId = GetAppId(Executable.Configuration, Executable.Architecture);

			XElement ExecutableElement = new(XName.Get("Executable"),
				new XAttribute("Name", RelativeExePath),
				new XAttribute("Id", AppId));

			string DisplayNameDefault = ReadIniString(IniSection_GeneralProjectSettings, "ProjectName", GetDefaultDisplayName());
			string ResourceEntrySuffix = (Executable.Architecture != null && Executable.Architecture != UnrealArch.X64) ? Executable.Architecture!.Value.ToString() : "";
			if (bIsDevBuild)
			{
				string ExecutableInfo = Executable.Configuration.ToString();
				if (Executable.Architecture != null && Executable.Architecture != UnrealArch.X64)
				{
					ExecutableInfo += " " + Executable.Architecture.ToString();
				}

				string DisplayNameSuffix = " (" + ExecutableInfo + ")";
				string ResourceEntryName = "ExeDisplayName" + Executable.Configuration.ToString() + ResourceEntrySuffix;
				string DisplayNameResource = AppXResources!.AddResourceString(ResourceEntryName, "ApplicationDisplayName", DisplayNameDefault, DisplayNameSuffix);
				ExecutableElement.Add(new XAttribute("OverrideDisplayName", DisplayNameResource));
			}
			else
			{
				string ResourceEntryName = "AppDisplayName" + ResourceEntrySuffix;
				string DisplayNameResource = AppXResources!.AddResourceString(ResourceEntryName, "ApplicationDisplayName", DisplayNameDefault);
				ExecutableElement.Add(new XAttribute("OverrideDisplayName", DisplayNameResource));
			}

			if (TargetDeviceFamily != null)
			{
				ExecutableElement.Add(new XAttribute("TargetDeviceFamily", TargetDeviceFamily));
			}

			if (Executable.Architecture != null && Platform.IsInGroup(UnrealPlatformGroup.Windows) && GDKEdition >= 251000)
			{
				ExecutableElement.Add(new XAttribute("Architecture", Executable.Architecture!.Value.bIsX64 ? "x64" : "ARM64"));
			}

			if (IsDevOnly)
			{
				ExecutableElement.Add(new XAttribute("IsDevOnly", "true"));
			}

			return ExecutableElement;
		}

		/// <summary>
		/// Get the protocol list element
		/// </summary>
		protected XElement? GetProtocolList(IEnumerable<AppXManifestExecutable> InExecutables)
		{
			XElement? ProtocolList = null;

			// only supported in March GDK onwards
			if (GDKEdition >= 230300)
			{
				// if there's multiple executables in the build, need to specify which one to associate with the protocol. it must use back-slashes
				// documentation for newer GDKs indicate that this element is optional and it will use the default executable in ExecutableList if it is not specified
				string? DefaultExecutable = null;
				if (InExecutables.Count() > 1 && GDKEdition < 251000)
				{
					AppXManifestExecutable MaxExecutable = InExecutables.MaxBy( X => X.Configuration ); // select the most optimized exe
					DefaultExecutable = MaxExecutable.ExecutablePath.Replace('/', '\\');
				}

				List<string> ProtocolNames = new();

				// read protocols
				if (EngineIni!.GetArray(IniSection_PlatformTargetSettings, "UriActivationProtocols", out List<string>? UriActivationProtocols) &&
					UriActivationProtocols.Count > 0)
				{
					foreach (string RawUriActivationProtocol in UriActivationProtocols)
					{
						if (ConfigHierarchy.TryParse(RawUriActivationProtocol, out Dictionary<string, string>? UriActivationProtocol) && UriActivationProtocol.ContainsKey("RegisteredProtocol"))
						{
#pragma warning disable CA1308 // Normalize strings to uppercase - protocol must be lowercase
							string RegisteredProtocol = UriActivationProtocol["RegisteredProtocol"]
								.Split(':')[0]      // ignore any suffix in case someone has added a :// etc.
								.ToLowerInvariant()
							;
#pragma warning restore CA1308
							ProtocolNames.Add(RegisteredProtocol);
						}
					}
				}

				// build the protocol list element
				if (ProtocolNames.Count > 0)
				{
					ProtocolList = new XElement(XName.Get("ProtocolList"));
					foreach (string ProtocolName in ProtocolNames)
					{
						XElement Protocol = new(XName.Get("Protocol"),
							new XAttribute(XName.Get("Name"), ProtocolName)
						);
						if (DefaultExecutable != null)
						{
							Protocol.Add(new XAttribute(XName.Get("Executable"), DefaultExecutable));
						}

						ProtocolList.Add(Protocol);
					}
				}
			}

			return ProtocolList;
		}

		

		/// <summary>
		/// Get the shell visuals element
		/// </summary>
		protected XElement GetShellVisuals()
		{
#pragma warning disable CA1308 // Normalize strings to uppercase - external format may be case-sensitive
			string ForegroundTextValue = GetConfigString("ApplicationForegroundColor", null, "dark").ToLowerInvariant();
			string BackgroundColorValue = GetConfigColor("ApplicationBackgroundColor", "#000040").ToLowerInvariant();
#pragma warning restore CA1308

			string PublisherDisplayName = GetConfigString("PublisherDisplayName", "CompanyName", "NoPublisher");
			string DefaultDisplayName = GetDefaultDisplayName(); //NB. this cannot be localized (Partner Center refuses the package otherwise)

			string DescriptionDefault = ReadIniString(IniSection_GeneralProjectSettings, "Description", "");
			string DescriptionResource = AppXResources!.AddResourceString("AppDescription", "ApplicationDescription", DescriptionDefault);

			if (!AppXResources!.AddResourceBinaryFileReference("StoreLogo.png"))
			{
				Logger.LogWarning("Unable to stage Store logo.");
			}
			if (!AppXResources!.AddResourceBinaryFileReference("Logo.png"))
			{
				Logger.LogError("Unable to stage application logo.");
			}
			if (!AppXResources!.AddResourceBinaryFileReference("SmallLogo.png"))
			{
				Logger.LogError("Unable to stage application small logo.");
			}
			if (!AppXResources!.AddResourceBinaryFileReference("SplashScreen.png"))
			{
				Logger.LogError("Unable to stage splash screen image.");
			}
			if (!AppXResources!.AddResourceBinaryFileReference("Square480x480Logo.png"))
			{
				Logger.LogError("Unable to stage application large logo.");
			}

			XElement ShellVisuals = new(XName.Get("ShellVisuals"),
				new XAttribute(XName.Get("DefaultDisplayName"), DefaultDisplayName),
				new XAttribute(XName.Get("PublisherDisplayName"), PublisherDisplayName),
				new XAttribute(XName.Get("StoreLogo"), BuildResourceSubPath + "\\StoreLogo.png"),
				new XAttribute(XName.Get("Square150x150Logo"), BuildResourceSubPath + "\\Logo.png"),
				new XAttribute(XName.Get("Square44x44Logo"), BuildResourceSubPath + "\\SmallLogo.png"),
				new XAttribute(XName.Get("Square480x480Logo"), BuildResourceSubPath + "\\Square480x480Logo.png"),
				new XAttribute(XName.Get("Description"), DescriptionResource),
				new XAttribute(XName.Get("ForegroundText"), ForegroundTextValue),
				new XAttribute(XName.Get("BackgroundColor"), BackgroundColorValue),
				new XAttribute(XName.Get("SplashScreenImage"), BuildResourceSubPath + "\\SplashScreen.png")
			);

			return ShellVisuals;
		}

		/// <summary>
		/// Get the default display name for the project
		/// </summary>
		protected string GetDefaultDisplayName()
		{
			if (IsDLC)
			{
				if (DLCConfig!.TryGetValue("DefaultDisplayName", out string? DefaultDisplayName) && !String.IsNullOrEmpty(DefaultDisplayName))
				{
					return DefaultDisplayName;
				}

				if (DLCFile != null)
				{
					Logger.LogWarning("Default display name for {DLC} is not specified", DLCFile!.GetFileNameWithoutExtension());
					return DLCFile!.GetFileNameWithoutAnyExtensions();
				}

				Logger.LogWarning("Default display name for {DLC} is not specified", DLCPackage);
				return DLCPackage!;
			}
			else
			{
				string DefaultName = (ProjectFile != null) ? ProjectFile.GetFileNameWithoutAnyExtensions() : (TargetName ?? "UnrealGame");

				string DefaultDisplayName = GetConfigString("DefaultDisplayName", "ProjectName", DefaultName);

				return DefaultDisplayName;
			}
		}

		/// <summary>
		/// Gets the package identity name string for the main package, disregarding any current DLC
		/// </summary>
		/// <returns></returns>
		protected string GetMainPackageIdentityPackageName()
		{
			return base.GetIdentityPackageName();
		}

		/// <summary>
		/// Get the package identity name string
		/// </summary>
		protected override string GetIdentityPackageName()
		{
			if (IsDLC)
			{
				if (DLCConfig!.TryGetValue("PackageName", out string? PackageName) && !String.IsNullOrEmpty(PackageName))
				{
					return PackageName;
				}

				if (DLCFile != null)
				{
					Logger.LogWarning("Package name for {DLC} is not specified", DLCFile!.GetFileNameWithoutExtension());
					return DLCFile!.GetFileNameWithoutAnyExtensions();
				}

				Logger.LogWarning("Package name for {DLC} is not specified", DLCPackage);
				return DLCPackage!;

			}

			return base.GetIdentityPackageName();
		}

		/// <summary>
		/// Get the package version string
		/// </summary>
		protected override string? GetIdentityVersionNumber()
		{
			if (IsDLC)
			{
				if (DLCConfig!.TryGetValue("PackageVersion", out string? PackageVersion) && !String.IsNullOrEmpty(PackageVersion))
				{
					// If specified in the project settings attempt to retrieve the current build number and increment the version number by that amount, accounting for overflows
					if (EngineIni!.GetBool(IniSection_PlatformTargetSettings, "bIncludeEngineVersionInPackageVersion", out bool bIncludeEngineVersionInPackageVersion) && bIncludeEngineVersionInPackageVersion)
					{
						PackageVersion = IncludeBuildVersionInPackageVersion(PackageVersion);
					}

					return PackageVersion;
				}
			}

			return base.GetIdentityVersionNumber();
		}

		/// <summary>
		/// Manifest version. 0 = original, 1 = March 2022 GDK and above. 
		/// Note: Version 0 has been deprecated as of October 2023 GDK
		/// </summary>
		protected int GetManifestVersion()
		{
			// As of October 2023 GDK, configVersion=0 has been deprecated. As a result
			// the minimum expected manifest version is 1. 
			const int MinManifestVersion = 1;

			int Result = MinManifestVersion; 

			// If developers need to use a deprecated manifest version, they can use this config
			// key to override it. 
			string? ManifestVersionDeprecated = GetConfigString("ManifestVersionDeprecated", null);
			if (!String.IsNullOrEmpty(ManifestVersionDeprecated))
			{
				ManifestVersionDeprecated = ManifestVersionDeprecated.Replace("Version", "", StringComparison.Ordinal);
				if (Int32.TryParse(ManifestVersionDeprecated, out int ResultOverride))
				{
					Result = ResultOverride;
				}
				else
				{
					Logger.LogError("Could not parse ManifestVersionDeprecated value '{ManifestVersionDeprecated}'", ManifestVersionDeprecated);
				}
			}

			return Result;
		}

		/// <summary>
		/// Perform any additional initialization once all parameters and configuration are ready
		/// </summary>
		protected override void PostConfigurationInit()
		{
			base.PostConfigurationInit();

			// find and cache the configuration for the selected DLC
			DLCConfig = null;
			if (IsDLC)
			{
				string DLCName = DLCPackage ?? DLCFile!.GetFileNameWithoutExtension();

				if (EngineIni!.GetArray(IniSection_PlatformTargetSettings, "DLCPackages", out List<string>? DLCPackages))
				{
					foreach (string DLCPackage in DLCPackages)
					{
						if (ConfigHierarchy.TryParse(DLCPackage, out Dictionary<string, string>? Properties)
							&& Properties.TryGetValue("DLCName", out string? ThisDLCName)
							&& DLCName == ThisDLCName)
						{
							// we have found the configuration for this DLC package
							DLCConfig = Properties;
							break;
						}
					}
				}

				// create a dummy configuration if we didn't find one
				if (DLCConfig == null)
				{
					string DummyStoreId = String.Concat("0123456789bcdfghjklmnpqrstvwxzBCDFGHJKLMNPQRSTVWXZ".AsEnumerable().OrderBy(x => Guid.NewGuid()).Take(12)); // 12 random items from valid character list (real one can have duplicate chars, but this is just a dummy value)
					Logger.LogWarning("No DLC configuration for {DLCName}. Creating temporary defaults, including StoreId={DummyStoreId}", DLCName, DummyStoreId);
					DLCConfig = new Dictionary<string, string>()
					{
						{ "StoreId",             DummyStoreId },
						{ "DefaultDisplayName",  DLCName },
						{ "PackageName",         DLCName },
						{ "PackageVersion",      "1.0.0.0" },
					};
				}
			}
		}

		/// <summary>
		/// Create all the localization data. Returns whether there is any localization data
		/// </summary>
		protected override bool BuildLocalizationData()
		{
			bool bResult = base.BuildLocalizationData();

			// replace base strings with DLC strings
			if (DLCConfig != null)
			{
				// reset per-culture strings and make sure the default culture entry exists
				AppXResources!.ClearStrings();

				// add all default strings
				if (DLCConfig.TryGetValue("DefaultStringResources", out string? DefaultStringResources) && ConfigHierarchy.TryParse(DefaultStringResources, out Dictionary<string, string>? DefaultStrings))
				{
					AppXResources!.AddDefaultStrings(DefaultStrings);
				}

				// add per culture strings
				if (DLCConfig.TryGetValue("PerCultureResources", out string? PerCultureResources) && ConfigHierarchy.TryParse(PerCultureResources, out string[]? PerCultureResourcesArray))
				{
					foreach (string CultureResources in PerCultureResourcesArray)
					{
						if (!ConfigHierarchy.TryParse(CultureResources, out Dictionary<string, string>? CultureProperties)
							|| !CultureProperties.ContainsKey("CultureStringResources")
							|| !CultureProperties.ContainsKey("StageId"))
						{
							Logger.LogWarning("Invalid per-culture resource value: {Culture}", CultureResources);
							continue;
						}

						string StageId = CultureProperties["StageId"];
						if (String.IsNullOrEmpty(StageId))
						{
							Logger.LogWarning("Missing StageId value: {Culture}", CultureResources);
							continue;
						}

						if (!UEStageIdToAppXCultureId.TryGetValue(StageId, out string? CultureId))
						{
							Logger.LogWarning("Unknown Culture {ID} in {Culture}. Check main PerCultureResources", StageId, CultureResources);
							continue;
						}

						// read culture strings
						if (!ConfigHierarchy.TryParse(CultureProperties["CultureStringResources"], out Dictionary<string, string>? CultureStringResources))
						{
							Logger.LogError("Invalid culture string resources: \"{Culture}\". Unable to add resource entry.", CultureResources);
							continue;
						}
						AppXResources!.AddCultureStrings(CultureId, CultureStringResources);
					}
				}
			}

			// add the feature display name
			if (EngineIni!.GetArray(IniSection_PlatformTargetSettings, "IntelligentDeliveryFeatures", out List<string>? Features) && Features != null && Features.Count > 0)
			{
				foreach (string Feature in Features)
				{
					if (ConfigHierarchy.TryParse(Feature, out Dictionary<string, string>? FeatureProperties))
					{
						// verify feature id
						if (!FeatureProperties.TryGetValue("Id", out string? Id) || Id.Length == 0)
						{
							throw new InvalidDataException("Missing or invalid Id in feature");
						}
						string ResourceEntryName = "Feature_" + Id;

						// read the default display name and skip if it isn't set
						if (FeatureProperties.TryGetValue("DefaultDisplayName", out string? DefaultDisplayName) && !String.IsNullOrEmpty(DefaultDisplayName))
						{
							AppXResources!.AddDefaultString(ResourceEntryName, DefaultDisplayName);
						}
						else
						{
							continue;
						}

						// read the per-culture resources for this feature
						if (FeatureProperties.TryGetValue("PerCultureResources", out string? PerCultureResourcesProperty))
						{
							if (!ConfigHierarchy.TryParse(PerCultureResourcesProperty, out string[]? PerCultureResources))
							{
								Logger.LogWarning("Invalid per-culture resources value in feature {Feature} : {Culture}", Id, PerCultureResourcesProperty);
								continue;
							}

							// parse all per culture resources
							foreach (string CultureResources in PerCultureResources)
							{
								if (!ConfigHierarchy.TryParse(CultureResources, out Dictionary<string, string>? CultureProperties)
									|| !CultureProperties.ContainsKey("StageId")
									|| !CultureProperties.ContainsKey("DisplayName"))
								{
									Logger.LogWarning("Invalid per-culture resource value in feature {Feature} : {Culture}", Id, CultureResources);
									continue;
								}

								string StageId = CultureProperties["StageId"];
								if (String.IsNullOrEmpty(StageId))
								{
									Logger.LogWarning("Missing StageId value in feature {Feature}: {Culture}", Id, CultureProperties);
									continue;
								}

								if (UEStageIdToAppXCultureId.TryGetValue(StageId, out string? CultureId))
								{
									string DisplayName = CultureProperties["DisplayName"];
									if (!String.IsNullOrEmpty(DisplayName))
									{
										AppXResources!.AddCultureString(CultureId, ResourceEntryName, DisplayName);
									}
								}
								else
								{
									throw new InvalidDataException($"Unknown StageId {StageId} in feature {Id}. Make sure there is an entry in the main PerCultureResources");
								}
							}
						}
					}
				}
			}

			return bResult;
		}

		/// <summary>
		/// Return the entire manifest element
		/// </summary>
		protected override XElement GetManifest(IEnumerable<AppXManifestExecutable> InExecutables, out string IdentityName)
		{
			return GetGame(InExecutables, out IdentityName);
		}

		/// <summary>
		/// 
		/// </summary>
		protected override void ProcessManifest(IEnumerable<AppXManifestExecutable> InExecutables, string ManifestName, string ManifestTargetPath, string ManifestIntermediatePath)
		{
			AddResourcesForPackaging();

			base.ProcessManifest(InExecutables, ManifestName, ManifestTargetPath, ManifestIntermediatePath);
		}

		/// <summary>
		/// Register the locations where resource binary files can be found
		/// </summary>
		protected override void PrepareResourceBinaryPaths()
		{
			// check for DLC plugin images first, in the <plugin>/Build/<plaform>/Resources/ folder (and Platform extension equivalent)
			if (DLCFile != null)
			{
				AppXResources!.ProjectBinaryResourceDirectories.Add(DirectoryReference.Combine(DLCFile.Directory, "Build", Platform.ToString(), BuildResourceSubPath));
				AppXResources!.ProjectBinaryResourceDirectories.Add(DirectoryReference.Combine(DLCFile.Directory, "Platforms", Platform.ToString(), "Build", BuildResourceSubPath));
			}

			// check for DLC package images next, in <project>/Build/<plaform>/<dlc>/Resources/ folder (and Platform extension equivalent)
			if (DLCPackage != null && ProjectFile != null)
			{
				AppXResources!.ProjectBinaryResourceDirectories.Add(DirectoryReference.Combine(ProjectFile.Directory, "Build", Platform.ToString(), DLCPackage, BuildResourceSubPath));
				AppXResources!.ProjectBinaryResourceDirectories.Add(DirectoryReference.Combine(ProjectFile.Directory, "Platforms", Platform.ToString(), "Build", DLCPackage, BuildResourceSubPath));
			}

			base.PrepareResourceBinaryPaths();
		}

		/// <summary>
		/// Perform any platform-specific processing on the manifest before it is saved
		/// </summary>
		protected void AddResourcesForPackaging()
		{
			// see if features and recipes are being used
			EngineIni!.GetBool(IniSection_PlatformTargetSettings, "bUseFeaturesAndRecipes", out bool bUseFeaturesAndRecipes);
			if (!bUseFeaturesAndRecipes)
			{
				return;
			}

			// get the list of all features
			EngineIni.GetArray(IniSection_PlatformTargetSettings, "IntelligentDeliveryFeatures", out List<string>? Features);
			if (Features == null || Features.Count == 0)
			{
				return;
			}

			// parse all features
			foreach (string Feature in Features)
			{
				if (ConfigHierarchy.TryParse(Feature, out Dictionary<string, string>? Properties))
				{
					// verify feature id
					if (!Properties.TryGetValue("Id", out string? Id) || Id.Length == 0)
					{
						throw new InvalidDataException("Missing or invalid Id in feature");
					}

					// copy the feature image. Image file name must match the one in GDKPackageGenerator.GenerateFeaturesAndRecipes
					// note that localized feature images are not currently supported by the GDK
					string ImageFilename = "Feature_" + Id + ".png";
					bool bHasFeatureImage = AppXResources!.DoesDefaultResourceBinaryFileExist(ImageFilename, AllowEngineFallback: false);
					if (bHasFeatureImage)
					{
						if (!AppXResources!.AddResourceBinaryFileReference(ImageFilename, AllowEngineFallback: false))
						{
							Logger.LogError("Unable to feature image {Image} for feature {Id}.", ImageFilename, Id);
						}
					}

					// add the string entry. note that the prefix must match the one in GDKPackageGenerator.GenerateFeaturesAndRecipes
					string ResourceEntryName = "Feature_" + Id;
					if (AppXResources!.HasString(ResourceEntryName))
					{
						AppXResources!.AddResourceString(ResourceEntryName, ResourceEntryName, "");
					}
				}
			}
		}

		#region packaging resource query helper

		/// <summary>
		/// Helper to find whether there is a packaging resource file available
		/// </summary>
		public static bool DoesDefaultResourceBinaryFileExistForPlatform(FileReference ProjectFile, UnrealTargetPlatform Platform, ILogger Logger, string ImageFilename, bool bAllowEngineFallback = true)
		{
			GDKGameConfigGenerator Generator = GDKGameConfigGeneratorFactory.CreateGenerator(Platform,Logger);
			Generator.AppXResources = new(Logger, Generator.GetMakePriBinaryPath());
			Generator.ProjectFile = ProjectFile;
			Generator.PrepareResourceBinaryPaths();

			return Generator.AppXResources!.DoesDefaultResourceBinaryFileExist(ImageFilename, bAllowEngineFallback);
		}

		#endregion

		#region Package name derivation code

		private void GetManifestElementsForProject(DirectoryReference InProjectDir, string? InTargetName, out string Name, out string? Version, out string Architecture, out string ResourceId, out string Publisher)
		{
			// Load up INI settings. We'll use engine settings to retrieve the manifest configuration, but these may reference
			// values in either game or engine settings, so we'll keep both.
			GameIni = ConfigCache.ReadHierarchy(ConfigHierarchyType.Game, InProjectDir, Platform, CustomConfig);
			EngineIni = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, InProjectDir, Platform, CustomConfig);
			TargetName = InTargetName;

			Name = GetIdentityPackageName();
			Version = GetIdentityVersionNumber();
			Architecture = DefaultArchitecture;
			ResourceId = "";
			Publisher = GetIdentityPublisherName();
		}

		/// <summary>
		/// Returns the package full name for the given project
		/// </summary>
		public static string GetPackageFullNameFromProject(UnrealTargetPlatform InPlatform, ILogger Logger, DirectoryReference InProjectDir, string? InTargetName, string InCustomConfig)
		{
			GDKGameConfigGenerator ManifestGenerator = GDKGameConfigGeneratorFactory.CreateGenerator(InPlatform, Logger);
			ManifestGenerator.CustomConfig = InCustomConfig;

			ManifestGenerator.GetManifestElementsForProject(InProjectDir, InTargetName, out string Name, out string? Version, out string Architecture, out string ResourceId, out string Publisher);

			return String.Format("{0}_{1}_{2}_{3}_{4}",
				Name,
				Version,
				Architecture,
				ResourceId,
				GDKGameConfigInfo.EncodePublisherNameForPackageName(Publisher)
				);
		}

		/// <summary>
		/// Returns the package family name for the given project
		/// </summary>
		public static string GetPackageFamilyNameFromProject(UnrealTargetPlatform InPlatform, ILogger Logger, DirectoryReference InProjectDir, string? InTargetName, string InCustomConfig)
		{
			GDKGameConfigGenerator ManifestGenerator = GDKGameConfigGeneratorFactory.CreateGenerator(InPlatform, Logger);
			ManifestGenerator.CustomConfig = InCustomConfig;

			ManifestGenerator.GetManifestElementsForProject(InProjectDir, InTargetName, out string Name, out _, out _, out _, out string Publisher);

			return String.Format("{0}_{1}",
				Name,
				GDKGameConfigInfo.EncodePublisherNameForPackageName(Publisher)
				);
		}

		/// <summary>
		/// Returns the AUMID for the given project and configuration
		/// </summary>
		public static string GetAUMIDFromProject(UnrealTargetPlatform InPlatform, ILogger Logger, DirectoryReference InProjectDir, UnrealTargetConfiguration InTargetConfig, UnrealArch? InArchitecture, string? InTargetName, string InCustomConfig)
		{
			GDKGameConfigGenerator ManifestGenerator = GDKGameConfigGeneratorFactory.CreateGenerator(InPlatform, Logger);
			ManifestGenerator.CustomConfig = InCustomConfig;

			ManifestGenerator.GetManifestElementsForProject(InProjectDir, InTargetName, out string Name, out _, out _, out _, out string Publisher);

			string AppId = ManifestGenerator.GetAppId(InTargetConfig, InArchitecture);

			return $"{Name}_{GDKGameConfigInfo.EncodePublisherNameForPackageName(Publisher)}!{AppId}";
		}

		#endregion
	}

	partial struct GDKGameConfigGeneratorFactory
	{
		private static readonly Dictionary<UnrealTargetPlatform, Type> PlatformGenerators = new();

		/// <summary>
		/// Get the manifest generator for the given platform
		/// </summary>
		public static GDKGameConfigGenerator CreateGenerator(UnrealTargetPlatform Platform, ILogger? Logger)
		{
			if (!PlatformGenerators.TryGetValue(Platform, out Type? GeneratorType) || GeneratorType == null)
			{
				throw new BuildException($"Platform {Platform} does not support GDK manifest generation");
			}

			GDKGameConfigGenerator? Generator = (GDKGameConfigGenerator?)Activator.CreateInstance(GeneratorType, Logger);
			if (Generator == null)
			{
				throw new BuildException($"Could not instantiate the GDK manifest generator for {Platform}");
			}

			return Generator!;
		}

		private static Type RegisterGenerator(UnrealTargetPlatform Platform, Type GeneratorType)
		{
			if (!GeneratorType.IsSubclassOf(typeof(GDKGameConfigGenerator)))
			{
				throw new BuildException($"{GeneratorType} is not a GDKGameConfigGenerator");
			}

			PlatformGenerators.Add(Platform, GeneratorType);
			return GeneratorType;
		}
	}

	/// <summary>
	/// ToolMode to create a MicrosoftGame.config for a single executable pair
	/// </summary>
	internal class GDKGameConfigGeneratorMode : IToolMode<GDKGameConfigGeneratorMode>
	{
		public static string Name => "GDKGameConfigGenerator";
		public static ToolModeOptions Options => ToolModeOptions.BuildPlatforms;

#pragma warning disable IDE0044 // Make field readonly - these private fields are set by the command line.
		[CommandLine("-Executable", Required = false)]
		string? Executable = null;

		[CommandLine("-Configuration", Required = false)]
		UnrealTargetConfiguration? Configuration = null;

		[CommandLine("-Architecture", Required = false)]
		UnrealArch? Architecture = null;

		[CommandLine("-Project", Required = false)]
		FileReference? ProjectFile = null;

		[CommandLine("-Platform", Required = true)]
		UnrealTargetPlatform? Platform = null;

		[CommandLine("-Target", Required = true)]
		string? TargetName = null;

		[CommandLine("-CustomConfig", Required = false)]
		string CustomConfig = "";

		[CommandLine("-OutputDirectory", Required = true)]
		DirectoryReference? OutputDirectory = null;
#pragma warning restore IDE0044

		/// <inheritdoc/>
		public Task<int> ExecuteAsync(CommandLineArguments Arguments, ILogger Logger)
		{
			Arguments.ApplyTo(this);
			Arguments.CheckAllArgumentsUsed();

			List<AppXManifestExecutable> Executables;
			if (Executable == null || Configuration == null)
			{
				Executables = [];
			}
			else
			{
				Executables = [ new (Configuration!.Value, Executable!, Architecture ) ];
			}

			GDKGameConfigGenerator.Create((UnrealTargetPlatform)Platform!, Logger, OutputDirectory!, TargetName, ProjectFile, Executables, CustomConfig: CustomConfig);

			return Task.FromResult(0);
		}

		/// <summary>
		/// Helper function to create an action that will run this ToolMode
		/// </summary>
		public static Action CreateAction(TargetMakefileBuilder MakefileBuilder, UnrealTargetPlatform Platform, DirectoryReference OutputDirectory, string TargetName, FileReference? ProjectFile, UnrealTargetConfiguration? Configuration = null, string? ExecutableFileName = null, UnrealArch? Architecture = null, string CustomConfig = "")
		{
			Action GenAction = MakefileBuilder.CreateAction(ActionType.PostBuildStep);
			GenAction.CommandPath = Unreal.DotnetPath;
			GenAction.CommandArguments = $"\"{Unreal.UnrealBuildToolDllPath}\" -Mode=GDKGameConfigGenerator -Platform={Platform} -Target={TargetName} -OutputDirectory=\"{OutputDirectory}\"";
			GenAction.WorkingDirectory = Unreal.EngineSourceDirectory;
			GenAction.ProducedItems.Add(FileItem.GetItemByFileReference(FileReference.Combine(OutputDirectory, "MicrosoftGame.config")));
			GenAction.ProducedItems.Add(FileItem.GetItemByFileReference(FileReference.Combine(OutputDirectory, "resources.pri")));
			GenAction.DeleteItems.UnionWith(GenAction.ProducedItems);
			GenAction.bCanExecuteRemotely = false;

			if (Configuration != null && ExecutableFileName != null)
			{
				GenAction.CommandArguments += $" -Executable =\"{ExecutableFileName}\" -Configuration={Configuration}";
			}

			if (Architecture != null)
			{
				GenAction.CommandArguments += $" -Architecture={Architecture}";
			}

			if (!String.IsNullOrEmpty(CustomConfig))
			{
				GenAction.CommandArguments += $" -CustomConfig={CustomConfig}";
			}

			if (ProjectFile != null)
			{
				GenAction.CommandArguments += $" -Project=\"{ProjectFile}\"";
			}

			MakefileBuilder.AddOutputFiles(GenAction.ProducedItems);

			return GenAction;
		}
	}
}
