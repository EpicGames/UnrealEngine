// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using System.Text.Json;
using System.Text.RegularExpressions;
using System.Xml;

namespace UnrealBuildTool.GDK
{
	/// <summary>
	/// Helper class used to manage GDK manifest identities
	/// </summary>
	public class GDKGameConfigInfo
	{
		/// Regular expression for standard Micrososoft full package name:  Name_Version_ProcessorArchitecture_ResourceId_PublisherId
		public const string PackageIdentityRegEx = @"([A-Za-z0-9\-\.]+)_(\d+\.\d+.\d+.\d+)_(x86|x64|arm|neutral)_([A-Za-z0-9\-\.]*)_([A-Za-z0-9]+)";

		/// Regular expression for standard Microsoft AUMID: Name_PublisherId!AppName
		public const string PackageAUMIDRegEx = @"([A-Za-z0-9\-\.]+)_([A-Za-z0-9]+)!(\S+)";

		/// Architecture to use when none is specified
		public const string DefaultArchitecture = "neutral";

		/// Version to use when none is specified
		public const string DefaultVersion = "0.0.0.0";

		/// The package identity. e.g. "GameName"
		public string Identity { get; protected set; }

		/// The package version. e.g. "1.0.0.0"
		public string Version { get; protected set; }

		/// The package Architecture. e.g. "neutral", "x64" etc.
		public string Architecture { get; protected set; }

		/// The package resource id. (not used on GDK)
		public string ResourceId { get; protected set; }

		/// The package publisher id. e.g. "z844xmwvfmpty"
		public string PublisherId { get; protected set; }

		/// The full Package Name. e.g. "GameName_1.0.0.0_neutral__z844xmwvfmpty"
		public string PackageFullName => Identity + "_" + Version + "_" + Architecture + "_" + ResourceId + "_" + PublisherId;

		/// The package family name. e.g. "GameName_z844xmwvfmpty"
		public string PackageFamilyName => Identity + "_" + PublisherId;

		/// only valid if we are created from a MicrosoftGame.config
		public readonly Dictionary<string, string>? IdToExecuatable;

		/// only valid if we are created from a MicrosoftGame.config or via ParseAppList
		public List<string>? AUMIDs => InternalAUMIDs;
		private readonly List<string>? InternalAUMIDs;

		/// <summary>
		/// Create empty manifest identity
		/// </summary>
		public GDKGameConfigInfo()
		{
			Identity = "";
			Version = DefaultVersion;
			Architecture = DefaultArchitecture;
			ResourceId = "";
			PublisherId = "";
		}

		/// <summary>
		/// Create a manifest based on a given package full name. e.g. "GameName_1.0.0.0_neutral__z844xmwvfmpty"
		/// </summary>
		public GDKGameConfigInfo(string InPackageFullName) : this()
		{
			if (!SetFromPackageFullName(InPackageFullName))
			{
				throw new Exception($"Invalid Package Full Name : {InPackageFullName}");
			}
		}

		/// <summary>
		/// Create a manifest identity from the given manifest file
		/// </summary>
		public GDKGameConfigInfo(FileReference ManifestFile) : this()
		{
			try
			{
				// read the Identity
				XmlDocument XmlDoc = new();
				XmlDoc.Load(ManifestFile.FullName);
				XmlNodeList IdentityNodes = XmlDoc.GetElementsByTagName("Identity");
				if (IdentityNodes.Count == 1)
				{
					XmlElement XmlIdentity = (XmlElement)IdentityNodes[0]!;
					SetFromIdentity(XmlIdentity);
				}
				else
				{
					throw new Exception($"Found more than one Identity node in manifest {ManifestFile.FullName}");
				}

				// read the executables (NB. MicrosoftGame.config only... old AppXManifest.xml uses "Applications")
				XmlNodeList ExecutableList = XmlDoc.GetElementsByTagName("ExecutableList");
				if (ExecutableList.Count > 0)
				{
					IdToExecuatable = new Dictionary<string, string>();
					InternalAUMIDs = new List<string>();

					foreach (XmlNode? Executable in ExecutableList[0]!.ChildNodes)
					{
						if (Executable != null)
						{
							string Name = Executable.Attributes!["Name"]!.Value;
							string Id = Executable.Attributes["Id"]!.Value;
							IdToExecuatable.Add(Id, Name);
							InternalAUMIDs.Add($"{PackageFamilyName}!{Id}");
						}
					}
				}
			}
			catch
			{
				throw new Exception($"Unable to parse Xml manifest {ManifestFile.FullName}");
			}
		}

		internal GDKGameConfigInfo(JsonElement JsonPkg) : this()
		{
			try
			{
				if (JsonPkg.TryGetStringProperty("PackageFullName", out string? PackageFullName) || JsonPkg.TryGetStringProperty("FullName", out PackageFullName))
				{
					if (SetFromPackageFullName(PackageFullName))
					{
						if (JsonPkg.TryGetProperty("Applications", out JsonElement JsonApplications))
						{
							IdToExecuatable = new Dictionary<string, string>();
							InternalAUMIDs = new List<string>();

							foreach (JsonElement JsonApp in JsonApplications.EnumerateArray())
							{
								if (JsonApp.TryGetStringProperty("Aumid", out string? AUMID))
								{
									InternalAUMIDs.Add(AUMID);

									if (JsonApp.TryGetStringProperty("Executable", out string? Executable) && JsonApp.TryGetStringProperty("Id", out string? Id))
									{
										IdToExecuatable.Add(Id, Executable);
									}
								}
							}
						}
					}
				}
			}
			catch (Exception)
			{
			}
		}

		internal bool SetFromPackageFullName(string InPackageFullName)
		{
			Match M = Regex.Match(InPackageFullName, PackageIdentityRegEx, RegexOptions.IgnoreCase);
			if (M.Success)
			{
				Identity = M.Groups[1].Value;
				Version = M.Groups[2].Value;
				Architecture = M.Groups[3].Value;
				ResourceId = M.Groups[4].Value;
				PublisherId = M.Groups[5].Value;
			}

			return M.Success;
		}

		internal bool SetFromIdentity(XmlElement XmlIdentity)
		{
			Identity = XmlIdentity.HasAttribute("Name") ? XmlIdentity.GetAttribute("Name") : "";
			Version = XmlIdentity.HasAttribute("Version") ? XmlIdentity.GetAttribute("Version") : "";
			ResourceId = XmlIdentity.HasAttribute("ResourceId") ? XmlIdentity.GetAttribute("ResourceId") : "";

			Architecture = DefaultArchitecture;
			if (XmlIdentity.HasAttribute("ProcessorArchitecture"))
			{
				Architecture = XmlIdentity.GetAttribute("ProcessorArchitecture");
			}
			else
			{
				XmlNodeList XmlProcessorArchitectureNodes = XmlIdentity.OwnerDocument.GetElementsByTagName("ProcessorArchitecture");
				if (XmlProcessorArchitectureNodes.Count == 1)
				{
					Architecture = XmlProcessorArchitectureNodes[0]!.InnerText;
				}
			}

			string Publisher = XmlIdentity.HasAttribute("Publisher") ? XmlIdentity.GetAttribute("Publisher") : "";
			PublisherId = EncodePublisherNameForPackageName(Publisher);

			return true;
		}

		/// <summary>
		/// Query whether there are executables available in this manifest
		/// </summary>
		public bool HasExecutables => IdToExecuatable != null;

		/// <summary>
		/// Look up the relative path to the given executable, if it is mentioned
		/// </summary>
		/// <returns></returns>
		public string? GetExecutableRelativePath(string ExecutableName)
		{
			if (IdToExecuatable != null)
			{
				string ExecutableFileName = Path.GetFileName(ExecutableName);
				foreach (KeyValuePair<string, string> Pair in IdToExecuatable)
				{
					if (Path.GetFileName(Pair.Value).Equals(ExecutableFileName, StringComparison.OrdinalIgnoreCase))
					{
						return Pair.Value;
					}
				}
			}

			return null;
		}

		/// <summary>
		/// Returns the AUMID for the given manifest and executable file name. e.g. GameName_z844xmwvfmpty!AppGameNameDevelopment
		/// </summary>
		public string? GetAUMIDForExecutable(string ExecutableName)
		{
			if (IdToExecuatable != null)
			{
				string ExecutableFileName = Path.GetFileName(ExecutableName);
				foreach (KeyValuePair<string, string> Pair in IdToExecuatable)
				{
					if (Path.GetFileName(Pair.Value).Equals(ExecutableFileName, StringComparison.OrdinalIgnoreCase))
					{
						return $"{PackageFamilyName}!{Pair.Key}";
					}
				}
			}

			return null;
		}

		/// <summary>
		/// Returns the AUMID that matches the given configuration
		/// </summary>
		public string GetAUMIDForConfiguration(UnrealTargetConfiguration Configuration)
		{
			if (IdToExecuatable != null && AUMIDs != null)
			{
				return GetAUMIDForConfiguration(AUMIDs, Configuration);
			}
			else
			{
				throw new Exception("executable list is not available");
			}
		}

		/// <summary>
		/// Returns the executable that matches the given configuration
		/// </summary>
		public string GetExecutableForConfiguration(UnrealTargetConfiguration Configuration)
		{
			if (IdToExecuatable == null)
			{
				throw new Exception("executable list is not available");
			}
			else if (IdToExecuatable.Values.Count == 0)
			{
				throw new Exception("no executables");
			}

			string? Executable = IdToExecuatable.Values.FirstOrDefault(X => X.EndsWith(Configuration.ToString(), StringComparison.Ordinal));
			// fall back to any executable not tagged as Dev/Test/Shipping
			Executable ??= IdToExecuatable.Values.FirstOrDefault(X =>
				!X.EndsWith(@"Development", StringComparison.Ordinal) &&
				!X.EndsWith(@"Test", StringComparison.Ordinal) &&
				!X.EndsWith(@"Shipping", StringComparison.Ordinal));

			if (Executable == null)
			{
				throw new Exception($"Unable to find executable for configuration {Configuration} from {String.Join(',', IdToExecuatable.Values)}");
			}

			return Executable!;
		}

		/// <summary>
		/// Returns the executable that matches the given AUMID
		/// </summary>
		public string? GetExecutableForAUMID(string AUMID)
		{
			if (IdToExecuatable != null)
			{
				foreach (KeyValuePair<string, string> Pair in IdToExecuatable)
				{
					string ThisAUMID = $"{PackageFamilyName}!{Pair.Key}";
					if (AUMID.Equals(ThisAUMID, StringComparison.OrdinalIgnoreCase))
					{
						return Pair.Value;
					}
				}
			}

			return null;
		}

		/// <summary>
		/// Standard ToString override
		/// </summary>
		public override string ToString()
		{
			return PackageFullName;
		}

		#region Publisher encoding helper functions from Microsoft

		/// <summary>
		/// Converts a human-readable Publisher Name into the Publisher Id used by MS app manifests
		/// </summary>
		public static string EncodePublisherNameForPackageName(string PublisherName)
		{
			byte[] PublisherBytes = Encoding.Unicode.GetBytes(PublisherName);
			byte[] PublisherBytesHash = System.Security.Cryptography.SHA256.HashData(PublisherBytes);

			byte[] FirstBytes = new byte[8];
			Array.Copy(PublisherBytesHash, 0, FirstBytes, 0, 8);
			return GetBase32CharsForPackageName(FirstBytes);
		}

		private static string GetBase32CharsForPackageName(byte[] Bytes)
		{
			StringBuilder Result = new();

			for (int ByteIndex = 0; ByteIndex < Bytes.Length; ByteIndex += 5)
			{
				byte FirstByte = Bytes[ByteIndex];
				byte SecondByte = (ByteIndex + 1) < Bytes.Length ? Bytes[ByteIndex + 1] : (byte)0;

				Result.Append(ValueToDigitForPackageName((byte)((FirstByte & 0xF8) >> 3)));
				Result.Append(ValueToDigitForPackageName((byte)(((FirstByte & 0x07) << 2) | ((SecondByte & 0xC0) >> 6))));

				if (ByteIndex + 1 < Bytes.Length)
				{
					byte thirdByte = (ByteIndex + 2) < Bytes.Length ? Bytes[ByteIndex + 2] : (byte)0;

					Result.Append(ValueToDigitForPackageName((byte)((SecondByte & 0x3E) >> 1)));
					Result.Append(ValueToDigitForPackageName((byte)(((SecondByte & 0x01) << 4) | ((thirdByte & 0xF0) >> 4))));

					if (ByteIndex + 2 < Bytes.Length)
					{
						byte fourthByte = (ByteIndex + 3) < Bytes.Length ? Bytes[ByteIndex + 3] : (byte)0;

						Result.Append(ValueToDigitForPackageName((byte)(((thirdByte & 0x0F) << 1) | ((fourthByte & 0x80) >> 7))));

						if (ByteIndex + 3 < Bytes.Length)
						{
							byte fifthByte = (ByteIndex + 4) < Bytes.Length ? Bytes[ByteIndex + 4] : (byte)0;

							Result.Append(ValueToDigitForPackageName((byte)((fourthByte & 0x7C) >> 2)));
							Result.Append(ValueToDigitForPackageName((byte)(((fourthByte & 0x03) << 3) | ((fifthByte & 0xE0) >> 5))));

							if (ByteIndex + 4 < Bytes.Length)
							{
								Result.Append(ValueToDigitForPackageName((byte)(fifthByte & 0x1F)));
							}
						}
					}
				}
			}

			return Result.ToString();
		}

		private static char ValueToDigitForPackageName(byte Value)
		{
			if (Value >= 0x20)
			{
				throw new ArgumentOutOfRangeException(nameof(Value));
			}

			const string Base32DigitList = "0123456789abcdefghjkmnpqrstvwxyz";
			return Base32DigitList[Value];
		}

		#endregion

		/// <summary>
		/// Returns the full package name from the given manifest. e.g.  GameName_1.0.0.0_neutral__z844xmwvfmpty
		/// </summary>
		public static string GetPackageFullNameFromManifest(string PathToManifest)
		{
			GDKGameConfigInfo Manifest = new(new FileReference(PathToManifest));
			return Manifest.PackageFullName;
		}

		/// <summary>
		/// Returns the package family name from the given manifest. e.g. GameName_z844xmwvfmpty
		/// </summary>
		public static string GetPackageFamilyNameFromManifest(string PathToManifest)
		{
			GDKGameConfigInfo Manifest = new(new FileReference(PathToManifest));
			return Manifest.PackageFamilyName;
		}

		/// <summary>
		/// Returns the AUMID for the given manifest and executable file name. e.g. GameName_z844xmwvfmpty!AppGameNameDevelopment
		/// </summary>
		public static string GetAUMIDFromManifest(string PathToManifest, string ExecutableName, string? AlternativeExecutableName = null)
		{
			GDKGameConfigInfo Manifest = new(new FileReference(PathToManifest));
			string? AUMID = Manifest.GetAUMIDForExecutable(ExecutableName);
			if (AUMID == null && AlternativeExecutableName != null)
			{
				AUMID = Manifest.GetAUMIDForExecutable(AlternativeExecutableName);
			}
			if (AUMID == null)
			{
				throw new Exception($"Unable to find AUMID for Executable {ExecutableName} {AlternativeExecutableName ?? ""} in manifest {PathToManifest}");
			}

			return AUMID;
		}

		/// <summary>
		/// Returns the AUMID that matches the given configuration from a list of AUMIDs
		/// </summary>
		public static string GetAUMIDForConfiguration(IEnumerable<string> AUMIDs, UnrealTargetConfiguration Configuration)
		{
			if (!AUMIDs.Any())
			{
				throw new Exception("No AUMIDs");
			}

			string? AUMID = AUMIDs.FirstOrDefault(A => A.EndsWith(Configuration.ToString(), StringComparison.Ordinal));
			// fall back on any AUMIDs that are not tagged as Dev/Test/shipping
			AUMID ??= AUMIDs.FirstOrDefault(A =>
				!A.EndsWith(@"Development", StringComparison.Ordinal) &&
				!A.EndsWith(@"Test", StringComparison.Ordinal) &&
				!A.EndsWith(@"Shipping", StringComparison.Ordinal));

			if (AUMID == null)
			{
				throw new Exception($"Unable to find AUMID for configuration {Configuration} from {String.Join(',', AUMIDs)}");
			}

			return AUMID;
		}
	}

	/// <summary>
	/// Structure containing information about an installed GDK application
	/// </summary>
	public class GDKInstalledApp
	{
		/// <summary>
		/// The type of application
		/// </summary>
		public enum AppType
		{
			/// Unknown
			Unknown,

			/// Loose folder
			Staged,

			/// Installed package
			Packaged,

			/// Network share
			NetworkShare,
		};

		internal GDKInstalledApp(JsonElement JsonPkg)
		{
			GameConfig = new GDKGameConfigInfo(JsonPkg);

			DisplayName = JsonPkg.TryGetStringProperty("DisplayName", out string? Temp) ? Temp : "unknown";
			DeployPath = JsonPkg.TryGetStringProperty("InstallPath", out Temp) || JsonPkg.TryGetStringProperty("DeployPath", out Temp) || JsonPkg.TryGetStringProperty("Network", out Temp) ? Temp : "";

			Type = AppType.Unknown;
			IsRegistered = false;
			if (JsonPkg.TryGetStringProperty("DeployType", out Temp))
			{
				string[] PackageTypes = { "MSIXVC", "Package", "Install" };
				string[] StagedTypes = { "Loose", "Folder" };
				string[] NetworkTypes = { "Network" };
				if (PackageTypes.Any(X => Temp.Contains(X, StringComparison.OrdinalIgnoreCase)))
				{
					Type = AppType.Packaged;
				}
				else if (StagedTypes.Any(X => Temp.Contains(X, StringComparison.OrdinalIgnoreCase)))
				{
					Type = AppType.Staged;
				}
				else if (NetworkTypes.Any(X => Temp.Contains(X, StringComparison.OrdinalIgnoreCase)))
				{
					Type = AppType.NetworkShare;
				}
				IsRegistered = !Temp.Contains("Unregistered", StringComparison.OrdinalIgnoreCase);
			}
		}

		/// core MicrosoftGame.config information
		public GDKGameConfigInfo GameConfig { get; private set; }

		/// deployment type - staged or packaged
		public AppType Type { get; private set; }

		/// registration state
		public bool IsRegistered { get; private set; }

		/// display name
		public string DisplayName { get; private set; }

		/// deployment path
		public string DeployPath { get; private set; }

		/// <inheritdoc/>
		public override string ToString()
		{
			return $"{GameConfig.PackageFullName} ({Type})";
		}

		/// <summary>
		/// Returns a list of all applications from the given app list /json output
		/// </summary>
		/// <param name="JsonAppList"></param>
		/// <returns>List of all applications</returns>
		public static IEnumerable<GDKInstalledApp> ParseAppList(JsonDocument JsonAppList)
		{
			foreach (JsonElement JsonPkg in JsonAppList.RootElement.GetProperty("Packages").EnumerateArray())
			{
				yield return new GDKInstalledApp(JsonPkg);
			}
		}
	}
}
