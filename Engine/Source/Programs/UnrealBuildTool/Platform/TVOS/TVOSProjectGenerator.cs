// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace UnrealBuildTool
{
	class TVOSProjectGenerator : IOSProjectGenerator
	{
		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="Arguments">Command line arguments passed to the project generator</param>
		/// <param name="Logger">Logger for output</param>
		public TVOSProjectGenerator(CommandLineArguments Arguments, ILogger Logger)
			: base(Arguments, Logger)
		{
		}

		/// <summary>
		/// Enumerate all the platforms that this generator supports
		/// </summary>
		public override IEnumerable<UnrealTargetPlatform> GetPlatforms()
		{
			yield return UnrealTargetPlatform.TVOS;
		}
	}
}
