// Copyright Epic Games, Inc. All Rights Reserved.

using Microsoft.Extensions.Logging;

namespace UnrealBuildTool
{
	class TVOSToolChain : IOSToolChain
	{
		protected override string XnuPlatformDefine => "XNU_PLATFORM_AppleTVOS";

		public TVOSToolChain(ReadOnlyTargetRules InTarget, TVOSProjectSettings InProjectSettings, ILogger InLogger)
			: base(InTarget, InProjectSettings, () => new TVOSToolChainSettings(InProjectSettings, InLogger), ClangToolChainOptions.None, InLogger)
		{
		}
	}
}
