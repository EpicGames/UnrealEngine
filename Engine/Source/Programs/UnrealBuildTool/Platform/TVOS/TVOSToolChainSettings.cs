// Copyright Epic Games, Inc. All Rights Reserved.

using Microsoft.Extensions.Logging;

namespace UnrealBuildTool
{
	class TVOSToolChainSettings : IOSToolChainSettings
	{
		public TVOSToolChainSettings(IOSProjectSettings ProjectSettings, ILogger Logger) 
			: base("AppleTVOS", "AppleTVSimulator", "tvos", ProjectSettings, Logger)
		{
		}
	}
}
