// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace UnrealBuildTool
{
	class TVOSProjectSettings : IOSProjectSettings
	{
		/// <summary>
		/// which devices the game is allowed to run on
		/// </summary>
		public override string RuntimeDevices => "3";

		public TVOSProjectSettings(FileReference? ProjectFile, string? Bundle)
			: base(ProjectFile, UnrealTargetPlatform.TVOS, Bundle)
		{
		}
	}

	class TVOSProvisioningData : IOSProvisioningData
	{
		public TVOSProvisioningData(TVOSProjectSettings ProjectSettings, bool bForDistribution, ILogger Logger)
			: base(ProjectSettings, true, bForDistribution, Logger)
		{
		}
	}

	class TVOSPlatform : IOSPlatform
	{
		public TVOSPlatform(UEBuildPlatformSDK InSDK, ILogger Logger)
			: base(InSDK, UnrealTargetPlatform.TVOS, Logger)
		{
		}

		public override void ValidateTarget(TargetRules Target)
		{
			base.ValidateTarget(Target);

			// make sure we add Metal, in case base class got it wrong
			if (Target.GlobalDefinitions.Contains("HAS_METAL=0"))
			{
				Target.GlobalDefinitions.Remove("HAS_METAL=0");
				Target.GlobalDefinitions.Add("HAS_METAL=1");
				Target.ExtraModuleNames.Add("MetalRHI");
			}
		}

		public new TVOSProjectSettings ReadProjectSettings(FileReference? ProjectFile, string Bundle = "")
		{
			return (TVOSProjectSettings)base.ReadProjectSettings(ProjectFile, Bundle);
		}

		protected override IOSProjectSettings CreateProjectSettings(FileReference? ProjectFile, string? Bundle)
		{
			return new TVOSProjectSettings(ProjectFile, Bundle);
		}

		public TVOSProvisioningData ReadProvisioningData(TVOSProjectSettings ProjectSettings, bool bForDistribution = false)
		{
			return (TVOSProvisioningData)base.ReadProvisioningData(ProjectSettings, bForDistribution);
		}

		protected override IOSProvisioningData CreateProvisioningData(IOSProjectSettings ProjectSettings, bool bForDistribution)
		{
			return new TVOSProvisioningData((TVOSProjectSettings)ProjectSettings, bForDistribution, Logger);
		}

		public override void ModifyModuleRulesForOtherPlatform(string ModuleName, ModuleRules Rules, ReadOnlyTargetRules Target)
		{
			base.ModifyModuleRulesForOtherPlatform(ModuleName, Rules, Target);

			// don't do any target platform stuff if SDK is not available
			if (!UEBuildPlatform.IsPlatformAvailableForTarget(Platform, Target))
			{
				return;
			}

			if ((Target.Platform == UnrealTargetPlatform.Win64) || (Target.Platform == UnrealTargetPlatform.Mac))
			{
				// allow standalone tools to use targetplatform modules, without needing Engine
				if (Target.bForceBuildTargetPlatforms)
				{
					// @todo tvos: Make the module
					// InModule.AddPlatformSpecificDynamicallyLoadedModule("TVOSTargetPlatform");
				}
			}
		}

		/// <summary>
		/// Setup the target environment for building
		/// </summary>
		/// <param name="Target">Settings for the target being compiled</param>
		/// <param name="CompileEnvironment">The compile environment for this target</param>
		/// <param name="LinkEnvironment">The link environment for this target</param>
		public override void SetUpEnvironment(ReadOnlyTargetRules Target, CppCompileEnvironment CompileEnvironment, LinkEnvironment LinkEnvironment)
		{
			base.SetUpEnvironment(Target, CompileEnvironment, LinkEnvironment);
			CompileEnvironment.Definitions.Add("PLATFORM_TVOS=1");
			CompileEnvironment.Definitions.Add("UE_PLATFORM_TVOS=1");

			// TVOS uses only IOS header files, so use it's platform headers
			CompileEnvironment.Definitions.Add("OVERRIDE_PLATFORM_HEADER_NAME=IOS");

		}

		/// <summary>
		/// Creates a toolchain instance for the given platform.
		/// </summary>
		/// <param name="Target">The target being built</param>
		/// <returns>New toolchain instance.</returns>
		public override UEToolChain CreateToolChain(ReadOnlyTargetRules Target)
		{
			TVOSProjectSettings ProjectSettings = ((TVOSPlatform)UEBuildPlatform.GetBuildPlatform(UnrealTargetPlatform.TVOS)).ReadProjectSettings(Target.ProjectFile);
			return new TVOSToolChain(Target, ProjectSettings, Logger);
		}

		public override void Deploy(TargetReceipt Receipt)
		{
			new UEDeployTVOS(Logger).PrepTargetForDeployment(Receipt);
		}
	}

	class TVOSPlatformFactory : UEBuildPlatformFactory
	{
		public override UnrealTargetPlatform TargetPlatform => UnrealTargetPlatform.TVOS;

		/// <summary>
		/// Register the platform with the UEBuildPlatform class
		/// </summary>
		public override void RegisterBuildPlatforms(ILogger Logger)
		{
			ApplePlatformSDK SDK = new ApplePlatformSDK(Logger);

			// Register this build platform for IOS
			UEBuildPlatform.RegisterBuildPlatform(new TVOSPlatform(SDK, Logger), Logger);
			UEBuildPlatform.RegisterPlatformWithGroup(UnrealTargetPlatform.TVOS, UnrealPlatformGroup.Apple);
			UEBuildPlatform.RegisterPlatformWithGroup(UnrealTargetPlatform.TVOS, UnrealPlatformGroup.IOS);
		}
	}
}

