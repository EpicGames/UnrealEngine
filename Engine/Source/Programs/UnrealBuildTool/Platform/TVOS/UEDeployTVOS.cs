// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;

using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace UnrealBuildTool
{
	class UEDeployTVOS : UEDeployIOS
	{
		public UEDeployTVOS(ILogger InLogger)
			: base(InLogger)
		{
		}

		protected override string GetTargetPlatformName()
		{
			return "TVOS";
		}

		public override bool GeneratePList(FileReference? ProjectFile, UnrealTargetConfiguration Config, string ProjectDirectory, bool bIsUnrealGame, string GameName, bool bIsClient, string ProjectName, string InEngineDir, string AppDirectory, List<string> UPLScripts, string? BundleID, bool bBuildAsFramework)
		{
			if (AppleExports.CreatingAppOnWindows(ProjectFile))
			{
				return AppleOnWindowsAppMaker.GenerateTVOSPList(ProjectDirectory, bIsUnrealGame, GameName, bIsClient, ProjectName, InEngineDir, AppDirectory, BundleID);
			}

			return base.GeneratePList(ProjectFile, Config, ProjectDirectory, bIsUnrealGame, GameName, bIsClient, ProjectName, InEngineDir, AppDirectory, UPLScripts, BundleID, bBuildAsFramework);		
		}
	}
}
