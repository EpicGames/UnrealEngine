// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.Linq;
using System.Runtime.InteropServices;
using System.Text.RegularExpressions;
using System.Xml.Linq;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// Information about an executable in an AppX manifest file
	/// </summary>
	public struct AppXManifestExecutable
	{
		/// Constructor
		public AppXManifestExecutable(UnrealTargetConfiguration InConfiguration, string InExecutablePath, UnrealArch? InArchitecture = null)
		{
			Configuration = InConfiguration;
			ExecutablePath = InExecutablePath;
			Architecture = InArchitecture;
		}

		/// The build configuration
		public readonly UnrealTargetConfiguration Configuration;

		/// The path to the executable - relative or absolute
		public readonly string ExecutablePath;

		/// The executable architecture (optional)
		public readonly UnrealArch? Architecture;
	};

	/// <summary>
	/// Base class for VC appx manifest generation
	/// </summary>
	public abstract class AppXManifestGeneratorBase
	{
#pragma warning disable CA1707 // Identifiers should not contain underscores - public facing API
		/// config section for platform-specific target settings
		protected virtual string IniSection_PlatformTargetSettings => String.Format("/Script/{0}PlatformEditor.{0}TargetSettings", Platform.ToString());

		/// config section for  platform-specific general target settings (i.e. settings with are unrelated to manifest generation)
		protected virtual string? IniSection_GeneralPlatformSettings => null;

		/// config section for general target settings
		protected virtual string IniSection_GeneralProjectSettings => "/Script/EngineSettings.GeneralProjectSettings";
#pragma warning restore CA1707

		/// default subdirectory for build resources
		protected const string BuildResourceSubPath = "Resources";

		/// default subdirectory for engine resources
		protected const string EngineResourceSubPath = "DefaultImages";

		/// Manifest compliance values
		protected const int MaxResourceEntries = 200;

		/// cached engine ini
		protected ConfigHierarchy? EngineIni;

		/// cached game ini
		protected ConfigHierarchy? GameIni;

		/// AppX package resource generator
		protected UEAppXResources? AppXResources;

		/// the default culture to use
		protected string? DefaultAppXCultureId;

		/// lookup table for UE's CultureToStage StageId to AppX CultureId
		protected Dictionary<string, string> UEStageIdToAppXCultureId = new Dictionary<string, string>();

		/// the platform to generate the manifest for
		protected UnrealTargetPlatform Platform;

		/// project file to use
		protected FileReference? ProjectFile;

		/// target name to use
		protected string? TargetName;

		/// Logger for output
		protected readonly ILogger Logger;

		/// CustomConfig to use when reading the ini files
		public string CustomConfig = "";

		/// <summary>
		/// Create a manifest generator for the given platform variant.
		/// </summary>
		protected AppXManifestGeneratorBase(UnrealTargetPlatform InPlatform, ILogger InLogger)
		{
			Platform = InPlatform;
			Logger = InLogger;
		}

		/// <summary>
		/// Returns a valid version of the given package version string
		/// </summary>
		protected string? ValidatePackageVersion(string InVersionNumber)
		{
			string WorkingVersionNumber = Regex.Replace(InVersionNumber, "[^.0-9]", "");
			string CompletedVersionString = "";
			if (WorkingVersionNumber != null)
			{
				string[] SplitVersionString = WorkingVersionNumber.Split(new char[] { '.' });
				int NumVersionElements = Math.Min(4, SplitVersionString.Length);
				for (int VersionElement = 0; VersionElement < NumVersionElements; VersionElement++)
				{
					string QuadElement = SplitVersionString[VersionElement];
					if (QuadElement.Length == 0 || !Int32.TryParse(QuadElement, out int QuadValue))
					{
						CompletedVersionString += "0";
					}
					else
					{
						if (QuadValue < 0)
						{
							QuadValue = 0;
						}
						if (QuadValue > 65535)
						{
							QuadValue = 65535;
						}
						CompletedVersionString += QuadValue;
					}
					if (VersionElement < 3)
					{
						CompletedVersionString += ".";
					}
				}
				for (int VersionElement = NumVersionElements; VersionElement < 4; VersionElement++)
				{
					CompletedVersionString += "0";
					if (VersionElement < 3)
					{
						CompletedVersionString += ".";
					}
				}
			}
			if (CompletedVersionString == null || CompletedVersionString.Length <= 0)
			{
				Logger.LogError("Invalid package version {Ver}. Package versions must be in the format #.#.#.# where # is a number 0-65535.", InVersionNumber);
				Logger.LogError("Consider setting [{IniSection}]:PackageVersion to provide a specific value.", IniSection_PlatformTargetSettings);
			}
			return CompletedVersionString;
		}

		/// <summary>
		/// Returns a valid version of the given application id
		/// </summary>
		protected string? ValidateProjectBaseName(string InApplicationId)
		{
			string ReturnVal = Regex.Replace(InApplicationId, "[^A-Za-z0-9]", "");
			if (ReturnVal != null)
			{
				// Remove any leading numbers (must start with a letter)
				ReturnVal = Regex.Replace(ReturnVal, "^[0-9]*", "");
			}
			if (ReturnVal == null || ReturnVal.Length <= 0)
			{
				Logger.LogError("Invalid application ID {AppId}. Application IDs must only contain letters and numbers. And they must begin with a letter.", InApplicationId);
			}
			return ReturnVal;
		}

		/// <summary>
		/// Reads an integer from the cached ini files
		/// </summary>
		[return: NotNullIfNotNull("DefaultValue")]
		protected string? ReadIniString(string? Key, string Section, string? DefaultValue = null)
		{
			if (Key == null)
			{
				return DefaultValue;
			}

			string Value;
			if (GameIni!.GetString(Section, Key, out Value) && !String.IsNullOrWhiteSpace(Value))
			{
				return Value;
			}

			if (EngineIni!.GetString(Section, Key, out Value) && !String.IsNullOrWhiteSpace(Value))
			{
				return Value;
			}

			return DefaultValue;
		}

		/// <summary>
		/// Reads a string from the cached ini files
		/// </summary>
		[return: NotNullIfNotNull("DefaultValue")]
		protected string? GetConfigString(string PlatformKey, string? GenericKey, string? DefaultValue = null)
		{
			string? GeneralPlatformValue = (IniSection_GeneralPlatformSettings != null) ? ReadIniString(PlatformKey, IniSection_GeneralPlatformSettings) : null;
			string? GenericValue = ReadIniString(GenericKey, IniSection_GeneralProjectSettings, DefaultValue);
			return GeneralPlatformValue ?? ReadIniString(PlatformKey, IniSection_PlatformTargetSettings, GenericValue);
		}

		/// <summary>
		/// Reads a bool from the cached ini files
		/// </summary>
		protected bool GetConfigBool(string PlatformKey, string? GenericKey, bool DefaultValue = false)
		{
			string? GeneralPlatformValue = (IniSection_GeneralPlatformSettings != null) ? ReadIniString(PlatformKey, IniSection_GeneralPlatformSettings) : null;
			string? GenericValue = ReadIniString(GenericKey, IniSection_GeneralProjectSettings, null);
			string? ResultStr = GeneralPlatformValue ?? ReadIniString(PlatformKey, IniSection_PlatformTargetSettings, GenericValue);

			if (ResultStr == null)
			{
				return DefaultValue;
			}

			ResultStr = ResultStr.Trim().ToUpperInvariant();

			return ResultStr == "TRUE" || ResultStr == "1" || ResultStr == "YES";
		}

		/// <summary>
		/// Reads a color from the cached ini files
		/// </summary>
		protected string GetConfigColor(string PlatformConfigKey, string DefaultValue)
		{
			string? ConfigValue = GetConfigString(PlatformConfigKey, null, null);
			if (ConfigValue == null)
			{
				return DefaultValue;
			}

			Dictionary<string, string>? Pairs;
			int R, G, B;
			if (ConfigHierarchy.TryParse(ConfigValue, out Pairs) &&
				Int32.TryParse(Pairs["R"], out R) &&
				Int32.TryParse(Pairs["G"], out G) &&
				Int32.TryParse(Pairs["B"], out B))
			{
				return "#" + R.ToString("X2") + G.ToString("X2") + B.ToString("X2");
			}

			Logger.LogWarning("Failed to parse color config value. Using default.");
			return DefaultValue;
		}

		/// <summary>
		/// Create all the localization data. Returns whether there is any per-culture data set up
		/// </summary>
		protected virtual bool BuildLocalizationData()
		{
			bool bHasPerCultureResources = false;

			// reset per-culture strings and make sure the default culture entry exists
			AppXResources!.ClearStrings();

			// add all default strings
			if (EngineIni!.GetString(IniSection_PlatformTargetSettings, "CultureStringResources", out string DefaultCultureScratchValue) && ConfigHierarchy.TryParse(DefaultCultureScratchValue, out Dictionary<string, string>? DefaultStrings))
			{
				AppXResources!.AddDefaultStrings(DefaultStrings);
			}

			// read StageId overrides
			if (EngineIni!.GetString(IniSection_PlatformTargetSettings, "StageIdOverrides", out string? StageIdOverridesString) &&
				ConfigHierarchy.TryParseAsMap(StageIdOverridesString, out Dictionary<string, string>? StageIdOverrides))
			{
				UEStageIdToAppXCultureId = StageIdOverrides;
				AppXResources!.AddCultures(UEStageIdToAppXCultureId.Values);
			}

			// add per culture strings
			if (EngineIni.GetArray(IniSection_PlatformTargetSettings, "PerCultureResources", out List<string>? PerCultureResources))
			{
				bHasPerCultureResources = true;

				foreach (string CultureResources in PerCultureResources)
				{
					if (!ConfigHierarchy.TryParse(CultureResources, out Dictionary<string, string>? CultureProperties)
						|| !CultureProperties.ContainsKey("CultureStringResources")
						|| !CultureProperties.ContainsKey("StageId"))
					{
						Logger.LogWarning("Invalid per-culture resource value: {Culture}", CultureResources);
						continue;
					}

					string StageId = CultureProperties["StageId"];
					if (String.IsNullOrEmpty(StageId))
					{
						Logger.LogWarning("Missing StageId value: {Culture}", CultureResources);
						continue;
					}

					string CultureId;
					if (UEStageIdToAppXCultureId.ContainsKey(StageId))
					{
						CultureId = UEStageIdToAppXCultureId[StageId];
					}
					else
					{
						CultureId = StageId;
						UEStageIdToAppXCultureId[StageId] = CultureId;
						AppXResources.AddCulture(CultureId);
					}

					// read culture strings
					if (!ConfigHierarchy.TryParse(CultureProperties["CultureStringResources"], out Dictionary<string, string>? CultureStringResources))
					{
						Logger.LogError("Invalid culture string resources: \"{Culture}\". Unable to add resource entry.", CultureResources);
						continue;
					}
					AppXResources!.AddCultureStrings(CultureId, CultureStringResources);
				}
			}

			return bHasPerCultureResources;
		}

		/// <summary>
		/// Register the locations where resource binary files can be found
		/// </summary>
		protected virtual void PrepareResourceBinaryPaths()
		{
			if (ProjectFile != null)
			{
				AppXResources!.ProjectBinaryResourceDirectories.Add(DirectoryReference.Combine(ProjectFile.Directory, "Build", Platform.ToString(), BuildResourceSubPath));
				AppXResources!.ProjectBinaryResourceDirectories.Add(DirectoryReference.Combine(ProjectFile.Directory, "Platforms", Platform.ToString(), "Build", BuildResourceSubPath));
			}

			AppXResources!.EngineFallbackBinaryResourceDirectories.Add(DirectoryReference.Combine(Unreal.EngineDirectory, "Build", Platform.ToString(), EngineResourceSubPath));
			AppXResources!.EngineFallbackBinaryResourceDirectories.Add(DirectoryReference.Combine(Unreal.EngineDirectory, "Platforms", Platform.ToString(), "Build", EngineResourceSubPath));
		}

		/// <summary>
		/// Get the resources element
		/// </summary>
		protected XElement GetResources()
		{
			List<string> ResourceCulturesList = AppXResources!.GetAllCultureIds().ToList();

			// Move the default culture to the front of the list
			ResourceCulturesList.Remove(DefaultAppXCultureId!);
			ResourceCulturesList.Insert(0, DefaultAppXCultureId!);

			// Check that we have a valid number of cultures
			if (ResourceCulturesList!.Count < 1 || ResourceCulturesList.Count >= MaxResourceEntries)
			{
				Logger.LogWarning("Incorrect number of cultures to stage. There must be between 1 and {MaxCultures} cultures selected.", MaxResourceEntries);
			}

			// Create the culture list. This list is unordered except that the default language must be first which we already took care of above.
			IEnumerable<XElement> CultureElements = ResourceCulturesList.Select(c =>
				new XElement("Resource", new XAttribute("Language", c)));

			return new XElement("Resources", CultureElements);
		}

		/// <summary>
		/// Get the package identity name string
		/// </summary>
		protected virtual string GetIdentityPackageName()
		{
			// Read the PackageName from config
			string DefaultName = (ProjectFile != null) ? ProjectFile.GetFileNameWithoutAnyExtensions() : (TargetName ?? "DefaultUEProject");
			string PackageName = Regex.Replace(GetConfigString("PackageName", "ProjectName", DefaultName), "[^-.A-Za-z0-9]", "");
			if (String.IsNullOrWhiteSpace(PackageName))
			{
				Logger.LogError("Invalid package name {Name}. Package names must only contain letters, numbers, dash, and period and must be at least one character long.", PackageName);
				Logger.LogError("Consider using the setting [{IniSection}]:PackageName to provide a specific value.", IniSection_PlatformTargetSettings);
			}

			// If specified in the project settings append the users machine name onto the package name to allow sharing of devkits without stomping of deploys
			bool bPackageNameUseMachineName;
			if (EngineIni!.GetBool(IniSection_PlatformTargetSettings, "bPackageNameUseMachineName", out bPackageNameUseMachineName) && bPackageNameUseMachineName)
			{
				string MachineName = Regex.Replace(Unreal.MachineName, "[^-.A-Za-z0-9]", "");
				PackageName = PackageName + ".NOT.SHIPPABLE." + MachineName;
			}

			return PackageName;
		}

		/// <summary>
		/// Get the publisher name string
		/// </summary>
		protected virtual string GetIdentityPublisherName()
		{
			string PublisherName = GetConfigString("PublisherName", "CompanyDistinguishedName", "CN=NoPublisher");
			return PublisherName;
		}

		/// <summary>
		/// Get the package version string
		/// </summary>
		protected virtual string? GetIdentityVersionNumber()
		{
			string? VersionNumber = GetConfigString("PackageVersion", "ProjectVersion", "1.0.0.0");
			VersionNumber = ValidatePackageVersion(VersionNumber);

			// If specified in the project settings attempt to retrieve the current build number and increment the version number by that amount, accounting for overflows
			bool bIncludeEngineVersionInPackageVersion;
			if (EngineIni!.GetBool(IniSection_PlatformTargetSettings, "bIncludeEngineVersionInPackageVersion", out bIncludeEngineVersionInPackageVersion) && bIncludeEngineVersionInPackageVersion)
			{
				VersionNumber = IncludeBuildVersionInPackageVersion(VersionNumber);
			}

			return VersionNumber;
		}

		/// <summary>
		/// Get the package identity element
		/// </summary>
		protected XElement GetIdentity(out string IdentityName)
		{
			string PackageName = GetIdentityPackageName();
			string PublisherName = GetIdentityPublisherName();
			string? VersionNumber = GetIdentityVersionNumber();

			IdentityName = PackageName;

			return new XElement("Identity",
				new XAttribute("Name", PackageName),
				new XAttribute("Publisher", PublisherName),
				new XAttribute("Version", VersionNumber!));
		}

		/// <summary>
		/// Updates the given package version to include the engine build version, if requested
		/// </summary>
		protected virtual string? IncludeBuildVersionInPackageVersion(string? VersionNumber)
		{
			BuildVersion? BuildVersionForPackage;
			if (VersionNumber != null && BuildVersion.TryRead(BuildVersion.GetDefaultFileName(), out BuildVersionForPackage) && BuildVersionForPackage.Changelist != 0)
			{
				// Break apart the version number into individual elements
				string[] SplitVersionString = VersionNumber.Split('.');
				VersionNumber = String.Format("{0}.{1}.{2}.{3}",
					SplitVersionString[0],
					SplitVersionString[1],
					BuildVersionForPackage.Changelist / 10000,
					BuildVersionForPackage.Changelist % 10000);
			}

			return VersionNumber;
		}

		/// <summary>
		/// Get the path to the makepri.exe tool
		/// </summary>
		protected virtual FileReference GetMakePriBinaryPath()
		{
			if (!RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
			{
				throw new BuildException("unsupported platform");
			}			
			
			if (!MicrosoftPlatformSDK.TryGetWindowsSdkDir(null, Logger, out VersionNumber? SdkVersion, out DirectoryReference? SdkDir))
			{
				throw new BuildException("Cannot get default Windows Sdk directory");
			}

			FileReference MakePriPath = FileReference.Combine(SdkDir, "bin", SdkVersion.ToString(), "x64", "makepri.exe");
			if (!FileReference.Exists(MakePriPath))
			{
				throw new BuildException($"{MakePriPath} - file not found");
			}

			return MakePriPath;
		}

		/// <summary>
		/// Get any additional platform-specific parameters for makepri.exe
		/// </summary>
		protected virtual string GetMakePriExtraCommandLine()
		{
			return "";
		}

		/// <summary>
		/// Return the entire manifest element
		/// </summary>
		protected abstract XElement GetManifest(IEnumerable<AppXManifestExecutable> InExecutables, out string IdentityName);

		/// <summary>
		/// Perform any platform-specific processing on the manifest before it is saved
		/// </summary>
		protected virtual void ProcessManifest(IEnumerable<AppXManifestExecutable> InExecutables, string ManifestName, string ManifestTargetPath, string ManifestIntermediatePath)
		{
		}

		/// <summary>
		/// Perform any additional initialization once all parameters and configuration are ready
		/// </summary>
		protected virtual void PostConfigurationInit()
		{
		}

		/// <summary>
		/// Create a manifest and return the list of modified files
		/// </summary>
		public List<string>? CreateManifest(string InManifestName, DirectoryReference OutputDirectory, string? InTargetName, FileReference? InProjectFile, IEnumerable<AppXManifestExecutable> InExecutables)
		{
			FileUtils.CreateDirectoryTree(OutputDirectory);

			DirectoryReference ProjectRoot = (InProjectFile != null) ? InProjectFile.Directory : Unreal.EngineDirectory;
			DirectoryReference IntermediateDirectory = DirectoryReference.Combine(ProjectRoot, "Intermediate", "Manifest", Platform.ToString());
			FileUtils.ForceDeleteDirectory(IntermediateDirectory);
			FileUtils.CreateDirectoryTree(IntermediateDirectory);

			TargetName = InTargetName;
			ProjectFile = InProjectFile;
			AppXResources = new(Logger, GetMakePriBinaryPath());
			PrepareResourceBinaryPaths();

			// Load up INI settings. We'll use engine settings to retrieve the manifest configuration, but these may reference
			// values in either game or engine settings, so we'll keep both.
			GameIni = ConfigCache.ReadHierarchy(ConfigHierarchyType.Game, DirectoryReference.FromFile(InProjectFile), Platform, CustomConfig);
			EngineIni = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, DirectoryReference.FromFile(InProjectFile), Platform, CustomConfig);
			PostConfigurationInit();

			// Load and verify/clean culture list
			List<string>? SelectedUECultureIds;
			string DefaultUECultureId;
			GameIni.GetArray("/Script/UnrealEd.ProjectPackagingSettings", "CulturesToStage", out SelectedUECultureIds);
			GameIni.GetString("/Script/UnrealEd.ProjectPackagingSettings", "DefaultCulture", out DefaultUECultureId);
			if (SelectedUECultureIds == null || SelectedUECultureIds.Count < 1)
			{
				Logger.LogError("At least one culture must be selected to stage.");
				return null;
			}
			SelectedUECultureIds = SelectedUECultureIds.Distinct().ToList();

			if (DefaultUECultureId == null || DefaultUECultureId.Length < 1)
			{
				DefaultUECultureId = SelectedUECultureIds[0];
				Logger.LogWarning("A default culture must be selected to stage. Using {DefaultCulture}.", DefaultUECultureId);
			}
			if (!SelectedUECultureIds.Contains(DefaultUECultureId))
			{
				DefaultUECultureId = SelectedUECultureIds[0];
				Logger.LogWarning("The default culture must be one of the staged cultures. Using {DefaultCulture}.", DefaultUECultureId);
			}

			BuildLocalizationData();

			// generate the list of AppX cultures to stage
			foreach (string UEStageId in SelectedUECultureIds)
			{
				if (!UEStageIdToAppXCultureId.TryGetValue(UEStageId, out string? AppXCultureId) || String.IsNullOrEmpty(AppXCultureId))
				{
					// use the culture directly - no remapping required
					AppXCultureId = UEStageId;
					UEStageIdToAppXCultureId[UEStageId] = AppXCultureId;
					AppXResources.AddCulture(UEStageId);
				}
			}

			// look up the default AppX culture
			if (!UEStageIdToAppXCultureId.TryGetValue(DefaultUECultureId, out DefaultAppXCultureId) || String.IsNullOrEmpty(DefaultAppXCultureId))
			{
				// use the default culture directly - no remapping required
				DefaultAppXCultureId = DefaultUECultureId;
				UEStageIdToAppXCultureId[DefaultUECultureId] = DefaultAppXCultureId;
			}

			// Create the manifest document
			string? IdentityName = null;
			XDocument ManifestXmlDocument = new XDocument(GetManifest(InExecutables, out IdentityName));

			// Export manifest to the intermediate directory and add it to the manifest resources files for copying
			FileReference ManifestIntermediateFile = FileReference.Combine(IntermediateDirectory, InManifestName);
			FileReference ManifestTargetFile = FileReference.Combine(OutputDirectory, InManifestName);
			ManifestXmlDocument.Save(ManifestIntermediateFile.FullName);
			AppXResources.AddFileReference(ManifestIntermediateFile, InManifestName);
			ProcessManifest(InExecutables, InManifestName, ManifestTargetFile.FullName, ManifestIntermediateFile.FullName);

			// Generate the package resource index and copy all resource files to the output
			FileReference ManifestTargetPath = FileReference.Combine(OutputDirectory, InManifestName);
			List<FileReference> UpdatedFiles = AppXResources.GenerateAppXResources(OutputDirectory, IntermediateDirectory, ManifestTargetFile, DefaultAppXCultureId, IdentityName);

			// Clean up and reutrn the list of updated files
			FileUtils.ForceDeleteDirectory(IntermediateDirectory);
			return UpdatedFiles.ConvertAll(X => X.FullName);
		}
	}
}
