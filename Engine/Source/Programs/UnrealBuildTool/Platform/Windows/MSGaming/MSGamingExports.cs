// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using Microsoft.Win32;
using System;
using System.Collections.Generic;
using System.Linq;
using System.Runtime.Versioning;

namespace UnrealBuildTool
{
	/// <summary>
	/// Public MSGaming functions exposed to UAT
	/// </summary>
	public static class MSGamingExports
	{
		#region custom install actions

		/// Type of custom install action
		public enum CustomInstallActionType
		{
			/// command for installing
			Install,
			/// command for repairing
			Repair,
			/// command for uninstalling
			Uninstall,
		};

#pragma warning disable CA1034 // Nested types should not be visible - preserve API of APIs exposed to UAT
		/// a custom install action
		public struct CustomInstallAction
		{
			/// location to copy the custom installs to when staging
			private const string DefaultTargetRootFolder = "CustomInstall";

			/// name of the install action
			public string Name;

			/// source folder, relative to engine root
			public DirectoryReference SourceFolder;

			/// target folder, relative to targe root folder
			public string TargetFolder;

			/// a command to run and its arguments
			public struct InstallCommand
			{
				/// command to run
				public FileReference Command;

				/// arguments to pass to the given command
				public string Arguments;
			};

			/// the commands to run for each action
			public Dictionary<CustomInstallActionType, InstallCommand> Commands;

			/// read the given config key and return the actions
			public static IEnumerable<CustomInstallAction> GetFromConfig(ConfigHierarchy EngineIni, string IniSection)
			{
				List<CustomInstallAction> CustomInstallActions = new();

				if (EngineIni!.TryGetValuesGeneric(IniSection, "CustomInstallActions", out CustomInstallAction[]? CustomInstallActionsArray) && CustomInstallActionsArray != null)
				{
					// set the default target folder
					for (int i = 0; i < CustomInstallActionsArray.Length; i++)
					{
						if (String.IsNullOrEmpty(CustomInstallActionsArray[i].TargetFolder))
						{
							CustomInstallActionsArray[i].TargetFolder = CustomInstallActionsArray[i].Name;
						}
						else if (CustomInstallActionsArray[i].TargetFolder == ".")
						{
							CustomInstallActionsArray[i].TargetFolder = "";
						}
					}

					CustomInstallActions.AddRange(CustomInstallActionsArray);
				}

				// inject the GameInput dependency if requested
				if (EngineIni.GetBool(IniSection, "bAlwaysIncludeGameInputDependency", out bool bAlwaysIncludeGameInputDependency) && bAlwaysIncludeGameInputDependency)
				{
					DirectoryReference GameInputRedistDir = DirectoryReference.Combine(UnrealBuildBase.Unreal.EngineDirectory, @"Extras\Redist\en-us\");
					FileReference GameInputRedist = FileReference.Combine(GameInputRedistDir, "GameInputRedist.msi");

					InstallCommand MSICommand = new()
					{
						Command = GameInputRedist,
						Arguments = ""
					};

					CustomInstallAction GameInputAction = new();
					GameInputAction.Name = "GameInput";
					GameInputAction.SourceFolder = GameInputRedistDir;
					GameInputAction.TargetFolder = "GameInput";
					GameInputAction.Commands = new()
					{
						{ CustomInstallActionType.Install, MSICommand },
						{ CustomInstallActionType.Uninstall, MSICommand },
						{ CustomInstallActionType.Repair, MSICommand }
					};

					CustomInstallActions.Add(GameInputAction);
				}

				return CustomInstallActions.ToArray();
			}

			/// the target root folder for staging custom install actions, relative to the package root
			public static string GetTargetRootFolder(ConfigHierarchy EngineIni, string IniSection)
			{
				if (EngineIni.TryGetValue(IniSection, "CustomInstallStagingRoot", out string? TargetRootFolder ) && !String.IsNullOrEmpty(TargetRootFolder))
				{
					return TargetRootFolder;
				}
				else
				{
					return DefaultTargetRootFolder;
				}
			}
		}
#pragma warning restore CA1034

		#endregion

		/// <summary>
		/// Determine whether Windows is in developer mode. PC GDK packages can't be sideloaded without it
		/// </summary>
		[SupportedOSPlatform("windows")]
		public static bool IsWindowsDeveloperMode()
		{
			try
			{
				// normal windows key
				using RegistryKey? Key = Registry.LocalMachine.OpenSubKey(@"SOFTWARE\Microsoft\Windows\CurrentVersion\AppModelUnlock");
				if (Key != null && Convert.ToInt64(Key.GetValue("AllowDevelopmentWithoutDevLicense", 0L)) != 0)
				{
					return true;
				}

				// group policy key
				using RegistryKey? GroupKey = Registry.LocalMachine.OpenSubKey(@"SOFTWARE\Policies\Microsoft\Windows\Appx");
				if (GroupKey != null && Convert.ToInt64(GroupKey.GetValue("AllowDevelopmentWithoutDevLicense", 0L)) != 0)
				{
					return true;
				}
			}
			catch (Exception)
			{
			}

			return false;
		}

		/// <summary>
		/// Determine if the Microsoft.GamingServices package is installed.
		/// </summary>
		[SupportedOSPlatform("windows")]
		public static bool IsMSGamingRuntimeInstalled()
		{
			try
			{
				using RegistryKey? Key = Registry.CurrentUser.OpenSubKey(@"SOFTWARE\Classes\Local Settings\Software\Microsoft\Windows\CurrentVersion\AppModel\Repository\Packages");
				if (Key != null)
				{
					return Key.GetSubKeyNames().Any( X => X.StartsWith("Microsoft.GamingServices_", StringComparison.Ordinal));
				}
			}
			catch (Exception)
			{
			}
		
			return false;
		}

		/// <summary>
		/// Determine whether the local version of Windows can support GDK
		/// </summary>
		[SupportedOSPlatform("windows")]
		public static bool IsWindowsVersionSupported()
		{
			if (System.Environment.OSVersion.Version.Major < 10 || (System.Environment.OSVersion.Version.Major == 10 && System.Environment.OSVersion.Version.Build < 18362)) //https://en.wikipedia.org/wiki/Template:Windows_10_versions
			{
				return false;
			}

			return true;
		}
	}
}
