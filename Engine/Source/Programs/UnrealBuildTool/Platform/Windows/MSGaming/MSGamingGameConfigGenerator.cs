// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using Microsoft.Extensions.Logging;
using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Xml.Linq;
using UnrealBuildBase;
using static UnrealBuildTool.MSGamingExports;

namespace UnrealBuildTool.GDK
{
	partial struct GDKGameConfigGeneratorFactory
	{
#pragma warning disable CA1823 // Avoid unused private fields - instantiation is side-effectful, preferable to a static ctor
#pragma warning disable IDE0052
		private static readonly Type MSGamingGen = RegisterGenerator(UnrealTargetPlatform.Win64, typeof(MSGamingGameConfigGenerator));
#pragma warning restore CA1823
#pragma warning restore IDE0052
	}

	class MSGamingGameConfigGenerator : GDKGameConfigGenerator
	{
		protected override string TargetDeviceFamily => "PC";
		protected override string DefaultArchitecture => "x64";
		protected override string IniSection_PlatformTargetSettings => "/Script/MSGamingSupport.MSGamingSettings";
		protected override string? IniSection_GeneralPlatformSettings => "/Script/WindowsTargetPlatform.WindowsTargetSettings"; // for querying DefaultGraphicsRHI etc.

		protected virtual string IniPlatformName => "Windows";

		/// <summary>
		/// Create a manifest generator for Win64 using MS Gaming Runtime and MS Game Store
		/// </summary>
		public MSGamingGameConfigGenerator(ILogger InLogger)
			: base(UnrealTargetPlatform.Win64, InLogger)
		{
		}

		/// <summary>
		/// Create a manifest generator for other platforms
		/// </summary>
		internal MSGamingGameConfigGenerator(UnrealTargetPlatform InPlatform, ILogger InLogger)
			: base(InPlatform, InLogger)
		{
		}

		/// <summary>
		/// Register the locations where resource binary files can be found
		/// </summary>
		protected override void PrepareResourceBinaryPaths()
		{
			base.PrepareResourceBinaryPaths();

			// check for DLC plugin images first, in the <plugin>/Build/<platform>/MSGaming/Resources/ folder (and Platform extension equivalent)
			if (DLCFile != null)
			{
				AppXResources!.ProjectBinaryResourceDirectories.Add(DirectoryReference.Combine(DLCFile.Directory, "Build", IniPlatformName, "MSGaming", BuildResourceSubPath));
				AppXResources!.ProjectBinaryResourceDirectories.Add(DirectoryReference.Combine(DLCFile.Directory, "Platforms", IniPlatformName, "Build", "MSGaming", BuildResourceSubPath));
			}

			// check for DLC package images next, in <project>/Build/<platform>/MSGaming/<dlc>/Resources/ folder (and Platform extension equivalent)
			if (DLCPackage != null && ProjectFile != null)
			{
				AppXResources!.ProjectBinaryResourceDirectories.Add(DirectoryReference.Combine(ProjectFile.Directory, "Build", IniPlatformName, "MSGaming", DLCPackage, BuildResourceSubPath));
				AppXResources!.ProjectBinaryResourceDirectories.Add(DirectoryReference.Combine(ProjectFile.Directory, "Platforms", IniPlatformName, "Build", "MSGaming", DLCPackage, BuildResourceSubPath));
			}

			// allow projects to keep their resources in an MSGaming subfolder to help with project organization
			if (ProjectFile != null)
			{
				AppXResources!.ProjectBinaryResourceDirectories.Add(DirectoryReference.Combine(ProjectFile.Directory, "Build", IniPlatformName, "MSGaming", BuildResourceSubPath));
				AppXResources!.ProjectBinaryResourceDirectories.Add(DirectoryReference.Combine(ProjectFile.Directory, "Platforms", IniPlatformName, "Build", "MSGaming", BuildResourceSubPath));
			}

			// default images are stored under MSGamingSupport plugin folder
			DirectoryReference MSGamingSupportResourcePath = DirectoryReference.Combine(Unreal.EngineDirectory, "Build", EngineResourceSubPath);
			PluginInfo? MSGamingSupportPlugin = Plugins.GetPlugin("MSGamingSupport");
			if (MSGamingSupportPlugin != null)
			{
				MSGamingSupportResourcePath = DirectoryReference.Combine( MSGamingSupportPlugin.File.Directory, "Build", EngineResourceSubPath);
			}
			AppXResources!.EngineFallbackBinaryResourceDirectories.Add(MSGamingSupportResourcePath);

			// fallback path because Program targets do not load plugins
			AppXResources!.EngineFallbackBinaryResourceDirectories.Add(DirectoryReference.Combine(Unreal.EngineDirectory, @"Plugins\Runtime\Windows\MSGamingSupport\Build\DefaultImages"));
		}

		protected override XElement GetGame(IEnumerable<AppXManifestExecutable> InExecutables, out string IdentityName)
		{
			XElement Game = base.GetGame(InExecutables, out IdentityName);

			Game.Add(GetDesktopRegistration());

			if (!IsDLC)
			{
				if (EngineIni!.GetBool(IniSection_PlatformTargetSettings, "bMSAFullTrust", out bool bMSAFullTrust) && bMSAFullTrust)
				{
					Game.Add(new XElement(XName.Get("MSAFullTrust"), "true"));
				}
			}

			return Game;
		}

		private XElement GetDesktopRegistration()
		{
			XElement DesktopRegistration = new(XName.Get("DesktopRegistration"),
				new XElement(XName.Get("ProcessorArchitecture"), DefaultArchitecture)
				);

			if (!IsDLC)
			{
				int ManifestVersion = GetManifestVersion();

				DesktopRegistration.Add(
					new XElement(XName.Get("MultiplayerProtocol"), "true"),
					GetDependencyList(),
					GetCustomInstallActions(),
					GetWindowsOSVersion()
					);

				// Note: Manifest Version 0 has been deprecated as of October 2023 GDK and is not supported,
				// but keeping this here in case Manifest version is overridden.  
				if (ManifestVersion == 0) // ModFolder is deprecated in Manifest Version 1
				{
					if (EngineIni!.GetBool(IniSection_PlatformTargetSettings, "bUseModFolder", out bool bUseModFolder) && bUseModFolder)
					{
						if (EngineIni.GetString(IniSection_PlatformTargetSettings, "ModFolder", out string ModFolder) && !String.IsNullOrEmpty(ModFolder))
						{
							DesktopRegistration.Add(new XElement(XName.Get("ModFolder"),
								new XAttribute(XName.Get("Name"), ModFolder)
								)
							);
						}
					}
				}
			}
			else
			{
				string MainPackageIdentityName = GetMainPackageIdentityPackageName();

				DesktopRegistration.Add(new XElement(XName.Get("MainPackageDependency"),
					new XAttribute(XName.Get("Name"), MainPackageIdentityName)
					)
				);
			}

			return DesktopRegistration;
		}

		private XElement? GetDependencyList()
		{
			XElement DependencyList = new(XName.Get("DependencyList"));

			if (NoBootstrapExe)
			{
				// Without a bootstrapper, need Windows to install VC runtime libraries during package install
				// this will cause VC DLL loads to be redirected from %System32% to %ProgramFiles%\WindowsApps\Microsoft.VCLibs.140.00.UWPDesktop_14.0.32530.0_x64__8wekyb3d8bbwe
				DependencyList.Add(new XElement(XName.Get("KnownDependency"), new XAttribute(XName.Get("Name"), "VC14")));
			}

			bool bNeedsDX11 = (GetConfigString("DefaultGraphicsRHI", null) == "DefaultGraphicsRHI_DX11") ||
							  (GetConfigBool("bAlwaysIncludeDX11Dependency", null));
			if (bNeedsDX11)
			{
				// this will cause DX DLL loads to be redirected from %System32% to %ProgramFiles%\WindowsApps\Microsoft.DirectXRuntime_9.29.1974.0_x64__8wekyb3d8bbwe
				Log.TraceInformationOnce("Adding KnownDependency on DX11 redist in MicrosoftGame.config");
				DependencyList.Add(new XElement(XName.Get("KnownDependency"), new XAttribute(XName.Get("Name"), "DX11")));
			}

			return DependencyList.HasElements ? DependencyList : null;
		}

		private XElement? GetCustomInstallActions()
		{
			IEnumerable<CustomInstallAction> CustomInstallActions = CustomInstallAction.GetFromConfig(EngineIni!, IniSection_PlatformTargetSettings);
			if (!CustomInstallActions.Any())
			{
				return null;
			}

			string TargetRoot = CustomInstallAction.GetTargetRootFolder(EngineIni!, IniSection_PlatformTargetSettings);

			List<XElement> InstallActionList = new();
			List<XElement> RepairActionList = new();
			List<XElement> UninstallActionList = new();
			foreach (CustomInstallAction Action in CustomInstallActions)
			{
				foreach (KeyValuePair<CustomInstallActionType, CustomInstallAction.InstallCommand> Command in Action.Commands)
				{
					List<XAttribute> Attribs = new List<XAttribute>
					{
						new("File",      Path.Combine( Action.TargetFolder, Command.Value.Command.MakeRelativeTo(Action.SourceFolder))),
						new("Name",      Action.Name),
					};
					if (!String.IsNullOrEmpty(Command.Value.Arguments))
					{
						Attribs.Add(new("Arguments", Command.Value.Arguments));
					}
					switch (Command.Key)
					{
						case CustomInstallActionType.Install: InstallActionList.Add(new XElement("InstallAction", Attribs)); break;
						case CustomInstallActionType.Repair: RepairActionList.Add(new XElement("RepairAction", Attribs)); break;
						case CustomInstallActionType.Uninstall: UninstallActionList.Add(new XElement("UninstallAction", Attribs)); break;
					}
				}
			}

			XElement CustomInstallActionsElement = new("CustomInstallActions",
				new XElement("Folder", new XText(TargetRoot)),
				InstallActionList.Any() ? new XElement("InstallActionList", InstallActionList) : null,
				RepairActionList.Any() ? new XElement("RepairActionList", RepairActionList) : null,
				UninstallActionList.Any() ? new XElement("UninstallActionList", UninstallActionList) : null
			);
			return CustomInstallActionsElement;
		}

		private XElement? GetWindowsOSVersion()
		{
			EngineIni!.GetString(IniSection_PlatformTargetSettings, "RequiredMinimumWindowsVersion", out string RequiredMinVersion);
			EngineIni!.GetString(IniSection_PlatformTargetSettings, "SuggestedMinimumWindowsVersion", out string SuggestedMinVersion);
			EngineIni!.GetString(IniSection_PlatformTargetSettings, "RecommendedWindowsVersion", out string RecommendedVersion);

			XElement WindowsOsVersion = new(XName.Get("WindowsOsVersion"));

			if (!String.IsNullOrWhiteSpace(RequiredMinVersion))
			{
				WindowsOsVersion.SetAttributeValue("RequiredMinimum", RequiredMinVersion);
			}

			if (!String.IsNullOrWhiteSpace(SuggestedMinVersion))
			{
				WindowsOsVersion.SetAttributeValue("SuggestedMinimum", SuggestedMinVersion);
			}

			if (!String.IsNullOrWhiteSpace(RecommendedVersion))
			{
				WindowsOsVersion.SetAttributeValue("Recommended", RecommendedVersion);
			}

			// The <WindowsOsVersion> element is optional. If none of the values were configured, omit it.
			return WindowsOsVersion.HasAttributes ? WindowsOsVersion : null;
		}

		protected override string? IncludeBuildVersionInPackageVersion(string? VersionNumber)
		{
			// GDK on Windows reserves the lowest version element (Revision) for internal use so package versioning needs to be a little different

			if (VersionNumber != null && BuildVersion.TryRead(BuildVersion.GetDefaultFileName(), out BuildVersion? BuildVersionForPackage) && BuildVersionForPackage.Changelist != 0)
			{
				// Break apart the version number into individual elements
				string[] SplitVersionString = VersionNumber.Split('.');
				VersionNumber = String.Format("{0}.{1}.{2}.{3}",
					SplitVersionString[0],
					BuildVersionForPackage.Changelist / 10000,
					BuildVersionForPackage.Changelist % 10000,
					0); //must be zero
			}

			return VersionNumber;
		}
	}
}
