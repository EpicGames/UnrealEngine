// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using Microsoft.Extensions.Logging;
using System;
using System.Collections.Generic;
using System.Linq;
using UnrealBuildTool.GDK;

namespace UnrealBuildTool
{
	class MSGamingToolChain : VCToolChain
	{
		protected virtual bool SupportsContentOnlyProjects => false; // there is no content-only project support for Win64+GDK because it requires bespoke target.cs files

		public MSGamingToolChain(ReadOnlyTargetRules Target, ILogger Logger)
			: base(Target, Logger)
		{
		}

		public override void GetVersionInfo(List<string> Lines)
		{
			base.GetVersionInfo(Lines);

			string? GDKEdition = GDKExports.GetSDKVersion();
			if (GDKEdition != null)
			{
				string CurrentGDKDir = GDKExports.GetCurrentGSDKDir();
				Lines.Add($"Using GDK version {GDKEdition} ({CurrentGDKDir}).");
			}
			else
			{
				Lines.Add("GameDKCoreLatest or GameDKLatest environment variable not found - is the GDK installed?");
			}
		}

		protected override void AppendLinkArguments(LinkEnvironment LinkEnvironment, List<string> Arguments)
		{
			base.AppendLinkArguments(LinkEnvironment, Arguments);

			// do not strip the PDB path from the executable if we want to debug it (otherwise it can't find the symbols when attempting to debug an installed package build)
			if (LinkEnvironment.bCreateDebugInfo && LinkEnvironment.Configuration != CppConfiguration.Shipping)
			{
				Arguments.Remove("/PDBALTPATH:%_PDB%");
			}
		}

		public override void FinalizeOutput(ReadOnlyTargetRules Target, TargetMakefileBuilder MakefileBuilder, DirectoryReference ExeDir)
		{
			base.FinalizeOutput(Target, MakefileBuilder, ExeDir);

			// Look up the project
			FileReference? ProjectFile = Target.ProjectFile;
			if (SupportsContentOnlyProjects)
			{
				ProjectFile = GDKContentOnlyProjectSupport.GetProjectFileFromTarget(Target, Logger) ?? ProjectFile;
			}

			// Create a placeholder MicrosoftGame.config to be used during F5 debugging, and for Program targets. It does not reference executables because it could be shared between several configurations.
			// This allows the game to run while using assets from the Saved/Cooked folder, without needing -basedir= to point to a Staged build
			if ((Environment.CommandLine.Contains("-frommsbuild", StringComparison.OrdinalIgnoreCase) && ProjectFile != null) || Target.Type == TargetType.Program)
			{
				DirectoryReference ExecutableDirectory = MakefileBuilder.Makefile.ExecutableFile.Directory;
				Action GenAction = GDKGameConfigGeneratorMode.CreateAction(MakefileBuilder, Target.Platform, ExecutableDirectory, Target.Name, ProjectFile, CustomConfig:Target.CustomConfig);
				GenAction.CommandDescription = "Generate MicrosoftGame.config";
				GenAction.bCanExecuteInUBA = false; // TODO: Fails with some missing file, revisit
			}
		}

		protected override void ModifyFinalLinkArguments(LinkEnvironment LinkEnvironment, List<string> Arguments, bool bBuildImportLibraryOnly)
		{
			base.ModifyFinalLinkArguments(LinkEnvironment, Arguments, bBuildImportLibraryOnly);

			// optionally audit the GDK editions of any libs we link to
			GDKEditionAudit.VerifyLinkEnvironment(EnvVars, LinkEnvironment, bBuildImportLibraryOnly, Logger);
		}

		public override void PrepareRuntimeDependencies(List<RuntimeDependency> RuntimeDependencies, Dictionary<FileReference, FileReference> TargetFileToSourceFile, DirectoryReference ExeDir)
		{
			base.PrepareRuntimeDependencies(RuntimeDependencies, TargetFileToSourceFile, ExeDir);

			// optionally audit the GDK editions of any libs we depend on
			GDKEditionAudit.VerifyRuntimeDependencies(EnvVars, Target.Platform, TargetFileToSourceFile.Values.Distinct(), Logger);
		}

		public override void SetUpGlobalEnvironment(ReadOnlyTargetRules Target, CppCompileEnvironment GlobalCompileEnvironment, LinkEnvironment GlobalLinkEnvironment)
		{
			base.SetUpGlobalEnvironment(Target, GlobalCompileEnvironment, GlobalLinkEnvironment);

			DirectoryReference platformGDK = new(GDKExports.GetCurrentGSDKDir());
			GlobalCompileEnvironment.RootPaths.AddFolderName(CppRootPathFolder.PlatformSDK, $"{Target.Platform}SDK");
			GlobalLinkEnvironment.RootPaths.AddFolderName(CppRootPathFolder.PlatformSDK, $"{Target.Platform}SDK");
			GlobalCompileEnvironment.RootPaths[CppRootPathFolder.PlatformSDK] = platformGDK;
			GlobalLinkEnvironment.RootPaths[CppRootPathFolder.PlatformSDK] = platformGDK;

			string? currentGDKVersion = GDKExports.GetSDKVersion();
			if (currentGDKVersion != null)
			{
				EpicGames.UBA.Utils.RegisterPathHash(platformGDK, currentGDKVersion);
			}
		}
	}
}