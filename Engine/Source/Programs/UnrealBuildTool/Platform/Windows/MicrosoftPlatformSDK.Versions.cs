// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics;
using System.Linq;
using EpicGames.Core;

namespace UnrealBuildTool
{
	////////////////////////////////////////////////////////////////////////////////
	// If you are looking for version numbers, see Engine/Config/Windows/Windows_SDK.json
	////////////////////////////////////////////////////////////////////////////////

	partial class MicrosoftPlatformSDK : UEBuildPlatformSDK
	{
		private static MicrosoftPlatformSDK SDK => GetSDKForPlatformOrMakeTemp<MicrosoftPlatformSDK>("Win64")!;

		/// <summary>
		/// The default compiler version to be used, if installed. 
		/// </summary>
		static readonly VersionNumberRange[] PreferredClangVersions = SDK.GetVersionNumberRangeArrayFromConfig("PreferredClangVersions");

		/// <summary>
		/// The minimum supported Clang compiler
		/// </summary>
		static VersionNumber MinimumClangVersion => SDK.GetRequiredVersionNumberFromConfig("MinimumClangVersion");

		/// <summary>
		/// Ranges of tested compiler toolchains to be used, in order of preference. If multiple toolchains in a range are present, the latest version will be preferred.
		/// Note that the numbers here correspond to the installation *folders* rather than precise executable versions.
		/// </summary>
		/// <seealso href="https://learn.microsoft.com/en-us/lifecycle/products/visual-studio-2022"/>
		static VersionNumberRange[] PreferredVisualCppVersions => SDK.GetVersionNumberRangeArrayFromConfig("PreferredVisualCppVersions");

		/// <summary>
		/// Minimum Clang version required for MSVC toolchain versions
		/// </summary>
		static Tuple<VersionNumber, VersionNumber>[] MinimumRequiredClangVersion => SDK.GetVersionNumberRangeArrayFromConfig("MinimumRequiredClangVersion").
			Select(x => new Tuple<VersionNumber, VersionNumber>(x.Min, x.Max)).ToArray();

		/// <summary>
		/// Tested compiler toolchains that should not be allowed.
		/// </summary>
		static VersionNumberRange[] BannedVisualCppVersions => SDK.GetVersionNumberRangeArrayFromConfig("BannedVisualCppVersions");

		/// <summary>
		/// The minimum supported MSVC compiler
		/// </summary>
		static VersionNumber MinimumVisualCppVersion => SDK.GetRequiredVersionNumberFromConfig("MinimumVisualCppVersion");

		/// <summary>
		/// The default compiler version to be used, if installed. 
		/// https://www.intel.com/content/www/us/en/developer/articles/tool/oneapi-standalone-components.html#dpcpp-cpp
		/// </summary>
		static VersionNumberRange[] PreferredIntelOneApiVersions => SDK.GetVersionNumberRangeArrayFromConfig("PreferredIntelOneApiVersions");

		/// <summary>
		/// The minimum supported Intel compiler
		/// </summary>
		static VersionNumber MinimumIntelOneApiVersion => SDK.GetRequiredVersionNumberFromConfig("MinimumIntelOneApiVersion");

		/// <summary>
		/// If a toolchain version is a preferred version
		/// </summary>
		/// <param name="toolchain">The toolchain type</param>
		/// <param name="version">The version number</param>
		/// <returns>If the version is preferred</returns>
		public static bool IsPreferredVersion(WindowsCompiler toolchain, VersionNumber version)
		{
			if (toolchain.IsMSVC())
			{
				return PreferredVisualCppVersions.Any(x => x.Contains(version));
			}
			else if (toolchain.IsIntel())
			{
				return PreferredIntelOneApiVersions.Any(x => x.Contains(version));
			}
			else if (toolchain.IsClang())
			{
				return PreferredClangVersions.Any(x => x.Contains(version));
			}
			return false;
		}

		/// <summary>
		/// Get the latest preferred toolchain version
		/// </summary>
		/// <param name="toolchain">The toolchain type</param>
		/// <returns>The version number</returns>
		public static VersionNumber GetLatestPreferredVersion(WindowsCompiler toolchain)
		{
			if (toolchain.IsMSVC())
			{
				return PreferredVisualCppVersions.Select(x => x.Min).Max()!;
			}
			else if (toolchain.IsIntel())
			{
				return PreferredIntelOneApiVersions.Select(x => x.Min).Max()!;
			}
			else if (toolchain.IsClang())
			{
				return PreferredClangVersions.Select(x => x.Min).Max()!;
			}
			return new VersionNumber(0);
		}

		/// <summary>
		/// The minimum supported Clang version for a given MSVC toolchain
		/// </summary>
		/// <param name="vcVersion"></param>
		/// <returns></returns>
		public static VersionNumber GetMinimumClangVersionForVcVersion(VersionNumber vcVersion)
		{
			return MinimumRequiredClangVersion.FirstOrDefault(x => vcVersion >= x.Item1)?.Item2 ?? MinimumClangVersion;
		}

		/// <summary>
		/// The base Clang version for a given Intel toolchain
		/// </summary>
		/// <param name="intelCompilerPath"></param>
		/// <returns></returns>
		public static VersionNumber GetClangVersionForIntelCompiler(FileReference intelCompilerPath)
		{
			FileReference ldLLdPath = FileReference.Combine(intelCompilerPath.Directory, "compiler", "ld.lld.exe");
			if (FileReference.Exists(ldLLdPath))
			{
				FileVersionInfo versionInfo = FileVersionInfo.GetVersionInfo(ldLLdPath.FullName);
				VersionNumber version = new VersionNumber(versionInfo.FileMajorPart, versionInfo.FileMinorPart, versionInfo.FileBuildPart);
				return version;
			}

			return MinimumClangVersion;
		}

		/// <summary>
		/// Whether toolchain errors should be ignored. Enable to ignore banned toolchains when generating projects,
		/// as components such as the recommended toolchain can be installed by opening the generated solution via the .vsconfig file.
		/// If enabled the error will be downgraded to a warning.
		/// </summary>
		public static bool IgnoreToolchainErrors { get; set; } = false;
	}
}
