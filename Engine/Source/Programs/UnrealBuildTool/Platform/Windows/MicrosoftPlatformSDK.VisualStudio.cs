// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using EpicGames.Core;

namespace UnrealBuildTool
{
	partial class MicrosoftPlatformSDK : UEBuildPlatformSDK
	{
		/// <summary>
		/// The default set of components that should be suggested to be installed for Visual Studio
		/// This or the 2022 specific components should be updated if the preferred visual cpp version changes
		/// </summary>
		static readonly string[] s_visualStudioSuggestedComponents = SDK.GetStringArrayFromConfig("VisualStudioSuggestedComponents");

		/// <summary>
		/// Additional set of components that should be suggested to be installed for Visual Studio
		/// to support the Linux platform.
		/// </summary>
		static readonly string[] s_visualStudioSuggestedLinuxComponents = SDK.GetStringArrayFromConfig("VisualStudioSuggestedLinuxComponents");

		/// <summary>
		/// Additional set of components that should be suggested to be installed for Visual Studio 2022.
		/// </summary>
		static readonly string[] s_visualStudio2022SuggestedComponents = SDK.GetStringArrayFromConfig("VisualStudio2022SuggestedComponents");

		/// <summary>
		/// Additional set of components that should be suggested to be installed for Visual Studio 2026.
		/// </summary>
		static readonly string[] s_visualStudio2026SuggestedComponents = SDK.GetStringArrayFromConfig("VisualStudio2026SuggestedComponents");

		/// <summary>
		/// The minimum supported Visual Studio 2022 IDE version
		/// </summary>
		static VersionNumber MinimumVisualStudio2022Version => SDK.GetRequiredVersionNumberFromConfig("MinimumVisualStudio2022Version");

		/// <summary>
		/// The minimum supported Visual Studio 2026 IDE version
		/// </summary>
		static VersionNumber MinimumVisualStudio2026Version => SDK.GetRequiredVersionNumberFromConfig("MinimumVisualStudio2026Version");

		/// <summary>
		/// Returns the list of suggested of components that should be suggested to be installed for Visual Studio.
		/// Used to generate a .vsconfig file which will prompt Visual Studio to ask the user to install these components.
		/// </summary>
		public static IEnumerable<string> GetVisualStudioSuggestedComponents(VCProjectFileFormat format)
		{
			bool platformLinuxValid = (InstalledPlatformInfo.IsValidPlatform(UnrealTargetPlatform.Linux) && UEBuildPlatform.IsPlatformAvailable(UnrealTargetPlatform.Linux))
				|| (InstalledPlatformInfo.IsValidPlatform(UnrealTargetPlatform.LinuxArm64) && UEBuildPlatform.IsPlatformAvailable(UnrealTargetPlatform.LinuxArm64));

			SortedSet<string> components = [.. s_visualStudioSuggestedComponents];

			switch (format)
			{
				case VCProjectFileFormat.VisualStudio2022:
					components.UnionWith(s_visualStudio2022SuggestedComponents);
					break;
				case VCProjectFileFormat.VisualStudio2026:
					components.UnionWith(s_visualStudio2026SuggestedComponents);
					break;
				default:
					throw new BuildException("Unsupported Visual Studio version {0}", format);
			}

			if (platformLinuxValid)
			{
				components.UnionWith(s_visualStudioSuggestedLinuxComponents);
			}

			return components;
		}
	}
}
