// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers.Binary;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Diagnostics;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using System.Linq;
using System.Runtime.InteropServices;
using System.Runtime.Versioning;
using System.Text.RegularExpressions;
using System.Threading;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using Microsoft.VisualStudio.Setup.Configuration;
using Microsoft.Win32;
using UnrealBuildBase;

///////////////////////////////////////////////////////////////////
// If you are looking for supported version numbers, look in the
// MicrosoftPlatformSDK.Versions.cs file next to this file
///////////////////////////////////////////////////////////////////

namespace UnrealBuildTool
{
	internal partial class MicrosoftPlatformSDK : UEBuildPlatformSDK
	{
		public MicrosoftPlatformSDK(ILogger Logger)
			: base(Logger)
		{
		}

		[SupportedOSPlatform("windows")]
		protected override string? GetInstalledSDKVersion()
		{
			if (!OperatingSystem.IsWindows())
			{
				return null;
			}

			// use the PreferredWindowsSdkVersions array to find the SDK version that UBT will use to build with - 
			VersionNumber? WindowsVersion;
			if (TryGetWindowsSdkDir(null, Logger, out WindowsVersion, out _))
			{
				return WindowsVersion.ToString();
			}

			// if that failed, we aren't able to build, so give up
			return null;
		}

		public override bool TryConvertVersionToInt(string? StringValue, out ulong OutValue, string? Hint)
		{
			OutValue = 0;

			if (StringValue == null)
			{
				return false;
			}

			Match Result = Regex.Match(StringValue, @"^(\d+).(\d+).(\d+)");
			if (Result.Success)
			{
				// 8 bits for major, 8 for minor, 16 for build - we ignore patch (ie the 1234 in 10.0.14031.1234)
				OutValue |= UInt64.Parse(Result.Groups[1].Value) << 24;
				OutValue |= UInt64.Parse(Result.Groups[2].Value) << 16;
				OutValue |= UInt64.Parse(Result.Groups[3].Value) << 0;

				return true;
			}

			Result = Regex.Match(StringValue, @"^(\d+).(\d+)");
			if (Result.Success)
			{
				// 8 bits for major, 8 for minor - ignore build and patch
				OutValue |= UInt64.Parse(Result.Groups[1].Value) << 24;
				OutValue |= UInt64.Parse(Result.Groups[2].Value) << 16;

				return true;
			}

			return false;
		}

		#region Windows Specific SDK discovery

		#region Cached Locations

		/// <summary>
		/// Cache of Visual C++ installation directories
		/// </summary>
		private static readonly ConcurrentDictionary<WindowsCompiler, List<ToolChainInstallation>> CachedToolChainInstallations = new ConcurrentDictionary<WindowsCompiler, List<ToolChainInstallation>>();

		/// <summary>
		/// Cache of Windows SDK installation directories
		/// </summary>
		private static IReadOnlyDictionary<VersionNumber, DirectoryReference>? CachedWindowsSdkDirs;

		/// <summary>
		/// Cache of Universal CRT installation directories
		/// </summary>
		private static IReadOnlyDictionary<VersionNumber, DirectoryReference>? CachedUniversalCrtDirs;

		/// <summary>
		/// Cache of Visual Studio installation directories
		/// </summary>
		private static IReadOnlyList<VisualStudioInstallation>? CachedVisualStudioInstallations;

		private static readonly Lock CachedVisualStudioInstallationsLock = new();

		/// <summary>
		/// Cache of DIA SDK installation directories
		/// </summary>
		private static readonly ConcurrentDictionary<WindowsCompiler, List<DirectoryReference>> CachedDiaSdkDirs = new ConcurrentDictionary<WindowsCompiler, List<DirectoryReference>>();

		#endregion

		#region WindowsSdk 

		#region Public Interface

		/// <summary>
		/// Finds all the installed Windows SDK versions
		/// </summary>
		/// <param name="Logger">Logger for output</param>
		/// <returns>Map of version number to Windows SDK directories</returns>
		[SupportedOSPlatform("windows")]
		public static IReadOnlyDictionary<VersionNumber, DirectoryReference> FindWindowsSdkDirs(ILogger Logger)
		{
			// Update the cache of install directories, if it's not set
			if (CachedWindowsSdkDirs == null)
			{
				UpdateCachedWindowsSdks(Logger);
			}
			return CachedWindowsSdkDirs!;
		}

		/// <summary>
		/// List all the installed Windows SDK versions
		/// </summary>
		/// <param name="Logger">Logger for output</param>
		[SupportedOSPlatform("windows")]
		public static void DumpWindowsSdkDirs(ILogger Logger)
		{
			IOrderedEnumerable<KeyValuePair<VersionNumber, DirectoryReference>> SortedSdkDirs = FindWindowsSdkDirs(Logger).OrderByDescending(x => x.Key);

			if (SortedSdkDirs.Any())
			{
				Logger.LogInformation("Available Windows SDKs ({Count}):", SortedSdkDirs.Count());
				foreach (KeyValuePair<VersionNumber, DirectoryReference> Item in SortedSdkDirs)
				{
					Logger.LogInformation(" * {SdkDir}\n    (Version={Version})", Item.Value, Item.Key);
				}
			}
			else
			{
				Logger.LogInformation("No available Windows SDKs found");
			}
		}

		/// <summary>
		/// Finds all the installed Universal CRT versions
		/// </summary>
		/// <param name="Logger">Logger for output</param>
		/// <returns>Map of version number to universal CRT directories</returns>
		[SupportedOSPlatform("windows")]
		public static IReadOnlyDictionary<VersionNumber, DirectoryReference> FindUniversalCrtDirs(ILogger Logger)
		{
			if (CachedUniversalCrtDirs == null)
			{
				UpdateCachedWindowsSdks(Logger);
			}
			return CachedUniversalCrtDirs!;
		}

		/// <summary>
		/// Determines the directory containing the Windows SDK toolchain
		/// </summary>
		/// <param name="DesiredVersion">The desired Windows SDK version. This may be "Latest", a specific version number, or null. If null, the function will look for DefaultWindowsSdkVersion. Failing that, it will return the latest version.</param>
		/// <param name="Logger">Logger for output</param>
		/// <param name="OutSdkVersion">Receives the version number of the selected Windows SDK</param>
		/// <param name="OutSdkDir">Receives the root directory for the selected SDK</param>
		/// <param name="MinVersion">Optional minimum required version. Ignored if DesiredVesrion is specified</param>
		/// <param name="MaxVersion">Optional maximum required version. Ignored if DesiredVesrion is specified</param>
		/// <returns>True if the toolchain directory was found correctly</returns>
		[SupportedOSPlatform("windows")]
		public static bool TryGetWindowsSdkDir(string? DesiredVersion, ILogger Logger, [NotNullWhen(true)] out VersionNumber? OutSdkVersion, [NotNullWhen(true)] out DirectoryReference? OutSdkDir, VersionNumber? MinVersion = null, VersionNumber? MaxVersion = null)
		{
			// Get a map of Windows SDK versions to their root directories
			/*IReadOnlyDictionary<VersionNumber, DirectoryReference> WindowsSdkDirs =*/
			FindWindowsSdkDirs(Logger);

			// Figure out which version number to look for
			VersionNumber? WindowsSdkVersion = null;
			if (!String.IsNullOrEmpty(DesiredVersion))
			{
				if (String.Equals(DesiredVersion, "Latest", StringComparison.OrdinalIgnoreCase) && CachedWindowsSdkDirs!.Count > 0)
				{
					WindowsSdkVersion = CachedWindowsSdkDirs.OrderBy(x => x.Key).Last().Key;
				}
				else if (!VersionNumber.TryParse(DesiredVersion, out WindowsSdkVersion))
				{
					throw new BuildException("Unable to find requested Windows SDK; '{0}' is an invalid version", DesiredVersion);
				}
			}
			else if (CachedWindowsSdkDirs!.Count > 0)
			{
				IEnumerable<KeyValuePair<VersionNumber, DirectoryReference>> AllowedSdkDirs = CachedWindowsSdkDirs.OrderBy(x => x.Key).Where(
					x =>
					(MinVersion == null || x.Key >= MinVersion) &&
					(MaxVersion == null || x.Key <= MaxVersion));

				// convert the desired version into a VersionNumber
				VersionNumber MainVersion = VersionNumber.Parse(GetSDKForPlatform("Win64")!.GetMainVersion());
				if (AllowedSdkDirs.Any(x => x.Key == MainVersion))
				{
					WindowsSdkVersion = MainVersion;
				}
				// if it's not an installed version, use the highest installed version
				else if (AllowedSdkDirs.Any())
				{
					WindowsSdkVersion = AllowedSdkDirs.Last().Key;
				}
			}

			// Get the actual directory for this version
			DirectoryReference? SdkDir;
			if (WindowsSdkVersion != null && CachedWindowsSdkDirs!.TryGetValue(WindowsSdkVersion, out SdkDir))
			{
				OutSdkDir = SdkDir;
				OutSdkVersion = WindowsSdkVersion;
				return true;
			}
			else
			{
				OutSdkDir = null;
				OutSdkVersion = null;
				return false;
			}
		}

		#endregion

		#region Private implementation

		/// <summary>
		/// Updates the CachedWindowsSdkDirs and CachedUniversalCrtDirs variables
		/// </summary>
		[SupportedOSPlatform("windows")]
		private static void UpdateCachedWindowsSdks(ILogger Logger)
		{
			Dictionary<VersionNumber, DirectoryReference> WindowsSdkDirs = new Dictionary<VersionNumber, DirectoryReference>();
			Dictionary<VersionNumber, DirectoryReference> UniversalCrtDirs = new Dictionary<VersionNumber, DirectoryReference>();

			// Enumerate the Windows 8.1 SDK, if present
			DirectoryReference? InstallDir_8_1;
			if (TryReadInstallDirRegistryKey32("Microsoft\\Microsoft SDKs\\Windows\\v8.1", "InstallationFolder", out InstallDir_8_1))
			{
				if (FileReference.Exists(FileReference.Combine(InstallDir_8_1, "Include", "um", "windows.h")))
				{
					Logger.LogDebug("Found Windows 8.1 SDK at {InstallDir_8_1}", InstallDir_8_1);
					VersionNumber Version_8_1 = new VersionNumber(8, 1);
					WindowsSdkDirs[Version_8_1] = InstallDir_8_1;
				}
			}

			// Find all the root directories for Windows 10 SDKs
			List<DirectoryReference> InstallDirs_10 = new List<DirectoryReference>();
			EnumerateSdkRootDirs(InstallDirs_10, Logger);

			// Enumerate all the Windows 10 SDKs
			foreach (DirectoryReference InstallDir_10 in InstallDirs_10.Distinct())
			{
				DirectoryReference IncludeRootDir = DirectoryReference.Combine(InstallDir_10, "Include");
				if (DirectoryReference.Exists(IncludeRootDir))
				{
					foreach (DirectoryReference IncludeDir in DirectoryReference.EnumerateDirectories(IncludeRootDir))
					{
						VersionNumber? IncludeVersion;
						if (VersionNumber.TryParse(IncludeDir.GetDirectoryName(), out IncludeVersion))
						{
							if (FileReference.Exists(FileReference.Combine(IncludeDir, "um", "windows.h")))
							{
								Logger.LogDebug("Found Windows 10 SDK version {IncludeVersion} at {InstallDir_10}", IncludeVersion, InstallDir_10);
								WindowsSdkDirs[IncludeVersion] = InstallDir_10;
							}
							if (FileReference.Exists(FileReference.Combine(IncludeDir, "ucrt", "corecrt.h")))
							{
								Logger.LogDebug("Found Universal CRT version {IncludeVersion} at {InstallDir_10}", IncludeVersion, InstallDir_10);
								UniversalCrtDirs[IncludeVersion] = InstallDir_10;
							}
						}
					}
				}
			}

			CachedWindowsSdkDirs = WindowsSdkDirs;
			CachedUniversalCrtDirs = UniversalCrtDirs;
		}

		/// <summary>
		/// Enumerates all the Windows 10 SDK root directories
		/// </summary>
		/// <param name="RootDirs">Receives all the Windows 10 sdk root directories</param>
		/// <param name="Logger">Logger for output</param>
		[SupportedOSPlatform("windows")]
		public static void EnumerateSdkRootDirs(List<DirectoryReference> RootDirs, ILogger Logger)
		{
			DirectoryReference? RootDir;
			if (TryReadInstallDirRegistryKey32("Microsoft\\Windows Kits\\Installed Roots", "KitsRoot10", out RootDir))
			{
				Logger.LogDebug("Found Windows 10 SDK root at {RootDir} (1)", RootDir);
				RootDirs.Add(RootDir);
			}
			if (TryReadInstallDirRegistryKey32("Microsoft\\Microsoft SDKs\\Windows\\v10.0", "InstallationFolder", out RootDir))
			{
				Logger.LogDebug("Found Windows 10 SDK root at {RootDir} (2)", RootDir);
				RootDirs.Add(RootDir);
			}

			DirectoryReference? HostAutoSdkDir;
			if (TryGetHostPlatformAutoSDKDir(out HostAutoSdkDir))
			{
				DirectoryReference WindowsKitsDirAutoSdk = DirectoryReference.Combine(HostAutoSdkDir, "Win64", "Windows Kits");
				if (DirectoryReference.Exists(WindowsKitsDirAutoSdk))
				{
					foreach (DirectoryReference RootDirAutoSdk in DirectoryReference.EnumerateDirectories(WindowsKitsDirAutoSdk))
					{
						DirectoryReference IncludeRootDir = DirectoryReference.Combine(RootDirAutoSdk, "Include");
						if (DirectoryReference.Exists(IncludeRootDir))
						{
							Logger.LogDebug("Found Windows 10 AutoSDK root at {RootDirAutoSdk}", RootDirAutoSdk);
							RootDirs.Add(RootDirAutoSdk);
						}
					}
				}
			}
		}
		#endregion

		#endregion

		#region NetFx

		/// <summary>
		/// Gets the installation directory for the NETFXSDK
		/// </summary>
		/// <param name="OutInstallDir">Receives the installation directory on success</param>
		/// <returns>True if the directory was found, false otherwise</returns>
		[SupportedOSPlatform("windows")]
		public static bool TryGetNetFxSdkInstallDir([NotNullWhen(true)] out DirectoryReference? OutInstallDir)
		{
			DirectoryReference? HostAutoSdkDir;
			string[] PreferredVersions = new string[] { "4.6.2", "4.6.1", "4.6" };
			if (UEBuildPlatformSDK.TryGetHostPlatformAutoSDKDir(out HostAutoSdkDir))
			{
				foreach (string PreferredVersion in PreferredVersions)
				{
					DirectoryReference NetFxDir = DirectoryReference.Combine(HostAutoSdkDir, "Win64", "Windows Kits", "NETFXSDK", PreferredVersion);
					if (FileReference.Exists(FileReference.Combine(NetFxDir, "Include", "um", "mscoree.h")))
					{
						OutInstallDir = NetFxDir;
						return true;
					}
				}
			}

			string NetFxSDKKeyName = "Microsoft\\Microsoft SDKs\\NETFXSDK";
			foreach (string PreferredVersion in PreferredVersions)
			{
				if (TryReadInstallDirRegistryKey32(NetFxSDKKeyName + "\\" + PreferredVersion, "KitsInstallationFolder", out OutInstallDir))
				{
					return true;
				}
			}

			// If we didn't find one of our preferred versions for NetFXSDK, use the max version present on the system
			Version? MaxVersion = null;
			string? MaxVersionString = null;
			foreach (string ExistingVersionString in ReadInstallDirSubKeys32(NetFxSDKKeyName))
			{
				Version? ExistingVersion;
				if (!Version.TryParse(ExistingVersionString, out ExistingVersion))
				{
					continue;
				}
				if (MaxVersion == null || ExistingVersion.CompareTo(MaxVersion) > 0)
				{
					MaxVersion = ExistingVersion;
					MaxVersionString = ExistingVersionString;
				}
			}

			if (MaxVersionString != null)
			{
				return TryReadInstallDirRegistryKey32(NetFxSDKKeyName + "\\" + MaxVersionString, "KitsInstallationFolder", out OutInstallDir);
			}

			OutInstallDir = null;
			return false;
		}

		#region Private implementation

		[SupportedOSPlatform("windows")]
		static readonly Lazy<KeyValuePair<RegistryKey, string>[]> InstallDirRoots = new Lazy<KeyValuePair<RegistryKey, string>[]>(() =>
			new KeyValuePair<RegistryKey, string>[]
			{
				new KeyValuePair<RegistryKey, string>(Registry.CurrentUser, "SOFTWARE\\"),
				new KeyValuePair<RegistryKey, string>(Registry.LocalMachine, "SOFTWARE\\"),
				new KeyValuePair<RegistryKey, string>(Registry.CurrentUser, "SOFTWARE\\Wow6432Node\\"),
				new KeyValuePair<RegistryKey, string>(Registry.LocalMachine, "SOFTWARE\\Wow6432Node\\")
			});

		/// <summary>
		/// Reads an install directory for a 32-bit program from a registry key. This checks for per-user and machine wide settings, and under the Wow64 virtual keys (HKCU\SOFTWARE, HKLM\SOFTWARE, HKCU\SOFTWARE\Wow6432Node, HKLM\SOFTWARE\Wow6432Node).
		/// </summary>
		/// <param name="KeySuffix">Path to the key to read, under one of the roots listed above.</param>
		/// <param name="ValueName">Value to be read.</param>
		/// <param name="InstallDir">On success, the directory corresponding to the value read.</param>
		/// <returns>True if the key was read, false otherwise.</returns>
		[SupportedOSPlatform("windows")]
		public static bool TryReadInstallDirRegistryKey32(string KeySuffix, string ValueName, [NotNullWhen(true)] out DirectoryReference? InstallDir)
		{
			foreach (KeyValuePair<RegistryKey, string> InstallRoot in InstallDirRoots.Value)
			{
				using (RegistryKey? Key = InstallRoot.Key.OpenSubKey(InstallRoot.Value + KeySuffix))
				{
					if (Key != null && TryReadDirRegistryKey(Key.Name, ValueName, out InstallDir))
					{
						return true;
					}
				}
			}
			InstallDir = null;
			return false;
		}

		/// <summary>
		/// For each root location relevant to install dirs, look for the given key and add its subkeys to the set of subkeys to return.
		/// This checks for per-user and machine wide settings, and under the Wow64 virtual keys (HKCU\SOFTWARE, HKLM\SOFTWARE, HKCU\SOFTWARE\Wow6432Node, HKLM\SOFTWARE\Wow6432Node).
		/// </summary>
		/// <param name="KeyName">The subkey to look for under each root location</param>
		/// <returns>A list of unique subkeys found under any of the existing subkeys</returns>
		[SupportedOSPlatform("windows")]
		static string[] ReadInstallDirSubKeys32(string KeyName)
		{
			HashSet<string> AllSubKeys = new HashSet<string>(StringComparer.Ordinal);
			foreach (KeyValuePair<RegistryKey, string> Root in InstallDirRoots.Value)
			{
				using (RegistryKey? Key = Root.Key.OpenSubKey(Root.Value + KeyName))
				{
					if (Key == null)
					{
						continue;
					}

					foreach (string SubKey in Key.GetSubKeyNames())
					{
						AllSubKeys.Add(SubKey);
					}
				}
			}
			return AllSubKeys.ToArray();
		}

		/// <summary>
		/// Attempts to reads a directory name stored in a registry key
		/// </summary>
		/// <param name="KeyName">Key to read from</param>
		/// <param name="ValueName">Value within the key to read</param>
		/// <param name="Value">The directory read from the registry key</param>
		/// <returns>True if the key was read, false if it was missing or empty</returns>
		[SupportedOSPlatform("windows")]
		static bool TryReadDirRegistryKey(string KeyName, string ValueName, [NotNullWhen(true)] out DirectoryReference? Value)
		{
			string? StringValue = Registry.GetValue(KeyName, ValueName, null) as string;
			if (String.IsNullOrEmpty(StringValue))
			{
				Value = null;
				return false;
			}
			else
			{
				Value = new DirectoryReference(StringValue);
				return true;
			}
		}

		#endregion

		#endregion

		#region Toolchain

		#region Public Interface

		/// <summary>
		/// Gets the MSBuild path, and throws an exception on failure.
		/// </summary>
		/// <returns>Path to MSBuild</returns>
		[SupportedOSPlatform("windows")]
		public static FileReference GetMsBuildToolPath(ILogger Logger)
		{
			FileReference? Location;
			if (!TryGetMsBuildPath(Logger, out Location))
			{
				throw new BuildException("Unable to find installation of MSBuild.");
			}
			return Location;
		}

		/// <summary>
		/// Determines if a given compiler is installed
		/// </summary>
		/// <param name="Compiler">Compiler to check for</param>
		/// <param name="Architecture">Architecture the compiler must support</param>
		/// <param name="Logger">Logger for output</param>
		/// <returns>True if the given compiler is installed</returns>
		public static bool HasCompiler(WindowsCompiler Compiler, UnrealArch Architecture, ILogger Logger)
		{
			return FindToolChainInstallations(Compiler, Logger).Where(x => (x.Architecture == Architecture)).Any();
		}

		/// <summary>
		/// Determines if a given compiler is installed and valid
		/// </summary>
		/// <param name="Compiler">Compiler to check for</param>
		/// <param name="Architecture">Architecture the compiler must support</param>
		/// <param name="Logger">Logger for output</param>
		/// <returns>True if the given compiler is installed and valid</returns>
		public static bool HasValidCompiler(WindowsCompiler Compiler, UnrealArch Architecture, ILogger Logger)
		{
			// since this is static, we get the instance for it
			MicrosoftPlatformSDK SDK = (MicrosoftPlatformSDK)GetSDKForPlatform("Win64")!;
			return FindToolChainInstallations(Compiler, Logger).Where(x => (x.Error == null && x.Architecture == Architecture)).Any();
		}

		/// <summary>
		/// Determines the directory containing the MSVC toolchain
		/// </summary>
		/// <param name="Compiler">Major version of the compiler to use</param>
		/// <param name="CompilerVersion">The minimum compiler version to use</param>
		/// <param name="Architecture">Architecture that is required</param>
		/// <param name="Logger">Logger for output</param>
		/// <param name="OutToolChainVersion">Receives the chosen toolchain version</param>
		/// <param name="OutToolChainDir">Receives the directory containing the toolchain</param>
		/// <param name="OutRedistDir">Receives the optional directory containing redistributable components</param>
		/// <returns>True if the toolchain directory was found correctly</returns>
		public static bool TryGetToolChainDir(WindowsCompiler Compiler, string? CompilerVersion, UnrealArch Architecture, ILogger Logger, [NotNullWhen(true)] out VersionNumber? OutToolChainVersion, [NotNullWhen(true)] out DirectoryReference? OutToolChainDir, out DirectoryReference? OutRedistDir)
		{
			// Find all the installed toolchains
			List<ToolChainInstallation> ToolChains = FindToolChainInstallations(Compiler, Logger);

			// Figure out the actual version number that we want
			ToolChainInstallation? ToolChain = null;
			if (String.IsNullOrEmpty(CompilerVersion))
			{
				ToolChain = SelectToolChain(ToolChains, x => x, Architecture);
				if (ToolChain == null)
				{
					OutToolChainVersion = null;
					OutToolChainDir = null;
					OutRedistDir = null;
					return false;
				}
			}
			else if (CompilerVersion.StartsWith('<') && VersionNumber.TryParse(CompilerVersion[1..], out VersionNumber? LessThanToolChainVersion))
			{
				ToolChain = SelectToolChain(ToolChains.Where(x => x.Version < LessThanToolChainVersion && x.Family < LessThanToolChainVersion), x => x, Architecture);
				if (ToolChain == null || !(ToolChain.Version < LessThanToolChainVersion || ToolChain.Family < LessThanToolChainVersion))
				{
					DumpToolChains(ToolChains, x => x.ThenByDescending(x => x.Version < LessThanToolChainVersion).ThenByDescending(x => x.Family < LessThanToolChainVersion), Architecture, Logger);
					throw new BuildException("Unable to find valid <{0} C++ toolchain for {1} {2}", LessThanToolChainVersion, Compiler, Architecture.ToString());
				}
			}
			else if (VersionNumber.TryParse(CompilerVersion, out VersionNumber? ToolChainVersion))
			{
				ToolChain = SelectToolChain(ToolChains, x => x.ThenByDescending(x => x.Version == ToolChainVersion).ThenByDescending(x => x.Family == ToolChainVersion), Architecture);
				if (ToolChain == null || !(ToolChain.Version == ToolChainVersion || ToolChain.Family == ToolChainVersion))
				{
					DumpToolChains(ToolChains, x => x.ThenByDescending(x => x.Version == ToolChainVersion).ThenByDescending(x => x.Family == ToolChainVersion), Architecture, Logger);
					throw new BuildException("Unable to find valid {0} C++ toolchain for {1} {2}", ToolChainVersion, Compiler, Architecture.ToString());
				}
			}

			foreach (WindowsCompilerChannel Channel in Enum.GetValues(typeof(WindowsCompilerChannel)))
			{
				if (String.Equals(CompilerVersion, Channel.ToString(), StringComparison.OrdinalIgnoreCase))
				{
					ToolChain = SelectToolChain(ToolChains.Where(x => x.ReleaseChannel.HasFlag(Channel)), x => x.ThenBy(x => x.ReleaseChannel).ThenByDescending(x => x.Version), Architecture);
					if (ToolChain == null)
					{
						DumpToolChains(ToolChains, x => x.ThenBy(x => x.ReleaseChannel).ThenByDescending(x => x.Version), Architecture, Logger);
						throw new BuildException("Unable to find valid {0} C++ toolchain for {1} {2}", Channel, GetCompilerName(Compiler), Architecture);
					}
					break;
				}
			}

			if (ToolChain == null)
			{
				DumpToolChains(ToolChains, x => x.ThenBy(x => x.ReleaseChannel).ThenByDescending(x => x.Version), Architecture, Logger);
				throw new BuildException("Unable to find {0} {1} C++ toolchain; '{2}' is an invalid version", GetCompilerName(Compiler), Architecture, CompilerVersion);
			}

			// Get the actual directory for this version
			OutToolChainVersion = ToolChain.Version;
			OutToolChainDir = ToolChain.BaseDir;
			OutRedistDir = ToolChain.RedistDir;
			return true;
		}

		/// <summary>
		/// Returns the human-readable name of the given compiler
		/// </summary>
		/// <param name="Compiler">The compiler value</param>
		/// <returns>Name of the compiler</returns>
		public static string GetCompilerName(WindowsCompiler Compiler)
		{
			switch (Compiler)
			{
				case WindowsCompiler.VisualStudio2022:
					return "Visual Studio 2022";
				case WindowsCompiler.VisualStudio2026:
					return "Visual Studio";
				default:
					return Compiler.ToString();
			}
		}

		#endregion

		#region Private implementation

		/// <summary>
		/// Select which toolchain to use, combining a custom preference with a default sort order
		/// </summary>
		/// <param name="ToolChains"></param>
		/// <param name="Preference">Ordering function</param>
		/// <param name="Architecture">Architecture that must be supported</param>
		/// <returns></returns>
		static ToolChainInstallation? SelectToolChain(IEnumerable<ToolChainInstallation> ToolChains, Func<IOrderedEnumerable<ToolChainInstallation>, IOrderedEnumerable<ToolChainInstallation>> Preference, UnrealArch Architecture)
		{
			ToolChainInstallation? ToolChain = Preference(ToolChains.Where(x => x.Architecture == Architecture && x.HostArchitecture == MicrosoftPlatformSDK.MSVCHostUnrealArch)
				.OrderByDescending(x => x.Error == null))
				.ThenBy(x => x.ReleaseChannel)
				.ThenBy(x => x.FamilyRank)
				.ThenByDescending(x => x.IsAutoSdk)
				.ThenByDescending(x => x.Version)
				.FirstOrDefault();

			// Allow x64 host toolchains on arm64 if no native arm64 toolchains are available, and not running via wine
			if (!RuntimePlatform.IsRunningOnWine() && MicrosoftPlatformSDK.MSVCHostUnrealArch == UnrealArch.Arm64 && (ToolChain == null || ToolChain.Error != null))
			{
				ToolChain = Preference(ToolChains.Where(x => x.Architecture == Architecture && x.HostArchitecture != MicrosoftPlatformSDK.MSVCHostUnrealArch)
					.OrderByDescending(x => x.Error == null))
					.ThenBy(x => x.ReleaseChannel)
					.ThenBy(x => x.FamilyRank)
					.ThenByDescending(x => x.IsAutoSdk)
					.ThenByDescending(x => x.Version)
					.FirstOrDefault();
			}

			if (ToolChain?.Error != null)
			{
				if (!IgnoreToolchainErrors)
				{
					throw new BuildException(ToolChain.Error);
				}
				// If errors are ignored, log the warning instead
				Log.TraceWarningOnce(ToolChain.Error);
			}

			return ToolChain;
		}

		/// <summary>
		/// Dump all available toolchain info, combining a custom preference with a default sort order
		/// </summary>
		/// <param name="ToolChains"></param>
		/// <param name="Preference">Ordering function</param>
		/// <param name="Architecture">Architecture that must be supported</param>
		/// <param name="Logger">The ILogger interface to write to</param>
		static void DumpToolChains(IEnumerable<ToolChainInstallation> ToolChains, Func<IOrderedEnumerable<ToolChainInstallation>, IOrderedEnumerable<ToolChainInstallation>> Preference, UnrealArch Architecture, ILogger Logger)
		{
			IOrderedEnumerable<ToolChainInstallation> SortedToolChains = Preference(ToolChains.Where(x => x.Architecture == Architecture)
				.OrderByDescending(x => x.Error == null))
				.ThenBy(x => x.ReleaseChannel)
				.ThenBy(x => x.FamilyRank)
				.ThenByDescending(x => x.IsAutoSdk)
				.ThenByDescending(x => x.Version)
				.ThenBy(x => x.HostArchitecture.ToString());

			if (SortedToolChains.Any())
			{
				IEnumerable<ToolChainInstallation> AvailableToolChains = SortedToolChains.Where(x => x.Error == null);
				IEnumerable<ToolChainInstallation> UnavailableToolChains = SortedToolChains.Where(x => x.Error != null);

				if (AvailableToolChains.Any())
				{
					Logger.LogInformation("Available {Architecture} toolchains ({Count}):", Architecture, AvailableToolChains.Count());
					foreach (ToolChainInstallation ToolChain in AvailableToolChains)
					{
						Logger.LogInformation(" * {ToolChainDir}\n    (Family={Family}, FamilyRank={FamilyRank}, Version={Version}, HostArchitecture={HostArchitecture}, ReleaseChannel={ReleaseChannel}, Architecture={Arch})", ToolChain.BaseDir, ToolChain.Family, ToolChain.FamilyRank, ToolChain.Version, ToolChain.HostArchitecture, ToolChain.ReleaseChannel, ToolChain.Architecture);
					}
				}

				if (UnavailableToolChains.Any())
				{
					Logger.LogWarning("Unavailable {Architecture} toolchains ({Count}):", Architecture, UnavailableToolChains.Count());
					foreach (ToolChainInstallation ToolChain in UnavailableToolChains.Where(x => x.Error != null))
					{
						Logger.LogWarning(" * {ToolChainDir}\n    (Family={Family}, FamilyRank={FamilyRank}, Version={Version}, HostArchitecture={HostArchitecture}, ReleaseChannel={ReleaseChannel}, Architecture={Arch}, Error=\"{Error}\")", ToolChain.BaseDir, ToolChain.Family, ToolChain.FamilyRank, ToolChain.Version, ToolChain.HostArchitecture, ToolChain.ReleaseChannel, ToolChain.Architecture, ToolChain.Error);
					}
				}
			}
			else
			{
				Logger.LogInformation("No available {Architecture} toolchains found", Architecture);
			}
		}

		/// <summary>
		/// Dump all available toolchain info for a compiler
		/// </summary>
		/// <param name="Compiler">The compiler to filter by</param>
		/// <param name="Architecture">Architecture that must be supported</param>
		/// <param name="Logger">The ILogger interface to write to</param>
		public static void DumpAllToolChainInstallations(WindowsCompiler Compiler, UnrealArch Architecture, ILogger Logger)
		{
			List<ToolChainInstallation>? ToolChains = CachedToolChainInstallations.GetValueOrDefault(Compiler);
			if (ToolChains != null)
			{
				DumpToolChains(ToolChains, x => x.ThenBy(x => x.ReleaseChannel).ThenByDescending(x => x.Version), Architecture, Logger);
			}
		}

		static List<ToolChainInstallation> FindToolChainInstallations(WindowsCompiler Compiler, ILogger Logger)
		{
			return CachedToolChainInstallations.GetOrAdd(Compiler, _ =>
			{
				List<ToolChainInstallation>? ToolChains = new List<ToolChainInstallation>();
				if (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Win64)
				{
					if (Compiler == WindowsCompiler.Clang)
					{
						// Check for a manual installation to the default directory
						DirectoryReference ManualInstallDir = DirectoryReference.Combine(DirectoryReference.GetSpecialFolder(Environment.SpecialFolder.ProgramFiles)!, "LLVM");
						AddClangToolChain(Compiler, ManualInstallDir, ToolChains, IsAutoSdk: false, Logger);

						// Check for a manual installation to a custom directory
						string? LlvmPath = Environment.GetEnvironmentVariable("LLVM_PATH");
						if (!String.IsNullOrEmpty(LlvmPath))
						{
							AddClangToolChain(Compiler, new DirectoryReference(LlvmPath), ToolChains, IsAutoSdk: false, Logger);
						}

						// Check for installations bundled with Visual Studio
						foreach (VisualStudioInstallation Installation in FindVisualStudioInstallations(WindowsCompiler.VisualStudio2022, Logger)
							.Concat(FindVisualStudioInstallations(WindowsCompiler.VisualStudio2026, Logger)))
						{
							AddClangToolChain(Compiler, DirectoryReference.Combine(Installation.BaseDir, "VC", "Tools", "Llvm"), ToolChains, IsAutoSdk: false, Logger);
							AddClangToolChain(Compiler, DirectoryReference.Combine(Installation.BaseDir, "VC", "Tools", "Llvm", "x64"), ToolChains, IsAutoSdk: false, Logger);
							AddClangToolChain(Compiler, DirectoryReference.Combine(Installation.BaseDir, "VC", "Tools", "Llvm", "ARM64"), ToolChains, IsAutoSdk: false, Logger);
						}

						// Check for AutoSDK paths
						DirectoryReference? AutoSdkDir;
						if (UEBuildPlatformSDK.TryGetHostPlatformAutoSDKDir(out AutoSdkDir))
						{
							DirectoryReference ClangBaseDir = DirectoryReference.Combine(AutoSdkDir, "Win64", "LLVM");
							if (DirectoryReference.Exists(ClangBaseDir))
							{
								foreach (DirectoryReference ToolChainDir in DirectoryReference.EnumerateDirectories(ClangBaseDir))
								{
									AddClangToolChain(Compiler, ToolChainDir, ToolChains, IsAutoSdk: true, Logger);
								}
							}
						}
					}
					else if (Compiler == WindowsCompiler.ClangRTFM)
					{
						// the AutoRTFM Clang compiler is in-tree
						DirectoryReference AutoRTFMBaseDir = DirectoryReference.Combine(Unreal.EngineDirectory, "Restricted", "NotForLicensees", "Binaries", "Win64", "AutoRTFM");
						foreach (DirectoryReference AutoRTFMDir in DirectoryReference.EnumerateDirectories(AutoRTFMBaseDir))
						{
							AddClangToolChain(Compiler, AutoRTFMDir, ToolChains, IsAutoSdk: false, Logger);
						}

						// Check for AutoSDK paths
						DirectoryReference? AutoSdkDir;
						if (UEBuildPlatformSDK.TryGetHostPlatformAutoSDKDir(out AutoSdkDir))
						{
							DirectoryReference ClangBaseDir = DirectoryReference.Combine(AutoSdkDir, "Win64", "AutoRTFM");
							if (DirectoryReference.Exists(ClangBaseDir))
							{
								foreach (DirectoryReference ToolChainDir in DirectoryReference.EnumerateDirectories(ClangBaseDir))
								{
									AddClangToolChain(Compiler, ToolChainDir, ToolChains, IsAutoSdk: true, Logger);
								}
							}
						}
					}
					else if (Compiler == WindowsCompiler.ClangInstrument)
					{
						// the UnrealInstrumentation Clang compiler is in-tree
						DirectoryReference UnrealInstrumentationDir = DirectoryReference.Combine(Unreal.EngineDirectory, "Binaries", "Win64", "UnrealInstrumentation");
						AddClangToolChain(Compiler, UnrealInstrumentationDir, ToolChains, IsAutoSdk: false, Logger);

						// Check for AutoSDK paths
						DirectoryReference? AutoSdkDir;
						if (UEBuildPlatformSDK.TryGetHostPlatformAutoSDKDir(out AutoSdkDir))
						{
							DirectoryReference ClangBaseDir = DirectoryReference.Combine(AutoSdkDir, "Win64", "UnrealInstrumentation");
							if (DirectoryReference.Exists(ClangBaseDir))
							{
								foreach (DirectoryReference ToolChainDir in DirectoryReference.EnumerateDirectories(ClangBaseDir))
								{
									AddClangToolChain(Compiler, ToolChainDir, ToolChains, IsAutoSdk: true, Logger);
								}
							}
						}
					}
					else if (Compiler == WindowsCompiler.ClangCustom)
					{
						// Check for a manual installation to a custom directory
						string? LlvmPath = Environment.GetEnvironmentVariable("LLVM_CUSTOM_PATH");
						if (!String.IsNullOrEmpty(LlvmPath))
						{
							AddClangToolChain(Compiler, new DirectoryReference(LlvmPath), ToolChains, IsAutoSdk: false, Logger);
						}
						else
						{
							throw new BuildException("ClangCustom needs LLVM_CUSTOM_PATH environment variable to be defined");
						}
					}
					else if (Compiler == WindowsCompiler.Intel)
					{
						DirectoryReference InstallDir = DirectoryReference.Combine(DirectoryReference.GetSpecialFolder(Environment.SpecialFolder.ProgramFiles)!, "Intel", "oneAPI", "compiler");
						FindIntelOneApiToolChains(InstallDir, ToolChains, IsAutoSdk: false, Logger);

						InstallDir = DirectoryReference.Combine(DirectoryReference.GetSpecialFolder(Environment.SpecialFolder.ProgramFilesX86)!, "Intel", "oneAPI", "compiler");
						FindIntelOneApiToolChains(InstallDir, ToolChains, IsAutoSdk: false, Logger);

						// Check for AutoSDK paths
						DirectoryReference? AutoSdkDir;
						if (UEBuildPlatformSDK.TryGetHostPlatformAutoSDKDir(out AutoSdkDir))
						{
							DirectoryReference IntelBaseDir = DirectoryReference.Combine(AutoSdkDir, "Win64", "Intel");
							if (DirectoryReference.Exists(IntelBaseDir))
							{
								FindIntelOneApiToolChains(IntelBaseDir, ToolChains, IsAutoSdk: true, Logger);
							}
						}
					}
					else if (Compiler.IsMSVC())
					{
						// Enumerate all the manually installed toolchains
						List<VisualStudioInstallation> Installations = FindVisualStudioInstallations(Compiler, Logger);
						foreach (VisualStudioInstallation Installation in Installations)
						{
							DirectoryReference ToolChainBaseDir = DirectoryReference.Combine(Installation.BaseDir, "VC", "Tools", "MSVC");
							DirectoryReference RedistBaseDir = DirectoryReference.Combine(Installation.BaseDir, "VC", "Redist", "MSVC");
							FindVisualStudioToolChains(ToolChainBaseDir, RedistBaseDir, Installation.ReleaseChannel, ToolChains, IsAutoSdk: false, Logger);
						}

						// Enumerate all the AutoSDK toolchains
						DirectoryReference? PlatformDir;
						if (UEBuildPlatformSDK.TryGetHostPlatformAutoSDKDir(out PlatformDir))
						{
							string VSDir = String.Empty;
							switch (Compiler)
							{
								case WindowsCompiler.VisualStudio2022: VSDir = "VS2022"; break;
								case WindowsCompiler.VisualStudio2026: VSDir = "VS2026"; break;
							}

							if (!String.IsNullOrEmpty(VSDir))
							{
								DirectoryReference ReleaseBaseDir = DirectoryReference.Combine(PlatformDir, "Win64", VSDir);
								FindVisualStudioToolChains(ReleaseBaseDir, null, WindowsCompilerChannel.Latest, ToolChains, IsAutoSdk: true, Logger);

								DirectoryReference PreviewBaseDir = DirectoryReference.Combine(PlatformDir, "Win64", $"{VSDir}-Preview");
								FindVisualStudioToolChains(PreviewBaseDir, null, WindowsCompilerChannel.Preview, ToolChains, IsAutoSdk: true, Logger);

								DirectoryReference ExperimentalBaseDir = DirectoryReference.Combine(PlatformDir, "Win64", $"{VSDir}-Experimental");
								FindVisualStudioToolChains(ExperimentalBaseDir, null, WindowsCompilerChannel.Experimental, ToolChains, IsAutoSdk: true, Logger);
							}
						}

						// Additionally allow msvc toolchains from older versions of Visual Studio
						if (Compiler == WindowsCompiler.VisualStudio2026)
						{
							ToolChains.AddRange(FindToolChainInstallations(WindowsCompiler.VisualStudio2022, Logger));
						}
					}
					else
					{
						throw new BuildException("Unsupported compiler version ({0})", Compiler);
					}

					foreach (ToolChainInstallation? ToolChain in ToolChains.Where(x => x.Error == null && x.HostArchitecture == MSVCHostUnrealArch))
					{
						ToolChainInstallation? CrossToolChain;
						if (Compiler.IsMSVC())
						{
							// MSVC has the same base dir. Version may not match between Hostx64 and Hostarm64.
							CrossToolChain = ToolChains.FirstOrDefault(x =>
								x.Error == null &&
								x.Architecture == ToolChain.Architecture &&
								x.ReleaseChannel == ToolChain.ReleaseChannel &&
								x.BaseDir == ToolChain.BaseDir &&
								x.HostArchitecture != MSVCHostUnrealArch);
						}
						else
						{
							// Clang the version should match.
							CrossToolChain = ToolChains.FirstOrDefault(x =>
								x.Error == null &&
								x.Architecture == ToolChain.Architecture &&
								x.ReleaseChannel == ToolChain.ReleaseChannel &&
								x.Version == ToolChain.Version &&
								x.HostArchitecture != MSVCHostUnrealArch);
						}

						if (CrossToolChain != null)
						{
							DirectoryReference HostBinDir;
							DirectoryReference CrossBinDir;

							if (Compiler.IsMSVC())
							{
								HostBinDir = DirectoryReference.Combine(ToolChain.BaseDir, "bin", UnrealArchToMSVCHostDirectoryName(ToolChain.HostArchitecture));
								CrossBinDir = DirectoryReference.Combine(CrossToolChain.BaseDir, "bin", UnrealArchToMSVCHostDirectoryName(CrossToolChain.HostArchitecture));
							}
							else
							{
								HostBinDir = DirectoryReference.Combine(ToolChain.BaseDir, MSVCHostUnrealArch.bIsX64 ? "bin" : "bin-woa64");
								CrossBinDir = DirectoryReference.Combine(CrossToolChain.BaseDir, !MSVCHostUnrealArch.bIsX64 ? "bin" : "bin-woa64");
							}

							if (FileReference.Exists(HostBinDir) && FileReference.Exists(CrossBinDir))
							{
								EpicGames.UBA.Utils.RegisterCrossArchitecturePath(HostBinDir, CrossBinDir);
							}
						}
					}
				}
				return ToolChains;
			});
		}

		/// <summary>
		/// Read the Visual Studio install directory for the given compiler version. Note that it is possible for the compiler toolchain to be installed without
		/// Visual Studio, and vice versa.
		/// </summary>
		/// <returns>List of directories containing Visual Studio installations</returns>
		public static IReadOnlyList<VisualStudioInstallation> FindVisualStudioInstallations(ILogger Logger)
		{
			lock (CachedVisualStudioInstallationsLock)
			{
				if (CachedVisualStudioInstallations != null)
				{
					return CachedVisualStudioInstallations;
				}

				List<VisualStudioInstallation> Installations = new List<VisualStudioInstallation>();
				if (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Win64)
				{
					try
					{
						SetupConfiguration Setup = new SetupConfiguration();
						IEnumSetupInstances Enumerator = Setup.EnumAllInstances();

						ISetupInstance[] Instances = new ISetupInstance[1];
						for (; ; )
						{
							int NumFetched;
							Enumerator.Next(1, Instances, out NumFetched);

							if (NumFetched == 0)
							{
								break;
							}

							ISetupInstance2 Instance = (ISetupInstance2)Instances[0];
							if ((Instance.GetState() & InstanceState.Local) != InstanceState.Local)
							{
								continue;
							}

							VersionNumber? Version;
							if (!VersionNumber.TryParse(Instance.GetInstallationVersion(), out Version))
							{
								continue;
							}

							int MajorVersion = Version.GetComponent(0);

							WindowsCompiler Compiler;
							if (Version >= MicrosoftPlatformSDK.MinimumVisualStudio2026Version) // Treat any newer versions as 2026, until we have an explicit enum for them
							{
								Compiler = WindowsCompiler.VisualStudio2026;
							}
							else if (Version >= MicrosoftPlatformSDK.MinimumVisualStudio2022Version)
							{
								Compiler = WindowsCompiler.VisualStudio2022;
							}
							else // Unsupported older versions still detected for legacy MSBuild projects, but will not be used to compile
							{
#pragma warning disable CS0618 // Type or member is obsolete
								Compiler = WindowsCompiler.VisualStudioUnsupported; // Unsupported but still detected for legacy MSBuild projects
#pragma warning restore CS0618 // Type or member is obsolete
							}

							ISetupInstanceCatalog? Catalog = Instance as ISetupInstanceCatalog;
							WindowsCompilerChannel ReleaseChannel = Catalog?.IsPrerelease() == true ? WindowsCompilerChannel.Preview : WindowsCompilerChannel.Latest;

							string ProductId = Instance.GetProduct().GetId();
							bool bCommunity = ProductId.Equals("Microsoft.VisualStudio.Product.Community", StringComparison.Ordinal);

							Dictionary<string, VisualStudioPackage> Packages = [];
							foreach (ISetupPackageReference Package in Instance.GetPackages())
							{
								if (System.Version.TryParse(Package.GetVersion(), out Version? version))
								{
									Packages[Package.GetId()] = new(Package.GetId(), version, Package.GetIsExtension());
								}
							}

							DirectoryReference BaseDir = new DirectoryReference(Instance.GetInstallationPath());
							Installations.Add(new VisualStudioInstallation(Compiler, Version, BaseDir, bCommunity, ReleaseChannel, Packages));

							Logger.LogDebug("Found {VisualStudio} installation: {BaseDir} (Product={ProductId}, Version={Version})", Compiler, BaseDir, ProductId, Version);
						}

						Installations = [.. Installations.OrderByDescending(x => x.Compiler)
							.ThenBy(x => x.bCommunity)
							.ThenBy(x => x.ReleaseChannel)
							.ThenByDescending(x => x.Version)
						];
					}
					catch (Exception Ex)
					{
						Logger.LogDebug(Ex, "Unable to enumerate Visual Studio installations");
					}
				}
				CachedVisualStudioInstallations = Installations;

				return Installations;
			}
		}

		/// <summary>
		/// Read the Visual Studio install directory for the given compiler version. Note that it is possible for the compiler toolchain to be installed without
		/// Visual Studio, and vice versa.
		/// </summary>
		/// <returns>List of directories containing Visual Studio installations</returns>
		public static List<VisualStudioInstallation> FindVisualStudioInstallations(WindowsCompiler Compiler, ILogger Logger)
		{
			return FindVisualStudioInstallations(Logger).Where(x => x.Compiler == Compiler).ToList();
		}

		/// <summary>
		/// Finds all the valid Visual Studio toolchains under the given base directory
		/// </summary>
		/// <param name="BaseDir">Base directory to search</param>
		/// <param name="OptionalRedistDir">Optional directory for redistributable components (DLLs etc)</param>
		/// <param name="ReleaseChannel">The release channel of this installation</param>
		/// <param name="ToolChains">Map of tool chain version to installation info</param>
		/// <param name="IsAutoSdk">Whether this folder contains AutoSDK entries</param>
		/// <param name="Logger">Logger for output</param>
		static void FindVisualStudioToolChains(DirectoryReference BaseDir, DirectoryReference? OptionalRedistDir, WindowsCompilerChannel ReleaseChannel, List<ToolChainInstallation> ToolChains, bool IsAutoSdk, ILogger Logger)
		{
			if (DirectoryReference.Exists(BaseDir))
			{
				foreach (DirectoryReference ToolChainDir in DirectoryReference.EnumerateDirectories(BaseDir))
				{
					VersionNumber? Version;
					if (IsValidToolChainDirMSVC(ToolChainDir, UnrealArch.X64, out Version))
					{
						DirectoryReference? RedistDir = FindVisualStudioRedistForToolChain(ToolChainDir, OptionalRedistDir);
						AddVisualCppToolChain(Version, UnrealArch.X64, ReleaseChannel, ToolChainDir, RedistDir, ToolChains, IsAutoSdk, Logger);
					}

					if (IsValidToolChainDirMSVC(ToolChainDir, UnrealArch.Arm64, out Version))
					{
						DirectoryReference? RedistDir = FindVisualStudioRedistForToolChain(ToolChainDir, OptionalRedistDir);
						AddVisualCppToolChain(Version, UnrealArch.Arm64, ReleaseChannel, ToolChainDir, RedistDir, ToolChains, IsAutoSdk, Logger);
					}
				}
			}
		}

		/// <summary>
		/// Finds the most appropriate redist directory for the given toolchain version
		/// </summary>
		/// <param name="ToolChainDir"></param>
		/// <param name="OptionalRedistDir"></param>
		/// <returns></returns>
		static DirectoryReference? FindVisualStudioRedistForToolChain(DirectoryReference ToolChainDir, DirectoryReference? OptionalRedistDir)
		{
			DirectoryReference? RedistDir;
			if (OptionalRedistDir == null)
			{
				RedistDir = DirectoryReference.Combine(ToolChainDir, "redist"); // AutoSDK keeps redist under the toolchain
			}
			else
			{
				RedistDir = DirectoryReference.Combine(OptionalRedistDir, ToolChainDir.GetDirectoryName());

				// exact redist not found - find highest version (they are backwards compatible)
				if (!DirectoryReference.Exists(RedistDir) && DirectoryReference.Exists(OptionalRedistDir))
				{
					RedistDir = DirectoryReference.EnumerateDirectories(OptionalRedistDir)
						.Where(X => VersionNumber.TryParse(X.GetDirectoryName(), out VersionNumber? DirVersion))
						.OrderByDescending(X => VersionNumber.Parse(X.GetDirectoryName()))
						.FirstOrDefault();
				}
			}

			if (RedistDir != null && DirectoryReference.Exists(RedistDir))
			{
				return RedistDir;
			}

			return null;
		}

		/// <summary>
		/// Adds a Visual C++ toolchain to a list of installations
		/// </summary>
		/// <param name="Version"></param>
		/// <param name="HostArchitecture"></param>
		/// <param name="ReleaseChannel"></param>
		/// <param name="ToolChainDir"></param>
		/// <param name="RedistDir"></param>
		/// <param name="ToolChains"></param>
		/// <param name="IsAutoSdk"></param>
		/// <param name="Logger"></param>
		static void AddVisualCppToolChain(VersionNumber Version, UnrealArch HostArchitecture, WindowsCompilerChannel ReleaseChannel, DirectoryReference ToolChainDir, DirectoryReference? RedistDir, List<ToolChainInstallation> ToolChains, bool IsAutoSdk, ILogger Logger)
		{
			VersionNumber? Family;
			if (!VersionNumber.TryParse(ToolChainDir.GetDirectoryName(), out Family))
			{
				Family = Version;
			}

			int FamilyRank = PreferredVisualCppVersions.TakeWhile(x => !x.Contains(Family)).Count();

			string? Error = null;

			VersionNumberRange? Banned = BannedVisualCppVersions.FirstOrDefault(x => x.Contains(Version));
			if (Banned != null)
			{
				Error = $"UnrealBuildTool has banned the MSVC {Banned} toolchains due to compiler issues. Please install a different toolchain such as {PreferredVisualCppVersions.Select(x => x.Min).Max()} by opening the generated solution and installing recommended components or from the Visual Studio installer.";
			}
			else if (Version < MinimumVisualCppVersion)
			{
				Error = $"UnrealBuildTool requires at minimum the MSVC {MinimumVisualCppVersion} toolchain. Please install a later toolchain such as {PreferredVisualCppVersions.Select(x => x.Min).Max()} from the Visual Studio installer.";
			}

			if (HasX64ToolChain(ToolChainDir, HostArchitecture))
			{
				Logger.LogDebug("Found Visual Studio toolchain: {ToolChainDir} (Family={Family}, FamilyRank={FamilyRank}, Version={Version}, HostArchitecture={HostArchitecture}, ReleaseChannel={ReleaseChannel}, Architecture={Arch}, Error={Error}, Redist={RedistDir})", ToolChainDir, Family, FamilyRank, Version, HostArchitecture, ReleaseChannel, UnrealArch.X64.ToString(), Error != null, RedistDir);
				ToolChains.Add(new ToolChainInstallation(Family, FamilyRank, Version, HostArchitecture, ReleaseChannel, UnrealArch.X64, Error, ToolChainDir, RedistDir, IsAutoSdk));
			}

			if (HasArm64ToolChain(ToolChainDir, HostArchitecture))
			{
				Logger.LogDebug("Found Visual Studio toolchain: {ToolChainDir} (Family={Family}, FamilyRank={FamilyRank}, Version={Version}, HostArchitecture={HostArchitecture}, ReleaseChannel={ReleaseChannel}, Architecture={Arch}, Error={Error}, Redist={RedistDir})", ToolChainDir, Family, FamilyRank, Version, HostArchitecture, ReleaseChannel, UnrealArch.Arm64.ToString(), Error != null, RedistDir);
				ToolChains.Add(new ToolChainInstallation(Family, FamilyRank, Version, HostArchitecture, ReleaseChannel, UnrealArch.Arm64, Error, ToolChainDir, RedistDir, IsAutoSdk));

				Logger.LogDebug("Found Visual Studio toolchain: {ToolChainDir} (Family={Family}, FamilyRank={FamilyRank}, Version={Version}, HostArchitecture={HostArchitecture}, ReleaseChannel={ReleaseChannel}, Architecture={Arch}, Error={Error}, Redist={RedistDir})", ToolChainDir, Family, FamilyRank, Version, HostArchitecture, ReleaseChannel, UnrealArch.Arm64ec.ToString(), Error != null, RedistDir);
				ToolChains.Add(new ToolChainInstallation(Family, FamilyRank, Version, HostArchitecture, ReleaseChannel, UnrealArch.Arm64ec, Error, ToolChainDir, RedistDir, IsAutoSdk));
			}
		}

		/// <summary>
		/// Add a Clang toolchain
		/// </summary>
		/// <param name="ToolChainDir"></param>
		/// <param name="CompilerFile"></param>
		/// <param name="ToolChains"></param>
		/// <param name="IsAutoSdk"></param>
		/// <param name="Logger"></param>
		static void AddClangToolChain(DirectoryReference ToolChainDir, FileReference CompilerFile, List<ToolChainInstallation> ToolChains, bool IsAutoSdk, ILogger Logger)
		{
			if (FileReference.Exists(CompilerFile))
			{
				FileVersionInfo VersionInfo = FileVersionInfo.GetVersionInfo(CompilerFile.FullName);
				VersionNumber Version = new VersionNumber(VersionInfo.FileMajorPart, VersionInfo.FileMinorPart, VersionInfo.FileBuildPart);
				VersionNumber Family = new VersionNumber(VersionInfo.FileMajorPart);

				int Rank = PreferredClangVersions.TakeWhile(x => !x.Contains(Version)).Count();

				string? Error = null;
				if (Version < MinimumClangVersion)
				{
					Error = $"UnrealBuildTool requires at minimum the Clang {MinimumClangVersion} toolchain. Please install a later toolchain such as {PreferredClangVersions.Select(x => x.Min).Max()} from LLVM.";
				}

				UnrealArch HostArchitecture;
				if (IsX64Executable(CompilerFile))
				{
					HostArchitecture = UnrealArch.X64;
				}
				else if (IsArm64Executable(CompilerFile))
				{
					HostArchitecture = UnrealArch.Arm64;
				}
				else
				{
					// Unknown binary architecture
					HostArchitecture = UnrealArch.Deprecated;
					Error = $"Clang binary is not x64 or arm64.";
				}

				Logger.LogDebug("Found Clang toolchain: {ToolChainDir} (Version={Version}, HostArchitecture={HostArchitecture}, Rank={Rank}, Error={Error})", ToolChainDir, Version, HostArchitecture, Rank, Error != null);
				ToolChains.Add(new ToolChainInstallation(Family, Rank, Version, HostArchitecture, WindowsCompilerChannel.Any, UnrealArch.X64, Error, ToolChainDir, null, IsAutoSdk));
				ToolChains.Add(new ToolChainInstallation(Family, Rank, Version, HostArchitecture, WindowsCompilerChannel.Any, UnrealArch.Arm64, Error, ToolChainDir, null, IsAutoSdk));
				ToolChains.Add(new ToolChainInstallation(Family, Rank, Version, HostArchitecture, WindowsCompilerChannel.Any, UnrealArch.Arm64ec, Error, ToolChainDir, null, IsAutoSdk));
			}
		}

		/// <summary>
		/// Add a Clang toolchain
		/// </summary>
		/// <param name="Compiler"></param>
		/// <param name="ToolChainDir"></param>
		/// <param name="ToolChains"></param>
		/// <param name="IsAutoSdk"></param>
		/// <param name="Logger"></param>
		static void AddClangToolChain(WindowsCompiler Compiler, DirectoryReference ToolChainDir, List<ToolChainInstallation> ToolChains, bool IsAutoSdk, ILogger Logger)
		{
			string CompilerFilename = Compiler == WindowsCompiler.ClangRTFM ? "verse-clang-cl.exe" : (Compiler == WindowsCompiler.ClangInstrument ? "instr-clang-cl.exe" : "clang-cl.exe");
			AddClangToolChain(ToolChainDir, FileReference.Combine(ToolChainDir, "bin", CompilerFilename), ToolChains, IsAutoSdk, Logger);
			AddClangToolChain(ToolChainDir, FileReference.Combine(ToolChainDir, "bin-woa64", CompilerFilename), ToolChains, IsAutoSdk, Logger);
		}

		/// <summary>
		/// Add an Intel OneAPI toolchain
		/// </summary>
		/// <param name="ToolChainDir"></param>
		/// <param name="ToolChains"></param>
		/// <param name="IsAutoSdk"></param>
		/// <param name="Logger"></param>
		static void AddIntelOneApiToolChain(DirectoryReference ToolChainDir, List<ToolChainInstallation> ToolChains, bool IsAutoSdk, ILogger Logger)
		{
			FileReference CompilerFile = FileReference.Combine(ToolChainDir, "bin", "icx.exe");
			if (FileReference.Exists(CompilerFile))
			{
				FileVersionInfo VersionInfo = FileVersionInfo.GetVersionInfo(CompilerFile.FullName);
				VersionNumber Version = new VersionNumber(VersionInfo.FileMajorPart, VersionInfo.FileMinorPart, VersionInfo.FileBuildPart);
				VersionNumber Family = new VersionNumber(VersionInfo.FileMajorPart);

				// The icx.exe version may be lower than the toolchain folder version, so use that instead if available
				if (VersionNumber.TryParse(ToolChainDir.GetDirectoryName(), out VersionNumber? FolderVersion) && FolderVersion > Version)
				{
					Version = FolderVersion;
				}

				int Rank = PreferredIntelOneApiVersions.TakeWhile(x => !x.Contains(Version)).Count();
				bool Is64Bit = IsX64Executable(CompilerFile);

				string? Error = null;
				if (Version < MinimumIntelOneApiVersion)
				{
					Error = $"UnrealBuildTool requires at minimum the Intel OneAPI {MinimumIntelOneApiVersion} toolchain. Please install a later toolchain such as {PreferredIntelOneApiVersions.Select(x => x.Min).Max()} from Intel.";
				}

				UnrealArch HostArchitecture;
				if (IsX64Executable(CompilerFile))
				{
					HostArchitecture = UnrealArch.X64;
				}
				else if (IsArm64Executable(CompilerFile))
				{
					HostArchitecture = UnrealArch.Arm64;
				}
				else
				{
					// Unknown binary architecture
					HostArchitecture = UnrealArch.Deprecated;
					Error = $"Clang binary is not x64 or arm64.";
				}

				Logger.LogDebug("Found Intel OneAPI toolchain: {ToolChainDir} (Version={Version}, HostArchitecture={HostArchitecture}, Rank={Rank}, Error={Error})", ToolChainDir, Version, Is64Bit, Rank, Error != null);
				ToolChains.Add(new ToolChainInstallation(Family, Rank, Version, HostArchitecture, WindowsCompilerChannel.Any, UnrealArch.X64, Error, ToolChainDir, null, IsAutoSdk));
			}
		}

		/// <summary>
		/// Finds all the valid Intel oneAPI toolchains under the given base directory
		/// </summary>
		/// <param name="BaseDir">Base directory to search</param>
		/// <param name="ToolChains">Map of tool chain version to installation info</param>
		/// <param name="IsAutoSdk"></param>
		/// <param name="Logger">Logger for output</param>
		static void FindIntelOneApiToolChains(DirectoryReference BaseDir, List<ToolChainInstallation> ToolChains, bool IsAutoSdk, ILogger Logger)
		{
			if (DirectoryReference.Exists(BaseDir))
			{
				foreach (DirectoryReference ToolChainDir in DirectoryReference.EnumerateDirectories(BaseDir))
				{
					AddIntelOneApiToolChain(ToolChainDir, ToolChains, IsAutoSdk, Logger);
				}
			}
		}

		/// <summary>
		/// The compiler host directory name for the running process architecture.
		/// </summary>
		public static UnrealArch MSVCHostUnrealArch => RuntimeInformation.ProcessArchitecture == Architecture.Arm64 ? UnrealArch.Arm64 : UnrealArch.X64;

		/// <summary>
		/// The compiler host directory name for an UnrealArch.
		/// </summary>
		/// <param name="HostArchitecture">The binary architecture for the toolchain</param>
		public static string UnrealArchToMSVCHostDirectoryName(UnrealArch HostArchitecture) => HostArchitecture.bIsX64 ? "Hostx64" : "Hostarm64";

		/// <summary>
		/// The compiler host directory name for the running process architecture.
		/// </summary>
		public static string MSVCHostDirectoryName => UnrealArchToMSVCHostDirectoryName(MSVCHostUnrealArch);

		/// <summary>
		/// Test whether an executable is a specific machine type
		/// </summary>
		/// <param name="File">Executable to test</param>
		/// <param name="MachineType">The machine type to check</param>
		/// <returns></returns>
		static bool IsExecutableMachineType(FileReference File, ushort MachineType)
		{
			using (FileStream Stream = new FileStream(File.FullName, FileMode.Open, FileAccess.Read, FileShare.Read | FileShare.Delete))
			{
				byte[] Header = new byte[64];
				if (Stream.Read(Header, 0, Header.Length) != Header.Length)
				{
					return false;
				}
				if (Header[0] != (byte)'M' || Header[1] != (byte)'Z')
				{
					return false;
				}

				int Offset = BinaryPrimitives.ReadInt32LittleEndian(Header.AsSpan(0x3c));
				if (Stream.Seek(Offset, SeekOrigin.Begin) != Offset)
				{
					return false;
				}

				byte[] PeHeader = new byte[6];
				if (Stream.Read(PeHeader, 0, PeHeader.Length) != PeHeader.Length)
				{
					return false;
				}
				if (BinaryPrimitives.ReadUInt32BigEndian(PeHeader.AsSpan()) != 0x50450000)
				{
					return false;
				}

				return MachineType == BinaryPrimitives.ReadUInt16LittleEndian(PeHeader.AsSpan(4));
			}
		}

		/// <summary>
		/// Test whether an executable is x64
		/// </summary>
		/// <param name="File">Executable to test</param>
		/// <returns></returns>
		static bool IsX64Executable(FileReference File)
		{
			const ushort IMAGE_FILE_MACHINE_AMD64 = 0x8664;
			return IsExecutableMachineType(File, IMAGE_FILE_MACHINE_AMD64);
		}

		/// <summary>
		/// Test whether an executable is arm64
		/// </summary>
		/// <param name="File">Executable to test</param>
		/// <returns></returns>
		static bool IsArm64Executable(FileReference File)
		{
			const ushort IMAGE_FILE_MACHINE_ARM64 = 0xaa64;
			return IsExecutableMachineType(File, IMAGE_FILE_MACHINE_ARM64);
		}

		/// <summary>
		/// Determines if the given path is a valid Visual C++ version number
		/// </summary>
		/// <param name="ToolChainDir">The toolchain directory</param>
		/// <param name="HostArchitecture">The binary architecture for the toolchain</param>
		/// <param name="Version">The version number for the toolchain</param>
		/// <returns>True if the path is a valid version</returns>
		static bool IsValidToolChainDirMSVC(DirectoryReference ToolChainDir, UnrealArch HostArchitecture, [NotNullWhen(true)] out VersionNumber? Version)
		{
			FileReference CompilerExe = FileReference.Combine(ToolChainDir, "bin", UnrealArchToMSVCHostDirectoryName(HostArchitecture), "x64", "cl.exe");
			if (!FileReference.Exists(CompilerExe))
			{
				CompilerExe = FileReference.Combine(ToolChainDir, "bin", UnrealArchToMSVCHostDirectoryName(HostArchitecture), "arm64", "cl.exe");
			}

			if (!FileReference.Exists(CompilerExe))
			{
				Version = null;
				return false;
			}

			FileVersionInfo VersionInfo = FileVersionInfo.GetVersionInfo(CompilerExe.FullName);
			if (VersionInfo.ProductMajorPart != 0)
			{
				Version = new VersionNumber(VersionInfo.ProductMajorPart, VersionInfo.ProductMinorPart, VersionInfo.ProductBuildPart);
				return true;
			}

			return VersionNumber.TryParse(ToolChainDir.GetDirectoryName(), out Version);
		}

		/// <summary>
		/// Checks if the given directory contains a x64 toolchain. 
		/// </summary>
		/// <param name="ToolChainDir">Directory to check</param>
		/// <param name="HostArchitecture">The host binary architecture to check</param>
		/// <returns>True if the given directory contains a 64-bit toolchain</returns>
		static bool HasX64ToolChain(DirectoryReference ToolChainDir, UnrealArch HostArchitecture)
		{
			return FileReference.Exists(FileReference.Combine(ToolChainDir, "bin", UnrealArchToMSVCHostDirectoryName(HostArchitecture), "x64", "cl.exe"));
		}

		/// <summary>
		/// Checks if the given directory contains an arm64 toolchain.
		/// </summary>
		/// <param name="ToolChainDir">Directory to check</param>
		/// <param name="HostArchitecture">The host binary architecture to check</param>
		/// <returns>True if the given directory contains the arm64 toolchain</returns>
		static bool HasArm64ToolChain(DirectoryReference ToolChainDir, UnrealArch HostArchitecture)
		{
			return FileReference.Exists(FileReference.Combine(ToolChainDir, "bin", UnrealArchToMSVCHostDirectoryName(HostArchitecture), "arm64", "cl.exe"));
		}

		/// <summary>
		/// Determines if an IDE for the given compiler is installed.
		/// </summary>
		/// <param name="Compiler">Compiler to check for</param>
		/// <param name="Logger">Logger for output</param>
		/// <returns>True if the given compiler is installed</returns>
		public static bool HasIDE(WindowsCompiler Compiler, ILogger Logger)
		{
			return FindVisualStudioInstallations(Compiler, Logger).Count > 0;
		}

		/// <summary>
		/// Gets the path to MSBuild. This mirrors the logic in GetMSBuildPath.bat.
		/// </summary>
		/// <param name="Logger">Logger for output</param>
		/// <param name="OutLocation">On success, receives the path to the MSBuild executable.</param>
		/// <returns>True on success.</returns>
		[SupportedOSPlatform("windows")]
		public static bool TryGetMsBuildPath(ILogger Logger, [NotNullWhen(true)] out FileReference? OutLocation)
		{
			// Get the Visual Studio install directory
#pragma warning disable CS0618 // Type or member is obsolete
			IEnumerable<DirectoryReference> InstallDirs = FindVisualStudioInstallations(WindowsCompiler.VisualStudio2022, Logger)
				.Concat(FindVisualStudioInstallations(WindowsCompiler.VisualStudio2026, Logger))
				.Concat(FindVisualStudioInstallations(WindowsCompiler.VisualStudioUnsupported, Logger))
				.Select(x => x.BaseDir);
#pragma warning restore CS0618 // Type or member is obsolete

			foreach (DirectoryReference InstallDir in InstallDirs)
			{
				FileReference MsBuildLocation = FileReference.Combine(InstallDir, "MSBuild", "Current", "Bin", "MSBuild.exe");
				if (FileReference.Exists(MsBuildLocation))
				{
					OutLocation = MsBuildLocation;
					return true;
				}
			}

			OutLocation = null;
			return false;
		}

		/// <summary>
		/// Function to query the registry under HKCU/HKLM Win32/Wow64 software registry keys for a certain install directory.
		/// This mirrors the logic in GetMSBuildPath.bat.
		/// </summary>
		/// <returns></returns>
		[SupportedOSPlatform("windows")]
		static bool TryReadMsBuildInstallPath(string KeyRelativePath, string KeyName, string MsBuildRelativePath, [NotNullWhen(true)] out FileReference? OutMsBuildPath)
		{
			string[] KeyBasePaths =
			{
				@"HKEY_CURRENT_USER\SOFTWARE\",
				@"HKEY_LOCAL_MACHINE\SOFTWARE\",
				@"HKEY_CURRENT_USER\SOFTWARE\Wow6432Node\",
				@"HKEY_LOCAL_MACHINE\SOFTWARE\Wow6432Node\"
			};

			foreach (string KeyBasePath in KeyBasePaths)
			{
				string? Value = Registry.GetValue(KeyBasePath + KeyRelativePath, KeyName, null) as string;
				if (Value != null)
				{
					FileReference MsBuildPath = FileReference.Combine(new DirectoryReference(Value), MsBuildRelativePath);
					if (FileReference.Exists(MsBuildPath))
					{
						OutMsBuildPath = MsBuildPath;
						return true;
					}
				}
			}

			OutMsBuildPath = null;
			return false;
		}

		#endregion // Toolchain Private

		#endregion // Toolchain

		#region Dia Sdk

		/// <summary>
		/// Determines the directory containing the MSVC toolchain
		/// </summary>
		/// <param name="Compiler">Major version of the compiler to use</param>
		/// <returns>Map of version number to directories</returns>
		public static List<DirectoryReference> FindDiaSdkDirs(WindowsCompiler Compiler)
		{
			return CachedDiaSdkDirs.GetOrAdd(Compiler, _ =>
			{
				List<DirectoryReference> DiaSdkDirs = [];

				// VisualStudio and VisualStudio2026 are synonymous, so try both VS2026 and VS2022 if that's specified.
				List<WindowsCompiler> CompilersToCheck = Compiler == WindowsCompiler.VisualStudio ?
					[WindowsCompiler.VisualStudio2026, WindowsCompiler.VisualStudio2022] :
					[Compiler];

				foreach (WindowsCompiler CompilerToCheck in CompilersToCheck)
				{
					DirectoryReference? PlatformDir;
					if (TryGetHostPlatformAutoSDKDir(out PlatformDir))
					{
						string VSDir = "";
						switch (CompilerToCheck)
						{
							case WindowsCompiler.VisualStudio2026: VSDir = "VS2026"; break;
							case WindowsCompiler.VisualStudio2022: VSDir = "VS2022"; break;
						}

						if (VSDir.Length != 0)
						{
							DirectoryReference DiaSdkDir = DirectoryReference.Combine(PlatformDir, "Win64", "DIA SDK", VSDir);
							if (IsValidDiaSdkDir(DiaSdkDir))
							{
								DiaSdkDirs.Add(DiaSdkDir);
							}
						}
					}

					List<DirectoryReference> VisualStudioDirs = FindVisualStudioInstallations(CompilerToCheck, Log.Logger).ConvertAll(x => x.BaseDir);
					foreach (DirectoryReference VisualStudioDir in VisualStudioDirs)
					{
						DirectoryReference DiaSdkDir = DirectoryReference.Combine(VisualStudioDir, "DIA SDK");
						if (IsValidDiaSdkDir(DiaSdkDir))
						{
							DiaSdkDirs.Add(DiaSdkDir);
						}
					}
				}
				return DiaSdkDirs;
			});
		}

		/// <summary>
		/// Determines if a directory contains a valid DIA SDK
		/// </summary>
		/// <param name="DiaSdkDir">The directory to check</param>
		/// <returns>True if it contains a valid DIA SDK</returns>
		static bool IsValidDiaSdkDir(DirectoryReference DiaSdkDir)
		{
			return FileReference.Exists(FileReference.Combine(DiaSdkDir, "bin", "amd64", "msdia140.dll"));
		}

		#endregion // Dia

		#endregion // Windows Specific SDK 
	}

	/// <summary>
	/// Information about a particular toolchain installation
	/// </summary>
	[DebuggerDisplay("{BaseDir}")]
	internal class ToolChainInstallation
	{
		/// <summary>
		/// The version "family" (ie. the nominal version number of the directory this toolchain is installed to)
		/// </summary>
		public VersionNumber Family { get; }

		/// <summary>
		/// Index into the preferred version range
		/// </summary>
		public int FamilyRank { get; }

		/// <summary>
		/// The actual version number of this toolchain
		/// </summary>
		public VersionNumber Version { get; }

		/// <summary>
		/// The host architecture of this ToolChainInstallation (multiple ToolChainInstallation instances may be created, one per host architecture).
		/// </summary>
		public UnrealArch HostArchitecture { get; }

		/// <summary>
		/// The release channel of this toolchain
		/// </summary>
		public WindowsCompilerChannel ReleaseChannel { get; }

		/// <summary>
		/// The architecture of this ToolChainInstallation (multiple ToolChainInstallation instances may be created, one per architecture).
		/// </summary>
		public UnrealArch Architecture { get; }

		/// <summary>
		/// Reason for this toolchain not being compatible
		/// </summary>
		public string? Error { get; }

		/// <summary>
		/// Base directory for the toolchain
		/// </summary>
		public DirectoryReference BaseDir { get; }

		/// <summary>
		/// Base directory for the redistributable components
		/// </summary>
		public DirectoryReference? RedistDir { get; }

		/// <summary>
		/// Whether this toolchain comes from AutoSDK.
		/// </summary>
		public bool IsAutoSdk { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="Family"></param>
		/// <param name="FamilyRank"></param>
		/// <param name="Version"></param>
		/// <param name="HostArchitecture">The architecture of the binaries in this toolchain</param>
		/// <param name="ReleaseChannel">The release channel of this toolchain</param>
		/// <param name="Architecture">The target architecture this toolchain can compile</param>
		/// <param name="Error"></param>
		/// <param name="BaseDir">Base directory for the toolchain</param>
		/// <param name="RedistDir">Optional directory for redistributable components (DLLs etc)</param>
		/// <param name="IsAutoSdk">Whether this toolchain comes from AutoSDK</param>
		public ToolChainInstallation(VersionNumber Family, int FamilyRank, VersionNumber Version, UnrealArch HostArchitecture, WindowsCompilerChannel ReleaseChannel, UnrealArch Architecture, string? Error, DirectoryReference BaseDir, DirectoryReference? RedistDir, bool IsAutoSdk)
		{
			this.Family = Family;
			this.FamilyRank = FamilyRank;
			this.Version = Version;
			this.HostArchitecture = HostArchitecture;
			this.ReleaseChannel = ReleaseChannel;
			this.Architecture = Architecture;
			this.Error = Error;
			this.BaseDir = BaseDir;
			this.RedistDir = RedistDir;
			this.IsAutoSdk = IsAutoSdk;
		}
	}
}
