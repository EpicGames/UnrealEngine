// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Text;
using System.Text.Json;
using System.Text.RegularExpressions;
using System.Threading.Tasks;
using System.Xml;
using System.Xml.Serialization;
using EpicGames.Core;
using EpicGames.LogMatchers;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Logging.Abstractions;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// Flags for the PVS analyzer mode
	/// </summary>
	[Flags]
#pragma warning disable CA1028 // uint data type more closely matches the underlying PVS-Studio value
	public enum PVSAnalysisModeFlags : uint
#pragma warning restore CA1028
	{
		/// <summary>
		/// No flags
		/// </summary>
		None = 0,

		/// <summary>
		/// Check for 64-bit portability issues
		/// </summary>
		Check64BitPortability = 1,

		/// <summary>
		/// Enable general analysis
		/// </summary>
		GeneralAnalysis = 4,

		/// <summary>
		/// Check for optimizations
		/// </summary>
		Optimizations = 8,

		/// <summary>
		/// Enable customer-specific rules
		/// </summary>
		CustomerSpecific = 16,

		/// <summary>
		/// Enable MISRA analysis
		/// </summary>
		MISRA = 32,
	}

#pragma warning disable CA1707 // Identifiers should not contain underscores - public facing API
	/// <summary>
	/// Supported PVS analyzer timeout values
	/// </summary>
	public enum AnalysisTimeout
	{
		/// <summary>
		/// Analysis timeout when not set (infinite)
		/// </summary>
		No_timeout = 0,
		/// <summary>
		/// Analysis timeout for file 10 minutes (600 seconds)
		/// </summary>
		After_10_minutes = 600,
		/// <summary>
		/// Analysis timeout for file 30 minutes (1800 seconds)
		/// </summary>
		After_30_minutes = 1800,
		/// <summary>
		/// Analysis timeout for file 60 minutes (3600 seconds)
		/// </summary>
		After_60_minutes = 3600,
	}
#pragma warning restore CA1707

	/// <summary>
	/// Partial representation of PVS-Studio main settings file
	/// </summary>
	[XmlRoot("ApplicationSettings")]
	public class PVSApplicationSettings
	{
		/// <summary>
		/// Masks for paths excluded for analysis
		/// </summary>
		public string[]? PathMasks;

		/// <summary>
		/// Registered username
		/// </summary>
		public string? UserName;

		/// <summary>
		/// Registered serial number
		/// </summary>
		public string? SerialNumber;

		/// <summary>
		/// Disable the 64-bit Analysis
		/// </summary>
		public bool Disable64BitAnalysis;

		/// <summary>
		/// Disable the General Analysis
		/// </summary>
		public bool DisableGAAnalysis;

		/// <summary>
		/// Disable the Optimization Analysis
		/// </summary>
		public bool DisableOPAnalysis;

		/// <summary>
		/// Disable the Customer's Specific diagnostic rules
		/// </summary>
		public bool DisableCSAnalysis;

		/// <summary>
		/// Disable the MISRA Analysis
		/// </summary>
		public bool DisableMISRAAnalysis;

		/// <summary>
		/// File analysis timeout
		/// </summary>
		public AnalysisTimeout AnalysisTimeout;

		/// <summary>
		/// Disable analyzer Level 3 (Low) messages
		/// </summary>
		public bool NoNoise;

		/// <summary>
		/// Enable the display of analyzer rules exceptions which can be specified by comments and .pvsconfig files.
		/// </summary>
		public bool ReportDisabledRules;

		/// <summary>
		/// Gets the analysis mode flags from the settings
		/// </summary>
		/// <returns>Mode flags</returns>
		public PVSAnalysisModeFlags GetModeFlags()
		{
			PVSAnalysisModeFlags Flags = 0;
			if (!Disable64BitAnalysis)
			{
				Flags |= PVSAnalysisModeFlags.Check64BitPortability;
			}
			if (!DisableGAAnalysis)
			{
				Flags |= PVSAnalysisModeFlags.GeneralAnalysis;
			}
			if (!DisableOPAnalysis)
			{
				Flags |= PVSAnalysisModeFlags.Optimizations;
			}
			if (!DisableCSAnalysis)
			{
				Flags |= PVSAnalysisModeFlags.CustomerSpecific;
			}
			if (!DisableMISRAAnalysis)
			{
				Flags |= PVSAnalysisModeFlags.MISRA;
			}
			return Flags;
		}

		/// <summary>
		/// Attempts to read the application settings from the default location
		/// </summary>
		/// <returns>Application settings instance, or null if no file was present</returns>
		internal static PVSApplicationSettings? Read()
		{
			FileReference SettingsPath = FileReference.Combine(new DirectoryReference(Environment.GetFolderPath(Environment.SpecialFolder.ApplicationData)), "PVS-Studio", "Settings.xml");
			if (FileReference.Exists(SettingsPath))
			{
				try
				{
					XmlSerializer Serializer = new(typeof(PVSApplicationSettings));
					using FileStream Stream = new(SettingsPath.FullName, FileMode.Open, FileAccess.Read, FileShare.Read);
					using XmlReader XmlReader = XmlReader.Create(Stream);
					return (PVSApplicationSettings?)Serializer.Deserialize(XmlReader);
				}
				catch (Exception Ex)
				{
					throw new BuildException(Ex, "Unable to read PVS-Studio settings file from {0}", SettingsPath);
				}
			}
			return null;
		}
	}

	/// <summary>
	/// Settings for the PVS Studio analyzer
	/// </summary>
	public class PVSTargetSettings
	{
		/// <summary>
		/// Returns the application settings
		/// </summary>
		internal Lazy<PVSApplicationSettings?> ApplicationSettings { get; } = new Lazy<PVSApplicationSettings?>(() => PVSApplicationSettings.Read());

		/// <summary>
		/// Whether to use application settings to determine the analysis mode
		/// </summary>
		public bool UseApplicationSettings { get; set; }

		/// <summary>
		/// Override for the analysis mode to use
		/// </summary>
		public PVSAnalysisModeFlags ModeFlags
		{
			get
			{
				if (ModePrivate.HasValue)
				{
					return ModePrivate.Value;
				}
				else if (UseApplicationSettings && ApplicationSettings.Value != null)
				{
					return ApplicationSettings.Value.GetModeFlags();
				}
				else
				{
					return PVSAnalysisModeFlags.GeneralAnalysis;
				}
			}
			set => ModePrivate = value;
		}

		/// <summary>
		/// Private storage for the mode flags
		/// </summary>
		PVSAnalysisModeFlags? ModePrivate;

		/// <summary>
		/// Override for the analysis timeoutFlag to use
		/// </summary>
		public AnalysisTimeout AnalysisTimeoutFlag
		{
			get
			{
				if (TimeoutPrivate.HasValue)
				{
					return TimeoutPrivate.Value;
				}
				else if (UseApplicationSettings && ApplicationSettings.Value != null)
				{
					return ApplicationSettings.Value.AnalysisTimeout;
				}
				else
				{
					return AnalysisTimeout.After_30_minutes;
				}
			}
			set => TimeoutPrivate = value;
		}

		/// <summary>
		/// Private storage for the analysis timeout
		/// </summary>
		AnalysisTimeout? TimeoutPrivate;

		/// <summary>
		/// Override for the disable Level 3 (Low) analyzer messages
		/// </summary>
		public bool EnableNoNoise
		{
			get
			{
				if (EnableNoNoisePrivate.HasValue)
				{
					return EnableNoNoisePrivate.Value;
				}
				else if (UseApplicationSettings && ApplicationSettings.Value != null)
				{
					return ApplicationSettings.Value.NoNoise;
				}
				else
				{
					return false;
				}
			}
			set => EnableNoNoisePrivate = value;
		}

		/// <summary>
		/// Private storage for the NoNoise analyzer setting
		/// </summary>
		bool? EnableNoNoisePrivate;

		/// <summary>
		/// Override for the enable the display of analyzer rules exceptions which can be specified by comments and .pvsconfig files.
		/// </summary>
		public bool EnableReportDisabledRules
		{
			get
			{
				if (EnableReportDisabledRulesPrivate.HasValue)
				{
					return EnableReportDisabledRulesPrivate.Value;
				}
				else if (UseApplicationSettings && ApplicationSettings.Value != null)
				{
					return ApplicationSettings.Value.ReportDisabledRules;
				}
				else
				{
					return false;
				}
			}
			set => EnableReportDisabledRulesPrivate = value;
		}

		/// <summary>
		/// Private storage for the ReportDisabledRules analyzer setting
		/// </summary>
		bool? EnableReportDisabledRulesPrivate;
	}

	/// <summary>
	/// Read-only version of the PVS toolchain settings
	/// </summary>
	/// <remarks>
	/// Constructor
	/// </remarks>
	/// <param name="Inner">The inner object</param>
	public class ReadOnlyPVSTargetSettings(PVSTargetSettings Inner)
	{
		/// <summary>
		/// Accessor for the Application settings
		/// </summary>
		internal PVSApplicationSettings? ApplicationSettings => Inner.ApplicationSettings.Value;

		/// <summary>
		/// Whether to use the application settings for the mode
		/// </summary>
		public bool UseApplicationSettings => Inner.UseApplicationSettings;

		/// <summary>
		/// Override for the analysis mode to use
		/// </summary>
		public PVSAnalysisModeFlags ModeFlags => Inner.ModeFlags;

		/// <summary>
		/// Override for the analysis timeout to use
		/// </summary>
		public AnalysisTimeout AnalysisTimeoutFlag => Inner.AnalysisTimeoutFlag;

		/// <summary>
		/// Override NoNoise analysis setting to use
		/// </summary>
		public bool EnableNoNoise => Inner.EnableNoNoise;

		/// <summary>
		/// Override EnableReportDisabledRules analysis setting to use
		/// </summary>
		public bool EnableReportDisabledRules => Inner.EnableReportDisabledRules;
	}

	/// <summary>
	/// Special mode for gathering all the messages into a single output file
	/// </summary>
	internal sealed class PVSGatherMode : IToolMode<PVSGatherMode>
	{
		public static string Name => "PVSGather";
		public static ToolModeOptions Options => ToolModeOptions.None;

#pragma warning disable IDE0044 // Make field readonly - these private static fields are set by command-line parsing.
		/// <summary>
		/// Path to the input file list
		/// </summary>
		[CommandLine("-Input", Required = true)]
		FileReference? _inputFileList = null;

		/// <summary>
		/// Output file to generate
		/// </summary>
		[CommandLine("-Output", Required = true)]
		FileReference? _outputFile = null;

		/// <summary>
		/// Path to file list of paths to ignore
		/// </summary>
		[CommandLine("-Ignored", Required = true)]
		FileReference? _ignoredFile = null;

		/// <summary>
		/// Path to file list of rootpaths,realpath mapping
		/// </summary>
		[CommandLine("-RootPaths", Required = true)]
		FileReference? _rootPathsFile = null;

		/// <summary>
		/// The maximum level of warnings to print
		/// </summary>
		[CommandLine("-PrintLevel")]
		int _printLevel = 1;
#pragma warning restore IDE0044

		/// <summary>
		/// If all ThirdParty code should be ignored
		/// </summary>
		const bool _ignoreThirdParty = true;

		readonly CaptureLogger _parseLogger = new();

		IEnumerable<DirectoryReference> _ignoredDirectories = [];

		CppRootPaths _rootPaths = new();

		private readonly JsonSerializerOptions _jsonSerializerOptions = new()
		{
			// PVS-Studio output is camelCased, so we require insensitive property names to deserialize to our
			// standard C# property names.
			PropertyNameCaseInsensitive = true,
		};

		/// <summary>
		/// Execute the command
		/// </summary>
		/// <param name="arguments">List of command line arguments</param>
		/// <returns>Always zero, or throws an exception</returns>
		/// <param name="logger"></param>
		public async Task<int> ExecuteAsync(CommandLineArguments arguments, ILogger logger)
		{
			arguments.ApplyTo(this);
			arguments.CheckAllArgumentsUsed();

			if (_inputFileList == null || _outputFile == null || _ignoredFile == null || _rootPathsFile == null)
			{
				throw new NullReferenceException();
			}

			// Read the input files
			string[] inputFileLines = await FileReference.ReadAllLinesAsync(_inputFileList);
			IEnumerable<FileReference> inputFiles = inputFileLines.Select(x => x.Trim()).Where(x => x.Length > 0).Select(x => new FileReference(x));

			// Read the ignore file
			string[] ignoreFileLines = await FileReference.ReadAllLinesAsync(_ignoredFile);
			_ignoredDirectories = ignoreFileLines.Select(x => x.Trim()).Where(x => x.Length > 0).Select(x => new DirectoryReference(x));

			// Read the root paths file
			_rootPaths = new(new BinaryArchiveReader(_rootPathsFile));

			// Remove analyzedSourceFiles array from each line so all the lines can be more efficiently deduped.
			Regex removeRegex = new("\"analyzedSourceFiles\":\\[.*?\\],", RegexOptions.Compiled);

			ParallelQuery<string> allLines = inputFiles.AsParallel().SelectMany(FileReference.ReadAllLines)
				.Select(x => removeRegex.Replace(x, String.Empty))
				.Distinct();

			// Task.Run to prevent blocking on parallel query
			Task writeTask = Task.Run(() => FileReference.WriteAllLines(_outputFile, allLines.OrderBy(x => x)));

			ParallelQuery<PVSErrorInfo> allErrors = allLines.Select(GetErrorInfo)
				.OfType<PVSErrorInfo>();

			OrderedParallelQuery<PVSErrorInfo> filteredErrors = allErrors
				.Where(x => x.FalseAlarm != true && x.Level <= _printLevel) // Ignore false alarm warnings, and limit printing by PrintLevel
				.Where(x => !String.IsNullOrWhiteSpace(x.Positions.FirstOrDefault()?.File)) // Ignore files with no position
				.OrderBy(x => x.Positions.FirstOrDefault()?.File)
				.ThenBy(x => x.Positions.FirstOrDefault()?.Lines.FirstOrDefault());

			using (LogEventParser parser = new(logger))
			{
				parser.AddMatchers(LogMatcherCollections.UBTMatchers);
				// Create the combined output file, and print the diagnostics to the log
				foreach (PVSErrorInfo errorInfo in filteredErrors)
				{
					string file = errorInfo.Positions.FirstOrDefault()?.File ?? Unreal.EngineDirectory.FullName;
					int lineNumber = errorInfo.Positions.FirstOrDefault()?.Lines.FirstOrDefault() ?? 1;
					parser.WriteLine($"{file}({lineNumber}): warning {errorInfo.Code}: {errorInfo.Message}");
				}
			}

			PVSErrorInfo? renewError = allErrors.FirstOrDefault(x => x.Code.Equals("Renew", StringComparison.Ordinal));
			if (renewError != null)
			{
				logger.LogInformation("PVS-Studio Renewal Notice: {WarningMessage}", renewError.Message);
				logger.LogWarning("Warning: PVS-Studio license will expire soon. See output log for details.");
			}

			await writeTask;
			int count = allLines.Count();
			logger.LogInformation("Written {NumItems} {Noun} to {File}.", count, (count == 1) ? "diagnostic" : "diagnostics", _outputFile.FullName);
			_parseLogger.RenderTo(logger);
			return 0;
		}

		// Ignore anything in the IgnoredDirectories folders or ThirdParty if ignored
		bool IsFileIgnored(FileReference? fileReference) => fileReference != null
			&& ((_ignoreThirdParty && fileReference.FullName.Contains("ThirdParty", StringComparison.OrdinalIgnoreCase)) || (_ignoredDirectories.Any() && _ignoredDirectories.Any(fileReference.IsUnderDirectory)));

		PVSErrorInfo? GetErrorInfo(string line)
		{
			try
			{
				PVSErrorInfo errorInfo = JsonSerializer.Deserialize<PVSErrorInfo>(line, _jsonSerializerOptions) ?? throw new FormatException();
				FileReference? fileReference = errorInfo.Positions.FirstOrDefault()?.UpdateFilePath(_rootPaths);
				return IsFileIgnored(fileReference) ? null : errorInfo;
			}
			catch (Exception ex)
			{
				_parseLogger.LogDebug(KnownLogEvents.Compiler, "warning: Unable to parse PVS output line '{Line}' ({Message})", line, ex.Message);
			}
			return null;
		}
	}

	class PVSPosition
	{
		public required string File { get; set; }

		public required int[] Lines { get; init; }

		public FileReference? UpdateFilePath(CppRootPaths rootPaths)
		{
			FileReference? fileReference = FileReference.FromString(File);
			if (fileReference != null && rootPaths.bUseVfs)
			{
				fileReference = rootPaths.GetLocalPath(fileReference);
				File = fileReference.FullName;
			}
			return fileReference;
		}
	}

	class PVSErrorInfo
	{
		public required string Code { get; init; }

		public required bool FalseAlarm { get; init; }

		public required int Level { get; init; }

		public required string Message { get; init; }

		public required PVSPosition[] Positions { get; init; }
	}

	class PVSToolChain : ISPCToolChain
	{
		readonly ReadOnlyTargetRules Target;
		readonly ReadOnlyPVSTargetSettings Settings;
		readonly PVSApplicationSettings? ApplicationSettings;
		readonly VCToolChain InnerToolChain;
		readonly FileReference AnalyzerFile;
		readonly FileReference? LicenseFile;
		readonly UnrealTargetPlatform Platform;
		readonly Version AnalyzerVersion;

		static readonly Version _minAnalyzerVersion = new Version("7.30");
		static readonly Version _analysisPathsSkipVersion = new Version("7.34");

		const string OutputFileExtension = ".PVS-Studio.log";

		public PVSToolChain(ReadOnlyTargetRules Target, VCToolChain InInnerToolchain, ILogger Logger)
			: base(Logger)
		{
			this.Target = Target;
			Platform = Target.Platform;
			InnerToolChain = InInnerToolchain;

			AnalyzerFile = FileReference.Combine(Unreal.RootDirectory, "Engine", "Restricted", "NoRedist", "Extras", "ThirdPartyNotUE", "PVS-Studio", "PVS-Studio.exe");
			if (!FileReference.Exists(AnalyzerFile))
			{
				FileReference InstalledAnalyzerFile = FileReference.Combine(new DirectoryReference(Environment.GetFolderPath(Environment.SpecialFolder.ProgramFilesX86)), "PVS-Studio", "x64", "PVS-Studio.exe");
				if (FileReference.Exists(InstalledAnalyzerFile))
				{
					AnalyzerFile = InstalledAnalyzerFile;
				}
				else
				{
					throw new BuildException("Unable to find PVS-Studio at {0} or {1}", AnalyzerFile, InstalledAnalyzerFile);
				}
			}

			AnalyzerVersion = GetAnalyzerVersion(AnalyzerFile);

			if (AnalyzerVersion < _minAnalyzerVersion)
			{
				throw new BuildLogEventException("PVS-Studio version {Version} is older than the minimum supported version {MinVersion}", AnalyzerVersion, _minAnalyzerVersion);
			}

			Settings = Target.WindowsPlatform.PVS;
			ApplicationSettings = Settings.ApplicationSettings;

			if (ApplicationSettings != null)
			{
				if (Settings.ModeFlags == 0)
				{
					throw new BuildException("All PVS-Studio analysis modes are disabled.");
				}
			}
			else
			{
				FileReference defaultLicenseFile = AnalyzerFile.ChangeExtension(".lic");
				if (FileReference.Exists(defaultLicenseFile))
				{
					LicenseFile = defaultLicenseFile;
				}
			}

			if (RuntimePlatform.IsRunningOnWine())
			{
				throw new BuildException("PVS-Studio is not supported with Wine.");
			}
		}

		/// <inheritdoc/>
		public override void GetVersionInfo(List<string> Lines)
		{
			InnerToolChain.GetVersionInfo(Lines);

			ReadOnlyPVSTargetSettings settings = Target.WindowsPlatform.PVS;
			Lines.Add($"Using PVS-Studio {AnalyzerVersion} at {AnalyzerFile} with analysis mode {(uint)settings.ModeFlags} ({settings.ModeFlags.ToString()})");
		}

		public override void GetExternalDependencies(HashSet<FileItem> ExternalDependencies)
		{
			InnerToolChain.GetExternalDependencies(ExternalDependencies);
			ExternalDependencies.Add(FileItem.GetItemByFileReference(FileReference.Combine(new DirectoryReference(Environment.GetFolderPath(Environment.SpecialFolder.ApplicationData)), "PVS-Studio", "Settings.xml")));
			ExternalDependencies.Add(FileItem.GetItemByFileReference(AnalyzerFile));
		}

		public override void SetUpGlobalEnvironment(ReadOnlyTargetRules Target, CppCompileEnvironment GlobalCompileEnvironment, LinkEnvironment GlobalLinkEnvironment)
		{
			base.SetUpGlobalEnvironment(Target, GlobalCompileEnvironment, GlobalLinkEnvironment);
			InnerToolChain.SetUpGlobalEnvironment(Target, GlobalCompileEnvironment, GlobalLinkEnvironment);

			if (!AnalyzerFile.IsUnderDirectory(Unreal.RootDirectory))
			{
				GlobalCompileEnvironment.RootPaths.AddExtraPath(("PVSAnalyzer", AnalyzerFile.Directory.FullName));
				GlobalLinkEnvironment.RootPaths.AddExtraPath(("PVSAnalyzer", AnalyzerFile.Directory.FullName));
			}

			if (LicenseFile != null && !LicenseFile.IsUnderDirectory(Unreal.RootDirectory))
			{
				GlobalCompileEnvironment.RootPaths.AddExtraPath(("PVSLicense", LicenseFile.Directory.FullName));
				GlobalLinkEnvironment.RootPaths.AddExtraPath(("PVSLicense", LicenseFile.Directory.FullName));
			}
		}
  
		public override void SetEnvironmentVariables()
		{
			Target.WindowsPlatform.Environment?.SetEnvironmentVariables();
		}
			 
		static Version GetAnalyzerVersion(FileReference AnalyzerPath)
		{
			string output = String.Empty;
			Version? analyzerVersion = new(0, 0);

			try
			{
				using (Process PvsProc = new())
				{
					PvsProc.StartInfo.FileName = AnalyzerPath.FullName;
					PvsProc.StartInfo.Arguments = "--version";
					PvsProc.StartInfo.UseShellExecute = false;
					PvsProc.StartInfo.CreateNoWindow = true;
					PvsProc.StartInfo.RedirectStandardOutput = true;

					PvsProc.Start();
					output = PvsProc.StandardOutput.ReadToEnd();
					PvsProc.WaitForExit();
				}

				const string versionPattern = @"\d+(?:\.\d+)+";
				Match match = Regex.Match(output, versionPattern);

				if (match.Success)
				{
					string versionStr = match.Value;
					if (!Version.TryParse(versionStr, out analyzerVersion))
					{
						throw new BuildLogEventException("Failed to parse PVS-Studio version: {Version}", versionStr);
					}
				}
			}
			catch (Exception ex)
			{
				if (ex is BuildException)
				{
					throw;
				}

				throw new BuildException(ex, "Failed to obtain PVS-Studio version.");
			}

			return analyzerVersion;
		}

		class ActionGraphCapture(IActionGraphBuilder inner, List<IExternalAction> actions) : ForwardingActionGraphBuilder(inner)
		{
			public override void AddAction(IExternalAction Action)
			{
				base.AddAction(Action);
				actions.Add(Action);
			}
		}

		const string CPP_20 = "c++20";
		const string CPP_23 = "c++23";

		public static string GetLangStandForCfgFile(CppStandardVersion cppStandard)
		{
			return cppStandard switch
			{
				CppStandardVersion.Cpp20 => CPP_20,
				CppStandardVersion.Cpp23 => CPP_23,
				CppStandardVersion.Latest => CPP_23,
				_ => CPP_20,
			};
		}

		public static bool ShouldCompileAsC(string compilerCommandLine, string sourceFileName)
		{
			int cFlagLastPosition = Math.Max(Math.Max(compilerCommandLine.LastIndexOf("/TC ", StringComparison.Ordinal), compilerCommandLine.LastIndexOf("/Tc ", StringComparison.Ordinal)),
												Math.Max(compilerCommandLine.LastIndexOf("-TC ", StringComparison.Ordinal), compilerCommandLine.LastIndexOf("-Tc ", StringComparison.Ordinal)));

			int cppFlagLastPosition = Math.Max(Math.Max(compilerCommandLine.LastIndexOf("/TP ", StringComparison.Ordinal), compilerCommandLine.LastIndexOf("/Tp ", StringComparison.Ordinal)),
												Math.Max(compilerCommandLine.LastIndexOf("-TP ", StringComparison.Ordinal), compilerCommandLine.LastIndexOf("-Tp ", StringComparison.Ordinal)));

			bool compileAsCCode = cFlagLastPosition == cppFlagLastPosition
				? Path.GetExtension(sourceFileName).Equals(".c", StringComparison.OrdinalIgnoreCase)
				: cFlagLastPosition > cppFlagLastPosition;

			return compileAsCCode;
		}

		public override CppCompileEnvironment CreateSharedResponseFile(CppCompileEnvironment compileEnvironment, FileReference outResponseFile, IActionGraphBuilder graph)
		{
			return compileEnvironment;
		}

		CPPOutput PreprocessCppFiles(CppCompileEnvironment compileEnvironment, IEnumerable<FileItem> inputFiles, DirectoryReference outputDir, string moduleName, IActionGraphBuilder graph, out List<IExternalAction> preprocessActions)
		{
			// Preprocess the source files with the regular toolchain
			CppCompileEnvironment preprocessCompileEnvironment = new(compileEnvironment)
			{
				bPreprocessOnly = true
			};
			preprocessCompileEnvironment.AdditionalArguments += " /wd4005 /wd4828 /wd5105";
			preprocessCompileEnvironment.Definitions.Add("PVS_STUDIO");
			preprocessCompileEnvironment.Definitions.Add($"PVS_VERSION_MAJOR={AnalyzerVersion.Major}");
			preprocessCompileEnvironment.Definitions.Add($"PVS_VERSION_MINOR={AnalyzerVersion.Minor}");
			preprocessCompileEnvironment.CppCompileWarnings.UndefinedIdentifierWarningLevel = WarningLevel.Off; // Not sure why THIRD_PARTY_INCLUDES_START doesn't pick this up; the _Pragma appears in the preprocessed output. Perhaps in preprocess-only mode the compiler doesn't respect these?

			preprocessActions = [];
			return InnerToolChain.CompileAllCPPFiles(preprocessCompileEnvironment, inputFiles, outputDir, moduleName, new ActionGraphCapture(graph, preprocessActions));
		}

		void AnalyzeCppFile(VCCompileAction preprocessAction, CppCompileEnvironment compileEnvironment, DirectoryReference outputDir, CPPOutput result, IActionGraphBuilder graph)
		{
			FileItem sourceFileItem = preprocessAction.SourceFile!;
			FileItem preprocessedFileItem = preprocessAction.PreprocessedFile!;

			// Write the PVS studio config file
			StringBuilder configFileContents = new();
			foreach (DirectoryReference includePath in Target.WindowsPlatform.Environment!.IncludePaths)
			{
				configFileContents.AppendFormat("exclude-path={0}\n", includePath.FullName);
			}
			if (ApplicationSettings != null && ApplicationSettings.PathMasks != null)
			{
				foreach (string pathMask in ApplicationSettings.PathMasks)
				{
					if (pathMask.Contains(':', StringComparison.Ordinal) || pathMask.Contains('\\', StringComparison.Ordinal) || pathMask.Contains('/', StringComparison.Ordinal))
					{
						if (Path.IsPathRooted(pathMask) && !pathMask.Contains(':', StringComparison.Ordinal))
						{
							configFileContents.AppendFormat("exclude-path=*{0}*\n", pathMask);
						}
						else
						{
							configFileContents.AppendFormat("exclude-path={0}\n", pathMask);
						}
					}
				}
			}
			if (Platform.IsInGroup(UnrealPlatformGroup.Microsoft))
			{
				configFileContents.Append("platform=x64\n");
			}
			else
			{
				throw new BuildException("PVS-Studio does not support this platform");
			}
			configFileContents.Append("preprocessor=visualcpp\n");

			bool shouldCompileAsC = ShouldCompileAsC(String.Join(" ", preprocessAction.Arguments), sourceFileItem.AbsolutePath);
			configFileContents.AppendFormat("language={0}\n", shouldCompileAsC ? "C" : "C++");

			configFileContents.Append("skip-cl-exe=yes\n");

			WindowsCompiler windowsCompiler = Target.WindowsPlatform.Compiler;
			bool isVisualCppCompiler = windowsCompiler.IsMSVC();
			if (!shouldCompileAsC)
			{
				string languageStandardForCfg = GetLangStandForCfgFile(compileEnvironment.CppStandard);

				configFileContents.AppendFormat("std={0}\n", languageStandardForCfg);

				bool disableMsExtensionsFromArgs = preprocessAction.Arguments.Any(arg => arg.Equals("/Za", StringComparison.Ordinal) || arg.Equals("-Za", StringComparison.Ordinal) || arg.Equals("/permissive-", StringComparison.Ordinal));
				bool disableMsExtensions = isVisualCppCompiler && (languageStandardForCfg == CPP_20 || disableMsExtensionsFromArgs);
				configFileContents.AppendFormat("disable-ms-extensions={0}\n", disableMsExtensions ? "yes" : "no");
			}

			if (isVisualCppCompiler && preprocessAction.Arguments.Any(arg => arg.StartsWith("/await", StringComparison.Ordinal)))
			{
				configFileContents.Append("msvc-await=yes\n");
			}

			if (Settings.EnableNoNoise)
			{
				configFileContents.Append("no-noise=yes\n");
			}

			if (Settings.EnableReportDisabledRules)
			{
				configFileContents.Append("report-disabled-rules=yes\n");
			}

			// TODO: Investigate into this disabled error
			if (sourceFileItem.Location.IsUnderDirectory(Unreal.RootDirectory))
			{
				configFileContents.AppendFormat("errors-off=V1102\n");
			}

			foreach (string error in compileEnvironment.StaticAnalyzerPVSDisabledErrors.OrderBy(x => x))
			{
				configFileContents.AppendFormat($"errors-off={error}\n");
			}

			int timeout = (int)Settings.AnalysisTimeoutFlag;
			configFileContents.AppendFormat("timeout={0}\n", timeout);
			configFileContents.Append("silent-exit-code-mode=yes\n");
			configFileContents.Append("new-output-format=yes\n");

			if (AnalyzerVersion.CompareTo(_analysisPathsSkipVersion) >= 0)
			{
				if (Target.bStaticAnalyzerProjectOnly)
				{
					configFileContents.Append($"analysis-paths=skip={Unreal.EngineSourceDirectory}\n");
				}
				if (!Target.bStaticAnalyzerIncludeGenerated)
				{
					configFileContents.Append($"analysis-paths=skip=*.gen.cpp\n");
					configFileContents.Append($"analysis-paths=skip=*.generated.h\n");
				}
			}

			string baseFileName = preprocessedFileItem.Location.GetFileName();

			FileReference configFileLocation = FileReference.Combine(outputDir, baseFileName + ".cfg");
			FileItem configFileItem = graph.CreateIntermediateTextFile(configFileLocation, configFileContents.ToString());

			// Run the analyzer on the preprocessed source file
			FileReference outputFileLocation = FileReference.Combine(outputDir, baseFileName + OutputFileExtension);
			FileItem outputFileItem = FileItem.GetItemByFileReference(outputFileLocation);

			Action analyzeAction = graph.CreateAction(ActionType.Compile);
			analyzeAction.CommandDescription = "Analyzing";
			analyzeAction.StatusDescription = baseFileName;
			analyzeAction.WorkingDirectory = Unreal.EngineSourceDirectory;
			analyzeAction.CommandPath = AnalyzerFile;
			analyzeAction.CommandVersion = AnalyzerVersion.ToString();

			List<string> arguments =
			[
				$"--source-file \"{sourceFileItem.AbsolutePath}\"",
				$"--output-file \"{outputFileItem.AbsolutePath}\"",
				$"--cfg \"{configFileItem.AbsolutePath}\"",
				$"--i-file=\"{preprocessedFileItem.AbsolutePath}\"",
				$"--analysis-mode {(uint)Settings.ModeFlags}",
				$"--lic-name \"{ApplicationSettings?.UserName}\" --lic-key \"{ApplicationSettings?.SerialNumber}\"",
			];

			// PVS-Studio loads global configuration files prior to the ones specified by the user.
			DirectoryReference? applicationData = DirectoryReference.GetSpecialFolder(Environment.SpecialFolder.ApplicationData);
			if (applicationData != null && DirectoryReference.Exists(applicationData))
			{
				DirectoryReference globalPVSConfigsDir = DirectoryReference.Combine(applicationData, "PVS-Studio");

				if (DirectoryReference.Exists(globalPVSConfigsDir))
				{
					foreach (FileReference configPath in DirectoryReference.EnumerateFiles(globalPVSConfigsDir, "*.pvsconfig", SearchOption.TopDirectoryOnly))
					{
						arguments.Add($"--rules-config=\"{configPath.FullName}\"");
						analyzeAction.PrerequisiteItems.Add(FileItem.GetItemByFileReference(configPath));
					}
				}
			}

			foreach (string configPath in Target.StaticAnalyzerPVSConfigs.Where(path => path.EndsWith(".pvsconfig", StringComparison.InvariantCultureIgnoreCase)))
			{
				string fullPVSConfigPath = configPath;

				if (!Path.IsPathRooted(fullPVSConfigPath) && Target.ProjectFile != null)
				{
					string projectDir = Path.GetDirectoryName(Target.ProjectFile.FullName)!;
					fullPVSConfigPath = Path.Combine(projectDir, fullPVSConfigPath);
				}

				if (Path.Exists(fullPVSConfigPath))
				{
					arguments.Add($"--rules-config=\"{fullPVSConfigPath}\"");
					analyzeAction.PrerequisiteItems.Add(FileItem.GetItemByPath(fullPVSConfigPath));
				}
			}

			analyzeAction.CommandArguments = String.Join(' ', arguments);

			analyzeAction.PrerequisiteItems.UnionWith(preprocessAction.AdditionalPrerequisiteItems);
			analyzeAction.PrerequisiteItems.Add(sourceFileItem);
			analyzeAction.PrerequisiteItems.Add(configFileItem);
			analyzeAction.PrerequisiteItems.Add(preprocessedFileItem);
			analyzeAction.ProducedItems.Add(outputFileItem);
			analyzeAction.DeleteItems.Add(outputFileItem); // PVS Studio will append by default, so need to delete produced items
			analyzeAction.bCanExecuteRemotely = true;
			analyzeAction.bCanExecuteRemotelyWithSNDBS = false;
			analyzeAction.bCanExecuteRemotelyWithXGE = false;

			analyzeAction.RootPaths = compileEnvironment.RootPaths;
			analyzeAction.CacheBucket = GetCacheBucket(Target, compileEnvironment);
			analyzeAction.ArtifactMode = ArtifactMode.Enabled;

			result.ObjectFiles.AddRange(analyzeAction.ProducedItems);
		}

		protected override CPPOutput CompileCPPFiles(CppCompileEnvironment compileEnvironment, IEnumerable<FileItem> inputFiles, DirectoryReference outputDir, string moduleName, IActionGraphBuilder graph)
		{
			if (compileEnvironment.bDisableStaticAnalysis)
			{
				return new CPPOutput();
			}

			// Use a subdirectory for PVS output, to avoid clobbering regular build artifacts
			outputDir = DirectoryReference.Combine(outputDir, "PVS");

			// Preprocess the source files with the regular toolchain
			CPPOutput result = PreprocessCppFiles(compileEnvironment, inputFiles, outputDir, moduleName, graph, out List<IExternalAction> PreprocessActions);

			// Run the source files through PVS-Studio
			for (int Idx = 0; Idx < PreprocessActions.Count; Idx++)
			{
				if (PreprocessActions[Idx] is not VCCompileAction PreprocessAction)
				{
					continue;
				}

				FileItem? sourceFileItem = PreprocessAction.SourceFile;
				if (sourceFileItem == null)
				{
					Logger.LogWarning("Unable to find source file from command producing: {File}", String.Join(", ", PreprocessActions[Idx].ProducedItems.Select(x => x.Location.GetFileName())));
					continue;
				}

				if (PreprocessAction.PreprocessedFile == null)
				{
					Logger.LogWarning("Unable to find preprocessed output file from {File}", sourceFileItem.Location.GetFileName());
					continue;
				}

				// We don't want to run these remotely since they are very lightweight but has lots of I/O
				PreprocessAction.bCanExecuteRemotely = false;

				AnalyzeCppFile(PreprocessAction, compileEnvironment, outputDir, result, graph);
			}
			return result;
		}

		public override void GenerateTypeLibraryHeader(CppCompileEnvironment compileEnvironment, ModuleRules.TypeLibrary typeLibrary, FileReference outputFile, FileReference? outputHeader, IActionGraphBuilder graph)
		{
			InnerToolChain.GenerateTypeLibraryHeader(compileEnvironment, typeLibrary, outputFile, outputHeader, graph);
		}

		public override FileItem LinkFiles(LinkEnvironment linkEnvironment, bool bBuildImportLibraryOnly, IActionGraphBuilder graph)
		{
			throw new BuildException("Unable to link with PVS toolchain.");
		}

		public override void FinalizeOutput(ReadOnlyTargetRules target, TargetMakefileBuilder makefileBuilder, DirectoryReference ExeDir)
		{
			string outputFileExtension = OutputFileExtension;
			FileReference outputFile = target.ProjectFile == null
				? FileReference.Combine(Unreal.EngineDirectory, "Saved", "PVS-Studio", $"{target.Name}{outputFileExtension}")
				: FileReference.Combine(target.ProjectFile.Directory, "Saved", "PVS-Studio", $"{target.Name}{outputFileExtension}");

			TargetMakefile makefile = makefileBuilder.Makefile;
			IEnumerable<FileReference> inputFiles = [.. makefile.OutputItems.Select(x => x.Location).Where(x => x.HasExtension(outputFileExtension))];

			// Collect the sourcefile items off of the Compile action added in CompileCPPFiles so that in SingleFileCompile mode the PVSGather step is also not filtered out
			IEnumerable<FileItem> compileSourceFiles = [.. makefile.Actions.OfType<VCCompileAction>().Select(x => x.SourceFile!)];
			IEnumerable<CppRootPaths> allRootPaths = makefile.Actions.OfType<VCCompileAction>().Where(x => x.RootPaths.bUseVfs && x.RootPaths.Any()).Select(x => x.RootPaths);
			CppRootPaths rootPaths = allRootPaths.FirstOrDefault() ?? new();
			foreach (CppRootPaths item in allRootPaths.Except([rootPaths]))
			{
				rootPaths.Merge(item);
			}

			// Store list of system paths that should be excluded
			IEnumerable<DirectoryReference> systemIncludePaths = [.. makefile.Actions.OfType<VCCompileAction>().SelectMany(x => x.SystemIncludePaths)];

			FileItem inputFileListItem = makefileBuilder.CreateIntermediateTextFile(FileReference.Combine(makefile.ProjectIntermediateDirectory, outputFile.ChangeExtension(".input").GetFileName()), inputFiles.Select(x => x.FullName).Distinct().Order());
			FileItem ignoredFileListItem = makefileBuilder.CreateIntermediateTextFile(FileReference.Combine(makefile.ProjectIntermediateDirectory, outputFile.ChangeExtension(".ignored").GetFileName()), systemIncludePaths.Select(x => x.FullName).Distinct().Order());
			FileItem rootPathsItem = FileItem.GetItemByFileReference(FileReference.Combine(makefile.ProjectIntermediateDirectory, outputFile.ChangeExtension(".rootpaths").GetFileName()));
			{
				using MemoryStream stream = new();
				using BinaryArchiveWriter binaryArchiveWriter = new(stream);
				rootPaths.Write(binaryArchiveWriter);
				binaryArchiveWriter.Flush();
				DirectoryReference.CreateDirectory(rootPathsItem.Location.Directory);
				FileReference.WriteAllBytesIfDifferent(rootPathsItem.Location, stream.ToArray());
			}

			string arguments = $"-Input=\"{inputFileListItem.Location}\" -Output=\"{outputFile}\" -Ignored=\"{ignoredFileListItem.Location}\" -RootPaths=\"{rootPathsItem.Location}\" -PrintLevel={target.StaticAnalyzerPVSPrintLevel}";

			Action finalizeAction = makefileBuilder.CreateRecursiveAction<PVSGatherMode>(ActionType.PostBuildStep, arguments);
			finalizeAction.CommandDescription = "Process PVS-Studio Results";
			finalizeAction.PrerequisiteItems.Add(inputFileListItem);
			finalizeAction.PrerequisiteItems.Add(ignoredFileListItem);
			finalizeAction.PrerequisiteItems.Add(rootPathsItem);
			finalizeAction.PrerequisiteItems.UnionWith(makefile.OutputItems);
			finalizeAction.PrerequisiteItems.UnionWith(compileSourceFiles);
			finalizeAction.ProducedItems.Add(FileItem.GetItemByFileReference(outputFile));
			finalizeAction.ProducedItems.Add(FileItem.GetItemByPath(outputFile.FullName + "_does_not_exist")); // Force the gather step to always execute
			finalizeAction.DeleteItems.UnionWith(finalizeAction.ProducedItems);

			makefile.OutputItems.AddRange(finalizeAction.ProducedItems);
		}

		public static IExternalAction? SingleFileFinalizeOutput(IExternalAction finalizeAction, IEnumerable<IExternalAction> prereqActions)
		{
			Match match = Regex.Match(finalizeAction.CommandArguments, @"-Input=""(.*?)"" -Output=""(.*?)"" -Ignored=""(.*?)"" -RootPaths=""(.*?)"" -PrintLevel=(\d+)");
			if (match.Success)
			{
				FileItem input = FileItem.GetItemByPath(match.Groups[1].Value);
				FileItem output = FileItem.GetItemByPath(match.Groups[2].Value);
				FileItem ignored = FileItem.GetItemByPath(match.Groups[3].Value);
				FileItem rootPaths = FileItem.GetItemByPath(match.Groups[4].Value);
				int printLevel = Int32.Parse(match.Groups[5].Value);

				FileItem newInput = FileItem.GetItemByFileReference(input.Location.ChangeExtension($".single{input.Location.GetExtension()}"));
				FileItem newOutput = FileItem.GetItemByFileReference(output.Location.ChangeExtension($".single{output.Location.GetExtension()}"));

				string[] inputFileLines = FileReference.ReadAllLines(input.Location);
				IEnumerable<FileItem> inputFiles = inputFileLines.Select(x => x.Trim()).Where(x => x.Length > 0).Select(FileItem.GetItemByPath);
				IEnumerable<FileItem> newInputFiles = inputFiles.Intersect(prereqActions.SelectMany(x => x.ProducedItems).Where(x => x.Location.HasExtension(OutputFileExtension)));

				Utils.WriteFileIfChanged(newInput, newInputFiles.Select(x => x.FullName), NullLogger.Instance);

				string arguments = $"-Input=\"{newInput.Location}\" -Output=\"{newOutput.Location}\" -Ignored=\"{ignored.Location}\" -RootPaths=\"{rootPaths.Location}\" -PrintLevel={printLevel}";
				Action singleFinalizeAction = new(finalizeAction);
				singleFinalizeAction.CommandDescription = "Process PVS-Studio Single File Results";
				singleFinalizeAction.CommandArguments = $"{singleFinalizeAction.CommandArguments[..singleFinalizeAction.CommandArguments.IndexOf("-Input=", StringComparison.Ordinal)]}{arguments}";
				singleFinalizeAction.PrerequisiteItems.Clear();
				singleFinalizeAction.PrerequisiteItems.Add(newInput);
				singleFinalizeAction.PrerequisiteItems.Add(ignored);
				singleFinalizeAction.PrerequisiteItems.Add(rootPaths);
				singleFinalizeAction.PrerequisiteItems.UnionWith(newInputFiles);
				singleFinalizeAction.ProducedItems.Clear();
				singleFinalizeAction.ProducedItems.Add(newOutput);
				singleFinalizeAction.ProducedItems.Add(FileItem.GetItemByPath(newOutput.FullName + "_does_not_exist")); // Force the gather step to always execute
				singleFinalizeAction.DeleteItems.Clear();
				singleFinalizeAction.DeleteItems.UnionWith(singleFinalizeAction.ProducedItems);

				return singleFinalizeAction;
			}
			return null;
		}

		public override List<string> GetISPCCompileTargets(UnrealTargetPlatform Platform, UnrealArch Arch) => InnerToolChain.GetISPCCompileTargets(Platform, Arch);

		public override string GetISPCOSTarget(UnrealTargetPlatform Platform) => InnerToolChain.GetISPCOSTarget(Platform);

		public override string GetISPCArchTarget(UnrealTargetPlatform Platform, UnrealArch Arch) => InnerToolChain.GetISPCArchTarget(Platform, Arch);

		public override string? GetISPCCpuTarget(UnrealTargetPlatform Platform) => InnerToolChain.GetISPCCpuTarget(Platform);

		public override string GetISPCObjectFileFormat(UnrealTargetPlatform Platform) => InnerToolChain.GetISPCObjectFileFormat(Platform);

		public override string GetISPCObjectFileSuffix(UnrealTargetPlatform Platform) => InnerToolChain.GetISPCObjectFileSuffix(Platform);

	}
}
