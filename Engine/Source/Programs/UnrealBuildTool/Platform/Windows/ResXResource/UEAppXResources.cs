// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using System.Linq;
using System.Text;
using System.Xml;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace UnrealBuildTool
{
	/// <summary>
	/// AppX Package Resource Index generator
	/// </summary>
	public class UEAppXResources
	{
		const string BuildResourceSubPath = "Resources";
		const string DefaultResources = "";

		class Resources
		{
			public Resources() { }

			public Dictionary<string, string> SourceStrings = new();    // raw source strings, as registered by AddCultureString etc.
			public Dictionary<string, string> StringResources = new();  // string resources that will be used, as defined by AddResourceString
		};

		readonly Dictionary<string, Resources> PerCultureResources = new();
		readonly Dictionary<FileReference, string> FilesToCopy = new(); // source file -> target file fragment (relative to OutputDir)

		readonly ILogger Logger;
		readonly FileReference MakePriExe; // path to makepri.exe

		/// <summary>
		/// Folders where binary resource files are located, in decreasing priority order
		/// </summary>
		public List<DirectoryReference> ProjectBinaryResourceDirectories { get; }

		/// <summary>
		/// Folders where fallback binary resource files are located, in decreasing priority order
		/// </summary>
		public List<DirectoryReference> EngineFallbackBinaryResourceDirectories { get; }

		/// <summary>
		/// Target OS version
		/// </summary>
		public string TargetOSVersion { get; set; }

		/// <summary>
		/// Create a new instance
		/// </summary>
		public UEAppXResources(ILogger InLogger, FileReference InMakePriExe)
		{
			Logger = InLogger;
			MakePriExe = InMakePriExe;
			ProjectBinaryResourceDirectories = new();
			EngineFallbackBinaryResourceDirectories = new();
			TargetOSVersion = "10.0.0";

			if (FileReference.Exists(MakePriExe) == false)
			{
				throw new BuildException($"Couldn't find the makepri executable: {MakePriExe}");
			}

			AddCulture(DefaultResources);
		}

		/// <summary>
		/// Adds a reference to the given culture
		/// </summary>
		public void AddCulture(string AppXCultureId)
		{
			if (!PerCultureResources.TryGetValue(AppXCultureId, out Resources? Result) || Result == null)
			{
				PerCultureResources.Add(AppXCultureId, new());
			}
		}

		/// <summary>
		/// Adds a reference to all of the given cultures
		/// </summary>
		/// <param name="AppXCultureIds"></param>
		public void AddCultures(IEnumerable<string> AppXCultureIds)
		{
			foreach (string AppXCultureId in AppXCultureIds)
			{
				AddCulture(AppXCultureId);
			}
		}

		/// <summary>
		/// Returns a collection of all registered cultures
		/// </summary>
		/// <returns></returns>
		public IEnumerable<string> GetAllCultureIds()
		{
			return PerCultureResources.Keys.Where(X => !String.IsNullOrEmpty(X));
		}

		/// <summary>
		/// Add the given string to the default culture
		/// </summary>
		public void AddDefaultString(string ConfigKey, string Value)
		{
			AddCultureString(DefaultResources, ConfigKey, Value);
		}

		/// <summary>
		/// Adds all of the given strings to the default culture
		/// </summary>
		public void AddDefaultStrings(Dictionary<string, string> Strings)
		{
			AddCultureStrings(DefaultResources, Strings);
		}

		/// <summary>
		/// Add the given string to the given culture
		/// </summary>
		public void AddCultureString(string AppXCultureId, string ConfigKey, string Value)
		{
			GetCultureResources(AppXCultureId).SourceStrings[ConfigKey] = Value;
		}

		/// <summary>
		/// Adds all of the given strings to the given culture
		/// </summary>
		public void AddCultureStrings(string AppXCultureId, Dictionary<string, string> Strings)
		{
			Resources ThisCultureResources = GetCultureResources(AppXCultureId);
			foreach (KeyValuePair<string, string> String in Strings)
			{
				ThisCultureResources.SourceStrings[String.Key] = String.Value;
			}
		}

		/// <summary>
		/// Returns whether the given string is known
		/// </summary>
		public bool HasString(string Key)
		{
			return GetDefaultResources().SourceStrings.ContainsKey(Key);
		}

		/// <summary>
		/// Clear all stored strings for all cultures
		/// </summary>
		public void ClearStrings()
		{
			foreach (KeyValuePair<string, Resources> Itr in PerCultureResources)
			{
				Itr.Value.SourceStrings.Clear();
			}
		}

		Resources GetCultureResources(string AppXCultureId)
		{
			if (PerCultureResources.TryGetValue(AppXCultureId, out Resources? Result) && Result != null)
			{
				return Result;
			}
			else
			{
				throw new Exception($"Culture {AppXCultureId} is not known. Missing call to AddCulture?");
			}
		}

		Resources GetDefaultResources()
		{
			return GetCultureResources(DefaultResources);
		}

		bool RunMakePri(string CommandLine)
		{
			StringBuilder ProcessOutput = new StringBuilder();
			void LocalProcessOutput(DataReceivedEventArgs Args)
			{
				if (Args != null && Args.Data != null)
				{
					ProcessOutput.AppendLine(Args.Data.TrimEnd());
				}
			}

			ProcessStartInfo StartInfo = new ProcessStartInfo(MakePriExe.FullName, CommandLine);
			StartInfo.UseShellExecute = false;
			StartInfo.CreateNoWindow = true;
			StartInfo.StandardOutputEncoding = Encoding.Unicode;
			StartInfo.StandardErrorEncoding = Encoding.Unicode;

			Process LocalProcess = new Process();
			LocalProcess.StartInfo = StartInfo;
			LocalProcess.OutputDataReceived += (Sender, Args) => { LocalProcessOutput(Args); };
			LocalProcess.ErrorDataReceived += (Sender, Args) => { LocalProcessOutput(Args); };
			int ExitCode = Utils.RunLocalProcess(LocalProcess);

			if (ExitCode == 0)
			{
				Logger.LogDebug("{Output}", ProcessOutput.ToString());
				return true;
			}
			else
			{
				Logger.LogInformation("{Output}", ProcessOutput.ToString());
				Logger.LogError("{File} returned an error.", MakePriExe.GetFileName());
				Logger.LogError("Exit code: {Code}", ExitCode);
				return false;
			}
		}

		private bool RemoveStaleResourceFiles(IEnumerable<string> RequiredFileFragments, DirectoryReference OutputDirectory)
		{
			DirectoryReference ResourceDirectory = DirectoryReference.Combine(OutputDirectory, BuildResourceSubPath);
			if (!DirectoryReference.Exists(ResourceDirectory))
			{
				return false;
			}

			// remove all files in the Resources/ subfolder that should not be included
			IEnumerable<FileReference> ExistingFiles = DirectoryReference.EnumerateFiles(ResourceDirectory, "*.*", SearchOption.AllDirectories);
			IEnumerable<FileReference> RequiredFiles = RequiredFileFragments
				.Select(X => FileReference.Combine(OutputDirectory, X))
				.Where(X => X.IsUnderDirectory(ResourceDirectory))
				;
			IEnumerable<FileReference> StaleResourceFiles = ExistingFiles.Except(RequiredFiles);
			if (!StaleResourceFiles.Any())
			{
				return false;
			}

			Logger.LogDebug("Removing stale manifest resource files...");
			foreach (FileReference StaleResourceFile in StaleResourceFiles)
			{
				// try to delete the file & the directory that contains it
				try
				{
					Logger.LogDebug("    removing {Path}", StaleResourceFile.MakeRelativeTo(OutputDirectory));
					FileUtils.ForceDeleteFile(StaleResourceFile);
					if (!Directory.EnumerateFileSystemEntries(StaleResourceFile.Directory.FullName).Any())
					{
						DirectoryReference.Delete(StaleResourceFile.Directory, false);
					}
				}
				catch (Exception E)
				{
					Logger.LogError("    Could not remove {StaleResourceFile} - {Message}.", StaleResourceFile, E.Message);
				}
			}

			return true;
		}

		/// <summary>
		/// Attempts to locate the given resource binary file in the previously-specified resource folders
		/// </summary>
		bool FindResourceBinaryFile([NotNullWhen(true)] out FileReference? SourceFilePath, string ResourceFileName, string CultureId = "", bool AllowEngineFallback = true)
		{
			// look in project binary resource directories
			foreach (DirectoryReference BinaryResourceDirectory in ProjectBinaryResourceDirectories)
			{
				FileReference BinaryResourceFile = FileReference.Combine(BinaryResourceDirectory, CultureId, ResourceFileName);
				if (FileReference.Exists(BinaryResourceFile))
				{
					SourceFilePath = BinaryResourceFile;
					return true;
				}
			}

			// look in Engine, if allowed
			if (AllowEngineFallback)
			{
				foreach (DirectoryReference BinaryResourceDirectory in EngineFallbackBinaryResourceDirectories)
				{
					FileReference BinaryResourceFile = FileReference.Combine(BinaryResourceDirectory, CultureId, ResourceFileName);
					if (FileReference.Exists(BinaryResourceFile))
					{
						SourceFilePath = BinaryResourceFile;
						return true;
					}
				}
			}

			// not found
			SourceFilePath = null;
			return false;
		}

		/// <summary>
		/// Determines whether the given resource binary file can be found in one of the known folder locations for the given culture
		/// </summary>
		public bool DoesCultureResourceBinaryFileExist(string ResourceFileName, string CultureId, bool AllowEngineFallback = true)
		{
			return FindResourceBinaryFile(out FileReference? _, ResourceFileName, CultureId, AllowEngineFallback);
		}

		/// <summary>
		/// Determines whether the given default resource binary file can be found in one of the known folder locations
		/// </summary>
		public bool DoesDefaultResourceBinaryFileExist(string ResourceFileName, bool AllowEngineFallback = true)
		{
			return DoesCultureResourceBinaryFileExist(ResourceFileName, DefaultResources, AllowEngineFallback);
		}

		/// <summary>
		/// Adds the given resource binary file(s) to the manifest files
		/// </summary>
		public bool AddResourceBinaryFileReference(string ResourceFileName, bool AllowEngineFallback = true)
		{
			// At least the default culture entry for any resource binary must always exist
			if (!FindResourceBinaryFile(out FileReference? SourceFilePath, ResourceFileName, DefaultResources, AllowEngineFallback))
			{
				return false;
			}
			AddFileReference(SourceFilePath, Path.Combine(BuildResourceSubPath, ResourceFileName));

			// Copy all per-culture resource files
			foreach (string CultureId in GetAllCultureIds())
			{
				if (FindResourceBinaryFile(out SourceFilePath, ResourceFileName, CultureId, AllowEngineFallback))
				{
					AddFileReference(SourceFilePath, Path.Combine(BuildResourceSubPath, CultureId, ResourceFileName));
				}
			}

			return true;
		}

		/// <summary>
		/// Adds the given file to the manifest files
		/// </summary>
		public void AddFileReference(FileReference SourcePath, string TargetPathFragment)
		{
			FilesToCopy.Add(SourcePath, TargetPathFragment);
		}

		private static bool AreFilesDifferent(FileReference File1, FileReference File2)
		{
			FileInfo FileInfo1 = File1.ToFileInfo();
			FileInfo FileInfo2 = File2.ToFileInfo();
			if (FileInfo1.Length != FileInfo2.Length)
			{
				return true;
			}

			byte[] FileContents1 = FileReference.ReadAllBytes(File1);
			byte[] FileContents2 = FileReference.ReadAllBytes(File2);
			return !Enumerable.SequenceEqual(FileContents1, FileContents2);
		}

		/// <summary>
		/// Copies all of the generated files to the output folder
		/// </summary>
		private List<FileReference> CopyFilesToOutput(DirectoryReference OutputDirectory)
		{
			List<FileReference> UpdatedFiles = new();
			if (!FilesToCopy.Any())
			{
				return UpdatedFiles;
			}

			Logger.LogDebug("Updating manifest resource files...");
			foreach (KeyValuePair<FileReference, string> FileToCopy in FilesToCopy)
			{
				FileReference SourceFile = FileToCopy.Key;
				FileReference TargetFile = FileReference.Combine(OutputDirectory, FileToCopy.Value);
				if (!FileReference.Exists(SourceFile))
				{
					Logger.LogError("    Source file not found {TargetFile}", TargetFile);
					continue;
				}
				if (FileReference.Exists(TargetFile) && !AreFilesDifferent(TargetFile, SourceFile))
				{
					continue;
				}

				// remove old version, if any
				bool bFileExists = FileReference.Exists(TargetFile);
				if (bFileExists)
				{
					try
					{
						FileUtils.ForceDeleteFile(TargetFile);
					}
					catch (Exception E)
					{
						Logger.LogError("    Could not replace file {TargetFile} - {Message}", TargetFile, E.Message);
					}
				}

				// copy new version
				try
				{
					if (bFileExists)
					{
						Logger.LogDebug("    updating {Path}", Utils.MakePathRelativeTo(TargetFile.FullName, OutputDirectory.FullName!));
					}
					else
					{
						Logger.LogDebug("    adding {Path}", Utils.MakePathRelativeTo(TargetFile.FullName, OutputDirectory.FullName!));
					}

					Directory.CreateDirectory(Path.GetDirectoryName(TargetFile.FullName)!);
					FileReference.Copy(SourceFile, TargetFile);
					FileReference.SetAttributes(TargetFile, FileAttributes.Normal);
					File.SetCreationTime(TargetFile.FullName, File.GetCreationTime(SourceFile.FullName));

					UpdatedFiles.Add(TargetFile);
				}
				catch (Exception E)
				{
					Logger.LogError("    Unable to copy file {TargetFile} - {Message}", TargetFile, E.Message);
				}
			}

			return UpdatedFiles;
		}

		/// <summary>
		/// Adds the given string to the culture string writers
		/// </summary>
		public string AddResourceString(string ResourceEntryName, string ConfigKey, string DefaultValue, string ValueSuffix = "")
		{
			// Get the default culture value
			Resources DefaultCultureResources = GetDefaultResources();
			string? DefaultCultureString = null;
			if (DefaultCultureResources.SourceStrings.TryGetValue(ConfigKey, out string? Value) && Value != null)
			{
				DefaultCultureString = Value;
			}
			if (String.IsNullOrEmpty(DefaultCultureString))
			{
				DefaultCultureString = DefaultValue;
			}
			DefaultCultureResources.StringResources.Add(ResourceEntryName, DefaultCultureString + ValueSuffix);

			// Get the localized culture values
			foreach (string CultureId in GetAllCultureIds())
			{
				Resources ThisCultureResources = GetCultureResources(CultureId);
				if (ThisCultureResources.SourceStrings.TryGetValue(ConfigKey, out string? CultureString) && !String.IsNullOrEmpty(CultureString))
				{
					ThisCultureResources.StringResources.Add(ResourceEntryName, CultureString + ValueSuffix);
				}
			}

			return "ms-resource:" + ResourceEntryName;
		}

		/// <summary>
		/// Generate the package resource index and copies all resources files to the output
		/// </summary>
		public List<FileReference> GenerateAppXResources(DirectoryReference OutputDirectory, DirectoryReference IntermediateDirectory, FileReference ManifestFile, string DefaultAppXCultureId, string? PackageIdentityName)
		{
			// Clean out the resources intermediate path so that we know there are no stale binary files.
			FileUtils.ForceDeleteDirectory(DirectoryReference.Combine(IntermediateDirectory, BuildResourceSubPath));

			// Finalize all cultures
			foreach (KeyValuePair<string, Resources> Itr in PerCultureResources)
			{
				string AppXCultureId = Itr.Key;

				// Create the culture folders
				DirectoryReference IntermediateCultureResourceDirectory = DirectoryReference.Combine(IntermediateDirectory, BuildResourceSubPath, AppXCultureId);
				DirectoryReference OutputCultureResourceDirectory = DirectoryReference.Combine(OutputDirectory, BuildResourceSubPath, AppXCultureId);
				FileUtils.CreateDirectoryTree(IntermediateCultureResourceDirectory);
				FileUtils.CreateDirectoryTree(OutputCultureResourceDirectory);

				// Export the resource tables & add them to the list of files to copy
				FileReference IntermediateResWIndexerFile = FileReference.Combine(IntermediateDirectory, BuildResourceSubPath, AppXCultureId, "resources.resw");
				UEResXWriter ResourceWriter = new(IntermediateResWIndexerFile.FullName);
				foreach (KeyValuePair<string, string> StringResourcePair in Itr.Value.StringResources)
				{
					ResourceWriter.AddResource(StringResourcePair.Key, StringResourcePair.Value);
				}
				ResourceWriter.Close();

				AddFileReference(IntermediateResWIndexerFile, Path.Combine(BuildResourceSubPath, AppXCultureId, "resources.resw"));
			}

			// The resource database is dependent on everything else calculated here (manifest, resource string tables, binary resources).
			// So if any file has been updated we'll need to run the config.
			bool bHadStaleResources = RemoveStaleResourceFiles(FilesToCopy.Values, OutputDirectory);
			List<FileReference> UpdatedFilePaths = CopyFilesToOutput(OutputDirectory);
			FileReference TargetResourceIndexFile = FileReference.Combine(OutputDirectory, "resources.pri");

			if (bHadStaleResources || UpdatedFilePaths.Any() || !FileReference.Exists(TargetResourceIndexFile))
			{
				// Create resource index configuration
				FileReference ResourceConfigFile = FileReference.Combine(IntermediateDirectory, "priconfig.xml");
				RunMakePri($"createconfig /cf \"{ResourceConfigFile}\" /dq {DefaultAppXCultureId} /o /pv {TargetOSVersion}");

				// Load the new resource index configuration
				XmlDocument PriConfig = new XmlDocument();
				PriConfig.Load(ResourceConfigFile.FullName);

				// remove the packaging node - we do not want to split the pri & only want one .pri file
				XmlNode PackagingNode = PriConfig.SelectSingleNode("/resources/packaging")!;
				PackagingNode.ParentNode!.RemoveChild(PackagingNode);

				// all required resources are explicitly listed in resources.resfiles, rather than relying on makepri to discover them
				FileReference ResourcesResFile = FileReference.Combine(IntermediateDirectory, "resources.resfiles");
				XmlNode PriIndexNode = PriConfig.SelectSingleNode("/resources/index")!;
				XmlAttribute PriStartIndex = PriIndexNode.Attributes!["startIndexAt"]!;
				PriStartIndex.Value = ResourcesResFile.FullName;

				// swap the folder indexer-config to a RESFILES indexer-config.
				XmlElement FolderIndexerConfigNode = (XmlElement)PriConfig.SelectSingleNode("/resources/index/indexer-config[@type='folder']")!;
				FolderIndexerConfigNode.SetAttribute("type", "RESFILES");
				FolderIndexerConfigNode.RemoveAttribute("foldernameAsQualifier");
				FolderIndexerConfigNode.RemoveAttribute("filenameAsQualifier");

				PriConfig.Save(ResourceConfigFile.FullName);

				// generate resources.resfiles
				IEnumerable<FileReference> Resources = DirectoryReference.EnumerateFiles(DirectoryReference.Combine(OutputDirectory, BuildResourceSubPath), "*.*", SearchOption.AllDirectories);
				System.Text.StringBuilder ResourcesList = new System.Text.StringBuilder();
				foreach (FileReference Resource in Resources)
				{
					ResourcesList.AppendLine(Resource.MakeRelativeTo(OutputDirectory));
				}
				File.WriteAllText(ResourcesResFile.FullName, ResourcesList.ToString());

				// remove old Package Resource Index
				FileUtils.ForceDeleteFile(TargetResourceIndexFile);

				// generate new Package Resource Index
				FileReference ResourceLogFile = FileReference.Combine(IntermediateDirectory, "ResIndexLog.xml");
				string MakePriCommandLine = $"new /pr \"{OutputDirectory}\" /cf \"{ResourceConfigFile}\" /mn \"{ManifestFile}\" /il \"{ResourceLogFile}\" /of \"{TargetResourceIndexFile}\" /o";
				if (PackageIdentityName != null)
				{
					MakePriCommandLine += $" /indexName \"{PackageIdentityName}\"";
				}

				Logger.LogDebug("    generating {Path}", TargetResourceIndexFile.MakeRelativeTo(OutputDirectory));
				RunMakePri(MakePriCommandLine);
				UpdatedFilePaths.Add(TargetResourceIndexFile);

				// .resw files are not needed - the data is embedded in the resources.pri
				foreach (FileReference ResW in DirectoryReference.EnumerateFiles(OutputDirectory, "*.resw", SearchOption.AllDirectories))
				{
					FileUtils.ForceDeleteFile(ResW);
				}
			}

			// Report if nothing was changed
			if (!bHadStaleResources && !UpdatedFilePaths.Any())
			{
				Logger.LogDebug($"Manifest resource files are up to date");
			}

			return UpdatedFilePaths;
		}
	}
}
