// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool
{
	/// <summary>
	/// Intentionally unimplemented, but declared so that it's type can be included in generated resx files
	/// </summary>
	public abstract class UEResXReader
	{
	}
}
