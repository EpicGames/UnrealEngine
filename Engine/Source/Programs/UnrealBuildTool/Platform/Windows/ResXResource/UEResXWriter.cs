// Copyright Epic Games, Inc. All Rights Reserved.

using System.Xml;

namespace UnrealBuildTool
{
	/// <summary>
	/// Resource writer for VC app manifest generator
	/// </summary>
	public class UEResXWriter
	{
		/// <summary>
		/// Create a new resource writer at the given path
		/// </summary>
		public UEResXWriter(string InPath)
		{

			Filename = InPath;
			Document = new XmlDocument();
			Document.AppendChild(Document.CreateXmlDeclaration("1.0", "utf-8", null));
			RootElement = Document.CreateElement("root");
			Document.AppendChild(RootElement);

			RootElement.AppendChild(CreateEntry("resmimetype", "text/microsoft-resx", null));
			RootElement.AppendChild(CreateEntry("version", "2.0", null));
			RootElement.AppendChild(CreateEntry("reader", typeof(UEResXReader).AssemblyQualifiedName!, null));
			RootElement.AppendChild(CreateEntry("writer", typeof(UEResXWriter).AssemblyQualifiedName!, null));
		}

		/// <summary>
		/// Closes the writer
		/// </summary>
		public void Close()
		{
			Document.Save(Filename);
		}

		/// <summary>
		/// Add the given resource reference
		/// </summary>
		public void AddResource(string InName, string InValue)
		{
			RootElement.AppendChild(CreateEntry(InName, InValue, "preserve"));
		}

		private XmlNode CreateEntry(string InName, string InValue, string? InSpace)
		{
			XmlElement Value = Document.CreateElement("value");
			Value.InnerText = InValue;

			XmlElement Data = Document.CreateElement("data");

			{
				XmlAttribute Attr = Document.CreateAttribute("name");
				Attr.Value = InName;
				Data.Attributes.Append(Attr);
			}

			if (!System.String.IsNullOrEmpty(InSpace))
			{
				XmlAttribute Attr = Document.CreateAttribute("xml:space");
				Attr.Value = "preserve";
				Data.Attributes.Append(Attr);
			}

			Data.AppendChild(Value);

			return Data;
		}

		private readonly string Filename;
		private readonly XmlDocument Document;
		private readonly XmlElement RootElement;
	}
}
