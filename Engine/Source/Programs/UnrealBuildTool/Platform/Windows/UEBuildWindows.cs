// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using System.Linq;
using System.Runtime.Versioning;
using System.Text;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Logging.Abstractions;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	partial struct UnrealArch
	{
		/// <summary>
		/// Version of Arm64 that can interop with X64 (Emulation Compatible)
		/// </summary>
		public static UnrealArch Arm64ec { get; } = FindOrAddByName("arm64ec", bIsX64: false);

		private struct WindowsArchInfo
		{
			public string ToolChain;
			public string LibDir;
			public string SystemLibDir;

			public WindowsArchInfo(string ToolChain, string LibDir, string SystemLibDir)
			{
				this.ToolChain = ToolChain;
				this.LibDir = LibDir;
				this.SystemLibDir = SystemLibDir;
			}
		}

		private static readonly IReadOnlyDictionary<UnrealArch, WindowsArchInfo> WindowsToolchainArchitectures = new Dictionary<UnrealArch, WindowsArchInfo>()
		{
			{ UnrealArch.Arm64,         new WindowsArchInfo("arm64", "arm64", "arm64") },
			{ UnrealArch.Arm64ec,       new WindowsArchInfo("arm64", "x64", "arm64") },
			{ UnrealArch.X64,           new WindowsArchInfo("x64", "x64", "x64") },
		};

		/// <summary>
		/// Windows-specific tool chain to compile with
		/// </summary>
		public string WindowsToolChain
		{
			get
			{
				if (WindowsToolchainArchitectures.ContainsKey(this))
				{
					return WindowsToolchainArchitectures[this].ToolChain;
				}

				throw new BuildException($"Unknown architecture {ToString()} passed to UnrealArch.WindowsToolChain");
			}
		}

		/// <summary>
		/// Windows-specific lib directory for this architecture
		/// </summary>
		public string WindowsLibDir
		{
			get
			{
				if (WindowsToolchainArchitectures.ContainsKey(this))
				{
					return WindowsToolchainArchitectures[this].LibDir;
				}

				throw new BuildException($"Unknown architecture {ToString()} passed to UnrealArch.WindowsLibDir");
			}
		}

		/// <summary>
		/// Windows-specific system lib directory for this architecture
		/// </summary>
		public string WindowsSystemLibDir
		{
			get
			{
				if (WindowsToolchainArchitectures.ContainsKey(this))
				{
					return WindowsToolchainArchitectures[this].SystemLibDir;
				}

				throw new BuildException($"Unknown architecture {ToString()} passed to UnrealArch.WindowsSystemLibDir");
			}
		}

		/// <summary>
		/// Windows-specific low level name for the generic platforms
		/// </summary>
		public string WindowsName
		{
			get
			{
				if (WindowsToolchainArchitectures.ContainsKey(this))
				{
					return WindowsToolchainArchitectures[this].ToolChain;
				}

				throw new BuildException($"Unknown architecture {ToString()} passed to UnrealArch.WindowsName");
			}
		}
	}

	/// <summary>
	/// Available compiler toolchains on Windows platform
	/// </summary>
	public enum WindowsCompiler
	{
		/// <summary>
		/// Use the default compiler. A specific value will always be used outside of configuration classes.
		/// </summary>
		Default,

		/// <summary>
		/// Use Clang for Windows, using the clang-cl driver.
		/// </summary>
		Clang,

		/// <summary>
		/// Use the RTFM (Rollback Transactions on Failure Memory) Clang variant for Verse on Windows, using the verse-clang-cl driver.
		/// </summary>
		ClangRTFM,

		/// <summary>
		/// Use the Clang variant for instrumentation.
		/// </summary>
		ClangInstrument,

		/// <summary>
		/// Use a custom version of Clang found using LLVM_CUSTOM_PATH environment variable.
		/// </summary>
		ClangCustom,

		/// <summary>
		/// Use the Intel oneAPI C++ compiler
		/// </summary>
		Intel,

		/// <summary>
		/// Visual Studio 2022 (Visual C++ 17.0)
		/// </summary>
		VisualStudio2022,

		/// <summary>
		/// Visual Studio 2026 (Visual C++ 19.0) or older
		/// </summary>
		VisualStudio2026,

		/// <summary>
		/// Any version of Visual Studio available
		/// </summary>
		VisualStudio = VisualStudio2026, // Should be set to the latest version of VisualStudio

		/// <summary>
		/// Unsupported Visual Studio
		/// Must be the last entry in WindowsCompiler enum and should only be used in limited circumstances
		/// </summary>
		[Obsolete("Unsupported Visual Studio WindowsCompiler, do not use this enum")]
		VisualStudioUnsupported,
	}

	/// <summary>
	/// Available runtime ray tracing modes on Windows platform
	/// </summary>
	public enum WindowsRayTracingRuntimeMode
	{
		/// <summary>
		/// Raytracing is disabled
		/// </summary>
		Disabled,

		/// <summary>
		/// Only inline raytracing is enabled
		/// </summary>
		Inline,

		/// <summary>
		/// Full raytracing with RTPSOs and hit shaders are available
		/// </summary>
		Full,
	}

	/// <summary>
	/// Enum describing the release channel of a compiler
	/// </summary>
	internal enum WindowsCompilerChannel
	{
		Latest,
		Preview,
		Experimental,
		Any = Latest | Preview | Experimental
	}

	/// <summary>
	/// Extension methods for WindowsCompiler enum
	/// </summary>
	public static class WindowsCompilerExtensions
	{
		/// <summary>
		/// Returns if this compiler toolchain based on Clang
		/// </summary>
		/// <param name="Compiler">The compiler to check</param>
		/// <returns>true if Clang based</returns>
		public static bool IsClang(this WindowsCompiler Compiler)
		{
			return Compiler == WindowsCompiler.Clang || Compiler == WindowsCompiler.ClangRTFM || Compiler == WindowsCompiler.ClangInstrument || Compiler == WindowsCompiler.ClangCustom || Compiler == WindowsCompiler.Intel;
		}

		/// <summary>
		/// Returns if this compiler toolchain based on Intel
		/// </summary>
		/// <param name="Compiler">The compiler to check</param>
		/// <returns>true if Intel based</returns>
		public static bool IsIntel(this WindowsCompiler Compiler)
		{
			return Compiler == WindowsCompiler.Intel;
		}

		/// <summary>
		/// Returns if this compiler toolchain based on MSVC
		/// </summary>
		/// <param name="Compiler">The compiler to check</param>
		/// <returns>true if MSVC based</returns>
		public static bool IsMSVC(this WindowsCompiler Compiler)
		{
			return Compiler >= WindowsCompiler.VisualStudio2022;
		}
	}

	/// <summary>
	/// Windows-specific target settings
	/// </summary>
	public class WindowsTargetRules
	{
		/// <summary>
		/// The target rules which owns this object. Used to resolve some properties.
		/// </summary>
		readonly TargetRules Target;

		/// <summary>
		/// If enabled will set the ProductVersion embeded in windows executables and dlls to contain BUILT_FROM_CHANGELIST and BuildVersion
		/// Enabled by default for all precompiled and Shipping configurations. Regardless of this setting, the versions from Build.version will be available via the BuildSettings module
		/// Note: Embedding these versions will cause resource files to be recompiled whenever changelist is updated which will cause binaries to relink
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/WindowsTargetPlatform.WindowsTargetSettings", "SetResourceVersions")]
		[XmlConfigFile(Category = "WindowsPlatform")]
		[CommandLine("-SetResourceVersions")]
		[CommandLine("-NoSetResourceVersions", Value = "false")]
		public bool bSetResourceVersions
		{
			get => bSetResourceVersionPrivate ?? Target.bPrecompile || Target.Configuration == UnrealTargetConfiguration.Shipping;
			set => bSetResourceVersionPrivate = value;
		}
		private bool? bSetResourceVersionPrivate = null;

		/// <summary>
		/// If enabled, resource files will be shared between Modular target dlls for non-formal builds rather than linking in different resources to each dll.
		/// Additionally the build will instead call cvtres.exe to convert a compiled resource .res file into an object up front rather than offloading to link.exe,
		/// This is an optimization for Modular builds to prevent reprocessing the same resource for every dll which can be slow and spawn two additional processes per link.exe
		/// </summary>
		public bool bSharedResourceFiles
		{
			get => bSharedResourceFilesPrivate ?? Target.LinkType != TargetLinkType.Monolithic && !Target.bFormalBuild;
			set => bSharedResourceFilesPrivate = value;
		}
		private bool? bSharedResourceFilesPrivate = null;

		/// <summary>
		/// If -PGOOptimize is specified but the linker flags have changed since the last -PGOProfile, this will emit a warning and build without PGO instead of failing during link with LNK1268. 
		/// </summary>
		[XmlConfigFile(Category = "WindowsPlatform")]
		[CommandLine("-IgnoreStalePGOData")]
		public bool bIgnoreStalePGOData = false;

		/// <summary>
		/// If specified along with -PGOProfile, then /FASTGENPROFILE will be used instead of /GENPROFILE. This usually means that the PGO data is generated faster, but the resulting data may not yield as efficient optimizations during -PGOOptimize
		/// </summary>
		[XmlConfigFile(Category = "WindowsPlatform")]
		[CommandLine("-PGOFastGen")]
		public bool bUseFastGenProfile = false;

		/// <summary>
		/// If specified along with -PGOOptimize, will use the specified per-merged pgd file instead of the usual pgd file with loose pgc files.
		/// </summary>
		[XmlConfigFile(Category = "WindowsPlatform")]
		[CommandLine("-PGOMergedPGD")]
		public string? PreMergedPgdFilename = null;

		/// <summary>
		/// If specified along with -PGOProfile, prevent the usage of extra counters. Please note that by default /FASTGENPROFILE doesnt use extra counters
		/// </summary>
		/// <seealso href="https://learn.microsoft.com/en-us/cpp/build/reference/genprofile-fastgenprofile-generate-profiling-instrumented-build">genprofile-fastgenprofile-generate-profiling-instrumented-build</seealso>
		[XmlConfigFile(Category = "WindowsPlatform")]
		[CommandLine("-PGONoExtraCounters")]
		public bool bPGONoExtraCounters = false;

		/// <summary>
		/// If specified along with -PGOProfile, use sample-based PGO instead of instrumented. Currently Intel oneAPI 2024.0+ only.
		/// </summary>
		[XmlConfigFile(Category = "WindowsPlatform")]
		[CommandLine("-SampleBasedPGO")]
		public bool bSampleBasedPGO = false;

		/// <summary>
		/// Which level to use for Inline Function Expansion when TargetRules.bUseInlining is enabled
		/// </summary>
		/// <seealso href="https://learn.microsoft.com/en-us/cpp/build/reference/ob-inline-function-expansion">ob-inline-function-expansion</seealso>
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/WindowsTargetPlatform.WindowsTargetSettings", "InlineFunctionExpansionLevel")]
		[XmlConfigFile(Category = "WindowsPlatform")]
		public int InlineFunctionExpansionLevel { get; set; } = 2;

		/// <summary>
		/// If specificed Visual Studio Dynamic Debugging support will be enabled, unless the compiler or linker is not MSVC or LTCG is enabled
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/WindowsTargetPlatform.WindowsTargetSettings", "bDynamicDebugging")]
		[XmlConfigFile(Category = "WindowsPlatform")]
		[CommandLine("-DynamicDebugging")]
		public bool bDynamicDebugging { get; set; }

		/// <summary>
		/// Version of the compiler toolchain to use on Windows platform. A value of "default" will be changed to a specific version at UBT start up.
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/WindowsTargetPlatform.WindowsTargetSettings", "Compiler")]
		[XmlConfigFile(Category = "WindowsPlatform")]
		[CommandLine("-2022", Value = nameof(WindowsCompiler.VisualStudio2022))]
		[CommandLine("-2026", Value = nameof(WindowsCompiler.VisualStudio2026))]
		[CommandLine("-Compiler=")]
		public WindowsCompiler Compiler { get; set; } = WindowsCompiler.Default;

		/// <summary>
		/// Version of the toolchain to use on Windows platform when a non-msvc Compiler is in use, to locate include paths etc.
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/WindowsTargetPlatform.WindowsTargetSettings", "Toolchain")]
		[XmlConfigFile(Category = "WindowsPlatform")]
		[CommandLine("-VCToolchain=")]
		public WindowsCompiler ToolChain { get; set; } = WindowsCompiler.Default;

		/// <summary>
		/// Order of preference when Compiler is set to Default. If populated and none listed are available this will cause a warning.
		/// </summary>
		[XmlConfigFile(Category = "WindowsPlatform")]
		public string[] PreferredCompilers
		{
			get => _preferredCompilersPrivate ?? DefaultPreferredCompilers;
			set => _preferredCompilersPrivate = value;
		}
		private string[]? _preferredCompilersPrivate = null;

#pragma warning disable IDE0044 // Make field readonly - these private static fields are set by XML configuration.
		/// <summary>
		/// Default order of preference when Compiler is set to Default. If populated and none listed are available this will cause a warning.
		/// </summary>
		[XmlConfigFile(Category = "WindowsPlatform")]
		private string[] DefaultPreferredCompilers = [];
#pragma warning restore IDE0044

		/// <summary>
		/// Architecture of Target.
		/// </summary>
		public UnrealArch Architecture
		{
			get;
			internal set;
		}
		= UnrealArch.X64;

		/// <summary>
		/// Warning level when reporting toolchains that are not in the preferred version list
		/// </summary>
		/// <seealso cref="MicrosoftPlatformSDK.PreferredVisualCppVersions"/>
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/WindowsTargetPlatform.WindowsTargetSettings", "ToolchainVersionWarningLevel")]
		[XmlConfigFile(Category = "WindowsPlatform")]
		[CommandLine("-ToolchainVersionWarningLevel=")]
		public WarningLevel ToolchainVersionWarningLevel { get; set; } = WarningLevel.Warning;

		/// <summary>
		/// The specific compiler version to use. This may be a specific version number (for example, "14.13.26128"), the string "Latest" to select the newest available version, or
		/// the string "Preview" to select the newest available preview version. By default, and if it is available, we use the toolchain version indicated by
		/// WindowsPlatform.DefaultToolChainVersion (otherwise, we use the latest version).
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/WindowsTargetPlatform.WindowsTargetSettings", "CompilerVersion")]
		[XmlConfigFile(Category = "WindowsPlatform")]
		[CommandLine("-CompilerVersion")]
		public string? CompilerVersion = null;

		/// <summary>
		/// The specific msvc toolchain version to use if the compiler is not msvc. This may be a specific version number (for example, "14.13.26128"), the string "Latest" to select the newest available version, or
		/// the string "Preview" to select the newest available preview version. By default, and if it is available, we use the toolchain version indicated by
		/// WindowsPlatform.DefaultToolChainVersion (otherwise, we use the latest version).
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/WindowsTargetPlatform.WindowsTargetSettings", "ToolchainVersion")]
		[XmlConfigFile(Category = "WindowsPlatform")]
		[CommandLine("-VCToolchainVersion")]
		public string? ToolchainVersion = null;

		/// <summary>
		/// True if /fastfail should be passed to the msvc compiler and linker
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/WindowsTargetPlatform.WindowsTargetSettings", "bVCFastFail")]
		[XmlConfigFile(Category = "WindowsPlatform")]
		[CommandLine("-VCFastFail")]
		public bool bVCFastFail = false;

		/// <summary>
		/// True if /d2ExtendedWarningInfo should be passed to the compiler and /d2:-ExtendedWarningInfo to the linker
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/WindowsTargetPlatform.WindowsTargetSettings", "bVCExtendedWarningInfo")]
		[XmlConfigFile(Category = "WindowsPlatform")]
		[CommandLine("-VCExtendedWarningInfo")]
		[CommandLine("-VCDisableExtendedWarningInfo", Value = "false")]
		public bool bVCExtendedWarningInfo = true;

		/// <summary>
		/// True if optimizations to reduce the size of debug information should be disabled. Disabling this option can decrease pdb size but may degrade the debugging experience.
		/// See https://clang.llvm.org/docs/UsersManual.html#cmdoption-fstandalone-debug for more information
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/WindowsTargetPlatform.WindowsTargetSettings", "bClangStandaloneDebug")]
		[XmlConfigFile(Category = "WindowsPlatform")]
		[CommandLine("-ClangStandaloneDebug")]
		[CommandLine("-NoClangStandaloneDebug", Value = "false")]
		public bool bClangStandaloneDebug = true;

		/// <summary>
		/// True if we should use the Clang linker (LLD) when we are compiling with Clang or Intel oneAPI, otherwise we use the MSVC linker.
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/WindowsTargetPlatform.WindowsTargetSettings", "bAllowClangLinker")]
		[XmlConfigFile(Category = "WindowsPlatform")]
		[CommandLine("-ClangLinker")]
		[CommandLine("-NoClangLinker", Value = "false")]
		public bool bAllowClangLinker = true;

		/// <summary>
		/// True if we should use the Rad linker
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/WindowsTargetPlatform.WindowsTargetSettings", "bAllowRadLinker")]
		[XmlConfigFile(Category = "WindowsPlatform")]
		[CommandLine("-RadLinker")]
		public bool bAllowRadLinker = false;

		/// <summary>
		/// The specific Windows SDK version to use. This may be a specific version number (for example, "8.1", "10.0" or "10.0.10150.0"), or the string "Latest", to select the newest available version.
		/// By default, and if it is available, we use the Windows SDK version indicated by WindowsPlatform.DefaultWindowsSdkVersion (otherwise, we use the latest version).
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/WindowsTargetPlatform.WindowsTargetSettings", "WindowsSDKVersion")]
		[XmlConfigFile(Category = "WindowsPlatform")]
		[CommandLine("-WindowsSDKVersion")]
		public string? WindowsSdkVersion = null;

		/// <summary>
		/// Value for the WINVER macro, defining the minimum supported Windows version.
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/WindowsTargetPlatform.WindowsTargetSettings", "TargetWindowsVersion")]
		public int TargetWindowsVersion
		{
			get => TargetWindowsVersionPrivate ?? GetTargetVersionFromPlatformSDK();
			set => TargetWindowsVersionPrivate = value;
		}
		private int? TargetWindowsVersionPrivate = null;
		private int GetTargetVersionFromPlatformSDK()
		{
			// due to some reflection property walking, this can actually be called with a non-Windows platform
			// in which case, just use Win7. This should never matter (but it's hard to skip due to reflection causing it)
			if (!Target.Platform.IsInGroup(UnrealPlatformGroup.Windows))
			{
				return 0x601;
			}

			// @todo some MS platforms are setting this hardcoded, they could move to their SDK.json
			string Key = Architecture.bIsX64 ? "MinimumWindowsX64TargetVersion" : "MinimumWindowsArm64TargetVersion";
			UEBuildPlatformSDK SDK = UEBuildPlatformSDK.GetSDKForPlatform(Target.Platform.ToString())!;
			// the string in the .json will be eg. 0x601, so convert the string from hex
			return Convert.ToInt32(SDK.GetRequiredVersionFromConfig(Key), 16);
		}

		/// <summary>
		/// Value for the NTDDI_VERSION macro, defining the minimum supported Windows version.
		/// https://learn.microsoft.com/en-us/windows/win32/winprog/using-the-windows-headers?redirectedfrom=MSDN#macros-for-conditional-declarations
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/WindowsTargetPlatform.WindowsTargetSettings", "TargetWindowsMinorVersion")]
		public int TargetWindowsMinorVersion
		{
			get => TargetWindowsMinorVersionPrivate ?? (TargetWindowsVersion << 16);
			set => TargetWindowsMinorVersionPrivate = value;
		}
		private int? TargetWindowsMinorVersionPrivate = null;

		/// <summary>
		/// Enable PIX debugging (automatically disabled in Shipping and Test configs)
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/WindowsTargetPlatform.WindowsTargetSettings", "bEnablePIXProfiling")]
		public bool bPixProfilingEnabled = true;

		/// <summary>
		/// Enable building with the Win10 SDK instead of the older Win8.1 SDK 
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/WindowsTargetPlatform.WindowsTargetSettings", "bUseWindowsSDK10")]
		[Obsolete("Deprecated in UE5.8. Set MainVersion in Windows_SDK.json")]
		public bool bUseWindowsSDK10 = false;

		/// <summary>
		/// Enable building with the C++/WinRT language projection
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/WindowsTargetPlatform.WindowsTargetSettings", "bUseCPPWinRT")]
		public bool bUseCPPWinRT = false;
		
		/// <summary>
		/// Enables runtime ray tracing support.
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/WindowsTargetPlatform.WindowsTargetSettings", "RayTracingMode")]
		public WindowsRayTracingRuntimeMode RayTracingMode = WindowsRayTracingRuntimeMode.Disabled;

		/// <summary>
		/// The name of the company (author, provider) that created the project.
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Game, "/Script/EngineSettings.GeneralProjectSettings", "CompanyName")]
		public string? CompanyName;

		/// <summary>
		/// The project's copyright and/or trademark notices.
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Game, "/Script/EngineSettings.GeneralProjectSettings", "CopyrightNotice")]
		public string? CopyrightNotice;

		/// <summary>
		/// The product name.
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Game, "/Script/EngineSettings.GeneralProjectSettings", "ProjectName")]
		public string? ProductName;

		/// <summary>
		/// Enables address sanitizer (ASan)
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration", Name = "bEnableAddressSanitizer")]
		[CommandLine("-EnableASan")]
		public bool bEnableAddressSanitizer = false;

		/// <summary>
		/// Enables undefined behavior sanitizer (UBSan)
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration", Name = "bEnableUndefinedBehaviorSanitizer")]
		[CommandLine("-EnableUBSan")]
		public bool bEnableUndefinedBehaviorSanitizer = false;

		/// <summary>
		/// Enables AutoRTFM sanitizer (AutoRTFMSan)
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration", Name = "bEnableAutoRTFMSanitizer")]
		[CommandLine("-EnableAutoRTFMSan")]
		[CommandLine("-EnableRTFMSan")]
		public bool bEnableAutoRTFMSanitizer = false;

		/// <summary>
		/// Enables instrumentation.
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration", Name = "bEnableInstrumentation")]
		[CommandLine("-EnableInstrumentation")]
		public bool bEnableInstrumentation = false;

		/// <summary>
		/// Enables LibFuzzer.
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration", Name = "bEnableLibFuzzer")]
		[CommandLine("-EnableLibFuzzer")]
		public bool bEnableLibFuzzer = false;

		/// <summary>
		/// Whether .sarif files containing errors and warnings are written alongside each .obj, if supported
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/WindowsTargetPlatform.WindowsTargetSettings", "bWriteSarif")]
		[XmlConfigFile(Category = "WindowsPlatform")]
		public bool bWriteSarif = true;

		/// <summary>
		/// Whether we should export a file containing .obj to source file mappings.
		/// </summary>
		[XmlConfigFile]
		[CommandLine("-ObjSrcMap")]
		public string? ObjSrcMapFile = null;

		/// <summary>
		/// Whether to have the linker or library tool to generate a link repro in the specified directory
		/// See https://learn.microsoft.com/en-us/cpp/build/reference/linkrepro for more information
		/// </summary>
		[CommandLine("-LinkRepro=")]
		public string? LinkReproDir = null;

		/// <summary>
		/// Provides a Module Definition File (.def) to the linker to describe various attributes of a DLL.
		/// Necessary when exporting functions by ordinal values instead of by name.
		/// </summary>
		public string? ModuleDefinitionFile;

		/// <summary>
		/// Specifies the path to a manifest file for the linker to embed. Defaults to the manifest in Engine/Build/Windows/Resources.
		/// </summary>
		public string? ManifestFile;

		/// <summary>
		/// Specifies the path to an .ico file to use as the appliction icon. Can be assigned to null and will default to Engine/Build/Windows/Resources/Default.ico for engine targets or Build/Windows/Application.ico for projects.
		/// </summary>
		public string? ApplicationIcon;

		/// <summary>
		/// Whether to define or not the resource icon to be used in Default.rc2.
		/// </summary>
		[CommandLine("-NoDefineIconResource", Value ="false")]
		public bool bDefineIconResource = true;

		/// <summary>
		/// Enables strict standard conformance mode (/permissive-).
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[XmlConfigFile(Category = "WindowsPlatform")]
		[CommandLine("-Strict")]
		public bool bStrictConformanceMode
		{
			get => bStrictConformanceModePrivate ?? Target.DefaultBuildSettings >= BuildSettingsVersion.V4;
			set => bStrictConformanceModePrivate = value;
		}
		private bool? bStrictConformanceModePrivate;

		/// <summary>
		/// Enables updated __cplusplus macro (/Zc:__cplusplus).
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[XmlConfigFile(Category = "WindowsPlatform")]
		[CommandLine("-UpdatedCPPMacro")]
		public bool bUpdatedCPPMacro { get; set; } = true;

		/// <summary>
		/// Enables inline conformance (Remove unreferenced COMDAT) (/Zc:inline).
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[XmlConfigFile(Category = "WindowsPlatform")]
		[CommandLine("-NoStrictInline", Value = "false")]
		public bool bStrictInlineConformance { get; set; } = true;

		/// <summary>
		/// Enables preprocessor conformance (/Zc:preprocessor).
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[Obsolete("Deprecated in UE5.8. Unreal Engine cannot be compiled without strict preprocessor conformance enabled.")]
		public bool bStrictPreprocessorConformance { get; set; } = true;

		/// <summary>
		/// Enables enum types conformance (/Zc:enumTypes).
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[XmlConfigFile(Category = "WindowsPlatform")]
		[CommandLine("-StrictEnumTypes")]
		[CommandLine("-NoStrictEnumTypes", Value = "false")]
		public bool bStrictEnumTypesConformance { get; set; } = false;

		/// <summary>
		/// Enables enforcing standard C++ ODR violations (/Zc:checkGwOdr).
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[XmlConfigFile(Category = "WindowsPlatform")]
		[CommandLine("-StrictODR")]
		public bool bStrictODRViolationConformance { get; set; } = false;

		/// <summary>
		/// Volatile Metadata is enabled by default and improves x64 emulation on arm64, but may come at a small performance cost (/volatileMetadata-).
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[XmlConfigFile(Category = "WindowsPlatform")]
		[CommandLine("-DisableVolatileMetadata")]
		public bool bDisableVolatileMetadata { get; set; } = false;

		/// <summary>
		/// Controls how the members of a structure are packed into memory and specifies the same packing for all structures in a module.
		/// See https://learn.microsoft.com/en-us/cpp/build/reference/zp-struct-member-alignment
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[XmlConfigFile(Category = "WindowsPlatform")]
		public int? StructMemberAlignment { get; set; } = 8;

		/// <summary>
		/// Whether to request the linker create a stripped pdb file as part of the build.
		/// If enabled the full debug pdb will have the extension .full.pdb
		/// </summary>
		[XmlConfigFile(Category = "WindowsPlatform")]
		[CommandLine("-StripPrivateSymbols")]
		public bool bStripPrivateSymbols = false;

		/// <summary>
		/// If you supply -NoDebugInfo, windows platforms still create debug info while linking. Set this to true to not create debug info while linking in this circumstance
		/// </summary>
		[XmlConfigFile(Category = "WindowsPlatform")]
		[CommandLine("-NoLinkerDebugInfo")]
		public bool bNoLinkerDebugInfo = false;

		/// <summary>
		/// Set page size to allow for larger than 4GB PDBs to be generated by the msvc linker.
		/// Will default to 16384 for monolithic editor builds.
		/// Values should be a power of two such as 4096, 8192, 16384, or 32768
		/// </summary>
		public uint? PdbPageSize = null;

		/// <summary>
		/// Specify an alternate location for the PDB file. This option does not change the location of the generated PDB file,
		/// it changes the name that is embedded into the executable. Path can contain %_PDB% which will be expanded to the original
		/// PDB file name of the target, without the directory.
		/// See https://learn.microsoft.com/en-us/cpp/build/reference/pdbaltpath-use-alternate-pdb-path
		/// </summary>
		[CommandLine("-PdbAltPath")]
		public string? PdbAlternatePath = null;

		/// VS2015 updated some of the CRT definitions but not all of the Windows SDK has been updated to match.
		/// Microsoft provides legacy_stdio_definitions library to enable building with VS2015 until they fix everything up.
		public bool bNeedsLegacyStdioDefinitionsLib => Compiler.IsMSVC() || Compiler.IsClang();

		/// <summary>
		/// The stack size when linking
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/WindowsTargetPlatform.WindowsTargetSettings")]
		public int DefaultStackSize = 12000000;

		/// <summary>
		/// The stack size to commit when linking
		/// </summary>
		[RequiresUniqueBuildEnvironment]
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/WindowsTargetPlatform.WindowsTargetSettings")]
		public int DefaultStackSizeCommit;

		/// <summary>
		/// Whether the embedded manifest will indicate the application is DPI aware
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/WindowsTargetPlatform.WindowsTargetSettings")]
		public bool bForceHighDPIInGameMode = false;

		/// <summary>
		/// Max number of slots FWindowsPlatformTLS::AllocTlsSlot can allocate.
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/WindowsRuntimeSettings.WindowsRuntimeSettings")]
		public int MaxNumTlsSlots = 0;

		/// <summary>
		/// Max number threads that can use FWindowsPlatformTLS at one time.
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/WindowsRuntimeSettings.WindowsRuntimeSettings")]
		public int MaxNumThreadsWithTlsSlots = 0;

		/// <summary>
		/// Determines the amount of memory that the compiler allocates to construct precompiled headers (/Zm).
		/// </summary>
		[XmlConfigFile(Category = "WindowsPlatform")]
		public int PCHMemoryAllocationFactor = 0;

		/// <summary>
		/// Allow the target to specify extra options for linking that aren't otherwise noted here
		/// </summary>
		[XmlConfigFile(Category = "WindowsPlatform")]
		public string AdditionalLinkerOptions = "";

		/// <summary>
		/// Create an image that can be hot patched (/FUNCTIONPADMIN)
		/// </summary>
		public bool bCreateHotPatchableImage
		{
			get => bCreateHotPatchableImagePrivate ?? Target.bWithLiveCoding;
			set => bCreateHotPatchableImagePrivate = value;
		}
		private bool? bCreateHotPatchableImagePrivate;

		/// <summary>
		/// Strip unreferenced symbols (/OPT:REF)
		/// </summary>
		public bool bStripUnreferencedSymbols
		{
			get => bStripUnreferencedSymbolsPrivate ?? ((Target.Configuration == UnrealTargetConfiguration.Test || Target.Configuration == UnrealTargetConfiguration.Shipping) && !Target.bWithLiveCoding);
			set => bStripUnreferencedSymbolsPrivate = value;
		}
		private bool? bStripUnreferencedSymbolsPrivate;

		/// <summary>
		/// Merge identical COMDAT sections together (/OPT:ICF)
		/// </summary>
		public bool bMergeIdenticalCOMDATs
		{
			get => bMergeIdenticalCOMDATsPrivate ?? ((Target.Configuration == UnrealTargetConfiguration.Test || Target.Configuration == UnrealTargetConfiguration.Shipping) && !Target.bWithLiveCoding);
			set => bMergeIdenticalCOMDATsPrivate = value;
		}
		private bool? bMergeIdenticalCOMDATsPrivate;

		/// <summary>
		/// Whether to put global symbols in their own sections (/Gw), allowing the linker to discard any that are unused.
		/// </summary>
		public bool bOptimizeGlobalData = true;

		/// <summary>
		/// Tells the compiler to check only the syntax of the source files on the command line. (/Zs)
		/// </summary>
		[XmlConfigFile(Category = "WindowsPlatform")]
		[CommandLine("-SyntaxCheckOnly")]
		public bool bSyntaxCheckOnly = false;

		/// <summary>
		/// Whether to reduce optimizations for huge functions over an instruction threshold to improve compile time
		/// https://devblogs.microsoft.com/cppblog/msvc-backend-updates-in-visual-studio-2019-versions-16-3-and-16-4/
		/// </summary>
		[XmlConfigFile(Category = "WindowsPlatform")]
		[RequiresUniqueBuildEnvironment]
		public bool bReducedOptimizeHugeFunctions = false;

		/// <summary>
		/// The instruction threshold to use when reducing optimizations for huge functions, default 20000.
		/// </summary>
		[XmlConfigFile(Category = "WindowsPlatform")]
		[RequiresUniqueBuildEnvironment]
		public int ReducedOptimizeHugeFunctionsThreshold = 20000;

		/// <summary>
		/// (Experimental) Appends the -ftime-trace argument to the command line for Clang to output a JSON file containing a timeline for the compile. 
		/// See http://aras-p.info/blog/2019/01/16/time-trace-timeline-flame-chart-profiler-for-Clang/ for more info.
		/// </summary>
		[XmlConfigFile(Category = "WindowsPlatform")]
		public bool bClangTimeTrace = false;

		/// <summary>
		/// Outputs compile timing information so that it can be analyzed.
		/// </summary>
		[XmlConfigFile(Category = "WindowsPlatform")]
		public bool bCompilerTrace = false;

		/// <summary>
		/// Print out files that are included by each source file
		/// </summary>
		[Obsolete("Deprecated in UE5.5 - Use bShowIncludes on TargetRules instead.")]
		public bool bShowIncludes = false;

		/// <summary>
		/// Bundle a working version of dbghelp.dll with the application, and use this to generate minidumps. This works around a bug with the Windows 10 Fall Creators Update (1709)
		/// where rich PE headers larger than a certain size would result in corrupt minidumps.
		/// </summary>
		public bool bUseBundledDbgHelp = true;

		/// <summary>
		/// Whether this build will use Microsoft's custom XCurl instead of libcurl
		/// Note that XCurl is not part of the normal Windows SDK and will require additional downloads
		/// </summary>
		[CommandLine("-UseXCurl")]
		public bool bUseXCurl = false;

		/// <summary>
		/// Settings for PVS studio
		/// </summary>
		public PVSTargetSettings PVS = new PVSTargetSettings();

		/// <summary>
		/// The Visual C++ environment to use for this target. Only initialized after all the target settings are finalized, in ValidateTarget().
		/// </summary>
		internal VCEnvironment? Environment;

		/// <summary>
		/// Directory containing the NETFXSDK
		/// </summary>
		[SuppressMessage("Interoperability", "CA1416:Validate platform compatibility", Justification = "Manually checked")]
#pragma warning disable CA1822 // Mark members as static - used in module build files, leave as-is
		public string? NetFxSdkDir
		{
			get
			{
				DirectoryReference? NetFxSdkDir;
				if (OperatingSystem.IsWindows() && MicrosoftPlatformSDK.TryGetNetFxSdkInstallDir(out NetFxSdkDir))
				{
					return NetFxSdkDir.FullName;
				}
				return null;
			}
		}
#pragma warning restore CA1822

		/// <summary>
		/// Directory containing the DIA SDK
		/// </summary>
		public string? DiaSdkDir => MicrosoftPlatformSDK.FindDiaSdkDirs(Environment!.ToolChain).Select(x => x.FullName).FirstOrDefault();

		/// <summary>
		/// Directory containing the IDE package (Professional, Community, etc...)
		/// </summary>
		public string? IDEDir => MicrosoftPlatformSDK.FindVisualStudioInstallations(Environment!.ToolChain, Target.Logger)
			.Select(x => x.BaseDir.FullName)
			.FirstOrDefault();

		/// <summary>
		/// Directory containing ThirdParty DirectX
		/// </summary>
#pragma warning disable CA1822 // Mark members as static - used in module build files, leave as-is
		public string DirectXDir => Path.Combine(Unreal.EngineSourceDirectory.FullName, "ThirdParty", "Windows", "DirectX");
#pragma warning restore CA1822

		/// <summary>
		/// Directory containing ThirdParty DirectX libs
		/// </summary>
		public string DirectXLibDir => Path.Combine(DirectXDir, "Lib", Target.Architecture.WindowsLibDir) + "/";

		/// <summary>
		/// Directory containing ThirdParty DirectX dlls
		/// </summary>
		public string DirectXDllDir => Path.Combine(Unreal.EngineDirectory.FullName, "Binaries", "ThirdParty", "Windows", "DirectX", Target.Architecture.WindowsLibDir) + "/";

		/// <summary>
		/// When using a Visual Studio compiler, returns the version name as a string
		/// </summary>
		/// <returns>The Visual Studio compiler version name (e.g. "2022")</returns>
		public string GetVisualStudioCompilerVersionName()
		{
			switch (Compiler)
			{
				case WindowsCompiler.Clang:
				case WindowsCompiler.ClangRTFM:
				case WindowsCompiler.ClangInstrument:
				case WindowsCompiler.ClangCustom:
				case WindowsCompiler.Intel:
				case WindowsCompiler.VisualStudio2022:
				case WindowsCompiler.VisualStudio2026:
					return "2015"; // VS2022+ is backwards compatible with VS2015 compiler

				default:
					throw new BuildException("Unexpected WindowsCompiler version for GetVisualStudioCompilerVersionName().  Either not using a Visual Studio compiler or switch block needs to be updated");
			}
		}

		/// <summary>
		/// Determines if a given compiler is installed and valid
		/// </summary>
		/// <param name="Compiler">Compiler to check for</param>
		/// <param name="Architecture">Architecture the compiler must support</param>
		/// <param name="Logger">Logger for output</param>
		/// <returns>True if the given compiler is installed and valid</returns>
		public static bool HasValidCompiler(WindowsCompiler Compiler, UnrealArch Architecture, ILogger Logger)
		{
			return MicrosoftPlatformSDK.HasValidCompiler(Compiler, Architecture, Logger);
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="Target">The target rules which owns this object</param>
		internal WindowsTargetRules(TargetRules Target)
		{
			this.Target = Target;
		}
	}

	/// <summary>
	/// Read-only wrapper for Windows-specific target settings
	/// </summary>
	public class ReadOnlyWindowsTargetRules
	{
		/// <summary>
		/// The private mutable settings object
		/// </summary>
		private readonly WindowsTargetRules Inner;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="Inner">The settings object to wrap</param>
		public ReadOnlyWindowsTargetRules(WindowsTargetRules Inner)
		{
			this.Inner = Inner;
			PVS = new ReadOnlyPVSTargetSettings(Inner.PVS);
		}

		/// <summary>
		/// Accessors for fields on the inner TargetRules instance
		/// </summary>
		#region Read-only accessor properties 
#pragma warning disable CS1591

		public bool bSetResourceVersions => Inner.bSetResourceVersions;

		public bool bSharedResourceFiles => Inner.bSharedResourceFiles;

		public bool bIgnoreStalePGOData => Inner.bIgnoreStalePGOData;

		public bool bUseFastGenProfile => Inner.bUseFastGenProfile;

		public string? PreMergedPgdFilename => Inner.PreMergedPgdFilename;

		public bool bPGONoExtraCounters => Inner.bPGONoExtraCounters;

		public bool bSampleBasedPGO => Inner.bSampleBasedPGO;

		public int InlineFunctionExpansionLevel => Inner.InlineFunctionExpansionLevel;

		public bool bDynamicDebugging => Inner.bDynamicDebugging;

		public WindowsCompiler Compiler => Inner.Compiler;

		public WindowsCompiler ToolChain => Inner.ToolChain;

		public UnrealArch Architecture => Inner.Architecture;

		public WarningLevel ToolchainVersionWarningLevel => Inner.ToolchainVersionWarningLevel;

		public string? CompilerVersion => Inner.CompilerVersion;

		public string? ToolchainVerison => Inner.ToolchainVersion;

		public string? WindowsSdkVersion => Inner.WindowsSdkVersion;

		public int TargetWindowsVersion => Inner.TargetWindowsVersion;

		public int TargetWindowsMinorVersion => Inner.TargetWindowsMinorVersion;

		public bool bPixProfilingEnabled => Inner.bPixProfilingEnabled;

		public bool bUseCPPWinRT => Inner.bUseCPPWinRT;

		public bool bVCFastFail => Inner.bVCFastFail;

		public bool bVCExtendedWarningInfo => Inner.bVCExtendedWarningInfo;

		public bool bClangStandaloneDebug => Inner.bClangStandaloneDebug;

		public bool bAllowClangLinker => Inner.bAllowClangLinker;

		public bool bAllowRadLinker => Inner.bAllowRadLinker;

		public WindowsRayTracingRuntimeMode RayTracingMode => Inner.RayTracingMode;

		public string? CompanyName => Inner.CompanyName;

		public string? CopyrightNotice => Inner.CopyrightNotice;

		public string? ProductName => Inner.ProductName;

		public bool bEnableAddressSanitizer => Inner.bEnableAddressSanitizer;

		public bool bEnableUndefinedBehaviorSanitizer => Inner.bEnableUndefinedBehaviorSanitizer;

		public bool bEnableAutoRTFMSanitizer => Inner.bEnableAutoRTFMSanitizer;

		public bool bEnableInstrumentation => Inner.bEnableInstrumentation;

		public bool bEnableLibFuzzer => Inner.bEnableLibFuzzer;

		public bool bWriteSarif => Inner.bWriteSarif;

		public string? ObjSrcMapFile => Inner.ObjSrcMapFile;

		public string? LinkReproDir => Inner.LinkReproDir;

		public string? ModuleDefinitionFile => Inner.ModuleDefinitionFile;

		public string? ManifestFile => Inner.ManifestFile;
		
		public string? ApplicationIcon => Inner.ApplicationIcon;
		
		public bool bDefineIconResource => Inner.bDefineIconResource;

		public bool bNeedsLegacyStdioDefinitionsLib => Inner.bNeedsLegacyStdioDefinitionsLib;

		public bool bStrictConformanceMode => Inner.bStrictConformanceMode;

		public bool bUpdatedCPPMacro => Inner.bUpdatedCPPMacro;

		public bool bStrictInlineConformance => Inner.bStrictInlineConformance;

		[Obsolete("Deprecated in UE5.8. Unreal Engine cannot be compiled without strict preprocessor conformance enabled.")]
		public bool bStrictPreprocessorConformance => Inner.bStrictPreprocessorConformance;

		public bool bStrictEnumTypesConformance => Inner.bStrictEnumTypesConformance;

		public bool bStrictODRViolationConformance => Inner.bStrictODRViolationConformance;

		public bool bDisableVolatileMetadata => Inner.bDisableVolatileMetadata;

		public int? StructMemberAlignment => Inner.StructMemberAlignment;

		public bool bStripPrivateSymbols => Inner.bStripPrivateSymbols;

		public bool bNoLinkerDebugInfo => Inner.bNoLinkerDebugInfo;

		public uint? PdbPageSize => Inner.PdbPageSize;

		public string? PdbAlternatePath => Inner.PdbAlternatePath;

		public int DefaultStackSize => Inner.DefaultStackSize;

		public int DefaultStackSizeCommit => Inner.DefaultStackSizeCommit;

		public int PCHMemoryAllocationFactor => Inner.PCHMemoryAllocationFactor;

		public string AdditionalLinkerOptions => Inner.AdditionalLinkerOptions;

		public bool bCreateHotpatchableImage => Inner.bCreateHotPatchableImage;

		public bool bStripUnreferencedSymbols => Inner.bStripUnreferencedSymbols;

		public bool bMergeIdenticalCOMDATs => Inner.bMergeIdenticalCOMDATs;

		public bool bOptimizeGlobalData => Inner.bOptimizeGlobalData;

		public bool bSyntaxCheckOnly => Inner.bSyntaxCheckOnly;

		public bool bReducedOptimizeHugeFunctions => Inner.bReducedOptimizeHugeFunctions;

		public int ReducedOptimizeHugeFunctionsThreshold => Inner.ReducedOptimizeHugeFunctionsThreshold;

		public bool bClangTimeTrace => Inner.bClangTimeTrace;

		public bool bCompilerTrace => Inner.bCompilerTrace;

		[Obsolete("Deprecated in UE5.5 - Use bShowIncludes on TargetRules instead.")]
		public bool bShowIncludes => Inner.bShowIncludes;

		public string GetVisualStudioCompilerVersionName()
		{
			return Inner.GetVisualStudioCompilerVersionName();
		}

		public bool bUseBundledDbgHelp => Inner.bUseBundledDbgHelp;

		public bool bUseXCurl => Inner.bUseXCurl;

		public ReadOnlyPVSTargetSettings PVS
		{
			get; private set;
		}

		internal VCEnvironment? Environment => Inner.Environment;

		public string? CompilerDir => Inner.Environment?.CompilerDir.FullName ?? null;

		public string? ToolChainDir => Inner.Environment?.ToolChainDir.FullName ?? null;

		public string? ToolChainVersion => Inner.Environment?.ToolChainVersion.ToString() ?? null;

		public string? WindowsSdkDir => Inner.Environment?.WindowsSdkDir.ToString() ?? null;

		public string? NetFxSdkDir => Inner.NetFxSdkDir;

		public string? DiaSdkDir => Inner.DiaSdkDir;

		public string? IDEDir => Inner.IDEDir;

		public string DirectXDir => Inner.DirectXDir;

		public string DirectXLibDir => Inner.DirectXLibDir;

		public string DirectXDllDir => Inner.DirectXDllDir;

		public int MaxNumTlsSlots => Inner.MaxNumTlsSlots;

		public int MaxNumThreadsWithTlsSlots => Inner.MaxNumThreadsWithTlsSlots;

#pragma warning restore CS1591
		#endregion
	}

	/// <summary>
	/// Information about a particular Visual Studio package
	/// </summary>
	[DebuggerDisplay("{Id},{Version}")]
	record VisualStudioPackage(string id, Version version, bool isExtension)
	{
		/// <summary>
		/// General package identifier
		/// </summary>
		public string Id { get; } = id;

		/// <summary>
		/// Version of the package
		/// </summary>
		public Version Version { get; } = version;

		/// <summary>
		/// Whether the package refers to an external extension
		/// </summary>
		public bool IsExtension { get; } = isExtension;
	}

	/// <summary>
	/// Information about a particular Visual Studio installation
	/// </summary>
	[DebuggerDisplay("{BaseDir}")]
	record VisualStudioInstallation(WindowsCompiler Compiler, VersionNumber Version, DirectoryReference BaseDir, bool bCommunity, WindowsCompilerChannel ReleaseChannel, IDictionary<string, VisualStudioPackage> Packages)
	{
		/// <summary>
		/// Compiler type
		/// </summary>
		public WindowsCompiler Compiler { get; } = Compiler;

		/// <summary>
		/// Version number for this installation
		/// </summary>
		public VersionNumber Version { get; } = Version;

		/// <summary>
		/// Base directory for the installation
		/// </summary>
		public DirectoryReference BaseDir { get; } = BaseDir;

		/// <summary>
		/// Whether it's a community edition of Visual Studio.
		/// </summary>
		public bool bCommunity { get; } = bCommunity;

		/// <summary>
		/// The release channel of this installation
		/// </summary>
		public WindowsCompilerChannel ReleaseChannel { get; } = ReleaseChannel;

		/// <summary>
		/// The packages added to this installation
		/// </summary>
		public IDictionary<string, VisualStudioPackage> Packages { get; } = Packages;
	}

	class WindowsArchitectureConfig : UnrealArchitectureConfig
	{
		public WindowsArchitectureConfig()
			: base(UnrealArchitectureMode.OneTargetPerArchitecture, new[] { UnrealArch.X64, UnrealArch.Arm64, UnrealArch.Arm64ec })
		{

		}

		public override UnrealArchitectures ActiveArchitectures(FileReference? ProjectFile, string? TargetName)
		{
			// for now always compile X64 unless overridden on commandline
			return new UnrealArchitectures(UnrealArch.X64);
		}

		public override bool RequiresArchitectureFilenames(UnrealArchitectures Architectures)
		{
			// for now x64 only uses architecture-less filenames.
			return Architectures.bIsMultiArch || !Architectures.Contains(UnrealArch.X64);
		}

		public override UnrealArch GetHostArchitecture()
		{
			switch (System.Runtime.InteropServices.RuntimeInformation.ProcessArchitecture)
			{
				case System.Runtime.InteropServices.Architecture.Arm64:
					return UnrealArch.Arm64;
				default:
					return UnrealArch.X64;
			}
		}
	}

	class WindowsPlatform : UEBuildPlatform
	{
		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="InPlatform">Creates a windows platform with the given enum value</param>
		/// <param name="InSDK">The installed Windows SDK</param>
		/// <param name="InLogger">Logger instance</param>
		public WindowsPlatform(UnrealTargetPlatform InPlatform, MicrosoftPlatformSDK InSDK, ILogger InLogger)
			: this(InPlatform, InSDK, new WindowsArchitectureConfig(), InLogger)
		{
		}

		/// <summary>
		/// Constructor that takes an archConfig (for use by subclasses)
		/// </summary>
		/// <param name="InPlatform">Creates a windows platform with the given enum value</param>
		/// <param name="InSDK">The installed Windows SDK</param>
		/// <param name="ArchConfig">Achitecture configuration</param>
		/// <param name="InLogger">Logger instance</param>
		public WindowsPlatform(UnrealTargetPlatform InPlatform, MicrosoftPlatformSDK InSDK, UnrealArchitectureConfig ArchConfig, ILogger InLogger)
			: base(InPlatform, InSDK, ArchConfig, InLogger)
		{
		}
		/// <summary>
		/// Reset a target's settings to the default
		/// </summary>
		/// <param name="Target"></param>
		public override void ResetTarget(TargetRules Target)
		{
			base.ResetTarget(Target);
		}

		/// <summary>
		/// Creates the VCEnvironment object used to control compiling and other tools. Virtual to allow other platforms to override behavior
		/// </summary>
		/// <param name="Target">Stanard target object</param>
		/// <returns></returns>
		[SupportedOSPlatform("windows")]
		protected virtual VCEnvironment CreateVCEnvironment(TargetRules Target)
		{
			return VCEnvironment.Create(Target.WindowsPlatform.Compiler, Target.WindowsPlatform.ToolChain, Platform, Target.WindowsPlatform.Architecture, Target.WindowsPlatform.CompilerVersion, Target.WindowsPlatform.ToolchainVersion, Target.WindowsPlatform.WindowsSdkVersion, null, Target.WindowsPlatform.bUseCPPWinRT, Target.WindowsPlatform.bAllowClangLinker, Target.WindowsPlatform.bAllowRadLinker, Logger);
		}

		/// <summary>
		/// Validate a target's settings
		/// </summary>
		public override void ValidateTarget(TargetRules Target)
		{
			base.ValidateTarget(Target);

			if (Platform == UnrealTargetPlatform.Win64)
			{
				Target.WindowsPlatform.Architecture = Target.Architecture;

				bool bCompilingForArm = !Target.Architecture.bIsX64;
				if (bCompilingForArm && Target.Name != "UnrealHeaderTool")
				{
					// VTune does not support ARM
					Target.GlobalDefinitions.Add("UE_EXTERNAL_PROFILING_ENABLED=0");
				}
			}

			// Disable Simplygon support if compiling against the NULL RHI.
			if (Target.GlobalDefinitions.Contains("USE_NULL_RHI=1"))
			{
				Target.bCompileCEF3 = false;
			}

			// Intel compiler only supports x64
			if (Target.WindowsPlatform.Compiler.IsIntel() && !Target.Architecture.bIsX64)
			{
				Target.WindowsPlatform.Compiler = WindowsCompiler.Default;
			}

			if (Target.WindowsPlatform.bEnableInstrumentation && !Target.bUseAutoRTFMCompiler)
			{
				Target.WindowsPlatform.Compiler = WindowsCompiler.ClangInstrument;

				// We always use clang linker with ClangInstrument.
				Target.WindowsPlatform.bAllowClangLinker = true;
			}

			if (Target.bUseAutoRTFMCompiler)
			{
				// We check the static analyzer, and only allow using the Auto-RTFM compiler if
				// we are not doing static analysis, or are explicitly wanting to use clang.
				switch (Target.StaticAnalyzer)
				{
					case StaticAnalyzer.PVSStudio:
						break;

					// All other analysis cases should use the ClangRTFM compiler as requested
					default:
						Target.WindowsPlatform.Compiler = WindowsCompiler.ClangRTFM;

						// We always use clang linker with AutoRTFM.
						Target.WindowsPlatform.bAllowClangLinker = true;
						break;
				}
			}

			if (Target.WindowsPlatform.bEnableUndefinedBehaviorSanitizer)
			{
				// UBSan currently requires a clang toolchain
				if (!Target.WindowsPlatform.Compiler.IsClang())
				{
					Target.WindowsPlatform.Compiler = WindowsCompiler.Clang;
				}
			}

			if (Target.WindowsPlatform.bEnableAutoRTFMSanitizer)
			{
				Target.WindowsPlatform.Compiler = WindowsCompiler.ClangRTFM;
			}

			// Set the compiler version if necessary
			if (Target.WindowsPlatform.Compiler == WindowsCompiler.Default)
			{
				Target.WindowsPlatform.Compiler = GetDefaultCompiler(Target, Target.WindowsPlatform.Architecture, Logger, !Platform.IsInGroup(UnrealPlatformGroup.Microsoft));
			}

			// Set the toolchain version if necessary
			if (Target.WindowsPlatform.ToolChain == WindowsCompiler.Default)
			{
				Target.WindowsPlatform.ToolChain = GetDefaultToolchain(Target, Target.WindowsPlatform.Architecture, Logger, !Platform.IsInGroup(UnrealPlatformGroup.Microsoft));
			}

			if (Target.StaticAnalyzer == StaticAnalyzer.VisualCpp
				|| (Target.StaticAnalyzer == StaticAnalyzer.Default && Target.WindowsPlatform.Compiler.IsMSVC()))
			{
				// VisualCpp static analysis requires MSVC
				if (!Target.WindowsPlatform.Compiler.IsMSVC())
				{
					Target.WindowsPlatform.Compiler = WindowsCompiler.VisualStudio2022;
				}
				Target.StaticAnalyzer = StaticAnalyzer.Default;
			}
			else if (Target.StaticAnalyzer == StaticAnalyzer.Clang
				|| (Target.StaticAnalyzer == StaticAnalyzer.Default && Target.WindowsPlatform.Compiler.IsClang()))
			{
				// Clang static analysis requires a clang toolchain
				if (!Target.WindowsPlatform.Compiler.IsClang())
				{
					Target.WindowsPlatform.Compiler = WindowsCompiler.Clang;
				}
				Target.StaticAnalyzer = StaticAnalyzer.Default;

				// Clang static analysis requires non unity builds
				Target.bUseUnityBuild = false;
				if (Target.bStaticAnalyzerIncludeGenerated)
				{
					Target.bAlwaysUseUnityForGeneratedFiles = false;
				}
			}
			else if (Target.StaticAnalyzer == StaticAnalyzer.PVSStudio)
			{
				// Disable PCHs for PVS studio analyzer.
				Target.bUsePCHFiles = false;

				// PVSStudio static analysis requires MSVC
				Target.WindowsPlatform.Compiler = WindowsCompiler.VisualStudio2022;
			}

			// Disable linking and ignore build outputs if we're using a static analyzer
			if (Target.StaticAnalyzer != StaticAnalyzer.None)
			{
				Target.bDisableLinking = true;
				Target.bIgnoreBuildOutputs = true;

				// Enable extended warnings when analyzing
				Target.WindowsPlatform.bVCExtendedWarningInfo = true;

				// Disable Dynamic Debugging when analyzing
				Target.WindowsPlatform.bDynamicDebugging = false;
			}

			// Disable chaining PCH files
			Target.bChainPCHs = false;

			// E&C support.
			if (Target.bSupportEditAndContinue || Target.bAdaptiveUnityEnablesEditAndContinue)
			{
				Target.bUseIncrementalLinking = true;
			}
			if (Target.bAdaptiveUnityEnablesEditAndContinue && !Target.bAdaptiveUnityDisablesPCH && !Target.bAdaptiveUnityCreatesDedicatedPCH)
			{
				throw new BuildException("bAdaptiveUnityEnablesEditAndContinue requires bAdaptiveUnityDisablesPCH or bAdaptiveUnityCreatesDedicatedPCH");
			}

			// for monolithic or merged module editor builds, add the PDBPAGESIZE option, (VS 16.11, VC toolchain 14.29.30133), but the pdb will be too large without this
			// some monolithic game builds could be too large as well, but they can be added in a .Target.cs with:
			// TargetRules.WindowsPlatform.PdbPageSize = 8192;
			if (!Target.WindowsPlatform.PdbPageSize.HasValue && (Target.LinkType == TargetLinkType.Monolithic || Target.bMergeModules) && Target.Type == TargetType.Editor)
			{
				Target.WindowsPlatform.PdbPageSize = 16384;
			}

			// If we're using PDB files and PCHs, the generated code needs to be compiled with the same options as the PCH.
			if ((Target.bUsePDBFiles || Target.bSupportEditAndContinue) && Target.bUsePCHFiles)
			{
				Target.bDisableDebugInfoForGeneratedCode = false;
			}

			if (Target.WindowsPlatform.bDynamicDebugging)
			{
				// Monolithic or merged module targets always require increasing the pdb page size with Dynamic Debugging or linking may fail
				if ((Target.LinkType == TargetLinkType.Monolithic || Target.bMergeModules) && (Target.WindowsPlatform.PdbPageSize ?? 0) < 16384)
				{
					Target.WindowsPlatform.PdbPageSize = 16384;
				}

				// '/DYNAMICDEOPT' combined with '/OPT:ICF' has a degraded debugging experience;
				if (Target.WindowsPlatform.bMergeIdenticalCOMDATs)
				{
					Target.WindowsPlatform.bMergeIdenticalCOMDATs = false;
				}
			}

			Target.bCompileISPC = true;

			if (OperatingSystem.IsWindows())
			{
				// Initialize the VC environment for the target, and set all the version numbers to the concrete values we chose
				Target.WindowsPlatform.Environment = CreateVCEnvironment(Target);

				// pull some things from it
				Target.WindowsPlatform.Compiler = Target.WindowsPlatform.Environment.Compiler;
				Target.WindowsPlatform.CompilerVersion = Target.WindowsPlatform.Environment.CompilerVersion.ToString();
				Target.WindowsPlatform.ToolChain = Target.WindowsPlatform.Environment.ToolChain;
				Target.WindowsPlatform.ToolchainVersion = Target.WindowsPlatform.Environment.ToolChainVersion.ToString();
				Target.WindowsPlatform.WindowsSdkVersion = Target.WindowsPlatform.Environment.WindowsSdkVersion.ToString();

				ValidateToolchainVersion(Target);
			}

			if (Target.StaticAllocator == StaticAllocatorType.None)
			{
				Target.StaticAllocator = StaticAllocatorType.Binned3;
			}
			if (Target.WindowsPlatform.bEnableAddressSanitizer)
			{
				Target.StaticAllocator = StaticAllocatorType.Ansi;
			}

			if (Target.WindowsPlatform.ManifestFile == null)
			{
				string Suffix = Target.Platform.ToString();
				switch (Target.Type)
				{
					case TargetType.Game:
					case TargetType.Client:
						if (Target.WindowsPlatform.bForceHighDPIInGameMode)
						{
							Suffix += "-dpiaware";
						}
						break;
				}
				if (Target.Platform == UnrealTargetPlatform.Win64 && !Target.Architecture.bIsX64)
				{
					Suffix += "-arm64";
				}
				Target.WindowsPlatform.ManifestFile = FileReference.Combine(Unreal.EngineDirectory, "Build", "Windows", "Resources", String.Format("Default-{0}.manifest", Suffix)).FullName;
			}
		}

		static bool _toolchainWarningLogged = false;
		void ValidateToolchainVersion(TargetRules Target)
		{
			if (_toolchainWarningLogged || Target.WindowsPlatform.Environment == null)
			{
				return;
			}

			// Ensure we're using a recent enough version of Clang given the MSVC version
			if (Target.WindowsPlatform.Compiler.IsClang() && !MicrosoftPlatformSDK.IgnoreToolchainErrors)
			{
				VersionNumber ClangVersion = Target.WindowsPlatform.Compiler == WindowsCompiler.Intel ? MicrosoftPlatformSDK.GetClangVersionForIntelCompiler(Target.WindowsPlatform.Environment.CompilerPath) : Target.WindowsPlatform.Environment.CompilerVersion;
				VersionNumber MinimumClang = MicrosoftPlatformSDK.GetMinimumClangVersionForVcVersion(Target.WindowsPlatform.Environment.ToolChainVersion);
				if (ClangVersion < MinimumClang)
				{
					throw new BuildException("MSVC toolchain version {0} requires Clang compiler version {1} or later. The current Clang compiler version was detected as: {2}", Target.WindowsPlatform.Environment.ToolChainVersion, MinimumClang, ClangVersion);
				}
			}

			if (Target.WindowsPlatform.ToolchainVersionWarningLevel != WarningLevel.Off)
			{
				if (!MicrosoftPlatformSDK.IsPreferredVersion(Target.WindowsPlatform.Compiler, Target.WindowsPlatform.Environment.CompilerVersion))
				{
					_toolchainWarningLogged = true;
					VersionNumber preferred = MicrosoftPlatformSDK.GetLatestPreferredVersion(Target.WindowsPlatform.Compiler);
					MicrosoftPlatformSDK.DumpAllToolChainInstallations(Target.WindowsPlatform.Compiler, Target.Architecture, Logger);
					if (Target.WindowsPlatform.ToolchainVersionWarningLevel == WarningLevel.Error)
					{
						throw new BuildLogEventException("{Compiler} compiler version {Version} is not a preferred version. Please use the latest preferred version {PreferredVersion}", WindowsPlatform.GetCompilerName(Target.WindowsPlatform.Compiler), Target.WindowsPlatform.Environment.CompilerVersion, preferred);
					}
					else if (Target.WindowsPlatform.Environment.CompilerVersion < preferred)
					{
						Logger.LogInformation("{Compiler} compiler version {Version} is not a preferred version. Please use the latest preferred version {PreferredVersion}", WindowsPlatform.GetCompilerName(Target.WindowsPlatform.Compiler), Target.WindowsPlatform.Environment.CompilerVersion, preferred);
					}
					else
					{
						Logger.LogInformation("{Compiler} compiler version {Version} is newer than latest preferred version {PreferredVersion}. Please use caution as this compiler has not been heavily tested", WindowsPlatform.GetCompilerName(Target.WindowsPlatform.Compiler), Target.WindowsPlatform.Environment.CompilerVersion, preferred);
					}
				}

				if (Target.WindowsPlatform.Compiler != Target.WindowsPlatform.ToolChain && !MicrosoftPlatformSDK.IsPreferredVersion(Target.WindowsPlatform.ToolChain, Target.WindowsPlatform.Environment.ToolChainVersion))
				{
					_toolchainWarningLogged = true;
					VersionNumber preferred = MicrosoftPlatformSDK.GetLatestPreferredVersion(Target.WindowsPlatform.ToolChain);
					if (Target.WindowsPlatform.ToolchainVersionWarningLevel == WarningLevel.Error)
					{
						throw new BuildLogEventException("{Toolchain} toolchain version {Version} is not a preferred version. Please use the latest preferred version {PreferredVersion}", WindowsPlatform.GetCompilerName(Target.WindowsPlatform.ToolChain), Target.WindowsPlatform.Environment.ToolChainVersion, preferred);
					}
					else if (Target.WindowsPlatform.Environment.ToolChainVersion < preferred)
					{
						Logger.LogInformation("{Toolchain} toolchain version {Version} is not a preferred version. Please use a preferred toolchain such as {PreferredVersion}", WindowsPlatform.GetCompilerName(Target.WindowsPlatform.ToolChain), Target.WindowsPlatform.Environment.ToolChainVersion, preferred);
					}
					else
					{
						Logger.LogInformation("{Toolchain} toolchain version {Version} is newer than latest preferred version {PreferredVersion}. Please use caution as this toolchain has not been heavily tested", WindowsPlatform.GetCompilerName(Target.WindowsPlatform.ToolChain), Target.WindowsPlatform.Environment.ToolChainVersion, preferred);
					}
				}
			}
		}

		/// <summary>
		/// Gets the default compiler which should be used, if it's not set explicitly by the target, command line, or config file.
		/// </summary>
		/// <returns>The default compiler version</returns>
		private static IEnumerable<WindowsCompiler> GetPreferredCompilers(TargetRules? TargetRules)
		{
			if (TargetRules == null)
			{
				yield break;
			}
			foreach (string enumString in TargetRules.WindowsPlatform.PreferredCompilers)
			{
				if (Enum.TryParse(enumString, out WindowsCompiler result))
				{
					yield return result;
				}
			}
		}

		/// <summary>
		/// Gets the default compiler which should be used, if it's not set explicitly by the target, command line, or config file.
		/// </summary>
		/// <returns>The default compiler version</returns>
		internal static WindowsCompiler GetDefaultCompiler(TargetRules? TargetRules, UnrealArch Architecture, ILogger Logger, bool bSkipWarning = false)
		{
			IEnumerable<WindowsCompiler> PreferredCompilers = GetPreferredCompilers(TargetRules);
			if (PreferredCompilers.Any())
			{
				WindowsCompiler Compiler = PreferredCompilers.FirstOrDefault(x => MicrosoftPlatformSDK.HasValidCompiler(x, Architecture, Logger));
				if (Compiler != default)
				{
					if (Compiler != PreferredCompilers.First())
					{
						Logger.Log(bSkipWarning || MicrosoftPlatformSDK.IgnoreToolchainErrors ? LogLevel.Debug : LogLevel.Warning, "{Target} {Platform} {Architecture} preferred compiler {Preferred} not available, compiler set to {Compiler}", TargetRules?.Name ?? "Default", TargetRules?.Platform ?? UnrealTargetPlatform.Win64, Architecture, PreferredCompilers.First(), Compiler);
					}
					else
					{
						Logger.LogDebug("{Target} {Platform} {Architecture} Compiler set to preferred compiler {Compiler}", TargetRules?.Name ?? "Default", TargetRules?.Platform ?? UnrealTargetPlatform.Win64, Architecture, Compiler);
					}
					return Compiler;
				}
				Logger.Log(bSkipWarning || MicrosoftPlatformSDK.IgnoreToolchainErrors ? LogLevel.Debug : LogLevel.Warning, "{Target} {Platform} {Architecture} No preferred compilers available [{Preferred}]", TargetRules?.Name ?? "Default", TargetRules?.Platform ?? UnrealTargetPlatform.Win64, Architecture, String.Join(',', PreferredCompilers));
			}

			return GetDefaultToolchain(TargetRules, Architecture, Logger, bSkipWarning);
		}

		/// <summary>
		/// Gets the default toolchain which should be used, if it's not set explicitly by the target, command line, or config file.
		/// </summary>
		/// <returns>The default toolchain version</returns>
		internal static WindowsCompiler GetDefaultToolchain(TargetRules? TargetRules, UnrealArch Architecture, ILogger Logger, bool bSkipWarning = false)
		{
			// If the compiler is set to MSVC, it should also be used as the toolchain
			if (TargetRules?.WindowsPlatform.Compiler.IsMSVC() == true)
			{
				return TargetRules.WindowsPlatform.Compiler;
			}

			// If there's no specific compiler set, try to pick the matching compiler for the selected IDE
			if (ProjectFileGeneratorSettings.Format != null)
			{
				foreach (ProjectFileFormat Format in ProjectFileGeneratorSettings.ParseFormatList(ProjectFileGeneratorSettings.Format, Logger))
				{
					if (Format == ProjectFileFormat.VisualStudio2022)
					{
						return WindowsCompiler.VisualStudio2022;
					}
					else if (Format == ProjectFileFormat.VisualStudio2026)
					{
						return WindowsCompiler.VisualStudio2026;
					}
				}
			}

			// Also check the default format for the Visual Studio project generator
			object? ProjectFormatObject;
			if (XmlConfig.TryGetValue(typeof(VCProjectFileSettings), "ProjectFileFormat", out ProjectFormatObject))
			{
				VCProjectFileFormat ProjectFormat = (VCProjectFileFormat)ProjectFormatObject;
				if (ProjectFormat == VCProjectFileFormat.VisualStudio2022)
				{
					return WindowsCompiler.VisualStudio2022;
				}
				else if (ProjectFormat == VCProjectFileFormat.VisualStudio2026)
				{
					return WindowsCompiler.VisualStudio2026;
				}
			}

			// Check the editor settings too
			ProjectFileFormat PreferredAccessor;
			if (ProjectFileGenerator.GetPreferredSourceCodeAccessor(TargetRules?.ProjectFile, out PreferredAccessor))
			{
				if (PreferredAccessor == ProjectFileFormat.VisualStudio)
				{
					return WindowsCompiler.VisualStudio;
				}
				else if (PreferredAccessor == ProjectFileFormat.VisualStudio2022)
				{
					return WindowsCompiler.VisualStudio2022;
				}
				else if (PreferredAccessor == ProjectFileFormat.VisualStudio2026)
				{
					return WindowsCompiler.VisualStudio2026;
				}
			}

			// Check if any valid compilers are installed so a warning can be shown if none are available
			bool compilerAvailable = MicrosoftPlatformSDK.HasValidCompiler(WindowsCompiler.VisualStudio, Architecture, NullLogger.Instance);
			if (!compilerAvailable && !bSkipWarning && !MicrosoftPlatformSDK.IgnoreToolchainErrors)
			{
				UEBuildPlatformSDK? SDK = GetSDK(UnrealTargetPlatform.Win64);
				VersionNumber? MinimumMsvcVersion = SDK?.GetVersionNumberFromConfig("MinimumVisualCppVersion");
				VersionNumber? MinimumVsVersion = SDK?.GetVersionNumberFromConfig("MinimumVisualStudio2022Version");
				VersionNumber? PreferredMsvcVersion = SDK?.GetVersionNumberRangeArrayFromConfig("PreferredVisualCppVersions")?.FirstOrDefault()?.Min;
				string MinimumMsvcVersionStr = MinimumMsvcVersion?.ToString() ?? "Latest";
				string MinimumVsVersionStr = MinimumVsVersion?.ToString() ?? "Latest";
				string PreferredMsvcVersionStr = PreferredMsvcVersion?.ToString() ?? "Latest";
				string ArchStr = Architecture == UnrealArch.X64 ? "x64/x86" : "ARM64/ARM64EC";
				string PreferredComponentStr = PreferredMsvcVersion != null ?
					$"MSVC Build Tools v{PreferredMsvcVersion.Components[0]}.{PreferredMsvcVersion.Components[1]} for {ArchStr}" :
					$"MSVC Build Tools for {ArchStr} (Latest)";

				// If we do have a Visual Studio installation, but we're missing just the C++ parts, warn about that.
				if (MicrosoftPlatformSDK.FindVisualStudioInstallations(NullLogger.Instance).Any())
				{
					Logger.LogWarning("Visual Studio is installed, but is out of date or missing a valid C++ toolchain (minimum version {MinVersion}, preferred version {PreferredVersion}). Please update Visual Studio to {MinimumVsVersionStr} or later and verify that the \"{Component}\" component is selected in the Visual Studio installation options.",
						MinimumMsvcVersionStr,
						PreferredMsvcVersionStr,
						MinimumVsVersionStr,
						PreferredComponentStr);
				}
				else
				{
					Logger.LogWarning("No valid Visual C++ toolchain was found (minimum version {MinVersion}, preferred version {PreferredVersion}). Please download and install Visual Studio {MinimumVsVersionStr} or later and verify that the \"{Component}\" component is selected in the Visual Studio installation options.",
						MinimumMsvcVersionStr,
						PreferredMsvcVersionStr,
						MinimumVsVersionStr,
						PreferredComponentStr);
				}
			}

			// Default to any available visual studio compiler
			return WindowsCompiler.VisualStudio;
		}

		/// <summary>
		/// Returns the human-readable name of the given compiler
		/// </summary>
		/// <param name="Compiler">The compiler value</param>
		/// <returns>Name of the compiler</returns>
		public static string GetCompilerName(WindowsCompiler Compiler)
		{
			return MicrosoftPlatformSDK.GetCompilerName(Compiler);
		}

		/// <summary>
		/// Get the first Visual Studio install directory for the given compiler version. Note that it is possible for the compiler toolchain to be installed without
		/// Visual Studio.
		/// </summary>
		/// <param name="Compiler">Version of the toolchain to look for.</param>
		/// <param name="Logger">Logger for output</param>
		/// <returns>True if the directory was found, false otherwise.</returns>
		public static IEnumerable<DirectoryReference>? TryGetVSInstallDirs(WindowsCompiler Compiler, ILogger Logger)
		{
			List<VisualStudioInstallation> Installations = MicrosoftPlatformSDK.FindVisualStudioInstallations(Compiler, Logger);
			if (Installations.Count == 0)
			{
				return null;
			}

			return Installations.Select(x => x.BaseDir);
		}

		/// <summary>
		/// Determines if a given compiler is installed and valid
		/// </summary>
		/// <param name="Compiler">Compiler to check for</param>
		/// <param name="Architecture">Architecture the compiler must support</param>
		/// <param name="Logger">Logger for output</param>
		/// <returns>True if the given compiler is installed and valid</returns>
		public static bool HasCompiler(WindowsCompiler Compiler, UnrealArch Architecture, ILogger Logger)
		{
			return MicrosoftPlatformSDK.HasCompiler(Compiler, Architecture, Logger);
		}

		/// <summary>
		/// Determines the directory containing the MSVC toolchain
		/// </summary>
		/// <param name="Compiler">Major version of the compiler to use</param>
		/// <param name="CompilerVersion">The minimum compiler version to use</param>
		/// <param name="Architecture">Architecture that is required</param>
		/// <param name="Logger">Logger for output</param>
		/// <param name="OutToolChainVersion">Receives the chosen toolchain version</param>
		/// <param name="OutToolChainDir">Receives the directory containing the toolchain</param>
		/// <param name="OutRedistDir">Receives the optional directory containing redistributable components</param>
		/// <returns>True if the toolchain directory was found correctly</returns>
		public static bool TryGetToolChainDir(WindowsCompiler Compiler, string? CompilerVersion, UnrealArch Architecture, ILogger Logger, [NotNullWhen(true)] out VersionNumber? OutToolChainVersion, [NotNullWhen(true)] out DirectoryReference? OutToolChainDir, out DirectoryReference? OutRedistDir)
		{
			return MicrosoftPlatformSDK.TryGetToolChainDir(Compiler, CompilerVersion, Architecture, Logger, out OutToolChainVersion, out OutToolChainDir, out OutRedistDir);
		}

		public static string GetArchitectureName(UnrealArch arch)
		{
			return arch.ToString();
		}

		[SupportedOSPlatform("windows")]
		public static bool TryGetWindowsSdkDir(string? DesiredVersion, ILogger Logger, [NotNullWhen(true)] out VersionNumber? OutSdkVersion, [NotNullWhen(true)] out DirectoryReference? OutSdkDir)
		{
			return MicrosoftPlatformSDK.TryGetWindowsSdkDir(DesiredVersion, Logger, out OutSdkVersion, out OutSdkDir);
		}

		/// <summary>
		/// Gets the platform name that should be used.
		/// </summary>
		public override string GetPlatformName()
		{
			return "Windows";
		}

		/// <summary>
		/// If this platform can be compiled with SN-DBS
		/// </summary>
		public override bool CanUseSNDBS()
		{
			return true;
		}

		public override IReadOnlyDictionary<string, string>? GetSNDBSToolTemplates()
		{
			return SNDBSToolTemplates;
		}

		/// <summary>
		/// If this platform can be compiled with FASTBuild
		/// </summary>
		public override bool CanUseFASTBuild()
		{
			return true;
		}

		/// <summary>
		/// Determines if the given name is a build product for a target.
		/// </summary>
		/// <param name="FileName">The name to check</param>
		/// <param name="NamePrefixes">Target or application names that may appear at the start of the build product name (eg. "UnrealEditor", "ShooterGameEditor")</param>
		/// <param name="NameSuffixes">Suffixes which may appear at the end of the build product name</param>
		/// <returns>True if the string matches the name of a build product, false otherwise</returns>
		public override bool IsBuildProduct(string FileName, string[] NamePrefixes, string[] NameSuffixes)
		{
			IEnumerable<string> extensions = [
				".exe",
				".dll",
				".dll.response",
				".dll.rsp",
				".lib",
				".pdb",
				".full.pdb",
				".exp",
				".obj",
				".map",
				".objpaths",
				".natvis",
				".natstepfilter",
				".natjmc"
			];

			return extensions.Any(x => IsBuildProductName(FileName, NamePrefixes, NameSuffixes, x))
				|| extensions.Select(x => $".alt{x}").Any(x => IsBuildProductName(FileName, NamePrefixes, NameSuffixes, x));

		}

		/// <summary>
		/// Get the extension to use for the given binary type
		/// </summary>
		/// <param name="InBinaryType"> The binrary type being built</param>
		/// <returns>string    The binary extenstion (ie 'exe' or 'dll')</returns>
		public override string GetBinaryExtension(UEBuildBinaryType InBinaryType)
		{
			switch (InBinaryType)
			{
				case UEBuildBinaryType.DynamicLinkLibrary:
					return ".dll";
				case UEBuildBinaryType.Executable:
					return ".exe";
				case UEBuildBinaryType.StaticLibrary:
					return ".lib";
			}
			return base.GetBinaryExtension(InBinaryType);
		}

		/// <summary>
		/// Get the extensions to use for debug info for the given binary type
		/// </summary>
		/// <param name="Target">The target being built</param>
		/// <param name="InBinaryType"> The binary type being built</param>
		/// <returns>string[]    The debug info extensions (i.e. 'pdb')</returns>
		public override string[] GetDebugInfoExtensions(ReadOnlyTargetRules Target, UEBuildBinaryType InBinaryType)
		{
			switch (InBinaryType)
			{
				case UEBuildBinaryType.DynamicLinkLibrary:
				case UEBuildBinaryType.Executable:
					return new string[] { ".pdb" };
			}
			return Array.Empty<string>();
		}

		public override bool HasDefaultBuildConfig(UnrealTargetPlatform Platform, DirectoryReference ProjectPath)
		{
			// check the base settings
			return base.HasDefaultBuildConfig(Platform, ProjectPath);
		}

		/// <summary>
		/// Modify the rules for a newly created module, where the target is a different host platform.
		/// This is not required - but allows for hiding details of a particular platform.
		/// </summary>
		/// <param name="ModuleName">The name of the module</param>
		/// <param name="Rules">The module rules</param>
		/// <param name="Target">The target being build</param>
		public override void ModifyModuleRulesForOtherPlatform(string ModuleName, ModuleRules Rules, ReadOnlyTargetRules Target)
		{
		}

		/// <summary>
		/// Gets the application icon for a given project
		/// </summary>
		/// <param name="ProjectFile">The project file</param>
		/// <returns>The icon to use for this project</returns>
		public static FileReference GetWindowsApplicationIcon(FileReference? ProjectFile)
		{
			// Check if there's a custom icon
			if (ProjectFile != null)
			{
				FileReference IconFile = FileReference.Combine(ProjectFile.Directory, "Build", "Windows", "Application.ico");
				if (FileReference.Exists(IconFile))
				{
					return IconFile;
				}
			}

			// Otherwise use the default
			return FileReference.Combine(Unreal.EngineDirectory, "Build", "Windows", "Resources", "Default.ico");
		}

		/// <summary>
		/// Gets the application icon for a given project
		/// </summary>
		/// <param name="ProjectFile">The project file</param>
		/// <returns>The icon to use for this project</returns>
		public virtual FileReference GetApplicationIcon(FileReference ProjectFile)
		{
			return GetWindowsApplicationIcon(ProjectFile);
		}

		/// <summary>
		/// Modify the rules for a newly created module, in a target that's being built for this platform.
		/// This is not required - but allows for hiding details of a particular platform.
		/// </summary>
		/// <param name="ModuleName">The name of the module</param>
		/// <param name="Rules">The module rules</param>
		/// <param name="Target">The target being build</param>
		public override void ModifyModuleRulesForActivePlatform(string ModuleName, ModuleRules Rules, ReadOnlyTargetRules Target)
		{
			bool bBuildShaderFormats = Target.bForceBuildShaderFormats;

			if (!Target.bBuildRequiresCookedData && Target.Type != TargetType.Program)
			{
				if (ModuleName == "TargetPlatform")
				{
					bBuildShaderFormats = true;
				}
			}

			// If instrumentation is active, we need to inject the instrumentation module in every other module
			if (Target.WindowsPlatform.bEnableInstrumentation && ModuleName != "Instrumentation")
			{
				Rules.PrivateDependencyModuleNames.Add("Instrumentation");
			}

			// allow standalone tools to use target platform modules, without needing Engine
			if (ModuleName == "TargetPlatform")
			{
				if (Target.bForceBuildTargetPlatforms)
				{
					Rules.DynamicallyLoadedModuleNames.Add("WindowsTargetPlatform");
					Rules.DynamicallyLoadedModuleNames.Add("WindowsTargetPlatformSettings");
					Rules.DynamicallyLoadedModuleNames.Add("WindowsTargetPlatformControls");
				}

				if (bBuildShaderFormats)
				{
					Rules.DynamicallyLoadedModuleNames.Add("ShaderFormatD3D");
					Rules.DynamicallyLoadedModuleNames.Add("ShaderFormatOpenGL");
					Rules.DynamicallyLoadedModuleNames.Add("ShaderFormatVectorVM");

					Rules.DynamicallyLoadedModuleNames.Remove("VulkanRHI");
					Rules.DynamicallyLoadedModuleNames.Add("VulkanShaderFormat");
				}
			}

			// Delay-load D3D12 so we can use the latest features and still run on downlevel versions of the OS
			Rules.PublicDelayLoadDLLs.Add("d3d12.dll");
		}

		/// <summary>
		/// Setup the target environment for building
		/// </summary>
		/// <param name="Target">Settings for the target being compiled</param>
		/// <param name="CompileEnvironment">The compile environment for this target</param>
		/// <param name="LinkEnvironment">The link environment for this target</param>
		public override void SetUpEnvironment(ReadOnlyTargetRules Target, CppCompileEnvironment CompileEnvironment, LinkEnvironment LinkEnvironment)
		{
			if (Target.WindowsPlatform.bEnableInstrumentation) 
			{
				CompileEnvironment.Definitions.Add("USING_INSTRUMENTATION=1");
			}

			// @todo Remove this hack to work around broken includes
			CompileEnvironment.Definitions.Add("NDIS_MINIPORT_MAJOR_VERSION=0");

			CompileEnvironment.Definitions.Add("WIN32=1");

			CompileEnvironment.Definitions.Add(String.Format("_WIN32_WINNT=0x{0:X4}", Target.WindowsPlatform.TargetWindowsVersion));
			CompileEnvironment.Definitions.Add(String.Format("WINVER=0x{0:X4}", Target.WindowsPlatform.TargetWindowsVersion));
			CompileEnvironment.Definitions.Add(String.Format("NTDDI_VERSION=0x{0:X8}", Target.WindowsPlatform.TargetWindowsMinorVersion));

			CompileEnvironment.Definitions.Add("PLATFORM_WINDOWS=1");
			CompileEnvironment.Definitions.Add("PLATFORM_MICROSOFT=1");
			CompileEnvironment.Definitions.Add("UE_PLATFORM_WINDOWS=1");
			CompileEnvironment.Definitions.Add("UE_PLATFORM_MICROSOFT=1");

			string? OverridePlatformHeaderName = GetOverridePlatformHeaderName();
			if (!String.IsNullOrEmpty(OverridePlatformHeaderName))
			{
				CompileEnvironment.Definitions.Add(String.Format("OVERRIDE_PLATFORM_HEADER_NAME={0}", OverridePlatformHeaderName));
			}

			if (Target.IsInPlatformGroup(UnrealPlatformGroup.Windows) && Target.WindowsPlatform.RayTracingMode != WindowsRayTracingRuntimeMode.Disabled && Target.Type != TargetType.Server)
			{
				CompileEnvironment.Definitions.Add("RHI_RAYTRACING=1");
			}

			// Explicitly exclude the MS C++ runtime libraries we're not using, to ensure other libraries we link with use the same
			// runtime library as the engine.
			bool bUseDebugCRT = Target.Configuration == UnrealTargetConfiguration.Debug && Target.bDebugBuildsActuallyUseDebugCRT;
			if (!Target.bUseStaticCRT || bUseDebugCRT)
			{
				LinkEnvironment.ExcludedLibraries.Add("LIBCMT");
				LinkEnvironment.ExcludedLibraries.Add("LIBCPMT");
			}
			if (!Target.bUseStaticCRT || !bUseDebugCRT)
			{
				LinkEnvironment.ExcludedLibraries.Add("LIBCMTD");
				LinkEnvironment.ExcludedLibraries.Add("LIBCPMTD");
			}
			if (Target.bUseStaticCRT || bUseDebugCRT)
			{
				LinkEnvironment.ExcludedLibraries.Add("MSVCRT");
				LinkEnvironment.ExcludedLibraries.Add("MSVCPRT");
			}
			if (Target.bUseStaticCRT || !bUseDebugCRT)
			{
				LinkEnvironment.ExcludedLibraries.Add("MSVCRTD");
				LinkEnvironment.ExcludedLibraries.Add("MSVCPRTD");
			}
			LinkEnvironment.ExcludedLibraries.Add("LIBC");
			LinkEnvironment.ExcludedLibraries.Add("LIBCP");
			LinkEnvironment.ExcludedLibraries.Add("LIBCD");
			LinkEnvironment.ExcludedLibraries.Add("LIBCPD");

			//@todo ATL: Currently, only VSAccessor requires ATL (which is only used in editor builds)
			// When compiling games, we do not want to include ATL - and we can't when compiling games
			// made with Launcher build due to VS 2012 Express not including ATL.
			// If more modules end up requiring ATL, this should be refactored into a BuildTarget flag (bNeedsATL)
			// that is set by the modules the target includes to allow for easier tracking.
			// Alternatively, if VSAccessor is modified to not require ATL than we should always exclude the libraries.
			if (Target.LinkType == TargetLinkType.Monolithic &&
				(Target.Type == TargetType.Game || Target.Type == TargetType.Client || Target.Type == TargetType.Server))
			{
				LinkEnvironment.ExcludedLibraries.Add("atl");
				LinkEnvironment.ExcludedLibraries.Add("atls");
				LinkEnvironment.ExcludedLibraries.Add("atlsd");
				LinkEnvironment.ExcludedLibraries.Add("atlsn");
				LinkEnvironment.ExcludedLibraries.Add("atlsnd");
			}

			// Add the library used for the delayed loading of DLLs.
			LinkEnvironment.SystemLibraries.Add("delayimp.lib");

			//@todo - remove once FB implementation uses Http module
			if (Target.bCompileAgainstEngine)
			{
				// link against wininet (used by FBX and Facebook)
				LinkEnvironment.SystemLibraries.Add("wininet.lib");
			}

			// Compile and link with Win32 API libraries.
			LinkEnvironment.SystemLibraries.Add("rpcrt4.lib");
			//LinkEnvironment.AdditionalLibraries.Add("wsock32.lib");
			LinkEnvironment.SystemLibraries.Add("ws2_32.lib");
			LinkEnvironment.SystemLibraries.Add("dbghelp.lib");
			LinkEnvironment.SystemLibraries.Add("comctl32.lib");
			LinkEnvironment.SystemLibraries.Add("Winmm.lib");
			LinkEnvironment.SystemLibraries.Add("kernel32.lib");
			LinkEnvironment.SystemLibraries.Add("user32.lib");
			LinkEnvironment.SystemLibraries.Add("gdi32.lib");
			LinkEnvironment.SystemLibraries.Add("winspool.lib");
			LinkEnvironment.SystemLibraries.Add("comdlg32.lib");
			LinkEnvironment.SystemLibraries.Add("advapi32.lib");
			LinkEnvironment.SystemLibraries.Add("shell32.lib");
			LinkEnvironment.SystemLibraries.Add("ole32.lib");
			LinkEnvironment.SystemLibraries.Add("oleaut32.lib");
			LinkEnvironment.SystemLibraries.Add("uuid.lib");
			LinkEnvironment.SystemLibraries.Add("odbc32.lib");
			LinkEnvironment.SystemLibraries.Add("odbccp32.lib");
			LinkEnvironment.SystemLibraries.Add("netapi32.lib");
			LinkEnvironment.SystemLibraries.Add("iphlpapi.lib");
			LinkEnvironment.SystemLibraries.Add("setupapi.lib"); //  Required for access monitor device enumeration
			LinkEnvironment.SystemLibraries.Add("synchronization.lib"); // Required for WaitOnAddress and WakeByAddressSingle

			// Windows 7 Desktop Windows Manager API for Slate Windows Compliance
			LinkEnvironment.SystemLibraries.Add("dwmapi.lib");

			// IME
			LinkEnvironment.SystemLibraries.Add("imm32.lib");

			// For 64-bit builds, we'll forcibly ignore a linker warning with DirectInput.  This is
			// Microsoft's recommended solution as they don't have a fixed .lib for us.
			LinkEnvironment.AdditionalArguments += " /ignore:4078";

			// Set up default stack size
			LinkEnvironment.DefaultStackSize = Target.WindowsPlatform.DefaultStackSize;
			LinkEnvironment.DefaultStackSizeCommit = Target.WindowsPlatform.DefaultStackSizeCommit;

			LinkEnvironment.ModuleDefinitionFile = Target.WindowsPlatform.ModuleDefinitionFile;

			if (Target.bPGOOptimize || Target.bPGOProfile)
			{
				// Win64 PGO folder is Windows, the rest match the platform name
				string PGOPlatform = Target.Platform == UnrealTargetPlatform.Win64 ? "Windows" : Target.Platform.ToString();

				CompileEnvironment.PGODirectory = DirectoryReference.Combine(Target.ProjectFile?.Directory ?? Unreal.WritableEngineDirectory, "Platforms", PGOPlatform, "Build", "PGO").FullName;
				CompileEnvironment.PGOFilenamePrefix = String.Format("{0}-{1}-{2}", Target.Name, Target.Platform, Target.Configuration);

				LinkEnvironment.PGODirectory = CompileEnvironment.PGODirectory;
				LinkEnvironment.PGOFilenamePrefix = CompileEnvironment.PGOFilenamePrefix;
				LinkEnvironment.PGOMergedFilenamePrefix = Target.Platform == UnrealTargetPlatform.Win64 ? Target.WindowsPlatform.PreMergedPgdFilename : null;
			}

			CompileEnvironment.Definitions.Add("WINDOWS_MAX_NUM_TLS_SLOTS=" + Target.WindowsPlatform.MaxNumTlsSlots.ToString());
			CompileEnvironment.Definitions.Add("WINDOWS_MAX_NUM_THREADS_WITH_TLS_SLOTS=" + Target.WindowsPlatform.MaxNumThreadsWithTlsSlots.ToString());

			// Validate PGO data
			if (CompileEnvironment.bPGOOptimize && CompileEnvironment.PGODirectory != null && Target.WindowsPlatform.Compiler.IsClang() && Target.WindowsPlatform.bIgnoreStalePGOData)
			{
				string CompilerVersionFilename = Path.Combine(CompileEnvironment.PGODirectory!, "PGOProfileCompilerInfo.txt");
				if (File.Exists(CompilerVersionFilename) && Target.WindowsPlatform.Environment != null)
				{
					string[] ProfileCompilerVersionLines = File.ReadAllLines(CompilerVersionFilename);
					if (ProfileCompilerVersionLines.Length >= 2 & Enum.TryParse(ProfileCompilerVersionLines[0].Trim(), false, out WindowsCompiler ProfileCompiler) && VersionNumber.TryParse(ProfileCompilerVersionLines[1].Trim(), out VersionNumber? ProfileVersion))
					{
						VersionNumber CompilerVersion = Target.WindowsPlatform.Compiler.IsIntel() ? MicrosoftPlatformSDK.GetClangVersionForIntelCompiler(Target.WindowsPlatform.Environment.CompilerPath) : Target.WindowsPlatform.Environment.CompilerVersion;
						if (ProfileCompiler.IsClang() && ProfileVersion.Components[0] > CompilerVersion.Components[0])
						{
							Logger.LogWarning("PGO Profile major LLVM version is newer than PGO Optimize LLVM major version. Profile Guided Optimization will be disabled for {Config} until new PGO Profile data is generated.", Target.Configuration.ToString());
							Logger.LogInformation("Profile: {ProfileCompiler} ({ProfileVersion}) Optimize: {OptimizeCompiler} ({OptimizeVersion})", ProfileCompiler, ProfileVersion, Target.WindowsPlatform.Compiler, CompilerVersion);
							CompileEnvironment.bPGOOptimize = false;
							LinkEnvironment.bPGOOptimize = false;
							CompileEnvironment.bAllowLTCG = true;
							LinkEnvironment.bAllowLTCG = true;
						}
					}
				}
			}
		}

		/// <summary>
		/// Setup the configuration environment for building
		/// </summary>
		/// <param name="Target"> The target being built</param>
		/// <param name="GlobalCompileEnvironment">The global compile environment</param>
		/// <param name="GlobalLinkEnvironment">The global link environment</param>
		public override void SetUpConfigurationEnvironment(ReadOnlyTargetRules Target, CppCompileEnvironment GlobalCompileEnvironment, LinkEnvironment GlobalLinkEnvironment)
		{
			base.SetUpConfigurationEnvironment(Target, GlobalCompileEnvironment, GlobalLinkEnvironment);

			// NOTE: Even when debug info is turned off, we currently force the linker to generate debug info
			//       anyway on Visual C++ platforms.  This will cause a PDB file to be generated with symbols
			//       for most of the classes and function/method names, so that crashes still yield somewhat
			//       useful call stacks, even though compiler-generate debug info may be disabled.  This gives
			//       us much of the build-time savings of fully-disabled debug info, without giving up call
			//       data completely.
			if (!Target.WindowsPlatform.bNoLinkerDebugInfo)
			{
				GlobalLinkEnvironment.bCreateDebugInfo = true;
			}
		}

		/// <summary>
		/// Whether this platform should create debug information or not
		/// </summary>
		/// <param name="Target">The target being built</param>
		/// <returns>bool    true if debug info should be generated, false if not</returns>
		public override bool ShouldCreateDebugInfo(ReadOnlyTargetRules Target)
		{
			switch (Target.Configuration)
			{
				case UnrealTargetConfiguration.Development:
				case UnrealTargetConfiguration.Shipping:
				case UnrealTargetConfiguration.Test:
					return !Target.bOmitPCDebugInfoInDevelopment;
				case UnrealTargetConfiguration.DebugGame:
				case UnrealTargetConfiguration.Debug:
				default:
					return true;
			}
		}

		/// <summary>
		/// Creates a toolchain instance for the given platform.
		/// </summary>
		/// <param name="Target">The target being built</param>
		/// <returns>New toolchain instance.</returns>
		public override UEToolChain CreateToolChain(ReadOnlyTargetRules Target)
		{
			VCToolChain toolchain = new VCToolChain(Target, Logger);
			if (Target.StaticAnalyzer == StaticAnalyzer.PVSStudio)
			{
				return new PVSToolChain(Target, toolchain, Logger);
			}
			else if (Target.bIWYU)
			{
				return new IWYUToolChain(Target, toolchain, Logger);
			}
				
			return toolchain;
		}

		/// <summary>
		/// Allows the platform to return various build metadata that is not tracked by other means. If the returned string changes, the makefile will be invalidated.
		/// </summary>
		/// <param name="ProjectFile">The project file being built</param>
		/// <param name="Metadata">String builder to contain build metadata</param>
		public override void GetExternalBuildMetadata(FileReference? ProjectFile, StringBuilder Metadata)
		{
			base.GetExternalBuildMetadata(ProjectFile, Metadata);

			if (ProjectFile != null)
			{
				Metadata.AppendLine("ICON: {0}", GetApplicationIcon(ProjectFile));
			}
		}

		/// <summary>
		/// Deploys the given target
		/// </summary>
		/// <param name="Receipt">Receipt for the target being deployed</param>
		public override void Deploy(TargetReceipt Receipt)
		{
			new UEDeployWindows(Logger).PrepTargetForDeployment(Receipt);
		}

		/// <summary>
		/// Allows the platform header name to be overridden to differ from the platform name.
		/// </summary>
		protected virtual string? GetOverridePlatformHeaderName()
		{
			// Both Win32 and Win64 use Windows headers, so we enforce that here.
			return GetPlatformName();
		}

		/// <summary>
		/// SN-DBS tool templates for Windows
		/// </summary>
		private static readonly IReadOnlyDictionary<string, string> SNDBSToolTemplates = new Dictionary<string, string>()
		{
			["cl-filter.exe"] = @"
[tool]
family=msvc
vc_major_version=14
use_surrogate=true
force_synchronous_pdb_writes=true
error_report_mode=prompt
response_file_content_pattern=\s--\s"".*?cl\.exe""\s(.*)

[group]
server={VC_TOOLCHAIN_DIR}\mspdbsrv.exe

[files]
main=cl-filter.exe
file01={VC_TOOLCHAIN_DIR}\c1.dll
file02={VC_TOOLCHAIN_DIR}\c1xx.dll
file03={VC_TOOLCHAIN_DIR}\c2.dll
file04={VC_TOOLCHAIN_DIR}\mspdb140.dll
file05={VC_TOOLCHAIN_DIR}\mspdbcore.dll
file06={VC_TOOLCHAIN_DIR}\mspdbsrv.exe
file07={VC_TOOLCHAIN_DIR}\mspft140.dll
file08={VC_TOOLCHAIN_DIR}\vcmeta.dll
file09={VC_TOOLCHAIN_DIR}\*\clui.dll
file10={VC_TOOLCHAIN_DIR}\*\mspft140ui.dll
file11={VC_TOOLCHAIN_DIR}\localespc.dll
file12={VC_TOOLCHAIN_DIR}\cppcorecheck.dll
file13={VC_TOOLCHAIN_DIR}\experimentalcppcorecheck.dll
file14={VC_TOOLCHAIN_DIR}\espxengine.dll
file15={VC_TOOLCHAIN_DIR}\c1.exe

[output-file-patterns]
outputfile01=\s*""([^ "",]+\.cpp\.txt)\""

[output-file-rules]
rule01=*\sqmcpp*.log|discard=true
rule02=*\vctoolstelemetry*.dat|discard=true
rule03=*\Microsoft\Windows\Temporary Internet Files\*|discard=true
rule04=*\Microsoft\Windows\INetCache\*|discard=true

[input-file-rules]
rule01=*\sqmcpp*.log|ignore_transient_errors=true;ignore_unexpected_input=true
rule02=*\vctoolstelemetry*.dat|ignore_transient_errors=true;ignore_unexpected_input=true
rule03=*\Microsoft\Windows\Temporary Internet Files\*|ignore_transient_errors=true;ignore_unexpected_input=true
rule04=*\Microsoft\Windows\INetCache\*|ignore_transient_errors=true;ignore_unexpected_input=true

[system-file-filters]
filter01=msvcr*.dll
filter02=msvcp*.dll
filter03=vcruntime140*.dll
filter04=appcrt140*.dll
filter05=desktopcrt140*.dll
filter06=concrt140*.dll",
			["cl.exe"] = @"
[tool]
family=msvc
vc_major_version=14
use_surrogate=true
force_synchronous_pdb_writes=true
error_report_mode=prompt

[group]
server={VC_TOOLCHAIN_DIR}\mspdbsrv.exe

[files]
main={VC_TOOLCHAIN_DIR}\cl.exe
file01={VC_TOOLCHAIN_DIR}\c1.dll
file02={VC_TOOLCHAIN_DIR}\c1xx.dll
file03={VC_TOOLCHAIN_DIR}\c2.dll
file04={VC_TOOLCHAIN_DIR}\mspdb140.dll
file05={VC_TOOLCHAIN_DIR}\mspdbcore.dll
file06={VC_TOOLCHAIN_DIR}\mspdbsrv.exe
file07={VC_TOOLCHAIN_DIR}\mspft140.dll
file08={VC_TOOLCHAIN_DIR}\vcmeta.dll
file09={VC_TOOLCHAIN_DIR}\*\clui.dll
file10={VC_TOOLCHAIN_DIR}\*\mspft140ui.dll
file11={VC_TOOLCHAIN_DIR}\localespc.dll
file12={VC_TOOLCHAIN_DIR}\cppcorecheck.dll
file13={VC_TOOLCHAIN_DIR}\experimentalcppcorecheck.dll
file14={VC_TOOLCHAIN_DIR}\espxengine.dll

[output-file-patterns]
outputfile01=\s*""([^ "",]+\.cpp\.txt\.json)\""

[output-file-rules]
rule01=*\sqmcpp*.log|discard=true
rule02=*\vctoolstelemetry*.dat|discard=true
rule03=*\Microsoft\Windows\Temporary Internet Files\*|discard=true
rule04=*\Microsoft\Windows\INetCache\*|discard=true

[input-file-rules]
rule01=*\sqmcpp*.log|ignore_transient_errors=true;ignore_unexpected_input=true
rule02=*\vctoolstelemetry*.dat|ignore_transient_errors=true;ignore_unexpected_input=true
rule03=*\Microsoft\Windows\Temporary Internet Files\*|ignore_transient_errors=true;ignore_unexpected_input=true
rule04=*\Microsoft\Windows\INetCache\*|ignore_transient_errors=true;ignore_unexpected_input=true

[system-file-filters]
filter01=msvcr*.dll
filter02=msvcp*.dll
filter03=vcruntime140*.dll
filter04=appcrt140*.dll
filter05=desktopcrt140*.dll
filter06=concrt140*.dll",
			["mspdbsrv.exe"] = @"
[tool]
use_cache=no

[files]
main={VC_TOOLCHAIN_DIR}\mspdbsrv.exe

[openmp]
omp=true
omp_min_threads=1",
		};
	}

	class UEDeployWindows : UEBuildDeploy
	{
		public UEDeployWindows(ILogger InLogger)
			: base(InLogger)
		{
		}

		public override bool PrepTargetForDeployment(TargetReceipt Receipt)
		{
			return base.PrepTargetForDeployment(Receipt);
		}
	}

	class WindowsPlatformFactory : UEBuildPlatformFactory
	{
		public override UnrealTargetPlatform TargetPlatform => UnrealTargetPlatform.Win64;

		/// <summary>
		/// Register the platform with the UEBuildPlatform class
		/// </summary>
		public override void RegisterBuildPlatforms(ILogger Logger)
		{
			MicrosoftPlatformSDK SDK = new MicrosoftPlatformSDK(Logger);

			// Register this build platform for Win64 (no more Win32)
			UEBuildPlatform.RegisterBuildPlatform(new WindowsPlatform(UnrealTargetPlatform.Win64, SDK, Logger), Logger);
			UEBuildPlatform.RegisterPlatformWithGroup(UnrealTargetPlatform.Win64, UnrealPlatformGroup.Windows);
			UEBuildPlatform.RegisterPlatformWithGroup(UnrealTargetPlatform.Win64, UnrealPlatformGroup.Microsoft);
			UEBuildPlatform.RegisterPlatformWithGroup(UnrealTargetPlatform.Win64, UnrealPlatformGroup.Desktop);
		}
	}
}
