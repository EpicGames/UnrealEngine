// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Collections.Immutable;
using EpicGames.Core;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// Interface for an action that compiles C++ source code
	/// </summary>
	interface ICppCompileAction : IExternalAction
	{
		/// <summary>
		/// Path to the compiled module interface file
		/// </summary>
		FileItem? CompiledModuleInterfaceFile { get; }
	}

	/// <summary>
	/// Serializer which creates a portable object file and allows caching it
	/// </summary>
	class VCCompileAction : ICppCompileAction
	{
		/// <summary>
		/// The action type
		/// </summary>
		public ActionType ActionType { get; set; } = ActionType.Compile;

		/// <summary>
		/// Artifact support for this step
		/// </summary>
		public ArtifactMode ArtifactMode { get; set; } = ArtifactMode.None;

		/// <summary>
		/// Path to the compiler
		/// </summary>
		public FileItem CompilerExe { get; }

		/// <summary>
		/// The type of compiler being used
		/// </summary>
		public WindowsCompiler CompilerType { get; }

		/// <summary>
		/// The version of the compiler being used
		/// </summary>
		public string CompilerVersion { get; set; }

		/// <summary>
		/// The version of the toolchain being used (if the compiler is not MSVC)
		/// </summary>
		public string ToolChainVersion { get; set; }

		/// <summary>
		/// Source file to compile
		/// </summary>
		public FileItem? SourceFile { get; set; }

		/// <summary>
		/// The object file to output
		/// </summary>
		public FileItem? ObjectFile { get; set; }

		/// <summary>
		/// The assembly file to output
		/// </summary>
		public FileItem? AssemblyFile { get; set; }

		/// <summary>
		/// The output preprocessed file
		/// </summary>
		public FileItem? PreprocessedFile { get; set; }

		/// <summary>
		/// The output analyze warning and error log file
		/// </summary>
		public FileItem? AnalyzeLogFile { get; set; }

		/// <summary>
		/// The output experimental warning and error log file
		/// </summary>
		public FileItem? ExperimentalLogFile { get; set; }

		/// <summary>
		/// The dependency list file
		/// </summary>
		public FileItem? DependencyListFile { get; set; }

		/// <summary>
		/// Compiled module interface
		/// </summary>
		public FileItem? CompiledModuleInterfaceFile { get; set; }

		/// <summary>
		/// For C++ source files, specifies a timing file used to track timing information.
		/// </summary>
		public FileItem? TimingFile { get; set; }

		/// <summary>
		/// Response file for the compiler
		/// </summary>
		public FileItem? ResponseFile { get; set; }

		/// <summary>
		/// The precompiled header file
		/// </summary>
		public FileItem? CreatePchFile { get; set; }

		/// <summary>
		/// The precompiled header file
		/// </summary>
		public FileItem? UsingPchFile { get; set; }

		/// <summary>
		/// The header which matches the PCH
		/// </summary>
		public FileItem? PchThroughHeaderFile { get; set; }

		/// <summary>
		/// List of additional response paths
		/// </summary>
		public List<FileItem> AdditionalResponseFiles { get; } = [];

		/// <summary>
		/// List of include paths
		/// </summary>
		public List<DirectoryReference> IncludePaths { get; } = [];

		/// <summary>
		/// List of system include paths
		/// </summary>
		public List<DirectoryReference> SystemIncludePaths { get; } = [];

		/// <summary>
		/// List of macro definitions
		/// </summary>
		public List<string> Definitions { get; } = [];

		/// <summary>
		/// List of force included files
		/// </summary>
		public List<FileItem> ForceIncludeFiles = [];

		/// <summary>
		/// Every file this action depends on.  These files need to exist and be up to date in order for this action to even be considered
		/// </summary>
		public List<FileItem> AdditionalPrerequisiteItems { get; } = [];

		/// <summary>
		/// The files that this action produces after completing
		/// </summary>
		public List<FileItem> AdditionalProducedItems { get; } = [];

		/// <summary>
		/// Arguments to pass to the compiler
		/// </summary>
		public List<string> Arguments { get; } = [];

		/// <summary>
		/// Whether to show included files to the console
		/// </summary>
		public bool bShowIncludes { get; set; }

		/// <summary>
		/// Whether to override the normal logic for UsingClFilter and force it on.
		/// </summary>
		public bool ForceClFilter = false;

		/// <summary>
		/// Architecture this is compiling for (used for display)
		/// </summary>
		public UnrealArch Architecture { get; set; }

		/// <summary>
		/// Whether this compile is static code analysis (used for display)
		/// </summary>
		public bool bIsAnalyzing { get; set; }

		public bool bWarningsAsErrors { get; set; }

		#region Public IAction implementation

		/// <summary>
		/// Items that should be deleted before running this action
		/// </summary>
		public List<FileItem> DeleteItems { get; } = [];

		/// <summary>
		/// Root paths for this action (generally engine root project root, toolchain root, sdk root)
		/// </summary>
		public CppRootPaths RootPaths { get; set; }

		/// <inheritdoc/>
		public bool bCanExecuteRemotely { get; set; }

		/// <inheritdoc/>
		public bool bCanExecuteRemotelyWithSNDBS { get; set; }

		/// <inheritdoc/>
		public bool bCanExecuteRemotelyWithXGE { get; set; } = true;

		/// <inheritdoc/>
		public bool bCanExecuteInUBA { get; set; } = true;

		/// <inheritdoc/>
		public bool bCanExecuteInUBACrossArchitecture { get; set; } = true;

		/// <inheritdoc/>
		public bool bUseActionHistory => true;

		/// <inheritdoc/>
		public bool bIsHighPriority => CreatePchFile != null;

		/// <inheritdoc/>
		public double Weight { get; set; } = 1.0;

		/// <inheritdoc/>
		public uint CacheBucket { get; set; }

		/// <inheritdoc/>
		public bool bShouldOutputLog { get; set; } = true;
		#endregion

		#region Implementation of IAction

		IEnumerable<FileItem> IExternalAction.DeleteItems => DeleteItems;
		public DirectoryReference WorkingDirectory => Unreal.EngineSourceDirectory;
		string IExternalAction.CommandDescription
		{
			get
			{
				if (PreprocessedFile != null)
				{
					return $"Preprocess [{Architecture}]";
				}
				else if (bIsAnalyzing)
				{
					return $"Analyze [{Architecture}]";
				}
				return $"Compile [{Architecture}]";
			}
		}
		bool IExternalAction.bIsClangCompiler => false;
		bool IExternalAction.bDeleteProducedItemsOnError => CompilerType.IsClang() && bIsAnalyzing && bWarningsAsErrors;
		bool IExternalAction.bForceWarningsAsError => CompilerType.IsClang() && bIsAnalyzing && bWarningsAsErrors;
		bool IExternalAction.bProducesImportLibrary => false;
		string IExternalAction.StatusDescription => SourceFile?.Location.GetFileName() ?? "Compiling";
		bool IExternalAction.bShouldOutputStatusDescription => CompilerType.IsClang();

		/// <inheritdoc/>
		IEnumerable<FileItem> IExternalAction.PrerequisiteItems
		{
			get
			{
				if (ResponseFile != null)
				{
					yield return ResponseFile;
				}
				if (SourceFile != null)
				{
					yield return SourceFile;
				}
				if (UsingPchFile != null)
				{
					yield return UsingPchFile;
				}
				foreach (FileItem additionalResponseFile in AdditionalResponseFiles)
				{
					yield return additionalResponseFile;
				}
				foreach (FileItem forceIncludeFile in ForceIncludeFiles)
				{
					yield return forceIncludeFile;
				}
				foreach (FileItem additionalPrerequisiteItem in AdditionalPrerequisiteItems)
				{
					yield return additionalPrerequisiteItem;
				}
			}
		}

		/// <inheritdoc/>
		IEnumerable<FileItem> IExternalAction.ProducedItems
		{
			get
			{
				if (ObjectFile != null)
				{
					yield return ObjectFile;
				}
				if (AssemblyFile != null)
				{
					yield return AssemblyFile;
				}
				if (PreprocessedFile != null)
				{
					yield return PreprocessedFile;
				}
				if (AnalyzeLogFile != null)
				{
					yield return AnalyzeLogFile;
				}
				if (ExperimentalLogFile != null)
				{
					yield return ExperimentalLogFile;
				}
				if (DependencyListFile != null)
				{
					yield return DependencyListFile;
				}
				if (TimingFile != null)
				{
					yield return TimingFile;
				}
				if (CreatePchFile != null)
				{
					yield return CreatePchFile;
				}
				foreach (FileItem additionalProducedItem in AdditionalProducedItems)
				{
					yield return additionalProducedItem;
				}
			}
		}

		/// <summary>
		/// Whether to use cl-filter
		/// </summary>
		bool UsingClFilter => ForceClFilter || (DependencyListFile != null && !DependencyListFile.HasExtension(".json") && !DependencyListFile.HasExtension(".d"));

		/// <inheritdoc/>
		FileReference IExternalAction.CommandPath =>
			UsingClFilter
				? FileReference.Combine(Unreal.EngineDirectory, "Build", "Windows", "cl-filter", "cl-filter.exe")
				: CompilerExe.Location;

		/// <inheritdoc/>
		string IExternalAction.CommandArguments => UsingClFilter ? GetClFilterArguments() : GetClArguments();

		/// <inheritdoc/>
		ImmutableArray<string> IExternalAction.ResponseFileContents => GetResponseFileContents();

		/// <inheritdoc/>
		string IExternalAction.CommandVersion =>
			CompilerType.IsMSVC()
				? CompilerVersion
				: $"{CompilerType} {CompilerVersion} MSVC {ToolChainVersion}";

		#endregion

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="environment">Compiler executable</param>
		public VCCompileAction(VCEnvironment environment)
		{
			CompilerExe = FileItem.GetItemByFileReference(environment.CompilerPath);
			CompilerType = environment.Compiler;
			CompilerVersion = environment.CompilerVersion.ToString();
			ToolChainVersion = environment.ToolChainVersion.ToString();

			RootPaths = new();
		}

		/// <summary>
		/// Copy constructor
		/// </summary>
		/// <param name="inAction">Action to copy from</param>
		public VCCompileAction(VCCompileAction inAction)
		{
			ActionType = inAction.ActionType;
			ArtifactMode = inAction.ArtifactMode;
			CompilerExe = inAction.CompilerExe;
			CompilerType = inAction.CompilerType;
			CompilerVersion = inAction.CompilerVersion;
			ToolChainVersion = inAction.ToolChainVersion;
			SourceFile = inAction.SourceFile;
			ObjectFile = inAction.ObjectFile;
			AssemblyFile = inAction.AssemblyFile;
			PreprocessedFile = inAction.PreprocessedFile;
			AnalyzeLogFile = inAction.AnalyzeLogFile;
			ExperimentalLogFile = inAction.ExperimentalLogFile;
			DependencyListFile = inAction.DependencyListFile;
			CompiledModuleInterfaceFile = inAction.CompiledModuleInterfaceFile;
			TimingFile = inAction.TimingFile;
			ResponseFile = inAction.ResponseFile;
			CreatePchFile = inAction.CreatePchFile;
			UsingPchFile = inAction.UsingPchFile;
			PchThroughHeaderFile = inAction.PchThroughHeaderFile;
			AdditionalResponseFiles = [.. inAction.AdditionalResponseFiles];
			IncludePaths = [.. inAction.IncludePaths];
			SystemIncludePaths = [.. inAction.SystemIncludePaths];
			Definitions = [.. inAction.Definitions];
			ForceIncludeFiles = [.. inAction.ForceIncludeFiles];
			Arguments = [.. inAction.Arguments];
			bShowIncludes = inAction.bShowIncludes;
			bCanExecuteRemotely = inAction.bCanExecuteRemotely;
			bCanExecuteRemotelyWithSNDBS = inAction.bCanExecuteRemotelyWithSNDBS;
			bCanExecuteRemotelyWithXGE = inAction.bCanExecuteRemotelyWithXGE;
			Architecture = inAction.Architecture;
			bIsAnalyzing = inAction.bIsAnalyzing;
			bWarningsAsErrors = inAction.bWarningsAsErrors;
			Weight = inAction.Weight;
			CacheBucket = inAction.CacheBucket;

			AdditionalPrerequisiteItems = [.. inAction.AdditionalPrerequisiteItems];
			AdditionalProducedItems = [.. inAction.AdditionalProducedItems];
			DeleteItems = [.. inAction.DeleteItems];
			RootPaths = new(inAction.RootPaths);
		}

		/// <summary>
		/// Serialize a cache handler from an archive
		/// </summary>
		/// <param name="reader">Reader to serialize from</param>
		public VCCompileAction(BinaryArchiveReader reader)
		{
			ActionType = (ActionType)reader.ReadInt();
			ArtifactMode = (ArtifactMode)reader.ReadByte();
			CompilerExe = reader.ReadFileItem()!;
			CompilerType = (WindowsCompiler)reader.ReadInt();
			CompilerVersion = reader.ReadString()!;
			ToolChainVersion = reader.ReadString()!;
			SourceFile = reader.ReadFileItem();
			ObjectFile = reader.ReadFileItem();
			AssemblyFile = reader.ReadFileItem();
			PreprocessedFile = reader.ReadFileItem();
			AnalyzeLogFile = reader.ReadFileItem();
			ExperimentalLogFile = reader.ReadFileItem();
			DependencyListFile = reader.ReadFileItem();
			CompiledModuleInterfaceFile = reader.ReadFileItem();
			TimingFile = reader.ReadFileItem();
			ResponseFile = reader.ReadFileItem();
			CreatePchFile = reader.ReadFileItem();
			UsingPchFile = reader.ReadFileItem();
			PchThroughHeaderFile = reader.ReadFileItem();
			AdditionalResponseFiles = reader.ReadList(reader.ReadFileItem)!;
			IncludePaths = reader.ReadList(reader.ReadCompactDirectoryReference)!;
			SystemIncludePaths = reader.ReadList(reader.ReadCompactDirectoryReference)!;
			Definitions = reader.ReadList(reader.ReadString)!;
			ForceIncludeFiles = reader.ReadList(reader.ReadFileItem)!;
			Arguments = reader.ReadList(reader.ReadString)!;
			bShowIncludes = reader.ReadBool();
			bCanExecuteRemotely = reader.ReadBool();
			bCanExecuteRemotelyWithSNDBS = reader.ReadBool();
			bCanExecuteRemotelyWithXGE = reader.ReadBool();
			bCanExecuteInUBA = reader.ReadBool();
			bCanExecuteInUBACrossArchitecture = reader.ReadBool();
			Architecture = UnrealArch.Parse(reader.ReadString()!);
			bIsAnalyzing = reader.ReadBool();
			bWarningsAsErrors = reader.ReadBool();
			Weight = reader.ReadDouble();
			CacheBucket = reader.ReadUnsignedInt();

			AdditionalPrerequisiteItems = reader.ReadList(reader.ReadFileItem)!;
			AdditionalProducedItems = reader.ReadList(reader.ReadFileItem)!;
			DeleteItems = reader.ReadList(reader.ReadFileItem)!;
			RootPaths = new(reader);
		}

		/// <inheritdoc/>
		public void Write(BinaryArchiveWriter writer)
		{
			writer.WriteInt((int)ActionType);
			writer.WriteByte((byte)ArtifactMode);
			writer.WriteFileItem(CompilerExe);
			writer.WriteInt((int)CompilerType);
			writer.WriteString(CompilerVersion);
			writer.WriteString(ToolChainVersion);
			writer.WriteFileItem(SourceFile);
			writer.WriteFileItem(ObjectFile);
			writer.WriteFileItem(AssemblyFile);
			writer.WriteFileItem(PreprocessedFile);
			writer.WriteFileItem(AnalyzeLogFile);
			writer.WriteFileItem(ExperimentalLogFile);
			writer.WriteFileItem(DependencyListFile);
			writer.WriteFileItem(CompiledModuleInterfaceFile);
			writer.WriteFileItem(TimingFile);
			writer.WriteFileItem(ResponseFile);
			writer.WriteFileItem(CreatePchFile);
			writer.WriteFileItem(UsingPchFile);
			writer.WriteFileItem(PchThroughHeaderFile);
			writer.WriteList(AdditionalResponseFiles, writer.WriteFileItem);
			writer.WriteList(IncludePaths, writer.WriteCompactDirectoryReference);
			writer.WriteList(SystemIncludePaths, writer.WriteCompactDirectoryReference);
			writer.WriteList(Definitions, writer.WriteString);
			writer.WriteList(ForceIncludeFiles, writer.WriteFileItem);
			writer.WriteList(Arguments, writer.WriteString);
			writer.WriteBool(bShowIncludes);
			writer.WriteBool(bCanExecuteRemotely);
			writer.WriteBool(bCanExecuteRemotelyWithSNDBS);
			writer.WriteBool(bCanExecuteRemotelyWithXGE);
			writer.WriteBool(bCanExecuteInUBA);
			writer.WriteBool(bCanExecuteInUBACrossArchitecture);
			writer.WriteString(Architecture.ToString());
			writer.WriteBool(bIsAnalyzing);
			writer.WriteBool(bWarningsAsErrors);
			writer.WriteDouble(Weight);
			writer.WriteUnsignedInt(CacheBucket);

			writer.WriteList(AdditionalPrerequisiteItems, writer.WriteFileItem);
			writer.WriteList(AdditionalProducedItems, writer.WriteFileItem);
			writer.WriteList(DeleteItems, writer.WriteFileItem);
			RootPaths.Write(writer);
		}

		/// <summary>
		/// Writes the response file with the action's arguments
		/// </summary>
		/// <param name="graph">The graph builder</param>
		public void WriteResponseFile(IActionGraphBuilder graph)
		{
			if (ResponseFile != null)
			{
				graph.CreateIntermediateTextFile(ResponseFile, GetCompilerArguments());
				Arguments.Clear();
			}
		}

		public List<string> GetCompilerArguments()
		{
			List<string> arguments = [];

			if (SourceFile != null)
			{
				VCToolChain.AddSourceFile(arguments, SourceFile, RootPaths);
			}

			foreach (FileItem additionalResponseFile in AdditionalResponseFiles)
			{
				VCToolChain.AddResponseFile(arguments, additionalResponseFile, RootPaths);
			}

			foreach (DirectoryReference includePath in IncludePaths)
			{
				VCToolChain.AddIncludePath(arguments, includePath, CompilerType, RootPaths);
			}

			foreach (DirectoryReference systemIncludePath in SystemIncludePaths)
			{
				VCToolChain.AddSystemIncludePath(arguments, systemIncludePath, CompilerType, RootPaths);
			}

			foreach (string definition in Definitions)
			{
				// Escape all quotation marks so that they get properly passed with the command line.
				string definitionArgument = definition.Contains('"', StringComparison.Ordinal) ? definition.Replace("\"", "\\\"", StringComparison.Ordinal) : definition;
				VCToolChain.AddDefinition(arguments, definitionArgument);
			}

			foreach (FileItem forceIncludeFile in ForceIncludeFiles)
			{
				VCToolChain.AddForceIncludeFile(arguments, forceIncludeFile, RootPaths);
			}

			if (CreatePchFile != null)
			{
				VCToolChain.AddCreatePchFile(arguments, PchThroughHeaderFile!, CreatePchFile, RootPaths);
			}

			if (UsingPchFile != null && CompilerType.IsMSVC())
			{
				VCToolChain.AddUsingPchFile(arguments, PchThroughHeaderFile!, UsingPchFile, RootPaths);
			}

			if (PreprocessedFile != null)
			{
				VCToolChain.AddPreprocessedFile(arguments, PreprocessedFile, RootPaths);
			}

			if (ObjectFile != null)
			{
				VCToolChain.AddObjectFile(arguments, ObjectFile, RootPaths);
			}

			if (AssemblyFile != null)
			{
				VCToolChain.AddAssemblyFile(arguments, AssemblyFile, RootPaths);
			}

			if (AnalyzeLogFile != null)
			{
				VCToolChain.AddAnalyzeLogFile(arguments, AnalyzeLogFile, RootPaths);
			}

			if (ExperimentalLogFile != null)
			{
				VCToolChain.AddExperimentalLogFile(arguments, ExperimentalLogFile, RootPaths);
			}

			// A better way to express this? .json is used as output for /sourceDependencies), but .md.json is used as output for /sourceDependencies:directives)
			if (DependencyListFile != null && DependencyListFile.HasExtension(".json") && !DependencyListFile.HasExtension(".md.json"))
			{
				VCToolChain.AddSourceDependenciesFile(arguments, DependencyListFile, RootPaths);
			}

			if (DependencyListFile != null && DependencyListFile.HasExtension(".d"))
			{
				VCToolChain.AddSourceDependsFile(arguments, DependencyListFile, RootPaths);
			}

			arguments.AddRange(Arguments);
			return arguments;
		}

		string GetClArguments()
		{
			if (ResponseFile == null)
			{
				return String.Join(' ', Arguments);
			}

			string responseFileString = VCToolChain.NormalizeCommandLinePath(ResponseFile, RootPaths);

			// cl.exe can't handle response files with a path longer than 260 characters, and relative paths can push it over the limit
			if (!System.IO.Path.IsPathRooted(responseFileString) && System.IO.Path.Combine(WorkingDirectory.FullName, responseFileString).Length > 260)
			{
				responseFileString = ResponseFile.FullName;
			}
			return $"@{Utils.MakePathSafeToUseWithCommandLine(responseFileString)} {String.Join(' ', Arguments)}";
		}

		private ImmutableArray<string> GetResponseFileContents()
		{
			if (ResponseFile != null)
			{
				return [.. GetCompilerArguments()];
			}
			return [];
		}

		string GetClFilterArguments()
		{
			List<string> arguments = [];
			string dependencyListFileString = VCToolChain.NormalizeCommandLinePath(DependencyListFile!, RootPaths);
			arguments.Add($"-dependencies={Utils.MakePathSafeToUseWithCommandLine(dependencyListFileString)}");

			if (TimingFile != null)
			{
				string timingFileString = VCToolChain.NormalizeCommandLinePath(TimingFile, RootPaths);
				arguments.Add($"-timing={Utils.MakePathSafeToUseWithCommandLine(timingFileString)}");
			}
			if (bShowIncludes)
			{
				arguments.Add("-showincludes");
			}

			arguments.Add($"-compiler={Utils.MakePathSafeToUseWithCommandLine(CompilerExe.AbsolutePath)}");
			arguments.Add("--");
			arguments.Add(Utils.MakePathSafeToUseWithCommandLine(CompilerExe.AbsolutePath));
			arguments.Add(GetClArguments());
			arguments.Add("/showIncludes");

			return String.Join(' ', arguments);
		}
	}

	/// <summary>
	/// Serializer for <see cref="VCCompileAction"/> instances
	/// </summary>
	class VCCompileActionSerializer : ActionSerializerBase<VCCompileAction>
	{
		/// <inheritdoc/>
		public override VCCompileAction Read(BinaryArchiveReader reader)
		{
			return new(reader);
		}

		/// <inheritdoc/>
		public override void Write(BinaryArchiveWriter writer, VCCompileAction action)
		{
			action.Write(writer);
		}
	}
}
