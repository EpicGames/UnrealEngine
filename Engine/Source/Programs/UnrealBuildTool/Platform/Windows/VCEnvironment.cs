// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Runtime.InteropServices;
using System.Runtime.Versioning;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// Stores information about a Visual C++ installation and compile environment
	/// </summary>
	class VCEnvironment
	{
		/// <summary>
		/// The compiler version
		/// </summary>
		public readonly WindowsCompiler Compiler;

		/// <summary>
		/// The compiler directory
		/// </summary>
		public readonly DirectoryReference CompilerDir;

		/// <summary>
		/// The compiler version
		/// </summary>
		public readonly VersionNumber CompilerVersion;

		/// <summary>
		/// The compiler Architecture
		/// </summary>
		public readonly UnrealArch Architecture;

		/// <summary>
		/// The underlying toolchain to use. Using Clang/ICL will piggy-back on a Visual Studio toolchain for the CRT, linker, etc...
		/// </summary>
		public readonly WindowsCompiler ToolChain;

		/// <summary>
		/// Root directory containing the toolchain
		/// </summary>
		public readonly DirectoryReference ToolChainDir;

		/// <summary>
		/// The toolchain version number
		/// </summary>
		public readonly VersionNumber ToolChainVersion;

		/// <summary>
		/// Root directory containing the Windows Sdk
		/// </summary>
		public readonly DirectoryReference WindowsSdkDir;

		/// <summary>
		/// Version number of the Windows Sdk
		/// </summary>
		public readonly VersionNumber WindowsSdkVersion;

		/// <summary>
		/// Use the CPP/WinRT language projection
		/// </summary>
		public readonly bool bUseCPPWinRT;

		/// <summary>
		/// Allow use of Clang linker
		/// </summary>
		public readonly bool bAllowClangLinker;

		/// <summary>
		/// Allow use of Rad linker
		/// </summary>
		public readonly bool bAllowRadLinker;

		/// <summary>
		/// The path to the compiler for compiling code
		/// </summary>
		public readonly FileReference CompilerPath;

		/// <summary>
		/// The path to the linker for linking executables
		/// </summary>
		public readonly FileReference LinkerPath;

		/// <summary>
		/// The path to the linker for linking libraries
		/// </summary>
		public readonly FileReference LibraryManagerPath;

		/// <summary>
		/// Path to the resource compiler from the Windows SDK
		/// </summary>
		public readonly FileReference ResourceCompilerPath;

		/// <summary>
		/// Path to the resource to object converter from the toolchain
		/// </summary>
		public readonly FileReference ResourceToObjectConverterPath;

		/// <summary>
		/// The path to the toolchain compiler for compiling code
		/// </summary>
		public readonly FileReference ToolchainCompilerPath;

		/// <summary>
		/// Optional directory containing redistributable items (DLLs etc)
		/// </summary>
		public readonly DirectoryReference? RedistDir = null;

		/// <summary>
		/// The default system include paths
		/// </summary>
		public readonly List<DirectoryReference> IncludePaths = [];

		/// <summary>
		/// The default system library paths
		/// </summary>
		public readonly List<DirectoryReference> LibraryPaths = [];

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="Params">Main constructor parameters</param>
		/// <param name="Logger">Logger for output</param>
		[SupportedOSPlatform("windows")]
		public VCEnvironment(VCEnvironmentParameters Params, ILogger Logger)
		{
			Compiler = Params.Compiler;
			CompilerDir = Params.CompilerDir;
			CompilerVersion = Params.CompilerVersion;
			Architecture = Params.Architecture;
			ToolChain = Params.ToolChain;
			ToolChainDir = Params.ToolChainDir;
			ToolChainVersion = Params.ToolChainVersion;
			WindowsSdkDir = Params.WindowsSdkDir;
			WindowsSdkVersion = Params.WindowsSdkVersion;
			RedistDir = Params.RedistDir;
			bUseCPPWinRT = Params.bUseCPPWinRT;
			bAllowClangLinker = Params.bAllowClangLinker;
			bAllowRadLinker = Params.bAllowRadLinker;

			// Get the compiler and linker paths from the Toolchain directory
			CompilerPath = GetCompilerToolPath(Compiler, Architecture, CompilerDir);
			LinkerPath = GetLinkerToolPath(Compiler, Architecture, CompilerDir, ToolChainDir);
			LibraryManagerPath = GetLibraryLinkerToolPath(Compiler, Architecture, ToolChainDir);
			ToolchainCompilerPath = GetCompilerToolPath(ToolChain, Architecture, ToolChainDir);
			ResourceToObjectConverterPath = GetResourceToObjectConverterToolPath(Architecture, ToolChainDir);

			// Get the resource compiler path from the Windows SDK
			ResourceCompilerPath = GetResourceCompilerToolPath(WindowsSdkDir, WindowsSdkVersion);

			// Register any executables for cross-architecture support with UBA
			RegisterCrossArchitectureToolPaths(WindowsSdkDir, WindowsSdkVersion);

			// Get all the system include paths
			SetupEnvironment(Logger);
		}

		/// <summary>
		/// Updates environment variables needed for running with this toolchain
		/// </summary>
		public void SetEnvironmentVariables()
		{
			// Add the compiler path and directory as environment variables for the process so they may be used elsewhere.
			Environment.SetEnvironmentVariable("VC_COMPILER_PATH", CompilerPath.FullName, EnvironmentVariableTarget.Process);
			Environment.SetEnvironmentVariable("VC_COMPILER_DIR", CompilerPath.Directory.FullName, EnvironmentVariableTarget.Process);
			Environment.SetEnvironmentVariable("VC_TOOLCHAIN_DIR", ToolchainCompilerPath.Directory.FullName, EnvironmentVariableTarget.Process);

			DirectoryReference.AddDirectoryToPath(GetVCToolPath(ToolChainDir, Architecture));
			if (Architecture == UnrealArch.Arm64)
			{
				// Add both toolchain paths to the PATH environment variable. There are some support DLLs which are only added to one of the paths, but which the toolchain in the other directory
				// needs to run (eg. mspdbcore.dll).
				DirectoryReference.AddDirectoryToPath(GetVCToolPath(ToolChainDir, UnrealArch.X64));
			}

			// Add the Windows SDK directory to the path too, for mt.exe.
			if (WindowsSdkVersion >= new VersionNumber(10))
			{
				string BuildHostArch = RuntimeInformation.ProcessArchitecture == System.Runtime.InteropServices.Architecture.Arm64 ? "arm64" : "x64";
				DirectoryReference.AddDirectoryToPath(DirectoryReference.Combine(WindowsSdkDir, "bin", WindowsSdkVersion.ToString(), BuildHostArch));

			}
		}

		/// <summary>
		/// Gets the path to the VC tool binaries.
		/// </summary>
		/// <param name="VCToolChainDir">Base directory for the VC toolchain</param>
		/// <param name="Architecture">Target Architecture</param>
		/// <returns>Directory containing the 64-bit toolchain binaries</returns>
		public static DirectoryReference GetVCToolPath(DirectoryReference VCToolChainDir, UnrealArch Architecture)
		{
			FileReference CompilerPath = FileReference.Combine(VCToolChainDir, "bin", MicrosoftPlatformSDK.MSVCHostDirectoryName, Architecture.WindowsToolChain, "cl.exe");
			FileItem FileItem = FileItem.GetItemByFileReference(CompilerPath); // FileItem is cached
			if (FileItem.Exists)
			{
				return CompilerPath.Directory;
			}

			throw new BuildException("No required {0} compiler toolchain found in {1}", Architecture, VCToolChainDir);
		}

		/// <summary>
		/// Gets the path to the Clang tool binaries.
		/// </summary>
		/// <param name="ClangToolChainDir">Base directory for the Clang toolchain</param>
		/// <returns>Directory containing the 64-bit toolchain binaries</returns>
		public static DirectoryReference GetClangToolPath(DirectoryReference ClangToolChainDir)
		{
			// If on an arm64 host, use arm64 binaries if available, otherwise use x64 
			if (!MicrosoftPlatformSDK.MSVCHostUnrealArch.bIsX64 && DirectoryReference.Exists(DirectoryReference.Combine(ClangToolChainDir, "bin-woa64")))
			{
				return DirectoryReference.Combine(ClangToolChainDir, "bin-woa64");
			}
			return DirectoryReference.Combine(ClangToolChainDir, "bin");
		}

		/// <summary>
		/// Gets the path to the compiler.
		/// </summary>
		static FileReference GetCompilerToolPath(WindowsCompiler Compiler, UnrealArch Architecture, DirectoryReference CompilerDir)
		{
			if (Compiler == WindowsCompiler.Clang)
			{
				return FileReference.Combine(GetClangToolPath(CompilerDir), "clang-cl.exe");
			}
			else if (Compiler == WindowsCompiler.ClangRTFM)
			{
				return FileReference.Combine(GetClangToolPath(CompilerDir), "verse-clang-cl.exe");
			}
			else if (Compiler == WindowsCompiler.ClangInstrument) 
			{
				return FileReference.Combine(GetClangToolPath(CompilerDir), "instr-clang-cl.exe");
			}
			else if (Compiler == WindowsCompiler.ClangCustom)
			{
				return FileReference.Combine(CompilerDir, "bin", "clang-cl.exe");
			}
			else if (Compiler == WindowsCompiler.Intel)
			{
				return FileReference.Combine(CompilerDir, "bin", "icx.exe");
			}
			return FileReference.Combine(GetVCToolPath(CompilerDir, Architecture), "cl.exe");
		}

		/// <summary>
		/// Gets the path to the linker.
		/// </summary>
		FileReference GetLinkerToolPath(WindowsCompiler Compiler, UnrealArch Architecture, DirectoryReference CompilerDir, DirectoryReference ToochainDir)
		{
			if (bAllowRadLinker)
			{
				return FileReference.Combine(Unreal.EngineDirectory, "Binaries", "Win64", "RadDebugger", Architecture.ToString(), "radlink.exe");
			}
			else if ((Compiler == WindowsCompiler.Clang || Compiler == WindowsCompiler.ClangRTFM || Compiler == WindowsCompiler.ClangInstrument || Compiler == WindowsCompiler.ClangCustom) && bAllowClangLinker)
			{
				return FileReference.Combine(GetClangToolPath(CompilerDir), "lld-link.exe");
			}
			else if (Compiler == WindowsCompiler.Intel && bAllowClangLinker)
			{
				return FileReference.Combine(CompilerDir, "bin", "compiler", "lld-link.exe");
			}
			return FileReference.Combine(GetVCToolPath(ToochainDir, Architecture), "link.exe");
		}

		/// <summary>
		/// Gets the path to the library linker.
		/// </summary>
		FileReference GetLibraryLinkerToolPath(WindowsCompiler Compiler, UnrealArch Architecture, DirectoryReference ToochainDir)
		{
			if (bAllowClangLinker && Compiler.IsClang())
			{
				// Since obj files could be LLVM IR Stream file format (when building with ltcg) we can't use link.exe. UbaObjTool support all formats and produce identical lib files as link.exe when running non-ltcg
#pragma warning disable CA1308 // Normalize strings to uppercase - the directory names are lowercased in VCS
				DirectoryReference Dir = DirectoryReference.Combine(Unreal.EngineDirectory, "Binaries", "Win64", "UnrealBuildAccelerator", RuntimeInformation.ProcessArchitecture.ToString().ToLowerInvariant());
#pragma warning restore CA1308
				return FileReference.Combine(Dir, "UbaObjTool.exe");
			}
			return FileReference.Combine(GetVCToolPath(ToochainDir, Architecture), "link.exe"); // We add /LIB to cmd line so we can use link.exe directly instead of going via lib.exe
		}

		/// <summary>
		/// Gets the path to the resource to object converter.
		/// </summary>
		private static FileReference GetResourceToObjectConverterToolPath(UnrealArch Architecture, DirectoryReference ToochainDir)
		{
			return FileReference.Combine(GetVCToolPath(ToochainDir, Architecture), "cvtres.exe");
		}

		/// <summary>
		/// Gets the path to the resource compiler.
		/// </summary>
		private static FileReference GetResourceCompilerToolPath(DirectoryReference WindowsSdkDir, VersionNumber WindowsSdkVersion)
		{
			string BuildHostArch = RuntimeInformation.ProcessArchitecture == System.Runtime.InteropServices.Architecture.Arm64 ? "arm64" : "x64";
			FileReference ResourceCompilerPath = FileReference.Combine(WindowsSdkDir, "bin", WindowsSdkVersion.ToString(), BuildHostArch, "rc.exe");
			FileItem FileItem = FileItem.GetItemByFileReference(ResourceCompilerPath);
			if (FileItem.Exists)
			{
				return ResourceCompilerPath;
			}

			throw new BuildException("Unable to find path to the Windows resource compiler under {0} (version {1})", WindowsSdkDir, WindowsSdkVersion);
		}

		/// <summary>
		/// Registers all SDK binaries with UBA for cross-architecture support
		/// </summary>
		static void RegisterCrossArchitectureToolPaths(DirectoryReference WindowsSdkDir, VersionNumber WindowsSdkVersion)
		{
			IEnumerable<DirectoryReference> ToolPaths = [
				 DirectoryReference.Combine(WindowsSdkDir, "bin", WindowsSdkVersion.ToString()),
				 DirectoryReference.Combine(Unreal.EngineDirectory, "Binaries", "Win64", "RadDebugger"),
				 DirectoryReference.Combine(Unreal.EngineDirectory, "Binaries", "Win64", "UnrealBuildAccelerator")
			];

			foreach (DirectoryReference ToolPath in ToolPaths)
			{
				DirectoryReference HostBinDir = DirectoryReference.Combine(ToolPath, MicrosoftPlatformSDK.MSVCHostUnrealArch.bIsX64 ? "x64" : "arm64");
				DirectoryReference CrossBinDir = DirectoryReference.Combine(ToolPath, !MicrosoftPlatformSDK.MSVCHostUnrealArch.bIsX64 ? "x64" : "arm64");
				if (DirectoryReference.Exists(HostBinDir) && DirectoryReference.Exists(CrossBinDir))
				{
					EpicGames.UBA.Utils.RegisterCrossArchitecturePath(HostBinDir, CrossBinDir);
				}
			}
		}

		/// <summary>
		/// Return the standard Clang C++ library path.
		/// </summary>
		private DirectoryReference GetClangLibsDir()
		{
			VersionNumber ClangVersion = Compiler.IsIntel() ? MicrosoftPlatformSDK.GetClangVersionForIntelCompiler(CompilerPath) : CompilerVersion;
			return DirectoryReference.Combine(CompilerDir, "lib", "clang", ClangVersion.GetComponent(0).ToString(), "lib", "windows");
		}

#pragma warning disable CA2214 // Do not call overridable methods in constructors - happens to be safe as of 2025-10-24, impractical to restructure code to avoid
		/// <summary>
		/// Return the standard Visual C++ library path.
		/// </summary>
		protected virtual DirectoryReference GetToolChainLibsDir() => DirectoryReference.Combine(ToolChainDir, "lib", Architecture.WindowsSystemLibDir);
#pragma warning restore CA2214

		/// <summary>
		/// Sets up the standard compile environment for the toolchain
		/// </summary>
		[SupportedOSPlatform("windows")]
		private void SetupEnvironment(ILogger Logger)
		{
			string ArchFolder = Architecture.WindowsSystemLibDir;

			// Add the standard Visual C++ include paths
			IncludePaths.Add(DirectoryReference.Combine(ToolChainDir, "INCLUDE"));

			if (Compiler.IsClang())
			{
				// Add the standard Clang C++ library path before the Visual C++ path. This includes compiler libraries required for PGO, ASAN, etc.
				LibraryPaths.Add(GetClangLibsDir());
			}

			// Add the standard Visual C++ library path
			LibraryPaths.Add(GetToolChainLibsDir());

			// If we're on >= Visual Studio 2015 and using pre-Windows 10 SDK, we need to find a Windows 10 SDK and add the UCRT include paths
			if (ToolChain.IsMSVC() && WindowsSdkVersion < new VersionNumber(10))
			{
				KeyValuePair<VersionNumber, DirectoryReference> Pair = MicrosoftPlatformSDK.FindUniversalCrtDirs(Logger).OrderByDescending(x => x.Key).FirstOrDefault();
				if (Pair.Key == null || Pair.Key < new VersionNumber(10))
				{
					throw new BuildException("{0} requires the Universal CRT to be installed.", WindowsPlatform.GetCompilerName(ToolChain));
				}

				DirectoryReference IncludeRootDir = DirectoryReference.Combine(Pair.Value, "include", Pair.Key.ToString());
				IncludePaths.Add(DirectoryReference.Combine(IncludeRootDir, "ucrt"));

				DirectoryReference LibraryRootDir = DirectoryReference.Combine(Pair.Value, "lib", Pair.Key.ToString());
				LibraryPaths.Add(DirectoryReference.Combine(LibraryRootDir, "ucrt", ArchFolder));
			}

			// Add the Windows SDK paths
			if (WindowsSdkVersion >= new VersionNumber(10))
			{
				DirectoryReference IncludeRootDir = DirectoryReference.Combine(WindowsSdkDir, "include", WindowsSdkVersion.ToString());
				IncludePaths.Add(DirectoryReference.Combine(IncludeRootDir, "ucrt"));
				IncludePaths.Add(DirectoryReference.Combine(IncludeRootDir, "shared"));
				IncludePaths.Add(DirectoryReference.Combine(IncludeRootDir, "um"));
				IncludePaths.Add(DirectoryReference.Combine(IncludeRootDir, "winrt"));
				if (bUseCPPWinRT)
				{
					IncludePaths.Add(DirectoryReference.Combine(IncludeRootDir, "cppwinrt"));
				}

				DirectoryReference LibraryRootDir = DirectoryReference.Combine(WindowsSdkDir, "lib", WindowsSdkVersion.ToString());
				LibraryPaths.Add(DirectoryReference.Combine(LibraryRootDir, "ucrt", ArchFolder));
				LibraryPaths.Add(DirectoryReference.Combine(LibraryRootDir, "um", ArchFolder));
			}
			else
			{
				DirectoryReference IncludeRootDir = DirectoryReference.Combine(WindowsSdkDir, "include");
				IncludePaths.Add(DirectoryReference.Combine(IncludeRootDir, "shared"));
				IncludePaths.Add(DirectoryReference.Combine(IncludeRootDir, "um"));
				IncludePaths.Add(DirectoryReference.Combine(IncludeRootDir, "winrt"));

				DirectoryReference LibraryRootDir = DirectoryReference.Combine(WindowsSdkDir, "lib", "winv6.3");
				LibraryPaths.Add(DirectoryReference.Combine(LibraryRootDir, "um", ArchFolder));
			}

			// Add path to Intel math libraries when using Intel oneAPI
			if (Compiler == WindowsCompiler.Intel)
			{
				VersionNumber ClangVersion = MicrosoftPlatformSDK.GetClangVersionForIntelCompiler(CompilerPath);
				IncludePaths.Add(DirectoryReference.Combine(CompilerDir, "compiler", "include"));
				IncludePaths.Add(DirectoryReference.Combine(CompilerDir, "lib", "clang", ClangVersion.GetComponent(0).ToString(), "include"));
				LibraryPaths.Add(DirectoryReference.Combine(CompilerDir, "lib"));
			}
		}

		/// <summary>
		/// Creates an environment with the given settings
		/// </summary>
		/// <param name="Compiler">The compiler version to use</param>
		/// <param name="ToolChain">The toolchain version to use, when a non-msvc compiler is used</param>
		/// <param name="Platform">The platform to target</param>
		/// <param name="Architecture">The Architecture to target</param>
		/// <param name="CompilerVersion">The specific compiler version to use</param>
		/// <param name="ToolchainVersion">The specific toolchain version to use (if the compiler isn't msvc)</param>
		/// <param name="WindowsSdkVersion">Version of the Windows SDK to use</param>
		/// <param name="SuppliedSdkDirectoryForVersion">If specified, this is the SDK directory to use, otherwise, attempt to look up via registry. If specified, the WindowsSdkVersion is used directly</param>
		/// <param name="bUseCPPWinRT">Include the CPP/WinRT language projection</param>
		/// <param name="bAllowClangLinker">Allow use of Clang linker</param>
		/// <param name="bAllowRadLinker">Allow use of Rad linker</param>
		/// <param name="Logger">Logger for output</param>
		/// <returns>New environment object with paths for the given settings</returns>
		[SupportedOSPlatform("windows")]
		public static VCEnvironment Create(WindowsCompiler Compiler, WindowsCompiler ToolChain, UnrealTargetPlatform Platform, UnrealArch Architecture, string? CompilerVersion, string? ToolchainVersion, string? WindowsSdkVersion, string? SuppliedSdkDirectoryForVersion, bool bUseCPPWinRT, bool bAllowClangLinker, bool bAllowRadLinker, ILogger Logger)
		{
			return Create(new VCEnvironmentParameters(Compiler, ToolChain, Platform, Architecture, CompilerVersion, ToolchainVersion, WindowsSdkVersion, SuppliedSdkDirectoryForVersion, bUseCPPWinRT, bAllowClangLinker, bAllowRadLinker, Logger), Logger);
		}

		/// <summary>
		/// Creates an environment with the given parameters
		/// </summary>
		[SupportedOSPlatform("windows")]
		public static VCEnvironment Create(VCEnvironmentParameters Params, ILogger Logger)
		{
			return new VCEnvironment(Params, Logger);
		}
	}

	/// <summary>
	/// Parameter structure for constructing VCEnvironment
	/// </summary>
	struct VCEnvironmentParameters
	{
		/// <summary>The platform to find the compiler for</summary>
		public UnrealTargetPlatform Platform;

		/// <summary>The compiler to use</summary>
		public WindowsCompiler Compiler;

		/// <summary>The compiler directory</summary>
		public DirectoryReference CompilerDir;

		/// <summary>The compiler version number</summary>
		public VersionNumber CompilerVersion;

		/// <summary>The compiler Architecture</summary>
		public UnrealArch Architecture;

		/// <summary>The base toolchain version</summary>
		public WindowsCompiler ToolChain;

		/// <summary>Directory containing the toolchain</summary>
		public DirectoryReference ToolChainDir;

		/// <summary>Version of the toolchain</summary>
		public VersionNumber ToolChainVersion;

		/// <summary>Root directory containing the Windows Sdk</summary>
		public DirectoryReference WindowsSdkDir;

		/// <summary>Version of the Windows Sdk</summary>
		public VersionNumber WindowsSdkVersion;

		/// <summary>Optional directory for redistributable items (DLLs etc)</summary>
		public DirectoryReference? RedistDir;

		/// <summary>Include the CPP/WinRT language projection</summary>
		public bool bUseCPPWinRT;

		/// <summary>Allow use of Clang linker</summary>
		public bool bAllowClangLinker;

		/// <summary>Allow use of Rad linker</summary>
		public bool bAllowRadLinker;

		/// <summary>
		/// Creates VC environment construction parameters with the given settings
		/// </summary>
		/// <param name="Compiler">The compiler version to use</param>
		/// <param name="ToolChain">The toolchain version to use, when a non-msvc compiler is used</param>
		/// <param name="Platform">The platform to target</param>
		/// <param name="Architecture">The Architecture to target</param>
		/// <param name="CompilerVersion">The specific compiler version to use</param>
		/// <param name="ToolchainVersion">The specific toolchain version to use (if the compiler isn't msvc)</param>
		/// <param name="WindowsSdkVersion">Version of the Windows SDK to use</param>
		/// <param name="SuppliedSdkDirectoryForVersion">If specified, this is the SDK directory to use, otherwise, attempt to look up via registry. If specified, the WindowsSdkVersion is used directly</param>
		/// <param name="bUseCPPWinRT">Include the CPP/WinRT language projection</param>
		/// <param name="bAllowClangLinker">Allow use of Clang linker</param>
		/// <param name="bAllowRadLinker">Allow use of Rad linker</param>
		/// <param name="Logger">Logger for output</param>
		/// <returns>Creation parameters for VC environment</returns>
		[SupportedOSPlatform("windows")]
		public VCEnvironmentParameters(WindowsCompiler Compiler, WindowsCompiler ToolChain, UnrealTargetPlatform Platform, UnrealArch Architecture, string? CompilerVersion, string? ToolchainVersion, string? WindowsSdkVersion, string? SuppliedSdkDirectoryForVersion, bool bUseCPPWinRT, bool bAllowClangLinker, bool bAllowRadLinker, ILogger Logger)
		{
			// Get the compiler version info
			if (!WindowsPlatform.TryGetToolChainDir(Compiler, CompilerVersion, Architecture, Logger, out VersionNumber? SelectedCompilerVersion, out DirectoryReference? SelectedCompilerDir, out DirectoryReference? SelectedRedistDir))
			{
				throw new BuildException("{0}{1} {2} must be installed in order to build this target.", WindowsPlatform.GetCompilerName(Compiler), String.IsNullOrEmpty(CompilerVersion) ? "" : String.Format(" ({0})", CompilerVersion), Architecture.ToString());
			}

			// Get the toolchain info
			VersionNumber? SelectedToolChainVersion;
			DirectoryReference? SelectedToolChainDir;
			if (Compiler.IsClang())
			{
				if (ToolChain.IsClang() || ToolChain == WindowsCompiler.Default)
				{
					throw new BuildException("{0} is not a valid ToolChain for Compiler {1}", WindowsPlatform.GetCompilerName(ToolChain), WindowsPlatform.GetCompilerName(Compiler));
				}

				if (!WindowsPlatform.TryGetToolChainDir(ToolChain, ToolchainVersion, Architecture, Logger, out SelectedToolChainVersion, out SelectedToolChainDir, out SelectedRedistDir))
				{
					throw new BuildException("{0} or newer must be installed in order to build this target.", WindowsPlatform.GetCompilerName(WindowsCompiler.VisualStudio2022));
				}

				VersionNumber? PreferredSelectedToolChainVersion = SelectedToolChainVersion;
				DirectoryReference? PreferredSelectedToolChainDir = SelectedToolChainDir;
				DirectoryReference? PreferredSelectedRedistDir = SelectedRedistDir;

				try
				{
					// If the preferred MSVC toolchain is not supported for the Clang compiler version, search for an older version that is supported
					VersionNumber SelectedClangVersion = !Compiler.IsIntel() ? SelectedCompilerVersion : MicrosoftPlatformSDK.GetClangVersionForIntelCompiler(FileReference.Combine(SelectedCompilerDir, "bin", "icx.exe"));
					while (MicrosoftPlatformSDK.GetMinimumClangVersionForVcVersion(SelectedToolChainVersion) > SelectedCompilerVersion)
					{
						if (!WindowsPlatform.TryGetToolChainDir(ToolChain, $"<{SelectedToolChainVersion.GetComponent(0)}.{SelectedToolChainVersion.GetComponent(1)}", Architecture, Logger, out SelectedToolChainVersion, out SelectedToolChainDir, out SelectedRedistDir))
						{
							throw new BuildException("{0} compiler or newer that supports Clang {1} must be installed in order to build this target.", WindowsPlatform.GetCompilerName(WindowsCompiler.VisualStudio2022));
						}
					}
				}
				catch (BuildException)
				{
					// In the event a valid toolchain is not found that is supported with the requested clang version, use the original preferred choice and the error will be handed later
					SelectedToolChainVersion = PreferredSelectedToolChainVersion;
					SelectedToolChainDir = PreferredSelectedToolChainDir;
					SelectedRedistDir = PreferredSelectedRedistDir;
				}
			}
			else
			{
				ToolChain = Compiler;
				SelectedToolChainVersion = SelectedCompilerVersion;
				SelectedToolChainDir = SelectedCompilerDir;
			}

			// Get the actual Windows SDK directory
			VersionNumber? SelectedWindowsSdkVersion;
			DirectoryReference? SelectedWindowsSdkDir;
			if (SuppliedSdkDirectoryForVersion != null)
			{
				SelectedWindowsSdkDir = new DirectoryReference(SuppliedSdkDirectoryForVersion);
				SelectedWindowsSdkVersion = VersionNumber.Parse(WindowsSdkVersion!);

				if (!DirectoryReference.Exists(SelectedWindowsSdkDir))
				{
					throw new BuildException("Windows SDK{0} must be installed at {1}.", String.IsNullOrEmpty(WindowsSdkVersion) ? "" : String.Format(" ({0})", WindowsSdkVersion), SuppliedSdkDirectoryForVersion);
				}
			}
			else
			{
				if (!WindowsPlatform.TryGetWindowsSdkDir(WindowsSdkVersion, Logger, out SelectedWindowsSdkVersion, out SelectedWindowsSdkDir))
				{
					MicrosoftPlatformSDK.DumpWindowsSdkDirs(Logger);
					throw new BuildException("Windows SDK{0} must be installed in order to build this target.", String.IsNullOrEmpty(WindowsSdkVersion) ? "" : String.Format(" ({0})", WindowsSdkVersion));
				}
			}

			// Store the final parameters
			this.Platform = Platform;
			this.Compiler = Compiler;
			CompilerDir = SelectedCompilerDir;
			this.CompilerVersion = SelectedCompilerVersion;
			this.Architecture = Architecture;
			this.ToolChain = ToolChain;
			ToolChainDir = SelectedToolChainDir;
			ToolChainVersion = SelectedToolChainVersion;
			WindowsSdkDir = SelectedWindowsSdkDir;
			this.WindowsSdkVersion = SelectedWindowsSdkVersion;
			RedistDir = SelectedRedistDir;
			this.bUseCPPWinRT = bUseCPPWinRT;
			this.bAllowClangLinker = bAllowClangLinker;
			this.bAllowRadLinker = bAllowRadLinker;
		}
	}
}
