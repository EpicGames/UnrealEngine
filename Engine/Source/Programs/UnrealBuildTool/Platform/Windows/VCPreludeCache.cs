// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Threading;
using EpicGames.Core;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// Caches predefined state ("prelude") for compiler versions and options
	/// </summary>
	static class VCPreludeCache
	{
		/// <summary>
		/// Set of options which can influence the prelude state. Any options matching entries in this set will be used to generate a separate prelude file.
		/// </summary>
		static readonly HashSet<string> FilterArguments = new HashSet<string>(StringComparer.OrdinalIgnoreCase)
		{
			"/EH",  // Exception handling mode
			"/GR",  // RTTI settings
			"/MD",  // Multithreaded DLL runtime library
			"/MDd", // Multithreaded debug DLL runtime library
			"/MT",  // Multithreaded static runtime library
			"/MTd", // Multithreaded debug static runtime library
		};

		/// <summary>
		/// Object used for ensuring only one thread can create items in the cache at once
		/// </summary>
		static readonly Lock LockObject = new();

		/// <summary>
		/// File containing a list of macros to be used to generate a prelude file
		/// </summary>
		static readonly FileReference MacroNamesFile = FileReference.Combine(Unreal.EngineDirectory, "Build", "Windows", "PreludeMacros.txt");

		/// <summary>
		/// Base directory for the cache
		/// </summary>
		static readonly DirectoryReference PreludeCacheDir = DirectoryReference.Combine(Unreal.EngineDirectory, "Saved", "UnrealBuildTool", "Prelude", "MSVC");

		/// <summary>
		/// The parsed list of macros
		/// </summary>
		static readonly List<string> MacroNames = new List<string>();

		/// <summary>
		/// Path to the file used to generate prelude headers
		/// </summary>
		static FileReference? PreludeGeneratorFile;

		/// <summary>
		/// Cached mapping from compiler filename to its version string
		/// </summary>
		static readonly Dictionary<FileReference, string> CompilerExeToVersion = new Dictionary<FileReference, string>();

		/// <summary>
		/// Get the prelude file (or create it) for the given compiler executable and set of options
		/// </summary>
		/// <param name="Compiler">Path to the compiler executable</param>
		/// <param name="Arguments">Command line arguments for generating the prelude file</param>
		/// <returns>Path to the prelude file</returns>
		public static FileReference GetPreludeHeader(FileReference Compiler, List<string> Arguments)
		{
			// Cache the compiler version
			string? CompilerVersion;
			lock (CompilerExeToVersion)
			{
				if (!CompilerExeToVersion.TryGetValue(Compiler, out CompilerVersion))
				{
					FileVersionInfo VersionInfo = FileVersionInfo.GetVersionInfo(Compiler.FullName);
					CompilerVersion = String.Format("{0} {1}", VersionInfo.ProductName, VersionInfo.ProductVersion);
				}
			}

			// Cache the macro list
			lock (MacroNames)
			{
				if (MacroNames.Count == 0)
				{
					string[] MacroListLines = FileReference.ReadAllLines(MacroNamesFile);
					foreach (string MacroListLine in MacroListLines)
					{
						string MacroName = MacroListLine.Trim();
						if (MacroName.Length > 0 && MacroName[0] != ';')
						{
							MacroNames.Add(MacroName);
						}
					}
				}
			}

			// Create a digest from the command line
			string CommandLine = String.Join(" ", Arguments.Where(x => FilterArguments.Contains(x)).OrderBy(x => x));
			ContentHash Digest = ContentHash.SHA1(String.Format("{0}\n{1}\n{2}", CompilerVersion, CommandLine, String.Join("\n", MacroNames)));

			// Get the prelude file
			FileReference PreludeHeader = FileReference.Combine(PreludeCacheDir, String.Format("{0}.h", Digest));
			if (!FileReference.Exists(PreludeHeader))
			{
				lock (LockObject)
				{
					if (!FileReference.Exists(PreludeHeader))
					{
						CreatePreludeHeader(PreludeHeader, Compiler, CompilerVersion, CommandLine);
					}
				}
			}
			return PreludeHeader;
		}

		/// <summary>
		/// Creates a prelude header
		/// </summary>
		/// <param name="PreludeHeader">Path to the file to create</param>
		/// <param name="Compiler">Path to the compiler</param>
		/// <param name="CompilerVersion">Version number of the compiler</param>
		/// <param name="CommandLine">Command line arguments used to generate the header</param>
		static void CreatePreludeHeader(FileReference PreludeHeader, FileReference Compiler, string CompilerVersion, string CommandLine)
		{
			// Make sure the cache directory exists
			DirectoryReference.CreateDirectory(PreludeCacheDir);

			// Make sure the prelude generator file exists
			if (PreludeGeneratorFile == null)
			{
				PreludeGeneratorFile = FileReference.Combine(PreludeCacheDir, "PreludeGenerator.cpp");
				using (StreamWriter Writer = new StreamWriter(PreludeGeneratorFile.FullName))
				{
					Writer.WriteLine("#define STRINGIZE_2(x) #x");
					Writer.WriteLine("#define STRINGIZE(x) STRINGIZE_2(x)");
					foreach (string MacroName in MacroNames)
					{
						Writer.WriteLine("");
						Writer.WriteLine("#ifdef {0}", MacroName);
						Writer.WriteLine("#pragma message(\"#define {0} \" STRINGIZE({0}))", MacroName);
						Writer.WriteLine("#endif");
					}
				}
			}

			// Invoke the compiler and capture the output
			using (ManagedProcessGroup ProcessGroup = new ManagedProcessGroup())
			{
				string FullCommandLine = String.Format("{0} /nologo /c {1}", CommandLine, Utils.MakePathSafeToUseWithCommandLine(PreludeGeneratorFile.FullName));
				using (ManagedProcess Process = new ManagedProcess(ProcessGroup, Compiler.FullName, FullCommandLine, PreludeCacheDir.FullName, null, null, ProcessPriorityClass.Normal))
				{
					// Filter the output
					FileReference TempPreludeFile = new FileReference(PreludeHeader.FullName + ".temp");
					using (StreamWriter Writer = new StreamWriter(TempPreludeFile.FullName))
					{
						Writer.WriteLine("// Compiler: {0}", CompilerVersion);
						Writer.WriteLine("// Arguments: {0}", CommandLine);
						Writer.WriteLine();

						string? Line;
						while (Process.TryReadLine(out Line))
						{
							Line = Line.Trim();
							if (Line.Length > 0)
							{
								if (Line.StartsWith("#define", StringComparison.Ordinal))
								{
									Writer.WriteLine(Line);
								}
								else if (Line.Trim() != PreludeGeneratorFile.GetFileName())
								{
									throw new BuildException("Unexpected output from compiler: {0}", Line);
								}
							}
						}
					}
					FileReference.Move(TempPreludeFile, PreludeHeader);
				}
			}
		}
	}
}
