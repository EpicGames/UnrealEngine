// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.IO;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	internal class VcSpecificFileAction : Action, ISpecificFileAction
	{
		readonly DirectoryReference SourceDir;
		readonly DirectoryReference OutputDir;
		readonly VCCompileAction BaseAction;

		// TODO: Entire CppCompileEnvironment needs to be saved with BinaryArchiveWriter, some options may still be unused
		readonly bool bPreprocessOnly;
		readonly bool bWithAssembly;

		readonly Dictionary<string, List<FileItem>> SingleFiles = new();

		internal VcSpecificFileAction(DirectoryReference Source, DirectoryReference Output, VCCompileAction Action, CppCompileEnvironment? CompileEnvironment = null) : base(ActionType.Compile)
		{
			SourceDir = Source;
			OutputDir = Output;
			BaseAction = Action;
			bPreprocessOnly = CompileEnvironment?.bPreprocessOnly ?? false;
			bWithAssembly = CompileEnvironment?.bWithAssembly ?? false;
		}

		public VcSpecificFileAction(BinaryArchiveReader Reader) : base(ActionType.Compile)
		{
			SourceDir = Reader.ReadCompactDirectoryReference();
			OutputDir = Reader.ReadCompactDirectoryReference();
			bPreprocessOnly = Reader.ReadBool();
			bWithAssembly = Reader.ReadBool();
			BaseAction = new VCCompileAction(Reader);
		}

		public new void Write(BinaryArchiveWriter Writer)
		{
			Writer.WriteCompactDirectoryReference(SourceDir);
			Writer.WriteCompactDirectoryReference(OutputDir);
			Writer.WriteBool(bPreprocessOnly);
			Writer.WriteBool(bWithAssembly);
			BaseAction.Write(Writer);
		}

		public DirectoryReference RootDirectory => SourceDir;

		public IExternalAction? CreateAction(FileItem SourceFile, ILogger Logger)
		{
			DirectoryReference IntermediateDirectory = DirectoryReference.Combine(OutputDir, "SingleFile");
			// Keep track of all specific files, so the output file can be renamed if there's a naming conflict
			string Filename = SourceFile.Name;
			if (!SingleFiles.ContainsKey(SourceFile.Name))
			{
				SingleFiles[SourceFile.Name] = new();
			}
			else
			{
				Filename = $"{Path.GetFileNameWithoutExtension(SourceFile.Name)}{SingleFiles[Filename].Count}{Path.GetExtension(SourceFile.Name)}";
			}
			SingleFiles[SourceFile.Name].Add(SourceFile);

			if (!SourceFile.HasExtension(".cpp") && !SourceFile.HasExtension(".h"))
			{
				return null;
			}

			DirectoryReference.CreateDirectory(IntermediateDirectory);
			VCCompileAction Action = new VCCompileAction(BaseAction);
			Action.ArtifactMode = ArtifactMode.None;
			Action.SourceFile = SourceFile;
			if (SourceFile.HasExtension(".h"))
			{
				if (BaseAction.CompilerType.IsClang())
				{
					ClangWarnings.GetHeaderDisabledWarnings(Action.Arguments);
				}

				string IncludeFileString = SourceFile.AbsolutePath;
				if (SourceFile.Location.IsUnderDirectory(Unreal.RootDirectory))
				{
					IncludeFileString = SourceFile.Location.MakeRelativeTo(Unreal.EngineSourceDirectory);
				}

				List<string> GeneratedHeaderCppContents = UEBuildModuleCPP.GenerateHeaderCpp(SourceFile.Name, IncludeFileString);
				Action.SourceFile = FileItem.GetItemByFileReference(FileReference.Combine(IntermediateDirectory, $"{SourceFile.Name}.cpp"));
				File.WriteAllLines(Action.SourceFile.FullName, GeneratedHeaderCppContents);
			}
			
			if (bPreprocessOnly)
			{
				Action.PreprocessedFile = FileItem.GetItemByFileReference(FileReference.Combine(IntermediateDirectory, $"{Filename}.i"));
			}
			else
			{
				Action.ObjectFile = FileItem.GetItemByFileReference(FileReference.Combine(IntermediateDirectory, $"{Filename}.obj"));
				if (BaseAction.CompilerType.IsMSVC() && bWithAssembly)
				{
					Action.AssemblyFile = FileItem.GetItemByFileReference(FileReference.Combine(IntermediateDirectory, $"{Filename}.asm"));
				}
			}

			if (BaseAction.CompilerType.IsMSVC())
			{
				Action.DependencyListFile = FileItem.GetItemByFileReference(FileReference.Combine(IntermediateDirectory, $"{Filename}.dep.json"));
			}
			else if (BaseAction.CompilerType.IsClang())
			{
				Action.DependencyListFile = FileItem.GetItemByFileReference(FileReference.Combine(IntermediateDirectory, $"{Filename}.d"));
			}
			Action.ResponseFile = FileItem.GetItemByFileReference(FileReference.Combine(IntermediateDirectory, $"{Filename}.rsp"));
			File.WriteAllLines(Action.ResponseFile.FullName, Action.GetCompilerArguments());
			Action.Arguments.Clear();

			return Action;
		}
	}

	class VcSpecificFileActionSerializer : ActionSerializerBase<VcSpecificFileAction>
	{
		/// <inheritdoc/>
		public override VcSpecificFileAction Read(BinaryArchiveReader Reader)
		{
			return new VcSpecificFileAction(Reader);
		}

		/// <inheritdoc/>
		public override void Write(BinaryArchiveWriter Writer, VcSpecificFileAction Action)
		{
			Action.Write(Writer);
		}
	}
}