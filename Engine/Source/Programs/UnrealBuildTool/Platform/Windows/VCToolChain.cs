// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Runtime.Versioning;
using System.Text;
using System.Threading;
using System.Xml.Linq;
using EpicGames.Core;
using Microsoft.CodeAnalysis;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	class VCToolChain : ISPCToolChain
	{
		/// <summary>
		/// The target being built
		/// </summary>
		protected ReadOnlyTargetRules Target;

		/// <summary>
		/// The Visual C++ environment
		/// </summary>
		protected VCEnvironment EnvVars;

		public VCToolChain(ReadOnlyTargetRules Target, ILogger Logger)
			: base(Logger)
		{
			this.Target = Target;
			EnvVars = Target.WindowsPlatform.Environment!;

			Logger.LogDebug("Compiler: {Path}", EnvVars.CompilerPath);
			Logger.LogDebug("Linker: {Path}", EnvVars.LinkerPath);
			Logger.LogDebug("Library Manager: {Path}", EnvVars.LibraryManagerPath);
			Logger.LogDebug("Resource Compiler: {Path}", EnvVars.ResourceCompilerPath);
			Logger.LogDebug("Resource Converter: {Path}", EnvVars.ResourceToObjectConverterPath);

			if (Target.WindowsPlatform.ObjSrcMapFile != null)
			{
				try
				{
					File.Delete(Target.WindowsPlatform.ObjSrcMapFile);
				}
				catch
				{
				}
			}
		}

		public override FileReference? GetCppCompilerPath()
		{
			return EnvVars.CompilerPath;
		}

		public IEnumerable<DirectoryReference> GetVCIncludePaths()
		{
			return EnvVars.IncludePaths;
		}

		protected bool IsDynamicDebuggingEnabled => Target.WindowsPlatform.bDynamicDebugging
			&& Target.WindowsPlatform.Compiler.IsMSVC()
			&& !(Target.WindowsPlatform.bAllowRadLinker)
			&& !(Target.WindowsPlatform.Environment != null && Target.WindowsPlatform.Environment.CompilerVersion < new VersionNumber(14, 44, 34918))
			&& !(Target.bAllowLTCG || Target.bPGOOptimize || Target.bPGOProfile)
			&& !(Target.Configuration == UnrealTargetConfiguration.Debug || Target.Configuration == UnrealTargetConfiguration.DebugGame)
			&& !(Target.Architecture == UnrealArch.Arm64ec)
			&& !(Target.WindowsPlatform.bEnableAddressSanitizer)
			&& !(Target.bSupportEditAndContinue);

		protected void DynamicDebuggingInfo(List<string> Lines)
		{
			if (Target.WindowsPlatform.bDynamicDebugging)
			{
				if (IsDynamicDebuggingEnabled)
				{
					Lines.Add("Visual Studio Dynamic Debugging is enabled");
					return;
				}

				Lines.Add("Visual Studio Dynamic Debugging has been disabled due to the following:");
				if (!Target.WindowsPlatform.Compiler.IsMSVC())
				{
					Lines.Add($" * Target.{nameof(Target.WindowsPlatform)}.{nameof(Target.WindowsPlatform.Compiler)} '{Target.WindowsPlatform.Compiler}' is not MSVC");
				}
				if (Target.WindowsPlatform.bAllowRadLinker)
				{
					Lines.Add($" * Target.{nameof(Target.WindowsPlatform)}.{nameof(Target.WindowsPlatform.bAllowRadLinker)} is enabled");
				}
				if (Target.WindowsPlatform.Environment != null && Target.WindowsPlatform.Environment.CompilerVersion < new VersionNumber(14, 44, 34918))
				{
					Lines.Add($" * MSVC compiler version {Target.WindowsPlatform.Environment.CompilerVersion} older than 14.44.34918");
				}
				if (Target.bAllowLTCG || Target.bPGOOptimize || Target.bPGOProfile)
				{
					Lines.Add($" * Target.{nameof(Target.bAllowLTCG)}, Target.{nameof(Target.bPGOOptimize)}, or Target.{nameof(Target.bPGOProfile)} is enabled");
				}
				if (Target.Configuration == UnrealTargetConfiguration.Debug || Target.Configuration == UnrealTargetConfiguration.DebugGame)
				{
					Lines.Add($" * Target.{nameof(Target.Configuration)} '{Target.Configuration}' is Debug or DebugGame. Development, Test, or Shipping configuration is recommended when Dynamic Debugging is requested");
				}
				if (Target.Architecture == UnrealArch.Arm64ec)
				{
					Lines.Add($" * Target.{nameof(Target.Architecture)} '{Target.Architecture}' does not currently support Dynamic Debugging.");
				}
				if (Target.WindowsPlatform.bEnableAddressSanitizer)
				{
					Lines.Add($" * Target.{nameof(Target.WindowsPlatform)}.{nameof(Target.WindowsPlatform.bEnableAddressSanitizer)} is enabled");
				}
				if (Target.WindowsPlatform.bEnableAutoRTFMSanitizer)
				{
					Lines.Add($" * Target.{nameof(Target.WindowsPlatform)}.{nameof(Target.WindowsPlatform.bEnableAutoRTFMSanitizer)} is enabled");
				}
				if (Target.bSupportEditAndContinue)
				{
					Lines.Add($" * Target.{nameof(Target.bSupportEditAndContinue)} is enabled");
				}
			}
		}

		/// <inheritdoc/>
		public override void SetEnvironmentVariables()
		{
			EnvVars.SetEnvironmentVariables();

			// This allows PGD files generated from this build to be portable to other machines. It needs to be set in all configurations so we don't split the environment
			HashSet<string> pgoPathTranslations = [];
			string? pgoPathTranslationEnv = Environment.GetEnvironmentVariable("PGO_PATH_TRANSLATION");
			if (!String.IsNullOrWhiteSpace(pgoPathTranslationEnv))
			{
				pgoPathTranslations= [.. pgoPathTranslationEnv.Split(';', StringSplitOptions.RemoveEmptyEntries)];
			}

			CppRootPaths RootPaths = new() { bUseVfs = true };
			RootPaths[CppRootPathFolder.Root] = Unreal.RootDirectory;
			if (Target.ProjectFile != null)
			{
				RootPaths[CppRootPathFolder.Project] = Target.ProjectFile.Directory;
			}
			foreach ((_, DirectoryReference vfs, DirectoryReference local) in RootPaths)
			{
				pgoPathTranslations.Add($"{local}={vfs.GetDirectoryName().ToUpperInvariant()}");
				pgoPathTranslations.Add($"{vfs}={vfs.GetDirectoryName().ToUpperInvariant()}");
			}
			Environment.SetEnvironmentVariable("PGO_PATH_TRANSLATION", String.Join(';', pgoPathTranslations.Order()));

			// Don't allow the INCLUDE environment variable to propagate. It's set by the IDE based on the IncludePath property in the project files which we
			// add to improve Visual Studio memory usage, but we don't actually need it to set when invoking the compiler. Doing so results in it being converted
			// into /I arguments by the CL driver, which results in errors due to the command line not fitting into the PDB debug record.
			Environment.SetEnvironmentVariable("INCLUDE", null);

			// Don't allow the CL or _CL_ environment variable to propagate.
			Environment.SetEnvironmentVariable("CL", null);
			Environment.SetEnvironmentVariable("_CL_", null);

			// Don't allow the LIBPATH environment variable to propagate.
			Environment.SetEnvironmentVariable("LIBPATH", null);
		}

		/// <inheritdoc/>
		public override void GetVersionInfo(List<string> Lines)
		{
			base.GetVersionInfo(Lines);
			if (EnvVars.Compiler == EnvVars.ToolChain)
			{
				Lines.Add($"Using {WindowsPlatform.GetCompilerName(EnvVars.Compiler)} {EnvVars.ToolChainVersion} toolchain ({EnvVars.ToolChainDir}) and Windows {EnvVars.WindowsSdkVersion} SDK ({EnvVars.WindowsSdkDir}).");
			}
			else if (EnvVars.Compiler == WindowsCompiler.Intel)
			{
				Lines.Add($"Using {WindowsPlatform.GetCompilerName(EnvVars.Compiler)} {EnvVars.CompilerVersion} compiler ({EnvVars.CompilerDir}) based on Clang {MicrosoftPlatformSDK.GetClangVersionForIntelCompiler(EnvVars.CompilerPath)} with {WindowsPlatform.GetCompilerName(EnvVars.ToolChain)} {EnvVars.ToolChainVersion} runtime ({EnvVars.ToolChainDir}) and Windows {EnvVars.WindowsSdkVersion} SDK ({EnvVars.WindowsSdkDir}).");
			}
			else
			{
				Lines.Add($"Using {WindowsPlatform.GetCompilerName(EnvVars.Compiler)} {EnvVars.CompilerVersion} compiler ({EnvVars.CompilerDir}) with {WindowsPlatform.GetCompilerName(EnvVars.ToolChain)} {EnvVars.ToolChainVersion} runtime ({EnvVars.ToolChainDir}) and Windows {EnvVars.WindowsSdkVersion} SDK ({EnvVars.WindowsSdkDir}).");
			}

			DynamicDebuggingInfo(Lines);

			if (Target.WindowsPlatform.ToolchainVersionWarningLevel == WarningLevel.Warning)
			{
				if (!MicrosoftPlatformSDK.IsPreferredVersion(EnvVars.Compiler, EnvVars.CompilerVersion))
				{
					Lines.Add($"Warning: {WindowsPlatform.GetCompilerName(EnvVars.Compiler)} compiler {EnvVars.CompilerVersion} is not a preferred version");
				}

				if (EnvVars.Compiler != EnvVars.ToolChain && !MicrosoftPlatformSDK.IsPreferredVersion(EnvVars.ToolChain, EnvVars.ToolChainVersion))
				{
					Lines.Add($"Notice: {WindowsPlatform.GetCompilerName(EnvVars.ToolChain)} runtime {EnvVars.ToolChainVersion} is not a preferred version");
				}
			}
		}

		public override void GetExternalDependencies(HashSet<FileItem> ExternalDependencies)
		{
			base.GetExternalDependencies(ExternalDependencies);

			UEBuildPlatformSDK? SDK = UEBuildPlatformSDK.GetSDKForPlatform(UnrealTargetPlatform.Win64.ToString());
			ExternalDependencies.UnionWith(SDK?.ExternalDependencies ?? []);

			ExternalDependencies.Add(FileItem.GetItemByFileReference(EnvVars.CompilerPath));
			ExternalDependencies.Add(FileItem.GetItemByFileReference(EnvVars.LinkerPath));
			ExternalDependencies.Add(FileItem.GetItemByFileReference(EnvVars.ResourceCompilerPath));
			if (EnvVars.Compiler != EnvVars.ToolChain)
			{
				ExternalDependencies.Add(FileItem.GetItemByFileReference(EnvVars.ToolchainCompilerPath));
			}
		}

		public static void AddDefinition(List<string> Arguments, string Definition)
		{
			// Split the definition into name and value
			int ValueIdx = Definition.IndexOf('=', StringComparison.Ordinal);
			if (ValueIdx == -1)
			{
				AddDefinition(Arguments, Definition, null);
			}
			else
			{
				AddDefinition(Arguments, Definition.Substring(0, ValueIdx), Definition.Substring(ValueIdx + 1));
			}
		}

		public static void AddDefinition(List<string> Arguments, string Variable, string? Value)
		{
			// If the value has a space in it and isn't wrapped in quotes, do that now
			if (Value != null && !Value.StartsWith("\"", StringComparison.Ordinal) && (Value.Contains(' ', StringComparison.Ordinal) || Value.Contains('$', StringComparison.Ordinal)))
			{
				Value = "\"" + Value + "\"";
			}

			if (Value != null)
			{
				Arguments.Add("/D" + Variable + "=" + Value);
			}
			else
			{
				Arguments.Add("/D" + Variable);
			}
		}

		public static new string NormalizeCommandLinePath(FileSystemReference Reference) =>
			// Try to use a relative path to shorten command line length and to enable remote distribution where absolute paths are not desired
			Reference.IsUnderDirectory(Unreal.EngineDirectory)
				? Reference.MakeRelativeTo(Unreal.EngineSourceDirectory).Replace('\\', '/')
				: Reference.FullName.Replace('\\', '/');

		public static new string NormalizeCommandLinePath(FileItem Item) => NormalizeCommandLinePath(Item.Location);
		
		public static new string NormalizeCommandLinePath(FileSystemReference Reference, CppRootPaths RootPaths) =>
			RootPaths.GetVfsOverlayPath(Reference, out string? vfsPath) ? vfsPath : NormalizeCommandLinePath(Reference);

		public static new string NormalizeCommandLinePath(FileItem Item, CppRootPaths RootPaths) => NormalizeCommandLinePath(Item.Location, RootPaths);

		public static void AddResponseFile(List<string> Arguments, FileItem ResponseFile, CppRootPaths RootPaths)
		{
			string ResponseFileString = NormalizeCommandLinePath(ResponseFile, RootPaths);
			Arguments.Add($"@{Utils.MakePathSafeToUseWithCommandLine(ResponseFileString)}");
		}

		public static void AddSourceFile(List<string> Arguments, FileItem SourceFile, CppRootPaths RootPaths)
		{
			string SourceFileString = NormalizeCommandLinePath(SourceFile, RootPaths);
			Arguments.Add(Utils.MakePathSafeToUseWithCommandLine(SourceFileString));
		}

		private static void AddIncludePath(List<string> Arguments, DirectoryReference IncludePath, WindowsCompiler Compiler, bool bSystemInclude, CppRootPaths RootPaths)
		{
			// Try to use a relative path to shorten command line length.
			string IncludePathString = NormalizeCommandLinePath(IncludePath, RootPaths);

			if (Compiler.IsClang() && bSystemInclude)
			{
				// Clang has special treatment for system headers; only system include directories are searched when include directives use angle brackets,
				// and warnings are disabled to allow compiler toolchains to be upgraded separately.
				Arguments.Add("/imsvc " + Utils.MakePathSafeToUseWithCommandLine(IncludePathString));
			}
			else if (Compiler.IsMSVC() && Compiler >= WindowsCompiler.VisualStudio2022 && bSystemInclude)
			{
				if (!Arguments.Contains("/external:W0"))
				{
					Arguments.Add("/external:W0");
				}
				// Defines a root directory that contains external headers.
				Arguments.Add("/external:I " + Utils.MakePathSafeToUseWithCommandLine(IncludePathString));
			}
			else
			{
				Arguments.Add("/I " + Utils.MakePathSafeToUseWithCommandLine(IncludePathString));
			}
		}

		public static void AddIncludePath(List<string> Arguments, DirectoryReference IncludePath, WindowsCompiler Compiler, CppRootPaths RootPaths)
		{
			AddIncludePath(Arguments, IncludePath, Compiler, false, RootPaths);
		}

		public static void AddSystemIncludePath(List<string> Arguments, DirectoryReference IncludePath, WindowsCompiler Compiler, CppRootPaths RootPaths)
		{
			AddIncludePath(Arguments, IncludePath, Compiler, true, RootPaths);
		}

		public static void AddForceIncludeFile(List<string> Arguments, FileItem ForceIncludeFile, CppRootPaths RootPaths)
		{
			string ForceIncludeFileString = NormalizeCommandLinePath(ForceIncludeFile, RootPaths);
			Arguments.Add($"/FI\"{ForceIncludeFileString}\"");
		}

		public static void AddCreatePchFile(List<string> Arguments, FileItem PchThroughHeaderFile, FileItem CreatePchFile, CppRootPaths RootPaths)
		{
			string PchThroughHeaderFilePath = PchThroughHeaderFile.Location.GetFileName();
			string CreatePchFilePath = NormalizeCommandLinePath(CreatePchFile, RootPaths);
			if (CreatePchFile.Name.EndsWith(".ifc", StringComparison.Ordinal))
			{
				Arguments.Add($"/exportHeader");
				Arguments.Add($"/translateInclude");
				Arguments.Add($"/dxifcInlineFunctions");
				Arguments.Add($"/ifcOutput {CreatePchFilePath}");
				Arguments.Add($"/dxifcSuppressDllImportTransform");
				Arguments.Add($"-d1experimental:ifcDebugRecords");
			}
			else
			{
				Arguments.Add($"/Yc\"{PchThroughHeaderFilePath}\"");
				Arguments.Add($"/Fp\"{CreatePchFilePath}\"");
			}
		}

		public static void AddUsingPchFile(List<string> Arguments, FileItem PchThroughHeaderFile, FileItem UsingPchFile, CppRootPaths RootPaths)
		{
			string PchThroughHeaderFilePath = NormalizeCommandLinePath(PchThroughHeaderFile, RootPaths);
			string UsingPchFilePath = NormalizeCommandLinePath(UsingPchFile, RootPaths);
			if (UsingPchFile.Name.EndsWith(".ifc", StringComparison.Ordinal))
			{
				Arguments.Add("/wd4127"); // with header units pragma warning disable is not propagated and this warning is very spammy so explicitly turned off here. Hoping ms will change so warning disable is propgated from HU
				Arguments.Add($"/translateInclude");
				Arguments.Add($"/headerUnit:quote {PchThroughHeaderFilePath}={UsingPchFilePath}");
				Arguments.Add($"/dxifcSuppressDllImportTransform");
				Arguments.Add($"-d1experimental:ifcDebugRecords");
			}
			else
			{
				Arguments.Add($"/Yu\"{PchThroughHeaderFilePath}\"");
				Arguments.Add($"/Fp\"{UsingPchFilePath}\"");
			}
		}

		public static void AddPreprocessedFile(List<string> Arguments, FileItem PreprocessedFile, CppRootPaths RootPaths)
		{
			string PreprocessedFileString = NormalizeCommandLinePath(PreprocessedFile, RootPaths);
			Arguments.Add("/P"); // Preprocess
			Arguments.Add("/C"); // Preserve comments when preprocessing
			Arguments.Add($"/Fi\"{PreprocessedFileString}\""); // Preprocess to a file
		}

		public static void AddObjectFile(List<string> Arguments, FileItem ObjectFile, CppRootPaths RootPaths)
		{
			string ObjectFileString = NormalizeCommandLinePath(ObjectFile, RootPaths);
			Arguments.Add($"/Fo\"{ObjectFileString}\"");
		}

		public static void AddAssemblyFile(List<string> Arguments, FileItem AssemblyFile, CppRootPaths RootPaths)
		{
			string AssemblyFileString = NormalizeCommandLinePath(AssemblyFile, RootPaths);
			Arguments.Add("/FAs");// Write out an assembly file (.asm) with the c++ code embedded in it via comments
			Arguments.Add($"/Fa\"{AssemblyFileString}\""); // Set the output patj for the asm file
		}

		public static void AddAnalyzeLogFile(List<string> Arguments, FileItem AnalyzeLogFile, CppRootPaths RootPaths)
		{
			string AnalyzeLogFileString = NormalizeCommandLinePath(AnalyzeLogFile, RootPaths);
			Arguments.Add("/analyze:log " + Utils.MakePathSafeToUseWithCommandLine(AnalyzeLogFileString));
		}

		public static void AddExperimentalLogFile(List<string> Arguments, FileItem ExperimentalLogFile, CppRootPaths RootPaths)
		{
			string ExperimentalLogFileString = NormalizeCommandLinePath(ExperimentalLogFile, RootPaths);
			Arguments.Add("/experimental:log " + Utils.MakePathSafeToUseWithCommandLine(ExperimentalLogFileString));
		}

		public static void AddSourceDependenciesFile(List<string> Arguments, FileItem SourceDependenciesFile, CppRootPaths RootPaths)
		{
			string SourceDependenciesFileString = NormalizeCommandLinePath(SourceDependenciesFile, RootPaths);
			Arguments.Add("/sourceDependencies " + Utils.MakePathSafeToUseWithCommandLine(SourceDependenciesFileString));
		}

		public static void AddSourceDependsFile(List<string> Arguments, FileItem SourceDependsFile, CppRootPaths RootPaths)
		{
			string SourceDependsFileString = NormalizeCommandLinePath(SourceDependsFile, RootPaths);
			Arguments.Add($"/clang:-MD /clang:-MF\"{SourceDependsFileString}\"");
		}

		protected static FileReference GetClangCompileProfDataFilename(CppCompileEnvironment CompileEnvironment)
		{
			string ProfDataFilename = Path.Combine(CompileEnvironment.PGODirectory!, CompileEnvironment.PGOFilenamePrefix!);
			if (File.Exists(ProfDataFilename)) 
			{
				return new FileReference(ProfDataFilename);
			}
			// If an exact match doesn't exist, fall back to an alternative. This is supported for building Test with Shipping PGO data for example
			string[] ProfDataFiles = Directory.GetFiles(CompileEnvironment.PGODirectory!, "*.profdata");
			if (ProfDataFiles.Length > 1)
			{
				throw new BuildException($"More than one .profdata file found in \"{CompileEnvironment.PGODirectory}\" and \"{ProfDataFilename}\" not found ");
			}
			if (ProfDataFiles.Length == 0)
			{
				throw new BuildException($"No .profdata files found in \"{CompileEnvironment.PGODirectory}\".");
			}
			return new FileReference(ProfDataFiles.First());
		}

		protected static FileReference GetClangLinkProfDataFilename(LinkEnvironment LinkEnvironment)
		{
			string ProfDataFilename = Path.Combine(LinkEnvironment.PGODirectory!, LinkEnvironment.PGOFilenamePrefix!);
			if (File.Exists(ProfDataFilename))
			{
				return new FileReference(ProfDataFilename);
			}
			// If an exact match doesn't exist, fall back to an alternative. This is supported for building Test with Shipping PGO data for example
			string[] ProfDataFiles = Directory.GetFiles(LinkEnvironment.PGODirectory!, "*.profdata");
			if (ProfDataFiles.Length > 1)
			{
				throw new BuildException($"More than one .profdata file found in \"{LinkEnvironment.PGODirectory}\" and \"{ProfDataFilename}\" not found ");
			}
			if (ProfDataFiles.Length == 0)
			{
				throw new BuildException($"No .profdata files found in \"{LinkEnvironment.PGODirectory}\".");
			}
			return new FileReference(ProfDataFiles.First());
		}

		private static void AddExceptionArguments(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			// Enable C++ exceptions when building with the editor or when building UHT.
			if (CompileEnvironment.bEnableExceptions)
			{
				// Enable C++ exception handling, but not C exceptions.
				Arguments.Add("/EHsc");
				Arguments.Add("/DPLATFORM_EXCEPTIONS_DISABLED=0");
			}
			else
			{
				// This is required to disable exception handling in VC platform headers.
				AddDefinition(Arguments, "_HAS_EXCEPTIONS=0");
				Arguments.Add("/DPLATFORM_EXCEPTIONS_DISABLED=1");
			}
		}

		private void AddClangResourceDirArgument(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			VersionNumber ClangVersion = Target.WindowsPlatform.Compiler == WindowsCompiler.Intel ? MicrosoftPlatformSDK.GetClangVersionForIntelCompiler(EnvVars.CompilerPath) : EnvVars.CompilerVersion;
			DirectoryReference ResourceDir = DirectoryReference.Combine(EnvVars.CompilerDir!, "lib", "clang", ClangVersion.GetComponent(0).ToString());
			Arguments.Add($"-resource-dir=\"{NormalizeCommandLinePath(ResourceDir, CompileEnvironment.RootPaths)}\"");
		}

		protected virtual void AppendCLArguments_Global(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			// Prevents the compiler from searching for include files in directories specified in the PATH and INCLUDE environment variables
			Arguments.Add("/X");

			// Workaround for MSVC 14.31 compiler crash
			if (EnvVars.Compiler.IsMSVC() && EnvVars.ToolChainVersion < new VersionNumber(14, 43) && EnvVars.ToolChainVersion >= new VersionNumber(14, 31))
			{
				Arguments.Add("/d2ssa-cfg-question-");
			}

			// @todo clang: Clang on Windows doesn't respect "#pragma warning (error: ####)", and we're not passing "/WX", so warnings are not
			// treated as errors when compiling on Windows using Clang right now.

			// NOTE re: clang: the arguments for clang-cl can be found at http://llvm.org/viewvc/llvm-project/cfe/trunk/include/clang/Driver/CLCompatOptions.td?view=markup
			// This will show the cl.exe options that map to clang.exe ones, which ones are ignored and which ones are unsupported.
			if (Target.WindowsPlatform.Compiler.IsClang())
			{
				// Sync the compatibility version with the MSVC toolchain version (14.xx which maps to advertised
				// compiler version of 19.xx).
				Arguments.Add($"-fms-compatibility-version=19.{EnvVars.ToolChainVersion.GetComponent(1)}");

				// We have 'this' vs nullptr comparisons that get optimized away for newer versions of Clang, which is undesirable until we refactor these checks.
				Arguments.Add("-fno-delete-null-pointer-checks");

				// The temp files are there to prevent corrupt output files but with uba we don't write output files until process has finished successfully so creating temp files just create wasteful IOPS
				Arguments.Add("-fno-temp-file");

				// Prevents clang-cl from reading a few files from installed visual studio folders (that we are not even using)
				Arguments.Add("-vctoolsdir undefined");

				if (Target.bWithLiveCoding)
				{
					Arguments.Add("-Z7");
					Arguments.Add("-fms-hotpatch");
					Arguments.Add("-Gy");
					Arguments.Add("-Xclang -mno-constructor-aliases");
				}

				if (Target.StaticAnalyzer == StaticAnalyzer.Default && CompileEnvironment.PrecompiledHeaderAction != PrecompiledHeaderAction.Create && !CompileEnvironment.bDisableStaticAnalysis)
				{
					Arguments.Add("-Wno-unused-command-line-argument");

					// Enable the static analyzer with default checks.
					Arguments.Add("--analyze");

					// Write out a pretty web page with navigation to understand how the analysis was derived if HTML is enabled.
#pragma warning disable CA1308 // Normalize strings to uppercase - Clang arg may be case-sensitive
					Arguments.Add($"-Xclang -analyzer-output={Target.StaticAnalyzerOutputType.ToString().ToLowerInvariant()}");
#pragma warning restore CA1308

					// Needed for some of the C++ checkers.
					Arguments.Add("-Xclang -analyzer-config -Xclang aggressive-binary-operation-simplification=true");

					// If writing to HTML, use the source filename as a basis for the report filename. 
					Arguments.Add("-Xclang -analyzer-config -Xclang stable-report-filename=true");
					Arguments.Add("-Xclang -analyzer-config -Xclang report-in-main-source-file=true");
					Arguments.Add("-Xclang -analyzer-config -Xclang path-diagnostics-alternate=true");

					// Run shallow analyze if requested.
					if (Target.StaticAnalyzerMode == StaticAnalyzerMode.Shallow)
					{
						Arguments.Add("-Xclang -analyzer-config -Xclang mode=shallow");
					}

					HashSet<string> ValidAnalyzerCheckers = ClangWarnings.ValidAnalyzerCheckers(EnvVars.CompilerPath);

					if (CompileEnvironment.StaticAnalyzerCheckers.Count > 0)
					{
						// Disable all default checks
						Arguments.Add("--analyzer-no-default-checks");

						// Only enable specific checks.
						foreach (string Checker in CompileEnvironment.StaticAnalyzerCheckers.Where(ValidAnalyzerCheckers.Contains))
						{
							Arguments.Add($"-Xclang -analyzer-checker -Xclang {Checker}");
						}
					}
					else
					{
						// Disable default checks.
						foreach (string Checker in CompileEnvironment.StaticAnalyzerDisabledCheckers.Where(ValidAnalyzerCheckers.Contains))
						{
							Arguments.Add($"-Xclang -analyzer-disable-checker -Xclang {Checker}");
						}
						// Enable additional non-default checks.
						foreach (string Checker in CompileEnvironment.StaticAnalyzerAdditionalCheckers.Where(ValidAnalyzerCheckers.Contains))
						{
							Arguments.Add($"-Xclang -analyzer-checker -Xclang {Checker}");
						}
					}
				}
				else if (Target.StaticAnalyzer == StaticAnalyzer.Default && CompileEnvironment.PrecompiledHeaderAction == PrecompiledHeaderAction.Create)
				{
					AddDefinition(Arguments, "__clang_analyzer__");
				}

				if (CompileEnvironment.RootPaths.bUseVfs)
				{
					Arguments.Add("-fcoverage-compilation-dir=.");
					Arguments.Add("-fdebug-compilation-dir=.");
					Arguments.Add("-vctoolsdir=.");
					Arguments.Add("-winsdkdir=.");
					Arguments.Add("-winsysroot=.");
					Arguments.Add("-Xclang -gno-codeview-command-line");
					Arguments.Add("-fintegrated-cc1");
				}

				// Resource dir is for built-in includes. We need it for vfs but also for uba include crawler to find includes to transfer
				AddClangResourceDirArgument(CompileEnvironment, Arguments);

			}
			else if (Target.StaticAnalyzer == StaticAnalyzer.Default && !CompileEnvironment.bDisableStaticAnalysis)
			{
				Arguments.Add("/analyze");

				if (CompileEnvironment.bStaticAnalyzerExtensions)
				{
					FileReference EspxEngine = FileReference.Combine(EnvVars.CompilerPath.Directory, "EspXEngine.dll");
					Arguments.Add($"/analyze:plugin\"{NormalizeCommandLinePath(EspxEngine)}\"");
				}

				foreach (FileReference Ruleset in CompileEnvironment.StaticAnalyzerRulesets)
				{
					Arguments.Add($"/analyze:ruleset\"{NormalizeCommandLinePath(Ruleset)}\"");
				}

				if (Target.WindowsPlatform.Compiler >= WindowsCompiler.VisualStudio2022)
				{
					// Ignore warnings in external headers
					Arguments.Add("/analyze:external-");
				}

				// Report functions that use a LOT of stack space. You can lower this value if you
				// want more aggressive checking for functions that use a lot of stack memory.
				Arguments.Add("/analyze:stacksize" + CompileEnvironment.AnalyzeStackSizeWarning);

				// Don't bother generating code, only analyze code (may report fewer warnings though.)
				//Arguments.Add("/analyze:only");

				// Re-evalulate new analysis warnings at a later time
				Arguments.Add("/wd6031"); // return value ignored: called-function could return unexpected value

				if (CompileEnvironment.SystemIncludePaths.Concat(CompileEnvironment.SharedSystemIncludePaths).Any(x => x.FullName.Contains("GoogleTest", StringComparison.OrdinalIgnoreCase)))
				{
					Arguments.Add("/wd6326"); // Potential comparison of a constant with another constant
				}

				// A lookup table of size 365 isn't sufficient to handle leap years https://learn.microsoft.com/en-us/cpp/code-quality/c6393
				Arguments.Add("/wd6393");
			}

			// Prevents the compiler from displaying its logo for each invocation.
			Arguments.Add("/nologo");

			// Enable intrinsic functions.
			Arguments.Add("/Oi");

			// Trace includes
			if (Target.bShowIncludes)
			{
				if (Target.WindowsPlatform.Compiler.IsClang())
				{
					Arguments.Add("/clang:--trace-includes");
				}
				else if (Target.WindowsPlatform.Compiler.IsMSVC())
				{
					Arguments.Add("/showIncludes");
				}
			}

			// Print absolute paths in diagnostics
			if (Target.WindowsPlatform.Compiler.IsClang())
			{
				Arguments.Add("-fdiagnostics-absolute-paths");
			}
			else if (Target.WindowsPlatform.Compiler.IsMSVC())
			{
				Arguments.Add("/FC");

				// Add column (in addition to line) to error messages
				Arguments.Add("/diagnostics:caret");
			}

			if (Target.WindowsPlatform.Compiler.IsClang() && Target.Platform.IsInGroup(UnrealPlatformGroup.Windows))
			{
				// Tell the Clang compiler to generate 64-bit code
				if (CompileEnvironment.Architecture.bIsX64)
				{
					Arguments.Add("--target=x86_64-pc-windows-msvc");

					// UE5 minspec is 4.2
					Arguments.Add("-msse4.2");

					// Use tpause on supported processors.
					Arguments.Add("-mwaitpkg");
				}
				else if (CompileEnvironment.Architecture == UnrealArch.Arm64)
				{
					Arguments.Add("--target=arm64-pc-windows-msvc");
				}
				else if (CompileEnvironment.Architecture == UnrealArch.Arm64ec)
				{
					Arguments.Add("--target=arm64ec-pc-windows-msvc");
				}
			}

			// Compile into an .obj file, and skip linking.
			if (!Target.WindowsPlatform.Compiler.IsClang() || !CompileEnvironment.bPreprocessOnly)
			{
				Arguments.Add("/c");
			}

			// Put symbols into different sections so the linker can remove them.
			if (Target.WindowsPlatform.bOptimizeGlobalData)
			{
				Arguments.Add("/Gw");
			}

			// Enable syntax check only mode
			if (Target.WindowsPlatform.bSyntaxCheckOnly)
			{
				Arguments.Add("/Zs");
			}

			// Reduce optimizations for huge functions, may improve compile time a the expense of speed for functions over the threshold
			if (Target.WindowsPlatform.Compiler.IsMSVC() && Target.WindowsPlatform.bReducedOptimizeHugeFunctions)
			{
				Arguments.Add("/d2ReducedOptimizeHugeFunctions");
				Arguments.Add($"/d2ReducedOptimizeThreshold:{Target.WindowsPlatform.ReducedOptimizeHugeFunctionsThreshold}");
			}

			if (IsDynamicDebuggingEnabled)
			{
				Arguments.Add("/dynamicdeopt");
			}

			// Separate functions for linker.
			Arguments.Add("/Gy");

			// Microsoft recommends not passing /Zm except in very limited circumstances, please see:
			// https://learn.microsoft.com/en-us/cpp/build/reference/zm-specify-precompiled-header-memory-allocation-limit
			if (Target.WindowsPlatform.PCHMemoryAllocationFactor > 0)
			{
				Arguments.Add($"/Zm{Target.WindowsPlatform.PCHMemoryAllocationFactor}");
			}

			// Fix Incredibuild errors with helpers using heterogeneous character sets
			Arguments.Add("/utf-8");

			// Disable "The file contains a character that cannot be represented in the current code page" warning for non-US windows.
			Arguments.Add("/wd4819");

			// Disable Microsoft extensions on VS2017+ for improved standards compliance.
			if (Target.WindowsPlatform.bStrictConformanceMode)
			{
				// This define is needed to ensure that MSVC static analysis mode doesn't declare attributes that are incompatible with strict conformance mode
				AddDefinition(Arguments, "SAL_NO_ATTRIBUTE_DECLARATIONS=1");

				Arguments.Add("/permissive-");
				if (Target.WindowsPlatform.Compiler.IsMSVC())
				{
					Arguments.Add("/Zc:strictStrings-"); // Have to disable strict const char* semantics due to Windows headers not being compliant.
				}
			}
			else if (Target.WindowsPlatform.Compiler.IsMSVC())
			{
				Arguments.Add("/Zc:hiddenFriend");
			}

			if (Target.WindowsPlatform.bUpdatedCPPMacro)
			{
				Arguments.Add("/Zc:__cplusplus");
			}

			if (CompileEnvironment.bVcRemoveUnreferencedComdat)
			{
				// Suppress generation of object code for unreferenced inline functions. Enabling this option is more standards compliant, and causes a big reduction
				// in object file sizes (and link times) due to the amount of stuff we inline.
				Arguments.Add("/Zc:inline");
			}

			if (Target.WindowsPlatform.Compiler.IsMSVC())
			{
				Arguments.Add("/Zc:preprocessor");
			}

			if (Target.WindowsPlatform.bStrictEnumTypesConformance && Target.WindowsPlatform.Compiler.IsMSVC())
			{
				Arguments.Add("/Zc:enumTypes");
			}

			if (Target.WindowsPlatform.bStrictODRViolationConformance && Target.WindowsPlatform.bOptimizeGlobalData && Target.WindowsPlatform.Compiler.IsMSVC())
			{
				Arguments.Add("/Zc:checkGwOdr");
			}

			// Ignore secure CRT warnings on Clang
			if (Target.WindowsPlatform.Compiler.IsClang())
			{
				AddDefinition(Arguments, "_CRT_SECURE_NO_WARNINGS");
			}

			// If compiling as a DLL, set the relevant defines
			if (CompileEnvironment.bIsBuildingDLL)
			{
				AddDefinition(Arguments, "_WINDLL");
			}

			// Maintain the old std::aligned_storage behavior from VS from v15.8 onwards, in case of prebuilt third party libraries are reliant on it
			AddDefinition(Arguments, "_DISABLE_EXTENDED_ALIGNED_STORAGE");

			// Do not allow inline method expansion if E&C support is enabled or inline expansion has been disabled, 
			// or if we are compiling in a debug build with `clang-cl`, since this will interfere with debugging capabilities.
			if (!CompileEnvironment.bSupportEditAndContinue && CompileEnvironment.bUseInlining
				&& !(Target.WindowsPlatform.Compiler.IsClang() && CompileEnvironment.Configuration == CppConfiguration.Debug))
			{
				Arguments.Add($"/Ob{Math.Clamp(Target.WindowsPlatform.InlineFunctionExpansionLevel, 1, 3)}");
			}
			else
			{
				// Specifically disable inline expansion to override /O1,/O2/ or /Ox if set
				Arguments.Add("/Ob0");
			}

			// Deterministic compile support
			if (CompileEnvironment.bDeterministic)
			{
				if (Target.WindowsPlatform.Compiler.IsMSVC())
				{
					Arguments.Add("/experimental:deterministic");

					// warning C5049: Embedding a full path may result in machine-dependent output
					Arguments.Add("/wd5049");
				}
			}

			if (Target.WindowsPlatform.Compiler.IsClang())
			{
				Arguments.Add("/Brepro");
			}

			if (Target.WindowsPlatform.Compiler.IsMSVC() && Target.WindowsPlatform.bVCFastFail)
			{
				Arguments.Add("/fastfail");
			}

			if (Target.WindowsPlatform.Compiler.IsMSVC() && Target.WindowsPlatform.bVCExtendedWarningInfo)
			{
				Arguments.Add("/d2ExtendedWarningInfo");
			}

			if (Target.WindowsPlatform.bEnableInstrumentation)
			{
				AddDefinition(Arguments, "USING_INSTRUMENTATION=1");

				// When AutoRTFM is also active, verse-clang-cl handles instrumentation
				// via -fepictsan instead of using instr-clang-cl.exe.
				if (Target.WindowsPlatform.Compiler == WindowsCompiler.ClangRTFM)
				{
					Arguments.Add("-fepictsan");
				}
			}

			// Address sanitizer
			// TODO(jamal.fanaian): We should probably not make this a global property instead?
			if (Target.WindowsPlatform.bEnableAddressSanitizer && !CompileEnvironment.bDisableDynamicAnalysis)
			{
				// Enable address sanitizer. This also requires companion libraries at link time.
				// Works for clang too.
				Arguments.Add("/fsanitize=address");

				if (Target.WindowsPlatform.Compiler.IsClang())
				{
					// Don't check global variables when address sanitizer is enabled. This can lead to false / positive with ASan mixing types of different size in its shadow space
					// (for instance, null character being shared for char and wchar_t but having different sizes (1 for char and 2 for wchar_t))
					Arguments.Add("-mllvm -asan-globals=0");
				}

				if (Target.WindowsPlatform.Compiler.IsMSVC())
				{
					// MSVC has no support for __has_feature(address_sanitizer)
					AddDefinition(Arguments, "USING_ADDRESS_SANITISER=1");
				}

				// Disabling a couple annotations to workaround an issue related to building third party libraries with different options than the main binary.
				// This fixes an error that looks like this: error LNK2038: mismatch detected for 'annotate_string': value '0' doesn't match value '1' in Module.Core.XX_of_YY.cpp.obj
				AddDefinition(Arguments, "_DISABLE_STRING_ANNOTATION=1");
				AddDefinition(Arguments, "_DISABLE_VECTOR_ANNOTATION=1");

				// Currently the ASan headers are not default around. They can be found at this location so lets use this until this is resolved in the toolchain
				// Jira with some more info and the MSVC bug at UE-144727
				AddSystemIncludePath(Arguments, DirectoryReference.Combine(EnvVars.CompilerDir, "crt", "src"), Target.WindowsPlatform.Compiler, CompileEnvironment.RootPaths);
			}

			if (Target.WindowsPlatform.bEnableUndefinedBehaviorSanitizer)
			{
				if (Target.WindowsPlatform.Compiler.IsClang())
				{
					Arguments.Add("-fsanitize=undefined");
				}
				else
				{
					Arguments.Add("/fsanitize=undefined");
				}
			}

			if (Target.WindowsPlatform.bEnableAutoRTFMSanitizer)
			{
				Arguments.Add("-fsanitize=autortfm");
			}

			if (Target.WindowsPlatform.bEnableLibFuzzer)
			{
				if (Target.WindowsPlatform.Compiler.IsClang())
				{
					Arguments.Add("-fsanitize=fuzzer");
				}
				else
				{
					Arguments.Add("/fsanitize=fuzzer");
				}
			}

			// Code Coverage Support
			if (CompileEnvironment.bCodeCoverage)
			{
				// Clang llvm-cov coverage flags
				if(Target.WindowsPlatform.Compiler.IsClang())
				{
					Arguments.Add("/Od");
					Arguments.Add("/clang:-fprofile-instr-generate");
					Arguments.Add("/clang:-fcoverage-mapping");
					Log.TraceInformationOnce("Enabling Clang code coverage flags");
				}
			}

			//
			//	Debug
			//
			if (CompileEnvironment.Configuration == CppConfiguration.Debug)
			{
				// Disable compiler optimization.
				Arguments.Add("/Od");

				// `/Os` causes `clang-cl` to effectively compile with `-Os` which enables optimizations,
				// rendering passing `/Od` to it useless.
				if (!Target.WindowsPlatform.Compiler.IsClang())
				{
					// Favor code size (especially useful for embedded platforms).
					Arguments.Add("/Os");
				}

				// Runtime checks and ASan are incompatible.
				if (!Target.WindowsPlatform.bEnableAddressSanitizer)
				{
					Arguments.Add("/RTCs");
				}
			}
			//
			//	Development
			//
			else
			{
				if (!CompileEnvironment.bOptimizeCode)
				{
					// Disable compiler optimization.
					Arguments.Add("/Od");
				}
				else
				{

					if (Target.WindowsPlatform.Compiler.IsClang())
					{
						switch (CompileEnvironment.OptimizationLevel)
						{
							case OptimizationMode.Size:
								{
									// We use Clang's -Oz here because it produces a smaller binary than /O1
									Arguments.Add("-Xclang -Oz");
								}
								break;
							case OptimizationMode.SizeAndSpeed:
								{
									// Typically the Cl compatible /Ox /Os args result in a smaller binary than just -XClang -Os (by 30MB or so in a Dev build)
									// However, when using PGO, -XClang -Os is the same size but marginally faster
									if ( CompileEnvironment.bPGOProfile || CompileEnvironment.bPGOOptimize )
									{
										Arguments.Add("-Xclang -Os");
									}
									else
									{
										Arguments.Add("/Ox");
										Arguments.Add("/Os");
									}
								}
								break;
							case OptimizationMode.Speed:
								{
									if ( CompileEnvironment.bPGOProfile || CompileEnvironment.bPGOOptimize )
									{
										// This is optimal both for speed and size when PGO is enabled
										Arguments.Add("-Xclang -Os");
									}
									else
									{
										// Maximum optimizations. We just use the MSVC flags and let the Clang-Cl driver translate
										Arguments.Add("/Ox");
										Arguments.Add("/Ot");
									}
								}
								break;
						}
					}
					else
					{
						// Maximum optimizations.
						Arguments.Add("/Ox");

						if (CompileEnvironment.OptimizationLevel != OptimizationMode.Speed)
						{
							Arguments.Add("/Os");
						}
						else
						{
							// Favor code speed.
							Arguments.Add("/Ot");
						}
					}

					// Coalesce duplicate strings
					Arguments.Add("/GF");

					// Only omit frame pointers on the PC (which is implied by /Ox) if wanted.
					if (CompileEnvironment.bOmitFramePointers == false)
					{
						Arguments.Add("/Oy-");
					}
				}
			}

			// Volatile Metadata is enabled by default and improves x64 emulation on arm64, but may come at a small perfomance cost
			if (Target.WindowsPlatform.Compiler.IsMSVC() && CompileEnvironment.Architecture == UnrealArch.X64 && Target.WindowsPlatform.bDisableVolatileMetadata)
			{
				Arguments.Add("/volatileMetadata-");
			}

			//
			// LTCG and PGO
			//
			bool bEnableLTCG =
				CompileEnvironment.bPGOProfile ||
				CompileEnvironment.bPGOOptimize ||
				CompileEnvironment.bAllowLTCG;
			if (bEnableLTCG && !Target.WindowsPlatform.Compiler.IsClang())
			{
				// Enable link-time code generation.
				Arguments.Add("/GL");
			}

			if (Target.WindowsPlatform.Compiler.IsClang())
			{
				if (bEnableLTCG && Target.WindowsPlatform.bAllowClangLinker)
				{
					// Enable link-time code generation. Requires Clang linker.
					Arguments.Add("-flto=thin");
				}

				if (CompileEnvironment.bPGOProfile)
				{
					// Generate instrumented code.
					if (Target.WindowsPlatform.bSampleBasedPGO)
					{
						if (Target.WindowsPlatform.Compiler.IsIntel())
						{
							Arguments.Add("-fprofile-sample-generate");
							Arguments.Add($"-fprofile-dwo-dir=\"{CompileEnvironment.PGODirectory!}\"");
							Arguments.Add("-gsplit-dwarf");
						}
						else
						{
							Arguments.Add("-gdwarf");
							Arguments.Add("-gsplit-dwarf");
							Arguments.Add("-gline-tables-only");
							Arguments.Add("/clang:-fdebug-info-for-profiling");
							Arguments.Add("/clang:-funique-internal-linkage-names");
						}
					}
					else
					{
						Arguments.Add("-fprofile-generate");

						// generate minimal debugging information for profiling
						Arguments.Add("-gline-tables-only");
					}
				}
				else if (CompileEnvironment.bPGOOptimize)
				{
					// Use a merged profdata file.
					if (Directory.Exists(CompileEnvironment.PGODirectory))
					{
						FileReference ProfDataFilename = GetClangCompileProfDataFilename(CompileEnvironment);
						Log.TraceInformationOnce($"Using PGO profile data \"{ProfDataFilename}\"");
						if (Target.WindowsPlatform.bSampleBasedPGO)
						{
							if (Target.WindowsPlatform.Compiler.IsIntel())
							{
								Arguments.Add($"-fprofile-sample-use=\"{ProfDataFilename}\"");
							}
							else
							{
								Arguments.Add($"/clang:-fprofile-sample-use=\"{ProfDataFilename}\"");
							}
						}
						else
						{
							Arguments.Add($"-fprofile-use=\"{ProfDataFilename}\"");
						}
					}
				}
			}

			//
			//	PC
			//
			if (CompileEnvironment.Architecture == UnrealArch.X64 && CompileEnvironment.MinCpuArchX64 != MinimumCpuArchitectureX64.None)
			{
				MinimumCpuArchitectureX64 MinCpuArchX64 = CompileEnvironment.MinCpuArchX64;
				// Define /arch:AVX[2,512] for the current compilation unit.  Machines without AVX support will crash on any SSE/AVX instructions if they run this compilation unit.
				if (Target.WindowsPlatform.Compiler.IsMSVC())
				{
					// Downgrade AVX if unsupported by MSVC version
					if (MinCpuArchX64 >= MinimumCpuArchitectureX64.AVX10_2 && EnvVars.CompilerVersion < new VersionNumber(14, 50))
					{
						MinCpuArchX64 = MinimumCpuArchitectureX64.AVX10_1;
					}
					if (MinCpuArchX64 >= MinimumCpuArchitectureX64.AVX10_1 && EnvVars.CompilerVersion < new VersionNumber(14, 42))
					{
						MinCpuArchX64 = MinimumCpuArchitectureX64.AVX512;
					}

					Arguments.Add($"/arch:{MinCpuArchX64}".Replace('_', '.'));
				}
				else if (Target.WindowsPlatform.Compiler.IsClang())
				{
					VersionNumber ClangVersion = Target.WindowsPlatform.Compiler.IsIntel() ? MicrosoftPlatformSDK.GetClangVersionForIntelCompiler(EnvVars.CompilerPath) : EnvVars.CompilerVersion;
					// Downgrade AVX if unsupported by Clang version
					if (MinCpuArchX64 >= MinimumCpuArchitectureX64.AVX10_2 && ClangVersion < new VersionNumber(20, 1))
					{
						MinCpuArchX64 = MinimumCpuArchitectureX64.AVX10_1;
					}
					if (MinCpuArchX64 >= MinimumCpuArchitectureX64.AVX10_1 && ClangVersion < new VersionNumber(18, 1))
					{
						MinCpuArchX64 = MinimumCpuArchitectureX64.AVX512;
					}

					if (MinCpuArchX64 >= MinimumCpuArchitectureX64.AVX)
					{
						// Apparently MSVC enables (a subset?) of BMI (bit manipulation instructions) when /arch:AVX is set. Some code relies on this, so mirror it by enabling BMI1
						Arguments.Add("-mavx");
						Arguments.Add("-mbmi");
					}
					if (MinCpuArchX64 >= MinimumCpuArchitectureX64.AVX2)
					{
						Arguments.Add("-mavx2");
						Arguments.Add("-mfma"); // Matching MSVC which includes FMA with /arch:AVX2
					}
					if (MinCpuArchX64 >= MinimumCpuArchitectureX64.AVX512)
					{
						// Match MSVC which says (https://learn.microsoft.com/en-us/cpp/build/reference/arch-x64?view=msvc-170):
						// > The __AVX512F__, __AVX512CD__, __AVX512BW__, __AVX512DQ__ and __AVX512VL__ preprocessor symbols are defined when the /arch:AVX512 compiler option is specified
						if (MinCpuArchX64 < MinimumCpuArchitectureX64.AVX10_1)
						{
							Arguments.Add("-mavx512f");
							Arguments.Add("-mavx512cd");
							Arguments.Add("-mavx512bw");
							Arguments.Add("-mavx512dq");
							Arguments.Add("-mavx512vl");
						}
					}
					if (MinCpuArchX64 >= MinimumCpuArchitectureX64.AVX10_1)
					{
						Arguments.Add("-mavx10.1");
					}
					if (MinCpuArchX64 >= MinimumCpuArchitectureX64.AVX10_2)
					{
						Arguments.Add("-mavx10.2");
					}
				}

				// AVX available implies sse4 and sse2 available.
				// Inform Unreal code that we have sse2, sse4, and AVX, both available to compile and available to run
				// By setting the ALWAYS_HAS defines, we direct Unreal code to skip cpuid checks to verify that the running hardware supports sse/avx.
				AddDefinition(Arguments, "PLATFORM_ENABLE_VECTORINTRINSICS=1");
				AddDefinition(Arguments, "PLATFORM_MAYBE_HAS_AVX=1");
				AddDefinition(Arguments, "PLATFORM_ALWAYS_HAS_AVX=1");
				if (MinCpuArchX64 >= MinimumCpuArchitectureX64.AVX2)
				{
					AddDefinition(Arguments, "PLATFORM_ALWAYS_HAS_AVX_2=1");
				}
				if (MinCpuArchX64 >= MinimumCpuArchitectureX64.AVX512)
				{
					AddDefinition(Arguments, "PLATFORM_ALWAYS_HAS_AVX_512=1");
				}
				if (MinCpuArchX64 >= MinimumCpuArchitectureX64.AVX10_1)
				{
					AddDefinition(Arguments, "PLATFORM_ALWAYS_HAS_AVX_10_1=1");
				}
				if (MinCpuArchX64 >= MinimumCpuArchitectureX64.AVX10_2)
				{
					AddDefinition(Arguments, "PLATFORM_ALWAYS_HAS_AVX_10_2=1");
				}
			}

			if (CompileEnvironment.Architecture == UnrealArch.X64 && CompileEnvironment.bMinCpuArchAPX)
			{
				if (Target.WindowsPlatform.Compiler.IsIntel())
				{
					Arguments.Add("-mapxf");
				}
				else if (Target.WindowsPlatform.Compiler.IsClang() && EnvVars.CompilerVersion >= new VersionNumber(21))
				{
					Arguments.Add("-mapxf");
				}
				else if (Target.WindowsPlatform.Compiler.IsMSVC() && EnvVars.CompilerVersion >= new VersionNumber(14, 51))
				{
					Arguments.Add("/feature:APX");
				}
			}

			if (CompileEnvironment.Architecture == UnrealArch.X64 && CompileEnvironment.MinCpuArchX64 == MinimumCpuArchitectureX64.None && Target.WindowsPlatform.Compiler.IsIntel())
			{
				// Intel oneAPI has /arch switch for sse4.2. Use it when no minimum is set.
				Arguments.Add("/arch:sse4.2");
			}

			// Prompt the user before reporting internal errors to Microsoft.
			Arguments.Add("/errorReport:prompt");

			AddExceptionArguments(CompileEnvironment, Arguments);

			// If enabled, create debug information.
			if (CompileEnvironment.bCreateDebugInfo)
			{
				/*
				Disabled until we can find a better means to limit the cost of /JMC as it can add considerable runtime overhead

				// Enable Just-My-Code in Debug or Development, unless on Clang
				if ((CompileEnvironment.Configuration == CppConfiguration.Debug || CompileEnvironment.Configuration == CppConfiguration.Development) && !Target.WindowsPlatform.Compiler.IsClang())
				{
					Arguments.Add("/JMC");
				}
				*/

				// Store debug info in .pdb files.
				// @todo clang: PDB files are emited from Clang but do not fully work with Visual Studio yet (breakpoints won't hit due to "symbol read error")
				// @todo clang (update): as of clang 3.9 breakpoints work with PDBs, and the callstack is correct, so could be used for crash dumps. However debugging is still impossible due to the large amount of unreadable variables and unpredictable breakpoint stepping behaviour
				if (CompileEnvironment.bUsePDBFiles || CompileEnvironment.bSupportEditAndContinue)
				{
					// Create debug info suitable for E&C if wanted.
					if (CompileEnvironment.bSupportEditAndContinue)
					{
						Arguments.Add("/ZI");
					}
					// Regular PDB debug information.
					else
					{
						Arguments.Add("/Zi");
					}
					// We need to add this so VS won't lock the PDB file and prevent synchronous updates. This forces serialization through MSPDBSRV.exe.
					// See http://msdn.microsoft.com/en-us/library/dn502518.aspx for deeper discussion of /FS switch.
					if (CompileEnvironment.bUseIncrementalLinking)
					{
						Arguments.Add("/FS");
					}
				}
				// Store C7-format debug info in the .obj files, which is faster.
				else
				{
					Arguments.Add("/Z7");
				}

				if (Target.WindowsPlatform.Compiler.IsClang())
				{
					VersionNumber ClangVersion = Target.WindowsPlatform.Compiler == WindowsCompiler.Intel ? MicrosoftPlatformSDK.GetClangVersionForIntelCompiler(EnvVars.CompilerPath) : EnvVars.CompilerVersion;

					// https://clang.llvm.org/docs/UsersManual.html#cmdoption-gline-tables-only
					if (CompileEnvironment.bDebugLineTablesOnly)
					{
						Arguments.Add("-gline-tables-only");
					}

					// https://clang.llvm.org/docs/UsersManual.html#cmdoption-fstandalone-debug
					if (Target.WindowsPlatform.bClangStandaloneDebug)
					{
						Arguments.Add("-fstandalone-debug");
					}

					if (CompileEnvironment.bDebugNoInlineLineTables)
					{
						Arguments.Add("-Xclang -gno-inline-line-tables");
					}

					if (CompileEnvironment.bDebugSimpleTemplateNames)
					{
						Arguments.Add("-Xclang -gsimple-template-names");
					}

					// https://clang.llvm.org/docs/UsersManual.html#cmdoption-feliminate-unused-debug-types
					// Intel ICX 2024.2 does not have this option, but reports Clang 19, so turn it off here
					// It also doesn't appear to be supported for Windows Arm64
					if (ClangVersion >= new VersionNumber(19) && !Target.WindowsPlatform.Compiler.IsIntel() && !CompileEnvironment.bDebugLineTablesOnly && Target.WindowsPlatform.Architecture.bIsX64)
					{
						Arguments.Add("-fno-eliminate-unused-debug-types");
					}
				}
			}

			// Specify the appropriate runtime library based on the platform and config.
			if (CompileEnvironment.bUseStaticCRT)
			{
				if (CompileEnvironment.bUseDebugCRT)
				{
					Arguments.Add("/MTd");
				}
				else
				{
					Arguments.Add("/MT");
				}
			}
			else
			{
				if (CompileEnvironment.bUseDebugCRT)
				{
					Arguments.Add("/MDd");
				}
				else
				{
					Arguments.Add("/MD");
				}
			}

			if (Target.WindowsPlatform.Compiler.IsMSVC())
			{
				// Allow large object files to avoid hitting the 2^16 section limit when running with -StressTestUnity.
				// Note: not needed for clang, it implicitly upgrades COFF files to bigobj format when necessary.
				Arguments.Add("/bigobj");

				// This flag changes so msvc delay instantiation of virtual methods until they are needed.
				// This brings msvc behavior closer to clang/gcc and also reduces pch/binary size a bit while increasing obj files a tiny bit (net win)
				Arguments.Add("/dximplicitlyinstantiatevirtualfunction-");
			}

			FPSemanticsMode FPSemantics = CompileEnvironment.FPSemantics;
			if (Target.WindowsPlatform.Compiler.IsClang())
			{
				// FMath::Sqrt calls get inlined and when reciprical is taken, turned into an rsqrtss instruction,
				// which is *too* imprecise for, e.g., TestVectorNormalize_Sqrt in UnrealMathTest.cpp
				// TODO: Observed in clang 7.0, presumably the same in Intel C++ Compiler?
				FPSemantics = FPSemanticsMode.Precise;
			}

			// If we are the editor we want to be precise to ensure more consistent data generation across builds.
			// Programs may get used as child processes for data generation.
			if (FPSemantics == FPSemanticsMode.Default && Target.DefaultBuildSettings >= BuildSettingsVersion.V7)
			{
				if (Target.Type == TargetType.Editor || Target.Type == TargetType.Program)
				{
					FPSemantics = FPSemanticsMode.Precise;
				}
			}

			switch (FPSemantics)
			{
				case FPSemanticsMode.Default: // Default is imprecise FP semantics.
				case FPSemanticsMode.Imprecise: Arguments.Add("/fp:fast"); break;
				case FPSemanticsMode.Precise:
					if (Target.WindowsPlatform.Compiler.IsClang())
					{
						// Clang defaults to -ffp-contract=on even with /fp:precise, which allows fusing multiplications and additions into FMAs.
						// Passing this alongside /fp:precise emits a warning, but /fp:precise is already the default so it's safe to omit.
						Arguments.Add("/clang:-ffp-contract=off");
					}
					else
					{
						Arguments.Add("/fp:precise");
					}
					break;
				default:
					throw new BuildException($"Unsupported FP semantics: {FPSemantics}");
			}

			if (Target.WindowsPlatform.Compiler.IsIntel())
			{
				Arguments.Add("/Qvec-peel-loops");
				Arguments.Add("/Qvec-remainder-loops");
				Arguments.Add("/Qvec-with-mask");
				Arguments.Add("/Qopt-dynamic-align");
				Arguments.Add("/Qunroll");
				Arguments.Add("/Qopt-streaming-stores:auto");
				Arguments.Add("/Qopt-jump-tables");
				Arguments.Add("/Qbranches-within-32B-boundaries");

				VersionNumber ClangVersion = MicrosoftPlatformSDK.GetClangVersionForIntelCompiler(EnvVars.CompilerPath);

				// Intel ICX 2025.1 reports ldexpf as part of math library, but comes up as unresolved
				if (ClangVersion >= new VersionNumber(20))
				{
					Arguments.Add("-fno-builtin-ldexpf");
				}
			}

			// Intel oneAPI compiler does not support /Zo
			if (CompileEnvironment.bOptimizeCode && Target.WindowsPlatform.Compiler != WindowsCompiler.Intel)
			{
				// Allow optimized code to be debugged more easily.  This makes PDBs a bit larger, but doesn't noticeably affect
				// compile times.  The executable code is not affected at all by this switch, only the debugging information.
				Arguments.Add("/Zo");
			}

			if (Target.WindowsPlatform.StructMemberAlignment != null)
			{
				Arguments.Add($"/Zp{Target.WindowsPlatform.StructMemberAlignment}");
			}

			if (CompileEnvironment.DefaultWarningLevel == WarningLevel.Error)
			{
				Arguments.Add("/WX");
			}

			if (CompileEnvironment.bUseHeaderUnitsForPch)
			{
				Arguments.Add("/wd4324"); // 'struct_name' : structure was padded due to __declspec(align())
				Arguments.Add("/wd4201"); // nonstandard extension used: nameless struct/union
				Arguments.Add("/wd4275"); // non - DLL-interface class 'class_1' used as base for DLL-interface class 'class_2'
				Arguments.Add("/wd4251"); // 'type' : class 'type1' needs to have dll-interface to be used by clients of class 'type2'
				Arguments.Add("/wd4702"); // unreachable code
				Arguments.Add("/wd4180"); // qualifier applied to function type has no meaning; ignored
				Arguments.Add("/wd4996"); // deprecation
				Arguments.Add("/wd4200"); // nonstandard extension used: zero - sized array in struct/union
				Arguments.Add("/wd4714"); // marked as __forceinline not inlined
				Arguments.Add("/wd5321"); // nonstandard extension used: encoding '\xE5' as a multi-byte utf-8 character
			}

			// Disable C4702: unreachable code when use inlining is disabled
			if (!CompileEnvironment.bUseInlining && EnvVars.Compiler.IsMSVC())
			{
				Arguments.Add("/wd4702"); // unreachable code
			}

			// Downgrade C4702: unreachable code to a warning when running LTCG or PGO
			if (CompileEnvironment.bPGOOptimize || CompileEnvironment.bPGOProfile || CompileEnvironment.bAllowLTCG)
			{
				Arguments.Add("/wd4702 /w44702");
			}

			AppendCLArguments_CompileWarnings(CompileEnvironment, Arguments);

			if (CompileEnvironment.Architecture == UnrealArch.Arm64ec)
			{
				Arguments.Add("/arm64EC");
				// The latest vc toolchain requires that these be manually set for arm64ec.
				AddDefinition(Arguments, "_ARM64EC_");
				AddDefinition(Arguments, "_ARM64EC_WORKAROUND_");
				AddDefinition(Arguments, "ARM64EC");
				AddDefinition(Arguments, "AMD64");
				AddDefinition(Arguments, "_AMD64_");
				AddDefinition(Arguments, "_WINDOWS");
				AddDefinition(Arguments, "WIN32");
			}
		}

		private void AddIncludePathArguments(CppCompileEnvironment CompileEnvironment, WindowsCompiler Compiler, List<string> Arguments)
		{
			foreach (DirectoryReference IncludePath in CompileEnvironment.UserIncludePaths)
			{
				AddIncludePath(Arguments, IncludePath, Compiler, CompileEnvironment.RootPaths);
			}

			foreach (DirectoryReference IncludePath in CompileEnvironment.SystemIncludePaths)
			{
				AddSystemIncludePath(Arguments, IncludePath, Compiler, CompileEnvironment.RootPaths);
			}

			foreach (DirectoryReference IncludePath in EnvVars.IncludePaths)
			{
				AddSystemIncludePath(Arguments, IncludePath, Compiler, CompileEnvironment.RootPaths);
			}
		}

		internal void AppendCLArguments_GlobalIWYU(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			AddIncludePathArguments(CompileEnvironment, Target.WindowsPlatform.Compiler, Arguments);

			Arguments.Add($"-fms-compatibility-version=19.{EnvVars.ToolChainVersion.GetComponent(1)}");
			Arguments.Add("-Wno-invalid-constexpr");

			AddDefinition(Arguments, "_CRT_USE_BUILTIN_OFFSETOF");
			AddDefinition(Arguments, "_CRT_SECURE_NO_WARNINGS");

			AddExceptionArguments(CompileEnvironment, Arguments);

			// This is a bit messed up. Since include-what-you-use exe and this path does not really match.
			// But some code includes cpuid.h behind #if PLATFORM_COMPILER_CLANG so we need this folder to find that file
			AddClangResourceDirArgument(CompileEnvironment, Arguments);

			AppendCLArguments_CompileWarnings(CompileEnvironment, Arguments);
		}

		internal virtual void AppendCLArguments_H(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			AppendCLArguments_CPP(CompileEnvironment, Arguments);
			
			if (Target.WindowsPlatform.Compiler.IsClang())
			{
				ClangWarnings.GetHeaderDisabledWarnings(Arguments);
			}
		}

		internal virtual void AppendCLArguments_CPP(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			// Explicitly compile the file as C++.
			Arguments.Add("/TP");

			if (Target.WindowsPlatform.Compiler.IsClang())
			{
				string FileSpecifier = "c++";
				if (CompileEnvironment.PrecompiledHeaderAction == PrecompiledHeaderAction.Create)
				{
					// Tell Clang to generate a PCH header
					FileSpecifier += "-header";
				}

				Arguments.Add($"-Xclang -x -Xclang \"{FileSpecifier}\"");
			}

			if (!CompileEnvironment.bEnableBufferSecurityChecks)
			{
				// This will disable buffer security checks (which are enabled by default) that the MS compiler adds around arrays on the stack,
				// Which can add some performance overhead, especially in performance intensive code
				// Only disable this if you know what you are doing, because it will be disabled for the entire module!
				Arguments.Add("/GS-");
			}

			// Configure RTTI
			if (CompileEnvironment.bUseRTTI)
			{
				// Enable C++ RTTI.
				Arguments.Add("/GR");
			}
			else
			{
				// Disable C++ RTTI.
				Arguments.Add("/GR-");
			}

			switch (CompileEnvironment.CppStandard)
			{
				case CppStandardVersion.Cpp20:
					Arguments.Add("/std:c++20");
					break;
				case CppStandardVersion.Cpp23:
					if (EnvVars.Compiler.IsMSVC() && EnvVars.CompilerVersion >= new VersionNumber(14, 43, 34808))
					{
						Arguments.Add("/std:c++23preview");
					}
					else
					{
						Arguments.Add("/std:c++latest");
					}
					break;
				case CppStandardVersion.Latest:
					Arguments.Add("/std:c++latest");
					break;
				default:
					throw new BuildException($"Unsupported C++ standard type set: {CompileEnvironment.CppStandard}");
			}

			if (CompileEnvironment.CppStandard >= CppStandardVersion.Cpp20)
			{
				// warning C5054: operator ___: deprecated between enumerations of different types
				// re: http://www.open-std.org/jtc1/sc22/wg21/docs/papers/2018/p1120r0.html

				// It seems unclear whether the deprecation will be enacted in C++23 or not
				// e.g. http://www.open-std.org/jtc1/sc22/wg21/docs/papers/2020/p2139r2.html
				// Until the path forward is clearer, it seems reasonable to leave things as they are.
				Arguments.Add("/wd5054");
			}

			if (CompileEnvironment.bEnableCoroutines)
			{
				if (Target.WindowsPlatform.Compiler.IsMSVC())
				{
					Arguments.Add("/await:strict");
				}
			}

			if (Target.WindowsPlatform.Compiler.IsClang())
			{
				if (!Target.WindowsPlatform.Architecture.bIsX64)
				{
					// Workaround for compiler error for arm64 & arm64ec : "unknown codeview register Q16_Q17_Q18_Q19"  #jira UE-300337
					// Use Dwarf instead of CodeView - debugging through VS is not possible
					Arguments.Add("-gdwarf");
					Arguments.Add("-gsplit-dwarf");
					Arguments.Add("-gline-tables-only");
				}
				
				if (Target.WindowsPlatform.bAllowClangLinker)
				{
					// Enable codeview ghash for faster lld links with Clang and Intel
					Arguments.Add("-Xclang -gcodeview-ghash");
				}

				if (CompileEnvironment.bEnableAutoRTFMInstrumentation)
				{
					Arguments.Add("-fautortfm");

					if (CompileEnvironment.bEnableAutoRTFMVerification)
					{
						Arguments.Add("-fautortfm-verify");
					}
				}

				if (CompileEnvironment.bAutoRTFMVerify)
				{
					Arguments.Add("-Xclang -mllvm -Xclang -autortfm-verify");
				}

				if (CompileEnvironment.bAutoRTFMClosedStaticLinkage)
				{
					Arguments.Add("-Xclang -mllvm -Xclang -autortfm-closed-static-linkage");
				}

				if (CompileEnvironment.bAutoRTFMRedirectionInstrumentation)
				{
					Arguments.Add("-Xclang -mllvm -Xclang -autortfm-enable-redirection-instrumentation");
				}
			}
		}

		internal void AppendCLArguments_C(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			// Explicitly compile the file as C.
			Arguments.Add("/TC");

			// Level 0 warnings.  Needed for external C projects that produce warnings at higher warning levels.
			// Forcibly remove "/W4" in order to avoid excessive warning that we know will happen.
			// > Command line warning D9025 : overriding '/W4' with '/W0'
			Arguments.Remove("/W4");
			Arguments.Add("/W0");

			// Select C Standard version available
			// Select C Standard version available
			switch (CompileEnvironment.CStandard)
			{
				case CStandardVersion.None:
				case CStandardVersion.C89:
				case CStandardVersion.C99:
					break;
				case CStandardVersion.C11:
					Arguments.Add("/std:c11");
					break;
				case CStandardVersion.C17:
				case CStandardVersion.Latest:
					Arguments.Add("/std:c17");
					break;
				default:
					throw new BuildException($"Unsupported C standard type set: {CompileEnvironment.CStandard}");
			}

			if (Target.WindowsPlatform.Compiler.IsClang())
			{
				if (CompileEnvironment.bEnableAutoRTFMInstrumentation)
				{
					Arguments.Add("-fautortfm");

					if (CompileEnvironment.bEnableAutoRTFMVerification)
					{
						Arguments.Add("-fautortfm-verify");
					}
				}

				if (CompileEnvironment.bAutoRTFMVerify)
				{
					Arguments.Add("-Xclang -mllvm -Xclang -autortfm-verify");
				}

				if (CompileEnvironment.bAutoRTFMClosedStaticLinkage)
				{
					Arguments.Add("-Xclang -mllvm -Xclang -autortfm-closed-static-linkage");
				}

				if (CompileEnvironment.bAutoRTFMRedirectionInstrumentation)
				{
					Arguments.Add("-Xclang -mllvm -Xclang -autortfm-enable-redirection-instrumentation");
				}
			}
		}

		void AppendCLArguments_CompileWarnings(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			// Set warning level.
			// Restrictive during regular compilation.
			Arguments.Add("/W4");

			// Treat warnings as errors
			if (CompileEnvironment.bWarningsAsErrors)
			{
				Arguments.Add("/WX");
			}

			if (Target.WindowsPlatform.Compiler.IsClang())
			{
				// Disable specific warnings that cause problems with Clang
				// NOTE: These must appear after we set the MSVC warning level

				// @todo clang: Ideally we want as few warnings disabled as possible

				// Treat all warnings as errors by default
				Arguments.Add("-Werror");   // https://clang.llvm.org/docs/UsersManual.html#cmdoption-werror

				VersionNumber ClangVersion = Target.WindowsPlatform.Compiler == WindowsCompiler.Intel ? MicrosoftPlatformSDK.GetClangVersionForIntelCompiler(EnvVars.CompilerPath) : EnvVars.CompilerVersion;

				ClangWarnings.GetWarnings(CompileEnvironment, ClangVersion, Arguments);
			}

			Arguments.AddRange(CompileEnvironment.CppCompileWarnings.GenerateWarningCommandLineArgs(CompileEnvironment, typeof(VCToolChain)));
		}

		protected virtual void AppendLinkArguments(LinkEnvironment LinkEnvironment, List<string> Arguments)
		{
			if (Target.WindowsPlatform.Compiler == WindowsCompiler.Clang && Target.WindowsPlatform.bAllowClangLinker)
			{
				// @todo clang: The following static libraries aren't linking correctly with Clang:
				//		tbbmalloc.lib, zlib_64.lib, libpng_64.lib, freetype2412MT.lib, IlmImf.lib
				//		LLD: Assertion failed: result.size() == 1, file ..\tools\lld\lib\ReaderWriter\FileArchive.cpp, line 71
				//

				// Only omit frame pointers on the PC (which is implied by /Ox) if wanted.
				if (!LinkEnvironment.bOmitFramePointers)
				{
					Arguments.Add("--disable-fp-elim");
				}
			}

			// Don't create a side-by-side manifest file for the executable.
			if (Target.WindowsPlatform.ManifestFile == null || LinkEnvironment.bIsBuildingDLL)
			{
				Arguments.Add("/MANIFEST:NO");
			}
			else
			{
				Arguments.Add("/MANIFEST:EMBED");
				FileItem ManifestFile = FileItem.GetItemByPath(Target.WindowsPlatform.ManifestFile);
				Arguments.Add($"/MANIFESTINPUT:\"{NormalizeCommandLinePath(ManifestFile, LinkEnvironment.RootPaths)}\"");

				// Embed a dependency on the SxS assembly manifest. this is for dependent runtime DLLs
				if (!Target.Architecture.bIsX64)
				{
#pragma warning disable CA1308 // Normalize strings to uppercase - external argument should use the correct case
					string AssemblyName = Target.Architecture.ToString().ToLowerInvariant();
#pragma warning restore CA1308
					Arguments.Add($"/MANIFESTDEPENDENCY:\"name='{AssemblyName}' processorArchitecture='arm64' version='1.0.0.0' type='win32'\"");
				}
			}

			// Prevents the linker from displaying its logo for each invocation.
			Arguments.Add("/NOLOGO");

			// On Clang, use DWARF debug info format for sample based PGO profile configs
			if (Target.WindowsPlatform.Compiler == WindowsCompiler.Clang && Target.WindowsPlatform.bAllowClangLinker && 
				Target.WindowsPlatform.bSampleBasedPGO && LinkEnvironment.bPGOProfile)
			{
				Arguments.Add("/DEBUG:DWARF");
			}
			// Address sanitizer requires debug info for symbolizing callstacks whether
			// we're building debug or shipping.
			else if (LinkEnvironment.bCreateDebugInfo || Target.WindowsPlatform.bEnableAddressSanitizer)
			{
				// Output debug info for the linked executable.
				if (Target.WindowsPlatform.Compiler.IsClang() && Target.WindowsPlatform.bAllowClangLinker)
				{
					Arguments.Add("/DEBUG:GHASH");
				}
				else
				{
					Arguments.Add("/DEBUG:FULL");
				}
			}

			// Prompt the user before reporting internal errors to Microsoft.
			if (!Target.WindowsPlatform.bAllowRadLinker)
			{
				Arguments.Add("/errorReport:prompt");
			}

			//
			//	PC
			//
			if (UseWindowsArchitecture(LinkEnvironment.Platform))
			{
				Arguments.Add($"/MACHINE:{WindowsExports.GetArchitectureName(Target.WindowsPlatform.Architecture)}");
				{
					if (LinkEnvironment.bIsBuildingConsoleApplication)
					{
						Arguments.Add("/SUBSYSTEM:CONSOLE");
					}
					else
					{
						Arguments.Add("/SUBSYSTEM:WINDOWS");
					}
				}

				if (LinkEnvironment.bIsBuildingConsoleApplication && !LinkEnvironment.bIsBuildingDLL && !String.IsNullOrEmpty(LinkEnvironment.WindowsEntryPointOverride))
				{
					// Use overridden entry point
					Arguments.Add($"/ENTRY:{LinkEnvironment.WindowsEntryPointOverride}");
				}

				// Allow the OS to load the EXE at different base addresses than its preferred base address.
				Arguments.Add("/FIXED:No");

				// Explicitly declare that the executable is compatible with Data Execution Prevention.
				Arguments.Add("/NXCOMPAT");
			}

			// Set the default stack size.
			if (LinkEnvironment.Platform.IsInGroup(UnrealPlatformGroup.Microsoft))
			{
				if (LinkEnvironment.DefaultStackSize > 0)
				{
					if (LinkEnvironment.DefaultStackSizeCommit > 0)
					{
						Arguments.Add($"/STACK:{LinkEnvironment.DefaultStackSize},{LinkEnvironment.DefaultStackSizeCommit}");
					}
					else
					{
						Arguments.Add($"/STACK:{LinkEnvironment.DefaultStackSize}");
					}
				}
			}

			// Allow delay-loaded DLLs to be explicitly unloaded.
			if (Target.WindowsPlatform.Architecture == UnrealArch.X64)
			{
				Arguments.Add("/DELAY:UNLOAD");
			}

			if (LinkEnvironment.bIsBuildingDLL)
			{
				Arguments.Add("/DLL");
			}

			if (String.IsNullOrEmpty(Target.WindowsPlatform.PdbAlternatePath))
			{
				// Don't embed the full PDB path; we want to be able to move binaries elsewhere. They will always be side by side.
				Arguments.Add("/PDBALTPATH:%_PDB%");
			}
			else
			{
				// Embed an alternate PDB path into the executable
				Arguments.Add($"/PDBALTPATH:\"{Target.WindowsPlatform.PdbAlternatePath}\"");
			}

			// Deterministic link support
			if (LinkEnvironment.bDeterministic)
			{
				if (Target.WindowsPlatform.Compiler.IsMSVC())
				{
					Arguments.Add("/experimental:deterministic");
				}
			}

			if (Target.WindowsPlatform.Compiler.IsClang())
			{
				Arguments.Add("/Brepro");
			}

			if (Target.WindowsPlatform.Compiler.IsMSVC() && Target.WindowsPlatform.bVCFastFail && !LinkEnvironment.bPGOOptimize && !LinkEnvironment.bPGOProfile)
			{
				Arguments.Add("/fastfail");
			}

			if (Target.WindowsPlatform.Compiler.IsMSVC() && Target.WindowsPlatform.bVCExtendedWarningInfo && !Target.WindowsPlatform.bAllowRadLinker)
			{
				Arguments.Add("/d2:-ExtendedWarningInfo");
			}

			// Allow for PDBs larger than 4GB
			if (Target.WindowsPlatform.PdbPageSize.HasValue)
			{
				Arguments.Add($"/PDBPAGESIZE:{System.Numerics.BitOperations.RoundUpToPowerOf2(Target.WindowsPlatform.PdbPageSize.Value)}");
				//Arguments.Add("/PDBCompress"); // Do not turn this on, it makes link times almost 2x slower. This is _only_ to save local disk space. Will _not_ make actual file smaller for network transfer
			}

			// Reduce optimizations for huge functions, may improve compile time a the expense of speed for functions over the threshold
			if (Target.WindowsPlatform.Compiler.IsMSVC() && Target.WindowsPlatform.bReducedOptimizeHugeFunctions)
			{
				Arguments.Add("/d2:\"-ReducedOptimizeHugeFunctions\"");
				Arguments.Add($"/d2:\"-ReducedOptimizeThreshold:{Target.WindowsPlatform.ReducedOptimizeHugeFunctionsThreshold}\"");
			}

			//
			//	Shipping & LTCG
			//
			if (LinkEnvironment.bAllowLTCG)
			{
				// Use link-time code generation.
				Arguments.Add("/LTCG");

				// This is where we add in the PGO-Lite linkorder.txt if we are using PGO-Lite
				//Result += " /ORDER:@linkorder.txt";
				//Result += " /VERBOSE";
			}

			//
			//	Shipping binary
			//
			if (LinkEnvironment.Configuration == CppConfiguration.Shipping)
			{
				// Generate an EXE checksum.
				Arguments.Add("/RELEASE");
			}

			// Eliminate unreferenced symbols.
			if (Target.WindowsPlatform.bStripUnreferencedSymbols)
			{
				Arguments.Add("/OPT:REF");
			}
			else
			{
				Arguments.Add("/OPT:NOREF");
			}

			// Identical COMDAT folding. Prevent Identical Code Folding when instrumentation is enabled to avoid thunks to all be folded together and prevent hotpatching.
			if (Target.WindowsPlatform.bMergeIdenticalCOMDATs && !(Target.WindowsPlatform.Compiler.IsClang() && Target.WindowsPlatform.bSampleBasedPGO && LinkEnvironment.bPGOProfile) && !Target.WindowsPlatform.bEnableInstrumentation)
			{
				Arguments.Add("/OPT:ICF");
			}
			else
			{
				Arguments.Add("/OPT:NOICF");
			}

			// Enable incremental linking if wanted. ( avoid /INCREMENTAL getting ignored (LNK4075) due to /LTCG, /RELEASE, and /OPT:ICF )
			if (LinkEnvironment.bUseIncrementalLinking &&
				LinkEnvironment.Configuration != CppConfiguration.Shipping &&
				!Target.WindowsPlatform.bMergeIdenticalCOMDATs &&
				!LinkEnvironment.bAllowLTCG &&
				Target.WindowsPlatform.Architecture == UnrealArch.X64)
			{
				Arguments.Add("/INCREMENTAL");
				Arguments.Add("/verbose:incr");
			}
			else
			{
				Arguments.Add("/INCREMENTAL:NO");
			}

			// Add any extra options from the target
			if (!String.IsNullOrEmpty(Target.WindowsPlatform.AdditionalLinkerOptions))
			{
				Arguments.Add(Target.WindowsPlatform.AdditionalLinkerOptions);
			}

			// Disable
			//LINK : warning LNK4199: /DELAYLOAD:nvtt_64.dll ignored; no imports found from nvtt_64.dll
			// type warning as we leverage the DelayLoad option to put third-party DLLs into a
			// non-standard location. This requires the module(s) that use said DLL to ensure that it
			// is loaded prior to using it.
			Arguments.Add("/ignore:4199");

			// Suppress warnings about missing PDB files for statically linked libraries.  We often don't want to distribute
			// PDB files for these libraries.
			Arguments.Add("/ignore:4099");      // warning LNK4099: PDB '<file>' was not found with '<file>'

			// Workaround for linker errors when linking against static libraries that were compiled with an older msvc
			// https://github.com/microsoft/STL/issues/2655
			Arguments.Add("/ALTERNATENAME:__imp___std_init_once_begin_initialize=__imp_InitOnceBeginInitialize");
			Arguments.Add("/ALTERNATENAME:__imp___std_init_once_complete=__imp_InitOnceComplete");

			if (Target.WindowsPlatform.bEnableAddressSanitizer)
			{
				// With clang we seemingly need to explicitly pass the .lib's to link against for ASan.
				if (Target.WindowsPlatform.Compiler.IsClang())
				{
					DirectoryReference ASanRuntimeDir;
					string ASanArchSuffix;
					if (EnvVars.Architecture == UnrealArch.X64)
					{
						VersionNumber ClangVersion = Target.WindowsPlatform.Compiler == WindowsCompiler.Intel ? MicrosoftPlatformSDK.GetClangVersionForIntelCompiler(EnvVars.CompilerPath) : EnvVars.CompilerVersion;
						ASanRuntimeDir = DirectoryReference.Combine(EnvVars.CompilerDir, "lib", "clang", $"{ClangVersion.Components[0]}", "lib", "windows");
						ASanArchSuffix = "x86_64";
					}
					else
					{
						throw new BuildException("Unsupported build architecture for Address Sanitizer");
					}

					string ASanRuntimeLib = $"clang_rt.asan_dynamic-{ASanArchSuffix}.lib";
					string ASanDebugRuntimeLib = $"clang_rt.asan_dbg_dynamic-{ASanArchSuffix}.lib";

					if (Target.bDebugBuildsActuallyUseDebugCRT)
					{
						LinkEnvironment.Libraries.Add(FileReference.Combine(ASanRuntimeDir, ASanDebugRuntimeLib));
					}
					else
					{
						LinkEnvironment.Libraries.Add(FileReference.Combine(ASanRuntimeDir, ASanRuntimeLib));
					}

					LinkEnvironment.Libraries.Add(FileReference.Combine(ASanRuntimeDir, $"clang_rt.asan_dynamic_runtime_thunk-{ASanArchSuffix}.lib"));
				}
			}

			if (Target.WindowsPlatform.bEnableUndefinedBehaviorSanitizer)
			{
				// With clang we seemingly need to explicitly pass the .lib's to link against for UBSan.
				if (Target.WindowsPlatform.Compiler.IsClang())
				{
					DirectoryReference UBSanRuntimeDir;
					string UBSanArchSuffix;
					if (EnvVars.Architecture == UnrealArch.X64)
					{
						VersionNumber ClangVersion = Target.WindowsPlatform.Compiler == WindowsCompiler.Intel ? MicrosoftPlatformSDK.GetClangVersionForIntelCompiler(EnvVars.CompilerPath) : EnvVars.CompilerVersion;
						UBSanRuntimeDir = DirectoryReference.Combine(EnvVars.CompilerDir, "lib", "clang", $"{ClangVersion.Components[0]}", "lib", "windows");
						//UBSanRuntimeDir = DirectoryReference.Combine(EnvVars.ToolChainDir, "lib", "x64");
						UBSanArchSuffix = "x86_64";
					}
					else
					{
						throw new BuildException("Unsupported build architecture for Undefined Behavior Sanitizer");
					}
					string UBSanRuntimeLib = $"clang_rt.ubsan_standalone-{UBSanArchSuffix}.lib";
					LinkEnvironment.Libraries.Add(FileReference.Combine(UBSanRuntimeDir, UBSanRuntimeLib));
				}
			}
			
			if (LinkEnvironment.bCodeCoverage)
			{
				// Link clang_rt.profile-x86_64.lib
				if (Target.WindowsPlatform.Compiler.IsClang())
				{
					DirectoryReference ProfileRuntimeDir;
					string ProfileArchSuffix;
					if (EnvVars.Architecture == UnrealArch.X64)
					{
						VersionNumber ClangVersion = Target.WindowsPlatform.Compiler == WindowsCompiler.Intel ? MicrosoftPlatformSDK.GetClangVersionForIntelCompiler(EnvVars.CompilerPath) : EnvVars.CompilerVersion;
						ProfileRuntimeDir = DirectoryReference.Combine(EnvVars.CompilerDir, "lib", "clang", $"{ClangVersion.Components[0]}", "lib", "windows");
						ProfileArchSuffix = "x86_64";
					}
					else
					{
						throw new BuildException("Unsupported build architecture for Code Coverage");
					}
					string ProfileRuntimeLib = $"clang_rt.profile-{ProfileArchSuffix}.lib";
					LinkEnvironment.Libraries.Add(FileReference.Combine(ProfileRuntimeDir, ProfileRuntimeLib));
				}
			}
		}

		protected virtual void AppendLibArguments(LinkEnvironment LinkEnvironment, List<string> Arguments)
		{
			// Prevents the linker from displaying its logo for each invocation.
			Arguments.Add("/NOLOGO");

			// Prompt the user before reporting internal errors to Microsoft.
			Arguments.Add("/errorReport:prompt");

			//
			//	PC
			//
			if (UseWindowsArchitecture(LinkEnvironment.Platform))
			{
				Arguments.Add($"/MACHINE:{WindowsExports.GetArchitectureName(Target.WindowsPlatform.Architecture)}");
				{
					if (LinkEnvironment.bIsBuildingConsoleApplication)
					{
						Arguments.Add("/SUBSYSTEM:CONSOLE");
					}
					else
					{
						Arguments.Add("/SUBSYSTEM:WINDOWS");
					}
				}
			}

			//
			//	Shipping & LTCG
			//
			if (LinkEnvironment.bAllowLTCG || LinkEnvironment.Configuration == CppConfiguration.Shipping)
			{
				// Use link-time code generation.
				Arguments.Add("/LTCG");
			}

			// Always build import libraries in deterministic mode, as no codegen is run this does not add any additional time
			if (!EnvVars.Compiler.IsClang() || !EnvVars.bAllowClangLinker)
			{
				Arguments.Add("/experimental:deterministic");
			}
		}

		private VCCompileAction CreateBaseCompileAction(CppCompileEnvironment CompileEnvironment)
		{
			VCCompileAction BaseCompileAction = new VCCompileAction(EnvVars);

			// TODO: Revisit this code. We want to use d1trimfile to make outputs machine independent.
			// but want to make sure we're not causing any frustration for devs (since __FILE__ will show a relative path with lines below)
			// Also need to find the equivalent for clang
#if false
			if (!Target.WindowsPlatform.Compiler.IsClang())
			{
				foreach (DirectoryItem rootPath in rootPaths)
				{
					string pathName = rootPath.FullName;
					if (pathName.Contains(' '))
						BaseCompileAction.Arguments.Add($"\"/d1trimfile:{pathName}\\\"");
					else
						BaseCompileAction.Arguments.Add($"/d1trimfile:{pathName}\\");
				}
			}
#endif
			BaseCompileAction.RootPaths = CompileEnvironment.RootPaths;

			// Add additional response files
			BaseCompileAction.AdditionalResponseFiles.AddRange(CompileEnvironment.AdditionalResponseFiles);

			BaseCompileAction.bIsAnalyzing = Target.StaticAnalyzer != StaticAnalyzer.None && !CompileEnvironment.bDisableStaticAnalysis && !(Target.WindowsPlatform.Compiler.IsClang() && CompileEnvironment.PrecompiledHeaderAction == PrecompiledHeaderAction.Create);
			BaseCompileAction.bWarningsAsErrors = CompileEnvironment.bWarningsAsErrors;

			// Add include paths to the argument list.
			BaseCompileAction.IncludePaths.AddRange(CompileEnvironment.UserIncludePaths);
			BaseCompileAction.SystemIncludePaths.AddRange(CompileEnvironment.SystemIncludePaths);

			if (!CompileEnvironment.bHasSharedResponseFile)
			{
				AppendCLArguments_Global(CompileEnvironment, BaseCompileAction.Arguments);
				BaseCompileAction.SystemIncludePaths.AddRange(EnvVars.IncludePaths);
			}
			else
			{
				// /nologo in the shared file doesn't work to suppress logging.
				BaseCompileAction.Arguments.Add("/nologo");
			}

			// Remember the architecture
			BaseCompileAction.Architecture = CompileEnvironment.Architecture;

			// Add preprocessor definitions to the argument list.
			BaseCompileAction.Definitions.AddRange(CompileEnvironment.Definitions);

			// Add the force included headers
			BaseCompileAction.ForceIncludeFiles.AddRange(CompileEnvironment.ForceIncludeFiles);

			// If we're using precompiled headers, set that up now
			if (CompileEnvironment.PrecompiledHeaderAction == PrecompiledHeaderAction.Include)
			{
				FileItem IncludeHeader = FileItem.GetItemByFileReference(CompileEnvironment.PrecompiledHeaderIncludeFilename!);
				BaseCompileAction.ForceIncludeFiles.Insert(0, IncludeHeader);

				BaseCompileAction.UsingPchFile = CompileEnvironment.PrecompiledHeaderFile;
				BaseCompileAction.PchThroughHeaderFile = IncludeHeader;

				if (Target.WindowsPlatform.Compiler.IsMSVC() && CompileEnvironment.PrecompiledHeaderFile != null && !CompileEnvironment.bUseHeaderUnitsForPch && Target.StaticAnalyzer == StaticAnalyzer.Default && !CompileEnvironment.bDisableStaticAnalysis)
				{
					BaseCompileAction.AdditionalPrerequisiteItems.Add(FileItem.GetItemByFileReference(new FileReference(CompileEnvironment.PrecompiledHeaderFile.FullName + "ast")));
				}
			}

			if (EnvVars.Compiler.IsClang() && CompileEnvironment.bPGOOptimize)
			{
				if (Directory.Exists(CompileEnvironment.PGODirectory))
				{
					BaseCompileAction.AdditionalPrerequisiteItems.Add(FileItem.GetItemByFileReference(GetClangCompileProfDataFilename(CompileEnvironment)));
				}
			}

			// Generate the timing info
			if (CompileEnvironment.bPrintTimingInfo || Target.WindowsPlatform.bCompilerTrace)
			{
				if (Target.WindowsPlatform.Compiler.IsMSVC())
				{
					if (CompileEnvironment.bPrintTimingInfo)
					{
						BaseCompileAction.Arguments.Add("/Bt+ /d2cgsummary");
					}

					BaseCompileAction.Arguments.Add("/d1reportTime");
				}
			}

			// MSVC uses multiple threads when compiling CPP files, so the "weight" is more than 1
			if (Target.WindowsPlatform.Compiler.IsMSVC())
			{
				// If deterministic is enabled, MSVC does not use multiple threads
				if (!CompileEnvironment.bDeterministic && !CompileEnvironment.bPreprocessOnly)
				{
					BaseCompileAction.Weight = Target.MSVCCompileActionWeight;

					// If we are building without unity files the multithread part balances out with the start/exit of all the actions
					if (!CompileEnvironment.bUseUnity)
					{
						// This is a very lightweight build. no pch or debug info. I/O is also low. Set weight to 1.0f
						if (CompileEnvironment.PrecompiledHeaderAction == PrecompiledHeaderAction.None && !CompileEnvironment.bCreateDebugInfo)
						{
							BaseCompileAction.Weight = 1.0f;
						}
						else
						{
							BaseCompileAction.Weight = 1.0f + (BaseCompileAction.Weight - 1.0f) * 0.5f;
						}
					}
				}
			}
			else if (Target.WindowsPlatform.Compiler.IsClang())
			{
				BaseCompileAction.Weight = Target.ClangCompileActionWeight;
			}

			// Don't farm out creation of precompiled headers as it is the critical path task.
			BaseCompileAction.bCanExecuteRemotely =
				CompileEnvironment.PrecompiledHeaderAction != PrecompiledHeaderAction.Create ||
				CompileEnvironment.bAllowRemotelyCompiledPCHs
				;

			// When compiling with SN-DBS, modules that contain a #import must be built locally
			BaseCompileAction.bCanExecuteRemotelyWithSNDBS = BaseCompileAction.bCanExecuteRemotely && !CompileEnvironment.bBuildLocallyWithSNDBS;

			if (Target.bAllowUbaCompression)
			{
				BaseCompileAction.CompilerVersion = $"{BaseCompileAction.CompilerVersion} Compressed";
			}

			// Calculate cache bucket. We want to use as few buckets as possible per build but we also
			// want to make sure that the buckets are not getting too big.
			BaseCompileAction.CacheBucket = GetCacheBucket(Target, CompileEnvironment);

			return BaseCompileAction;
		}

		public override FileItem? CopyDebuggerVisualizer(FileItem SourceFile, DirectoryReference IntermediateDirectory, IActionGraphBuilder Graph)
		{
			if (SourceFile.HasExtension(".natvis") || SourceFile.HasExtension(".natstepfilter") || SourceFile.HasExtension(".natjmc"))
			{
				FileReference IntermediateFile = FileReference.Combine(IntermediateDirectory, SourceFile.Name);
				if (!Unreal.IsFileInstalled(IntermediateFile))
				{
					FileItem Item = FileItem.GetItemByFileReference(IntermediateFile);
					Graph.CreateCopyAction(SourceFile, Item);
					return Item; // Only return the item if we are responsible for copying it, for addition to makefile/manifests
				}
			}
			return null;
		}

		public override FileItem? LinkDebuggerVisualizer(FileItem SourceFile, DirectoryReference IntermediateDirectory)
		{
			if (SourceFile.HasExtension(".natvis") || SourceFile.HasExtension(".natstepfilter") || SourceFile.HasExtension(".natjmc"))
			{
				FileItem IntermediateFile = FileItem.GetItemByFileReference(FileReference.Combine(IntermediateDirectory, SourceFile.Name));
				return IntermediateFile;
			}
			return null;
		}

		protected override CPPOutput CompileCPPFiles(CppCompileEnvironment CompileEnvironment, IEnumerable<FileItem> InputFiles, DirectoryReference OutputDir, string ModuleName, IActionGraphBuilder Graph)
		{
			CPPOutput Result = new CPPOutput();

			if (Target.StaticAnalyzer != StaticAnalyzer.None && CompileEnvironment.bDisableStaticAnalysis)
			{
				return Result;
			}

			VCCompileAction BaseCompileAction = CreateBaseCompileAction(CompileEnvironment);

			// Create a compile action for each source file.
			List<VCCompileAction> Actions = new List<VCCompileAction>();
			foreach (FileItem SourceFile in InputFiles)
			{
				VCCompileAction CompileAction = new VCCompileAction(BaseCompileAction);
				CompileAction.SourceFile = SourceFile;

				bool bIsPlainCFile = Path.GetExtension(SourceFile.AbsolutePath).ToUpperInvariant() == ".C";
				bool bIsHeaderFile = Path.GetExtension(SourceFile.AbsolutePath).ToUpperInvariant() == ".H";

				if (CompileEnvironment.PrecompiledHeaderAction == PrecompiledHeaderAction.Create)
				{
					// Generate a CPP File that just includes the precompiled header.
					string PrecompiledHeaderIncludeFilenameString = CompileEnvironment.PrecompiledHeaderIncludeFilename!.GetFileName();
					List<string> PchCppFileContents = [
						"// Generated by UnrealBuildTool (VCToolChain.cs)",
						$"// Compiler: {(EnvVars.Compiler.IsMSVC() ? $"MSVC {EnvVars.CompilerVersion}" : $"{EnvVars.Compiler} {EnvVars.CompilerVersion}, MSVC {EnvVars.ToolChainVersion}")}",
						$"#include \"{PrecompiledHeaderIncludeFilenameString}\""
					];
					CompileAction.SourceFile = FileItem.GetItemByFileReference(CompileEnvironment.PrecompiledHeaderIncludeFilename!.ChangeExtension(".cpp"));
					Graph.CreateIntermediateTextFile(CompileAction.SourceFile, PchCppFileContents);

					// Add the precompiled header file to the produced items list.
					string PchExtension = CompileEnvironment.bUseHeaderUnitsForPch ? ".ifc" : ".pch";
					CompileAction.CreatePchFile = FileItem.GetItemByFileReference(FileReference.Combine(OutputDir, SourceFile.Location.GetFileName() + PchExtension));
					CompileAction.PchThroughHeaderFile = FileItem.GetItemByFileReference(CompileEnvironment.PrecompiledHeaderIncludeFilename);

					if (Target.WindowsPlatform.Compiler.IsMSVC() && !CompileEnvironment.bUseHeaderUnitsForPch && Target.StaticAnalyzer == StaticAnalyzer.Default && !CompileEnvironment.bDisableStaticAnalysis)
					{
						CompileAction.AdditionalProducedItems.Add(FileItem.GetItemByFileReference(FileReference.Combine(OutputDir, SourceFile.Location.GetFileName() + PchExtension + "ast")));
					}

					// If we're creating a PCH that will be used to compile source files for a library, we need
					// the compiled modules to retain a reference to PCH's module, so that debugging information
					// will be included in the library.  This is also required to avoid linker warning "LNK4206"
					// when linking an application that uses this library.
					if (CompileEnvironment.bIsBuildingLibrary)
					{
						// NOTE: The symbol name we use here is arbitrary, and all that matters is that it is
						// unique per PCH module used in our library
						string FakeUniquePCHSymbolName = CompileEnvironment.PrecompiledHeaderIncludeFilename.GetFileNameWithoutExtension();
						CompileAction.Arguments.Add($"/Yl{FakeUniquePCHSymbolName}");
					}

					if (CompileEnvironment.bUseHeaderUnitsForPch)
					{
						string ifcPath = NormalizeCommandLinePath(CompileEnvironment.PrecompiledHeaderIncludeFilename!, CompileEnvironment.RootPaths);
						string tomlString =
							$"[[header-unit]]\r\n" +
							$"name = [\"quote\", '{ifcPath.Replace('/', '\\')}']\r\n" +
							$"ifc = \"{CompileEnvironment.PrecompiledHeaderIncludeFilename!.FullName.Replace('\\', '/') + ".ifc"}\"\r\n";
						FileReference tomlFileRef = FileReference.Combine(OutputDir, SourceFile.Location.GetFileName() + ".toml");
						FileItem tomlFile = Graph.CreateIntermediateTextFile(tomlFileRef, tomlString);
						CompileAction.AdditionalPrerequisiteItems.Add(tomlFile);
						Result.ObjectFiles.Add(tomlFile);
					}
				}

				string FileName = SourceFile.Name;
				if (CompileEnvironment.CollidingNames != null && CompileEnvironment.CollidingNames.Contains(SourceFile))
				{
					string HashString = ContentHash.MD5(SourceFile.AbsolutePath.Substring(Unreal.RootDirectory.FullName.Length)).GetHashCode().ToString("X4");
					FileName = Path.GetFileNameWithoutExtension(FileName) + "_" + HashString + Path.GetExtension(FileName);
				}

				if (CompileEnvironment.bPreprocessOnly)
				{
					CompileAction.PreprocessedFile = FileItem.GetItemByFileReference(FileReference.Combine(OutputDir, FileName + ".i"));
					CompileAction.ResponseFile = FileItem.GetItemByFileReference(GetResponseFileName(CompileAction.PreprocessedFile));
				}
				else if (Target.WindowsPlatform.Compiler.IsClang() && Target.StaticAnalyzer == StaticAnalyzer.Default && CompileEnvironment.PrecompiledHeaderAction != PrecompiledHeaderAction.Create)
				{
					// Clang analysis does not actually create an object, use the dependency list as the response filename
					string DependencyListFilename = FileName + ".d";
					FileItem DependencyListFile = FileItem.GetItemByFileReference(FileReference.Combine(OutputDir, DependencyListFilename));
					CompileAction.ResponseFile = FileItem.GetItemByFileReference(GetResponseFileName(DependencyListFile));
				}
				else
				{
					// Add the object file to the produced item list.
					string ObjectLeafFilename = FileName + ".obj";
					FileItem ObjectFile = FileItem.GetItemByFileReference(FileReference.Combine(OutputDir, ObjectLeafFilename));

					CompileAction.ObjectFile = ObjectFile;
					CompileAction.ResponseFile = FileItem.GetItemByFileReference(GetResponseFileName(ObjectFile));

					if (Target.WindowsPlatform.ObjSrcMapFile != null)
					{
						using (StreamWriter Writer = File.AppendText(Target.WindowsPlatform.ObjSrcMapFile))
						{
							Writer.WriteLine($"\"{ObjectLeafFilename}\" -> \"{SourceFile.AbsolutePath}\"");
						}
					}

					if (IsDynamicDebuggingEnabled)
					{
						CompileAction.AdditionalProducedItems.Add(FileItem.GetItemByFileReference(FileReference.Combine(OutputDir, FileName + ".alt.obj")));
					}

					if (Target.WindowsPlatform.Compiler.IsMSVC() && CompileEnvironment.bWithAssembly && SourceFile.HasExtension(".cpp"))
					{
						CompileAction.AssemblyFile = FileItem.GetItemByFileReference(FileReference.Combine(OutputDir, FileName + ".asm"));
					}

					// Experimental: support for JSON output of timing data
					if (Target.WindowsPlatform.Compiler.IsClang() && (Target.bPrintToolChainTimingInfo || Target.WindowsPlatform.bClangTimeTrace))
					{
						CompileAction.Arguments.Add("-ftime-trace");
						CompileAction.AdditionalProducedItems.Add(FileItem.GetItemByFileReference(ObjectFile.Location.ChangeExtension(".json")));
					}
				}

				{
					CompileAction.ArtifactMode = ArtifactMode.Enabled;

					if (CompileEnvironment.PrecompiledHeaderAction != PrecompiledHeaderAction.None)
					{
						if (!Target.bUseVFS)
						{
							// Unfortunately we require matching absolute paths for pch to be cached if not using vfs
							CompileAction.ArtifactMode |= ArtifactMode.AbsolutePath;
						}

						if (Target.WindowsPlatform.Compiler.IsClang())
						{
							CompileAction.Arguments.Add("-Xclang -fno-pch-timestamp"); // This is needed to prevent check on timestamp stored inside pch
							CompileAction.Arguments.Add("-Xclang -fvalidate-ast-input-files-content"); // Validate PCH inputs by content if mtime check fails
						}
					}
				}

				// Create PDB files if we were configured to do that.
				if (CompileEnvironment.bUsePDBFiles || CompileEnvironment.bSupportEditAndContinue)
				{
					FileReference PDBLocation;
					if (CompileEnvironment.PrecompiledHeaderAction == PrecompiledHeaderAction.Include)
					{
						// All files using the same PCH are required to share the same PDB that was used when compiling the PCH
						PDBLocation = CompileEnvironment.PrecompiledHeaderFile!.Location.ChangeExtension(".pdb");

						// Enable synchronous file writes, since we'll be modifying the existing PDB
						CompileAction.Arguments.Add("/FS");
					}
					else if (CompileEnvironment.PrecompiledHeaderAction == PrecompiledHeaderAction.Create)
					{
						// Files creating a PCH use a PDB per file.
						PDBLocation = FileReference.Combine(OutputDir, CompileEnvironment.PrecompiledHeaderIncludeFilename!.GetFileName() + ".pdb");

						// Enable synchronous file writes, since we'll be modifying the existing PDB
						CompileAction.Arguments.Add("/FS");
					}
					else if (!bIsPlainCFile)
					{
						// Ungrouped C++ files use a PDB per file.
						PDBLocation = FileReference.Combine(OutputDir, FileName + ".pdb");
					}
					else
					{
						// Group all plain C files that doesn't use PCH into the same PDB
						PDBLocation = FileReference.Combine(OutputDir, "MiscPlainC.pdb");
					}

					// Specify the PDB file that the compiler should write to.
					CompileAction.Arguments.Add($"/Fd\"{NormalizeCommandLinePath(PDBLocation, CompileEnvironment.RootPaths)}\"");

					// Don't allow remote execution when PDB files are enabled; we need to modify the same files. XGE works around this by generating separate
					// PDB files per agent, but this functionality is only available with the Visual C++ extension package (via the VCCompiler=true tool option).
					CompileAction.bCanExecuteRemotely = false;
				}

				// Add C or C++ specific compiler arguments.
				if (bIsPlainCFile)
				{
					AppendCLArguments_C(CompileEnvironment, CompileAction.Arguments);
				}
				else if (bIsHeaderFile)
				{
					AppendCLArguments_H(CompileEnvironment, CompileAction.Arguments);
				}
				else
				{
					AppendCLArguments_CPP(CompileEnvironment, CompileAction.Arguments);
				}

				// Add additional arguments to the argument list, must be the final arguments added
				if (!String.IsNullOrEmpty(CompileEnvironment.AdditionalArguments))
				{
					CompileAction.Arguments.Add(CompileEnvironment.AdditionalArguments);
				}

				if (Target.WindowsPlatform.Compiler.IsClang())
				{
					// If we are using the AutoRTFM compiler, we make the compile action depend on the version of the compiler itself.
					// This lets us update the compiler (which might not cause a version update of the compiler, which instead tracks
					// the LLVM versioning scheme that Clang uses), but ensure that we rebuild the source if the compiler has changed.
					// AutoRTFM also depends on external mapping files, which requires a full rebuild when changed.
					if (CompileEnvironment.bUseAutoRTFMCompiler)
					{
						FileReference? CompilerPath = GetCppCompilerPath();
						if (null != CompilerPath)
						{
							CompileAction.AdditionalPrerequisiteItems.Add(FileItem.GetItemByFileReference(CompilerPath));
							foreach (FileItem File in CompileEnvironment.AutoRTFMExternalMappingFiles)
							{
								CompileAction.AdditionalPrerequisiteItems.Add(File);
								CompileAction.Arguments.Add($"-Xclang -autortfm-mappings -Xclang \"{NormalizeCommandLinePath(File.Location, CompileEnvironment.RootPaths)}\"");
							}
						}
					}
				}

				if (CompileEnvironment.FileInlineSourceMap.TryGetValue(SourceFile, out HashSet<FileItem>? InlinedFiles))
				{
					CompileAction.AdditionalPrerequisiteItems.AddRange(InlinedFiles);
				}

				CompileAction.AdditionalPrerequisiteItems.AddRange(CompileEnvironment.AdditionalPrerequisites);

				if (SourceFile.HasExtension(".ixx"))
				{
					FileItem IfcFile = FileItem.GetItemByFileReference(FileReference.Combine(GetModuleInterfaceDir(OutputDir), SourceFile.Location.ChangeExtension(".ifc").GetFileName()));
					DirectoryReference.CreateDirectory(IfcFile.Directory.Location);
					CompileAction.Arguments.Add("/interface");
					CompileAction.Arguments.Add($"/ifcOutput \"{IfcFile.Location}\"");
					CompileAction.CompiledModuleInterfaceFile = IfcFile;

					FileItem IfcDepsFile = FileItem.GetItemByFileReference(FileReference.Combine(OutputDir, FileName + ".md.json"));

					VCCompileAction CompileDepsAction = new VCCompileAction(CompileAction);
					CompileDepsAction.ActionType = ActionType.GatherModuleDependencies;
					CompileDepsAction.ResponseFile = FileItem.GetItemByFileReference(GetResponseFileName(IfcDepsFile));
					CompileDepsAction.ObjectFile = null;
					CompileDepsAction.DependencyListFile = IfcDepsFile;
					CompileDepsAction.Arguments.Add($"/sourceDependencies:directives \"{IfcDepsFile.Location}\"");
					CompileDepsAction.AdditionalPrerequisiteItems.Add(SourceFile);
					CompileDepsAction.AdditionalPrerequisiteItems.AddRange(CompileEnvironment.AdditionalPrerequisites);
					CompileDepsAction.AdditionalProducedItems.Add(IfcDepsFile);
					Graph.AddAction(CompileDepsAction);

					if (!ProjectFileGenerator.bGenerateProjectFiles)
					{
						CompileDepsAction.WriteResponseFile(Graph);
					}

					CompileAction.ActionType = ActionType.CompileModuleInterface;
					CompileAction.AdditionalPrerequisiteItems.Add(IfcDepsFile); // Force the dependencies file into the action graph
					CompileAction.AdditionalProducedItems.Add(IfcFile);
					CompileAction.CompiledModuleInterfaceFile = IfcFile;
				}

				if ((Target.bPrintToolChainTimingInfo || Target.WindowsPlatform.bCompilerTrace) && Target.WindowsPlatform.Compiler.IsMSVC())
				{
					CompileAction.ForceClFilter = true;
					CompileAction.TimingFile = FileItem.GetItemByFileReference(FileReference.Combine(OutputDir, $"{FileName}.timing"));
					GenerateParseTimingInfoAction(SourceFile, CompileAction.TimingFile, Graph);
				}

				if (Target.WindowsPlatform.Compiler.IsMSVC() && !CompileAction.ForceClFilter)
				{
					CompileAction.DependencyListFile = FileItem.GetItemByFileReference(FileReference.Combine(OutputDir, $"{FileName}.dep.json"));
				}
				else if (Target.WindowsPlatform.Compiler.IsClang())
				{
					CompileAction.DependencyListFile = FileItem.GetItemByFileReference(FileReference.Combine(OutputDir, $"{FileName}.d"));
				}
				else
				{
					CompileAction.DependencyListFile = FileItem.GetItemByFileReference(FileReference.Combine(OutputDir, $"{FileName}.txt"));
					CompileAction.bShowIncludes = Target.bShowIncludes;
				}

				// Write cl errors and warnings to a file
				if (Target.WindowsPlatform.Compiler.IsMSVC() && Target.WindowsPlatform.bWriteSarif)
				{
					if (Target.StaticAnalyzer == StaticAnalyzer.Default && !CompileEnvironment.bDisableStaticAnalysis)
					{
						CompileAction.AnalyzeLogFile = FileItem.GetItemByFileReference(FileReference.Combine(OutputDir, $"{FileName}.sa.sarif"));
					}

					CompileAction.ExperimentalLogFile = FileItem.GetItemByFileReference(FileReference.Combine(OutputDir, $"{FileName}.sarif"));
				}

				// Pch are not compatible between different architectures (msvc pch is just a memory dump)
				if (Target.WindowsPlatform.Compiler.IsMSVC() && CompileEnvironment.PrecompiledHeaderAction != PrecompiledHeaderAction.None)
				{
					CompileAction.bCanExecuteInUBACrossArchitecture = false;
				}

				// Allow derived toolchains to make further changes
				ModifyFinalCompileAction(CompileAction, CompileEnvironment, SourceFile, OutputDir, ModuleName);

				if (!ProjectFileGenerator.bGenerateProjectFiles)
				{
					CompileAction.WriteResponseFile(Graph);
				}

				// Must be added after response file is created just to make sure it ends up on the command line and not in the response file
				if (Target.bMergeModules)
				{
					// EXTRACTEXPORTS can only be interpreted by UBA.. so this action won't build outside uba
					CompileAction.Arguments.Add("/EXTRACTEXPORTS");
					FileItem SymFile = FileItem.GetItemByFileReference(FileReference.Combine(OutputDir, FileName + ".exi"));
					CompileAction.AdditionalProducedItems.Add(SymFile);
				}

				CompileAction.bIsAnalyzing = Target.StaticAnalyzer != StaticAnalyzer.None && !CompileEnvironment.bDisableStaticAnalysis && !(Target.WindowsPlatform.Compiler.IsClang() && CompileEnvironment.PrecompiledHeaderAction == PrecompiledHeaderAction.Create);
				CompileAction.bWarningsAsErrors = CompileEnvironment.bWarningsAsErrors;

				// Update the output
				Graph.AddAction(CompileAction);
				Actions.Add(CompileAction);
			}

			Result.ObjectFiles.AddRange(Actions.Where(x => x.ObjectFile != null || x.PreprocessedFile != null).Select(x => x.ObjectFile != null ? x.ObjectFile! : x.PreprocessedFile!));
			// Clang static analysis doesn't create object files, so treat the dependency list file as the output
			if (Target.WindowsPlatform.Compiler.IsClang() && Target.StaticAnalyzer == StaticAnalyzer.Default)
			{
				Result.ObjectFiles.AddRange(Actions.Where(x => x.DependencyListFile != null).Select(x => x.DependencyListFile!));
			}
			Result.CompiledModuleInterfaces.AddRange(Actions.Where(x => x.CompiledModuleInterfaceFile != null).Select(x => x.CompiledModuleInterfaceFile!));
			Result.PrecompiledHeaderFile = Actions.Select(x => x.CreatePchFile).Where(x => x != null).FirstOrDefault();
			return Result;
		}

		protected virtual void ModifyFinalCompileAction(VCCompileAction CompileAction, CppCompileEnvironment CompileEnvironment, FileItem SourceFile, DirectoryReference OutputDir, string ModuleName)
		{
		}

		private Action GenerateParseTimingInfoAction(FileItem SourceFile, FileItem TimingFile, IActionGraphBuilder Graph)
		{
			FileItem TimingJsonFile = FileItem.GetItemByPath(Path.ChangeExtension(TimingFile.AbsolutePath, ".cta"));

			string ParseTimingArguments = $"-TimingFile=\"{TimingFile}\"";
			if (Target.bParseTimingInfoForTracing)
			{
				ParseTimingArguments += " -Tracing";
			}

			Action ParseTimingInfoAction = Graph.CreateRecursiveAction<ParseMsvcTimingInfoMode>(ActionType.ParseTimingInfo, ParseTimingArguments);
			ParseTimingInfoAction.StatusDescription = Path.GetFileName(TimingFile.AbsolutePath);
			ParseTimingInfoAction.bCanExecuteRemotely = true;
			ParseTimingInfoAction.bCanExecuteRemotelyWithSNDBS = true;
			ParseTimingInfoAction.PrerequisiteItems.Add(SourceFile);
			ParseTimingInfoAction.PrerequisiteItems.Add(TimingFile);
			ParseTimingInfoAction.ProducedItems.Add(TimingJsonFile);
			return ParseTimingInfoAction;
		}

		public override void FinalizeOutput(ReadOnlyTargetRules Target, TargetMakefileBuilder MakefileBuilder, DirectoryReference ExeDir)
		{
			if (Target.bPGOOptimize)
			{
				// Win64 PGO folder is Windows, the rest match the platform name
				string PGOPlatform = Target.Platform == UnrealTargetPlatform.Win64 ? "Windows" : Target.Platform.ToString();
				DirectoryReference PGODirectory = DirectoryReference.Combine(Target.ProjectFile?.Directory ?? Unreal.WritableEngineDirectory, "Platforms", PGOPlatform, "Build", "PGO");
				MakefileBuilder.AddInternalDependency(FileItem.GetItemByFileReference(FileReference.Combine(PGODirectory, "PGOProfileCompilerInfo.txt")));
			}

			if (Target.bPrintToolChainTimingInfo || Target.WindowsPlatform.bCompilerTrace)
			{
				TargetMakefile Makefile = MakefileBuilder.Makefile;

				List<FileItem> TimingJsonFiles = new List<FileItem>();

				if (Target.WindowsPlatform.Compiler.IsMSVC())
				{
					List<IExternalAction> ParseTimingActions = Makefile.Actions.Where(x => x.ActionType == ActionType.ParseTimingInfo).ToList();
					TimingJsonFiles = ParseTimingActions.SelectMany(a => a.ProducedItems.Where(i => i.HasExtension(".cta"))).ToList();
				}
				else if (Target.WindowsPlatform.Compiler.IsClang())
				{
					List<IExternalAction> CompileActions = Makefile.Actions.Where(x => x.ActionType == ActionType.Compile && x.ProducedItems.Any(i => i.HasExtension(".json"))).ToList();
					TimingJsonFiles = CompileActions.SelectMany(a => a.ProducedItems.Where(i => i.HasExtension(".json"))).ToList();
				}

				Makefile.OutputItems.AddRange(TimingJsonFiles);

				// Handing generating aggregate timing information if we compiled more than one file.
				if (TimingJsonFiles.Count > 1)
				{
					// Generate the file manifest for the aggregator.
					if (!DirectoryReference.Exists(Makefile.ProjectIntermediateDirectory))
					{
						DirectoryReference.CreateDirectory(Makefile.ProjectIntermediateDirectory);
					}

					if (Target.WindowsPlatform.Compiler.IsMSVC())
					{
						FileReference ManifestFile = FileReference.Combine(Makefile.ProjectIntermediateDirectory, $"{Target.Name}TimingManifest.txt");
						File.WriteAllLines(ManifestFile.FullName, TimingJsonFiles.Select(f => f.AbsolutePath));

						FileReference ExpectedCompileTimeFile = new(Path.Combine(Makefile.ProjectIntermediateDirectory.FullName, $"{Target.Name}.json"));
						List<string> ActionArgs = new List<string>()
						{
							$"-Name={Target.Name}",
							$"-ManifestFile={ManifestFile.FullName}",
							$"-CompileTimingFile={ExpectedCompileTimeFile}",
						};

						Action AggregateTimingInfoAction = MakefileBuilder.CreateRecursiveAction<AggregateParsedTimingInfo>(ActionType.ParseTimingInfo, String.Join(" ", ActionArgs));
						AggregateTimingInfoAction.StatusDescription = $"Aggregating {TimingJsonFiles.Count} Timing File(s)";
						AggregateTimingInfoAction.PrerequisiteItems.UnionWith(TimingJsonFiles);

						FileItem AggregateOutputFile = FileItem.GetItemByFileReference(FileReference.Combine(Makefile.ProjectIntermediateDirectory, $"{Target.Name}.cta"));
						AggregateTimingInfoAction.ProducedItems.Add(AggregateOutputFile);
						Makefile.OutputItems.Add(AggregateOutputFile);
					}
					else if (Target.WindowsPlatform.Compiler.IsClang())
					{
						FileReference ManifestFile = FileReference.Combine(Makefile.ProjectIntermediateDirectory, $"{Target.Name}TimingManifest.csv");
						File.WriteAllLines(ManifestFile.FullName, TimingJsonFiles.Select(f => f.FullName.Remove(f.FullName.Length - ".json".Length)));

						FileItem AggregateOutputFile = FileItem.GetItemByFileReference(FileReference.Combine(Makefile.ProjectIntermediateDirectory, $"{Target.Name}.trace.csv"));
						FileItem HeadersOutputFile = FileItem.GetItemByFileReference(FileReference.Combine(Makefile.ProjectIntermediateDirectory, $"{Target.Name}.headers.csv"));
						List<string> AggregateActionArgs = new List<string>()
						{
							$"-ManifestFile={ManifestFile.FullName}",
							$"-AggregateFile={AggregateOutputFile.FullName}",
							$"-HeadersFile={HeadersOutputFile.FullName}",
						};

						Action AggregateTimingInfoAction = MakefileBuilder.CreateRecursiveAction<AggregateClangTimingInfo>(ActionType.ParseTimingInfo, String.Join(" ", AggregateActionArgs));
						AggregateTimingInfoAction.StatusDescription = $"Aggregating {TimingJsonFiles.Count} Timing File(s)";
						AggregateTimingInfoAction.PrerequisiteItems.UnionWith(TimingJsonFiles);

						AggregateTimingInfoAction.ProducedItems.Add(AggregateOutputFile);
						AggregateTimingInfoAction.ProducedItems.Add(HeadersOutputFile);
						Makefile.OutputItems.AddRange(AggregateTimingInfoAction.ProducedItems);

						FileItem ArchiveOutputFile = FileItem.GetItemByFileReference(FileReference.Combine(Makefile.ProjectIntermediateDirectory, $"{Target.Name}.traces.zip"));
						List<string> ArchiveActionArgs = new List<string>()
						{
							$"-ManifestFile={ManifestFile.FullName}",
							$"-ArchiveFile={ArchiveOutputFile.FullName}",
						};

						Action ArchiveTimingInfoAction = MakefileBuilder.CreateRecursiveAction<AggregateClangTimingInfo>(ActionType.ParseTimingInfo, String.Join(" ", ArchiveActionArgs));
						ArchiveTimingInfoAction.StatusDescription = $"Archiving {TimingJsonFiles.Count} Timing File(s)";
						ArchiveTimingInfoAction.PrerequisiteItems.UnionWith(TimingJsonFiles);

						ArchiveTimingInfoAction.ProducedItems.Add(ArchiveOutputFile);
						Makefile.OutputItems.AddRange(ArchiveTimingInfoAction.ProducedItems);

						// Extract CompileScore data from traces
						FileReference ScoreDataExtractor = FileReference.Combine(Unreal.RootDirectory, "Engine", "Extras", "ThirdPartyNotUE", "CompileScore", "ScoreDataExtractor.exe");
						if (OperatingSystem.IsWindows() && FileReference.Exists(ScoreDataExtractor))
						{
							FileItem CompileScoreOutput = FileItem.GetItemByFileReference(FileReference.Combine(Makefile.ProjectIntermediateDirectory, $"{Target.Name}.scor"));

							Action CompileScoreExtractorAction = MakefileBuilder.CreateAction(ActionType.ParseTimingInfo);
							CompileScoreExtractorAction.WorkingDirectory = Unreal.EngineSourceDirectory;
							CompileScoreExtractorAction.StatusDescription = $"Extracting CompileScore";
							CompileScoreExtractorAction.bCanExecuteRemotely = false;
							CompileScoreExtractorAction.bCanExecuteRemotelyWithSNDBS = false;
							CompileScoreExtractorAction.bCanExecuteInUBA = false; // TODO: Unknown if supported
							CompileScoreExtractorAction.PrerequisiteItems.UnionWith(TimingJsonFiles);
							CompileScoreExtractorAction.CommandPath = ScoreDataExtractor;
							CompileScoreExtractorAction.CommandArguments = $"-clang -verbosity 0 -timelinepack 1000000 -extract -i \"{NormalizeCommandLinePath(Makefile.ProjectIntermediateDirectory)}\" -o \"{NormalizeCommandLinePath(CompileScoreOutput)}\"";

							CompileScoreExtractorAction.ProducedItems.Add(CompileScoreOutput);
							CompileScoreExtractorAction.ProducedItems.Add(FileItem.GetItemByFileReference(FileReference.Combine(Makefile.ProjectIntermediateDirectory, $"{Target.Name}.scor.gbl")));
							CompileScoreExtractorAction.ProducedItems.Add(FileItem.GetItemByFileReference(FileReference.Combine(Makefile.ProjectIntermediateDirectory, $"{Target.Name}.scor.incl")));
							CompileScoreExtractorAction.ProducedItems.Add(FileItem.GetItemByFileReference(FileReference.Combine(Makefile.ProjectIntermediateDirectory, $"{Target.Name}.scor.t0000")));
							Makefile.OutputItems.AddRange(CompileScoreExtractorAction.ProducedItems);
						}
					}
				}
			}

			OptionalUpdateSxSManifest(Target, MakefileBuilder, ExeDir);
		}

		private static readonly ConcurrentDictionary<FileReference, Lock> ManifestLocks = [];

		protected static void OptionalUpdateSxSManifest(ReadOnlyTargetRules Target, TargetMakefileBuilder MakefileBuilder, DirectoryReference ExeDir)
		{
			if (Target.WindowsPlatform.ManifestFile != null && !Target.bShouldCompileAsDLL && !Target.Architecture.bIsX64)
			{
#pragma warning disable CA1308 // Normalize strings to uppercase - file path should use the correct case
				string AssemblyName = Target.Architecture.ToString().ToLowerInvariant();
#pragma warning restore CA1308

				// collect the DLLs that should be included in the SxS manifest
				DirectoryReference ManifestDir = DirectoryReference.Combine(MakefileBuilder.Makefile.ExecutableFile.Directory, AssemblyName);
				IEnumerable<FileItem> DLLs = MakefileBuilder.Makefile.OutputItems.Where( Item =>
					Item.HasExtension(".dll") &&
					Item.Location.IsUnderDirectory(ManifestDir)
				)
				.Distinct();

				// load or create the intermediate SxS manifest
				bool bDirty = false;
				FileReference IntermediateManifest = GetSxSIntermediateManifest(Target, ExeDir);
				FileReference TemplateManifest = FileReference.Combine(Unreal.EngineDirectory, "Build", "Windows", "Resources", AssemblyName, AssemblyName + ".manifest");

				Lock Lock = ManifestLocks.GetOrAdd(IntermediateManifest, _ => new());
				lock (Lock)
				{
					XDocument XmlManifest;

					if (FileReference.Exists(IntermediateManifest))
					{
						XmlManifest = XDocument.Load(IntermediateManifest.FullName);
					}
					else
					{
						XmlManifest = XDocument.Load(TemplateManifest.FullName);
						bDirty = true;
					}

					// update & save the intermediate SxS manifest
					if (XmlManifest.Root != null && XmlManifest.Root.Name.LocalName == "assembly")
					{
						XNamespace XmlNs = XmlManifest.Root.Name.Namespace;
						HashSet<string> ExistingDLLNames = [.. XmlManifest.Root.Elements(XmlNs + "file").Select(X => X.Attribute("name")!.Value)];

						foreach (FileItem DLL in DLLs)
						{
							string DLLName = DLL.Location.MakeRelativeTo(ManifestDir);
							if (!ExistingDLLNames.Contains(DLLName))
							{
								XmlManifest.Root.Add(new XElement(XmlNs + "file", new XAttribute("name", DLLName)));
								bDirty = true;
							}
						}
					}
					if (bDirty)
					{
						DirectoryReference.CreateDirectory(IntermediateManifest.Directory);
						XmlManifest.Save(IntermediateManifest.FullName);
					}
				}
				// include the intermediate SxS manifest as a dependency
				MakefileBuilder.Makefile.InternalDependencies.Add( FileItem.GetItemByFileReference(IntermediateManifest) );
			}
		}

		public override void PrepareRuntimeDependencies(List<RuntimeDependency> RuntimeDependencies, Dictionary<FileReference, FileReference> TargetFileToSourceFile, DirectoryReference ExeDir)
		{
			// If ASan is enabled we need to copy the companion helper libraries from the MSVC tools bin folder to the
			// target executable folder.
			if (Target.WindowsPlatform.bEnableAddressSanitizer)
			{
				DirectoryReference ASanRuntimeDir;
				string ASanArchSuffix;
				if (EnvVars.Architecture == UnrealArch.X64)
				{
					if (Target.WindowsPlatform.Compiler.IsClang())
					{
						VersionNumber ClangVersion = Target.WindowsPlatform.Compiler == WindowsCompiler.Intel ? MicrosoftPlatformSDK.GetClangVersionForIntelCompiler(EnvVars.CompilerPath) : EnvVars.CompilerVersion;
						ASanRuntimeDir = DirectoryReference.Combine(EnvVars.CompilerDir, "lib", "clang", $"{ClangVersion.Components[0]}", "lib", "windows");
					}
					else
					{
						ASanRuntimeDir = DirectoryReference.Combine(EnvVars.ToolChainDir, "bin", "Hostx64", "x64");
					}
					ASanArchSuffix = "x86_64";
				}
				else
				{
					throw new BuildException("Unsupported build architecture for Address Sanitizer");
				}

				string ASanRuntimeDLL = $"clang_rt.asan_dynamic-{ASanArchSuffix}.dll";
				string ASanDebugRuntimeDLL = $"clang_rt.asan_dbg_dynamic-{ASanArchSuffix}.dll";

				RuntimeDependencies.Add(new RuntimeDependency(FileReference.Combine(ExeDir, ASanRuntimeDLL), StagedFileType.NonUFS));
				TargetFileToSourceFile[FileReference.Combine(ExeDir, ASanRuntimeDLL)] = FileReference.Combine(ASanRuntimeDir, ASanRuntimeDLL);
				if (Target.bDebugBuildsActuallyUseDebugCRT)
				{
					RuntimeDependencies.Add(new RuntimeDependency(FileReference.Combine(ExeDir, ASanDebugRuntimeDLL), StagedFileType.NonUFS));
					TargetFileToSourceFile[FileReference.Combine(ExeDir, ASanDebugRuntimeDLL)] = FileReference.Combine(ASanRuntimeDir, ASanDebugRuntimeDLL);
				}
			}

			// copy PGO support binaries for Windows from $(VC_PGO_RunTime_Dir)
			if (Target.bPGOProfile && Target.Platform.IsInGroup(UnrealPlatformGroup.Windows) && !Target.WindowsPlatform.Compiler.IsClang())
			{
				// Both ARM64 and ARM64EC use the same files. This requires the DLLs are all built as ARM64X.
				string[] PGOFiles = {
					"pgort140.dll",
					"pgosweep.exe",
				};

				// Non-x64 files are placed in sub-directories. A manifest is used to load DLLs from this directory,
				// and the EXEs just need to not stomp each other when building multiple architectures at once.
				DirectoryReference DstDir = ExeDir;
				if (!Target.Architecture.bIsX64)
				{
					DstDir = DirectoryReference.Combine(ExeDir, Target.Architecture.ToString());
				}

				foreach (string PGOFile in PGOFiles)
				{
					FileReference SrcFile = EnvVars.Architecture.bIsX64
						? FileReference.Combine(EnvVars.ToolChainDir, "bin", "Hostx64", "x64", PGOFile)
						: FileReference.Combine(EnvVars.ToolChainDir, "bin", "arm64", PGOFile);
					FileReference DstFile = FileReference.Combine(DstDir, PGOFile);
					TargetFileToSourceFile[DstFile] = SrcFile;
					RuntimeDependencies.Add(new RuntimeDependency(DstFile, StagedFileType.NonUFS));
				}
			}

			// add a dependency on the SxS manifest
			if (Target.WindowsPlatform.ManifestFile != null && !Target.bShouldCompileAsDLL && !Target.Architecture.bIsX64)
			{
#pragma warning disable CA1308 // Normalize strings to uppercase - file path should use the correct case
				string AssemblyName = Target.Architecture.ToString().ToLowerInvariant();
#pragma warning restore CA1308
				FileReference SrcFile = GetSxSIntermediateManifest(Target, ExeDir);
				FileReference DstFile = FileReference.Combine(ExeDir, AssemblyName, AssemblyName + ".manifest");

				TargetFileToSourceFile[DstFile] = SrcFile;
				RuntimeDependencies.Add(new RuntimeDependency(DstFile, StagedFileType.NonUFS));
			}

			if (Target.WindowsPlatform.Compiler.IsIntel())
			{
				FileReference SrcFile = FileReference.Combine(EnvVars.CompilerDir, "bin", "libmmd.dll");
				FileReference DstFile = FileReference.Combine(ExeDir, "libmmd.dll");

				TargetFileToSourceFile[DstFile] = SrcFile;
				RuntimeDependencies.Add(new RuntimeDependency(DstFile, StagedFileType.NonUFS));
			}
		}

		public virtual FileReference GetApplicationIcon(FileReference? ProjectFile)
		{
			if (Target.WindowsPlatform.ApplicationIcon != null)
			{
				return new FileReference(Target.WindowsPlatform.ApplicationIcon);
			}
			return WindowsPlatform.GetWindowsApplicationIcon(ProjectFile);
		}

		protected virtual bool UseWindowsArchitecture(UnrealTargetPlatform Platform)
		{
			return Platform.IsInGroup(UnrealPlatformGroup.Windows);
		}

		public override CPPOutput CompileRCFiles(CppCompileEnvironment CompileEnvironment, IEnumerable<FileItem> InputFiles, DirectoryReference OutputDir, IActionGraphBuilder Graph)
		{
			if (!InputFiles.Any())
			{
				return new();
			}

			CPPOutput Result = new();
			CppRootPaths RootPaths = CompileEnvironment.RootPaths;

			foreach (FileItem RCFile in InputFiles)
			{
				Action CompileAction = Graph.CreateAction(ActionType.Compile);
				CompileAction.RootPaths = CompileEnvironment.RootPaths;
				CompileAction.CommandDescription = $"Compile Resource [{CompileEnvironment.Architecture.WindowsName}]";
				CompileAction.WorkingDirectory = Unreal.EngineSourceDirectory;
				CompileAction.CommandPath = EnvVars.ResourceCompilerPath;
				CompileAction.StatusDescription = Path.GetFileName(RCFile.AbsolutePath);
				CompileAction.PrerequisiteItems.UnionWith([
					.. CompileEnvironment.ForceIncludeFiles,
					.. CompileEnvironment.AdditionalPrerequisites,
					// track known included headers as resource compiler does not support writing input dependencies to a file
					FileItem.GetItemByFileReference(FileReference.Combine(Unreal.EngineSourceDirectory, "Runtime", "Core", "Public", "HAL", "PreprocessorHelpers.h")),
					FileItem.GetItemByFileReference(FileReference.Combine(Unreal.EngineSourceDirectory, "Runtime", "Launch", "Resources", "Windows", "resource.h")),
					FileItem.GetItemByFileReference(FileReference.Combine(Unreal.EngineSourceDirectory, "Runtime", "Launch", "Resources", "Version.h")),
				]);
				CompileAction.PrerequisiteItems.UnionWith(CompileEnvironment.AdditionalPrerequisites);
				CompileAction.ArtifactMode = ArtifactMode.Enabled;
				CompileAction.CacheBucket = GetCacheBucket(Target, CompileEnvironment);

				// Resource tool can run remotely if possible
				CompileAction.bCanExecuteRemotely = true;
				CompileAction.bCanExecuteRemotelyWithSNDBS = false; // no tool template for SN-DBS results in warnings

				List<string> Arguments = [];

				// Suppress header spew
				Arguments.Add("/nologo");

				// If we're compiling for 64-bit Windows, also add the _WIN64 definition to the resource
				// compiler so that we can switch on that in the .rc file using #ifdef.
				AddDefinition(Arguments, "_WIN64");

				// Language
				Arguments.Add("/l 0x409");

				// Include paths. Don't use AddIncludePath() here, since it uses the full path and exceeds the max command line length.
				foreach (DirectoryReference IncludePath in CompileEnvironment.UserIncludePaths)
				{
					Arguments.Add($"/I \"{NormalizeCommandLinePath(IncludePath, RootPaths)}\"");
				}

				// System include paths.
				foreach (DirectoryReference SystemIncludePath in CompileEnvironment.SystemIncludePaths)
				{
					Arguments.Add($"/I \"{NormalizeCommandLinePath(SystemIncludePath, RootPaths)}\"");
				}
				foreach (DirectoryReference SystemIncludePath in EnvVars.IncludePaths)
				{
					Arguments.Add($"/I \"{NormalizeCommandLinePath(SystemIncludePath, RootPaths)}\"");
				}

				// Preprocessor definitions.
				foreach (string Definition in CompileEnvironment.Definitions)
				{
					if (!Definition.Contains("_API", StringComparison.Ordinal))
					{
						AddDefinition(Arguments, Definition);
					}
				}

				// Set the other version fields if requested
				if (Target.WindowsPlatform.bSetResourceVersions)
				{
					if (Target.Version.Changelist != 0)
					{
						AddDefinition(Arguments, $"BUILT_FROM_CHANGELIST={Target.Version.Changelist}");
					}
					if (!String.IsNullOrEmpty(Target.Version.BranchName))
					{
						AddDefinition(Arguments, $"BRANCH_NAME={Target.Version.BranchName}");
					}
					if (!String.IsNullOrEmpty(Target.BuildVersion))
					{
						AddDefinition(Arguments, $"BUILD_VERSION={Target.BuildVersion}");
					}
				}

				if (Target.WindowsPlatform.bDefineIconResource)
				{
					// Figure the icon to use. We can only use a custom icon when compiling to a project-specific intermediate directory (and not for the shared editor executable, for example).
					FileItem IconFile = FileItem.GetItemByFileReference(GetApplicationIcon(!CompileEnvironment.bUseSharedBuildEnvironment ? Target.ProjectFile : null));
					CompileAction.PrerequisiteItems.Add(IconFile);

					// Setup the compile environment, setting the icon to use via a macro. This is used in Default.rc2.
					AddDefinition(Arguments, $"BUILD_ICON_FILE_NAME=\"\\\"{NormalizeCommandLinePath(IconFile, RootPaths).Replace("/", "\\\\", StringComparison.Ordinal)}\\\"\"");
				}

				// Apply the target settings for the resources
				if (!CompileEnvironment.bUseSharedBuildEnvironment)
				{
					if (!String.IsNullOrEmpty(Target.WindowsPlatform.CompanyName))
					{
						AddDefinition(Arguments, $"PROJECT_COMPANY_NAME={SanitizeMacroValue(Target.WindowsPlatform.CompanyName)}");
					}

					if (!String.IsNullOrEmpty(Target.WindowsPlatform.CopyrightNotice))
					{
						AddDefinition(Arguments, $"PROJECT_COPYRIGHT_STRING={SanitizeMacroValue(Target.WindowsPlatform.CopyrightNotice)}");
					}

					if (!String.IsNullOrEmpty(Target.WindowsPlatform.ProductName))
					{
						AddDefinition(Arguments, $"PROJECT_PRODUCT_NAME={SanitizeMacroValue(Target.WindowsPlatform.ProductName)}");
					}

					if (Target.ProjectFile != null)
					{
						AddDefinition(Arguments, $"PROJECT_PRODUCT_IDENTIFIER={SanitizeMacroValue(Target.ProjectFile.GetFileNameWithoutExtension())}");
					}
				}

				// Add the RES file to the produced item list.
				FileItem CompiledResourceFile = FileItem.GetItemByFileReference(FileReference.Combine(OutputDir, $"{Path.GetFileName(RCFile.AbsolutePath)}.res"));
				CompileAction.ProducedItems.Add(CompiledResourceFile);
				Arguments.Add($"/fo \"{NormalizeCommandLinePath(CompiledResourceFile, RootPaths)}\"");
				Result.ObjectFiles.Add(CompiledResourceFile);

				// Add the RC file as a prerequisite of the action.
				Arguments.Add($"\"{NormalizeCommandLinePath(RCFile, RootPaths)}\"");

				// Create a response file for the resource compilier for informational purposes as it cannot be used
				FileItem ResponseFile = FileItem.GetItemByFileReference(GetResponseFileName(CompiledResourceFile));
				Graph.CreateIntermediateTextFile(ResponseFile, Arguments);
				CompileAction.PrerequisiteItems.Add(ResponseFile);

				CompileAction.CommandArguments = String.Join(' ', Arguments);

				// Add the C++ source file and its included files to the prerequisite item list.
				CompileAction.PrerequisiteItems.Add(RCFile);
			}

			return Result;
		}

		public override CPPOutput ConvertRCFiles(LinkEnvironment LinkEnvironment, IEnumerable<FileItem> InputFiles, DirectoryReference OutputDir, IActionGraphBuilder Graph)
		{
			CPPOutput Result = new();

			// Do not convert resource files if the input list is empty, compiling an application, or of shared resource files are not in use
			// Converting resource files to objects when using the clang linker is not supported
			if (!InputFiles.Any()
				|| !(LinkEnvironment.bIsBuildingDLL || LinkEnvironment.bIsBuildingLibrary)
				|| !Target.WindowsPlatform.bSharedResourceFiles
				|| (EnvVars.Compiler.IsClang() && EnvVars.bAllowClangLinker))
			{
				Result.ObjectFiles.AddRange(InputFiles);
				return Result;
			}

			FileItem ConvertedResourceObjectFile = FileItem.GetItemByFileReference(FileReference.Combine(OutputDir, $"{InputFiles.First().Name}.obj"));

			List<string> ConvertArguments = [
				"/nologo",
				"/folddups",
				$"/machine:{LinkEnvironment.Architecture.WindowsName}",
				$"/out:\"{NormalizeCommandLinePath(ConvertedResourceObjectFile, LinkEnvironment.RootPaths)}\"",
				.. InputFiles.Select(x => $"\"{NormalizeCommandLinePath(x, LinkEnvironment.RootPaths)}\"")
			];

			Action ConvertAction = Graph.CreateAction(ActionType.Compile);
			ConvertAction.RootPaths = LinkEnvironment.RootPaths;
			ConvertAction.CommandDescription = $"Convert Resource [{LinkEnvironment.Architecture.WindowsName}]";
			ConvertAction.WorkingDirectory = Unreal.EngineSourceDirectory;
			ConvertAction.CommandPath = EnvVars.ResourceToObjectConverterPath;
			ConvertAction.CommandArguments = String.Join(' ', ConvertArguments);
			ConvertAction.StatusDescription = Path.GetFileName(InputFiles.First().AbsolutePath);
			ConvertAction.PrerequisiteItems.UnionWith(InputFiles);
			ConvertAction.ProducedItems.Add(ConvertedResourceObjectFile);
			ConvertAction.ArtifactMode = ArtifactMode.Enabled;
			ConvertAction.CacheBucket = GetCacheBucket(Target, LinkEnvironment);

			// Convert Resource tool can run remotely if possible
			ConvertAction.bCanExecuteRemotely = true;
			ConvertAction.bCanExecuteRemotelyWithSNDBS = false; // no tool template for SN-DBS results in warnings

			// Create a response file for the resource converter arguments for informational purposes as it cannot be used
			FileItem ResponseFile = FileItem.GetItemByFileReference(GetResponseFileName(LinkEnvironment, ConvertedResourceObjectFile));
			Graph.CreateIntermediateTextFile(ResponseFile, ConvertArguments);
			ConvertAction.PrerequisiteItems.Add(ResponseFile);

			Result.ObjectFiles.Add(ConvertedResourceObjectFile);
			return Result;
		}

		public override void GenerateTypeLibraryHeader(CppCompileEnvironment CompileEnvironment, ModuleRules.TypeLibrary TypeLibrary, FileReference OutputFile, FileReference? OutputHeader, IActionGraphBuilder Graph)
		{
			CppRootPaths RootPaths = CompileEnvironment.RootPaths;

			FileItem TypeLibraryFile = FileItem.GetItemByPath(TypeLibrary.FileName);

			// Create the input file
			StringBuilder Contents = new StringBuilder();
			Contents.AppendLine("#include <windows.h>");
			Contents.AppendLine("#include <unknwn.h>");
			Contents.AppendLine();

			Contents.AppendFormat("#import \"{0}\"", NormalizeCommandLinePath(TypeLibraryFile, RootPaths).Replace('\\', '/'));
			if (!String.IsNullOrEmpty(TypeLibrary.Attributes))
			{
				Contents.Append(' ');
				Contents.Append(TypeLibrary.Attributes);
			}
			Contents.AppendLine();

			FileItem InputFile = Graph.CreateIntermediateTextFile(OutputFile.ChangeExtension(".cpp"), Contents.ToString(), false);

			// Build the argument list for the compiler
			FileItem ObjectFile = FileItem.GetItemByFileReference(OutputFile.ChangeExtension(".obj"));

			List<string> Arguments = new List<string>
			{
				$"\"{NormalizeCommandLinePath(InputFile, RootPaths)}\"",
				"/c",
				"/nologo",
				$"/Fo\"{NormalizeCommandLinePath(ObjectFile, RootPaths)}\""
			};

			AddIncludePathArguments(CompileEnvironment, EnvVars.ToolChain, Arguments);

			// Create the batch file action that will compile then touch the output file
			Action CompileAction = Graph.CreateAction(ActionType.Compile);
			CompileAction.ResponseFileContents = [.. Arguments];
			FileItem ResponseFile = Graph.CreateIntermediateTextFile(OutputFile.ChangeExtension(ResponseExt), CompileAction.ResponseFileContents, false);

			// Build the command for touching the output file(s) to update the time stamp
			string OutfileTlhFullName = NormalizeCommandLinePath(OutputFile, RootPaths).Replace('/', '\\'); // Must be backward slash
			string TouchTlhActionCommand = $"if exist \"{OutfileTlhFullName}\" ( copy /b \"{OutfileTlhFullName}\" + NUL \"{OutfileTlhFullName}\">nul )";
			string OutfileTliFullName = NormalizeCommandLinePath(OutputFile.ChangeExtension(".tli"), RootPaths).Replace('/', '\\'); // Must be backward slash
			string TouchTliActionCommand = $"if exist \"{OutfileTliFullName}\" ( copy /b \"{OutfileTliFullName}\" + NUL \"{OutfileTliFullName}\">nul )";

			// Build the batch file
			StringBuilder BatchFileContents = new();
			BatchFileContents.AppendLine("@echo off");
			BatchFileContents.AppendLine($"\"{NormalizeCommandLinePath(EnvVars.ToolchainCompilerPath, RootPaths)}\" @\"{NormalizeCommandLinePath(ResponseFile, RootPaths)}\"");
			BatchFileContents.AppendLine(TouchTlhActionCommand);
			BatchFileContents.AppendLine(TouchTliActionCommand);

			FileItem BatchFile = Graph.CreateIntermediateTextFile(OutputFile.ChangeExtension(".bat"), BatchFileContents.ToString(), false);

			CompileAction.CommandDescription = "GenerateTLH";
			CompileAction.PrerequisiteItems.Add(FileItem.GetItemByFileReference(EnvVars.ToolchainCompilerPath));
			CompileAction.PrerequisiteItems.Add(TypeLibraryFile);
			CompileAction.PrerequisiteItems.Add(InputFile);
			CompileAction.PrerequisiteItems.Add(ResponseFile);
			CompileAction.PrerequisiteItems.Add(BatchFile);
			CompileAction.ProducedItems.Add(ObjectFile);
			CompileAction.ProducedItems.Add(FileItem.GetItemByFileReference(OutputFile));
			if (OutputHeader != null)
			{
				CompileAction.ProducedItems.Add(FileItem.GetItemByFileReference(OutputHeader));
			}

			CompileAction.DeleteItems.Add(FileItem.GetItemByFileReference(OutputFile));
			CompileAction.DeleteItems.Add(FileItem.GetItemByFileReference(OutputFile.ChangeExtension(".tli"))); // May not be created, depending on Attributes
			CompileAction.StatusDescription = TypeLibrary.Header;
			CompileAction.WorkingDirectory = Unreal.EngineSourceDirectory;
			CompileAction.CommandPath = BuildHostPlatform.Current.Shell;
			CompileAction.CommandArguments = $"/C \"{BatchFile}\"";
			CompileAction.CommandVersion = EnvVars.ToolChainVersion.ToString();
			if (Target.bAllowUbaCompression)
			{
				CompileAction.CommandVersion = $"{CompileAction.CommandVersion} Compressed";
			}
			CompileAction.bShouldOutputStatusDescription = false;
			CompileAction.bCanExecuteRemotely = false; // Incompatible with remote distribution
			CompileAction.RootPaths = CompileEnvironment.RootPaths;
			CompileAction.ArtifactMode = ArtifactMode.Enabled;
			CompileAction.CacheBucket = GetCacheBucket(Target, CompileEnvironment);
		}

		public override IEnumerable<string> GetGlobalCommandLineArgs(CppCompileEnvironment CompileEnvironment)
		{
			List<string> Arguments = new();
			
			AppendCLArguments_Global(new(CompileEnvironment), Arguments);
			return Arguments;
		}

		public override IEnumerable<string> GetCPPCommandLineArgs(CppCompileEnvironment CompileEnvironment)
		{
			List<string> Arguments = new();

			AppendCLArguments_CPP(CompileEnvironment, Arguments);

			return Arguments;
		}

		public override IEnumerable<string> GetCCommandLineArgs(CppCompileEnvironment CompileEnvironment)
		{
			List<string> Arguments = new();

			AppendCLArguments_C(CompileEnvironment, Arguments);
			return Arguments;
		}

		public override CppCompileEnvironment CreateSharedResponseFile(CppCompileEnvironment CompileEnvironment, FileReference OutResponseFile, IActionGraphBuilder Graph)
		{
			CppCompileEnvironment NewCompileEnvironment = new CppCompileEnvironment(CompileEnvironment);
			List<string> Arguments = new List<string>();

			AppendCLArguments_Global(NewCompileEnvironment, Arguments);
			AddIncludePathArguments(NewCompileEnvironment, Target.WindowsPlatform.Compiler, Arguments);

			// Stash shared include paths for validation purposes
			NewCompileEnvironment.SharedUserIncludePaths = new(NewCompileEnvironment.UserIncludePaths);
			NewCompileEnvironment.SharedSystemIncludePaths = new(NewCompileEnvironment.SystemIncludePaths);
			NewCompileEnvironment.SharedSystemIncludePaths.UnionWith(EnvVars.IncludePaths);

			NewCompileEnvironment.UserIncludePaths.Clear();
			NewCompileEnvironment.SystemIncludePaths.Clear();

			FileItem FileItem = FileItem.GetItemByFileReference(OutResponseFile);
			Graph.CreateIntermediateTextFile(FileItem, Arguments);

			NewCompileEnvironment.AdditionalResponseFiles.Add(FileItem);
			NewCompileEnvironment.bHasSharedResponseFile = true;

			return NewCompileEnvironment;
		}

		public override void CreateSpecificFileAction(CppCompileEnvironment CompileEnvironment, DirectoryReference SourceDir, DirectoryReference OutputDir, IActionGraphBuilder Graph)
		{
			// This is not supported for now.. If someone wants it we can implement it
			if (CompileEnvironment.Architectures.bIsMultiArch)
			{
				return;
			}

			VCCompileAction BaseCompileAction = CreateBaseCompileAction(CompileEnvironment);
			AppendCLArguments_CPP(CompileEnvironment, BaseCompileAction.Arguments);
			BaseCompileAction.AdditionalPrerequisiteItems.AddRange(CompileEnvironment.AdditionalPrerequisites); // Primarily for ispc.generated.h files
			Graph.AddAction(new VcSpecificFileAction(SourceDir, OutputDir, BaseCompileAction, CompileEnvironment));
		}

		/// <summary>
		/// Macros passed via the command line have their quotes stripped, and are tokenized before being re-stringized by the compiler. This conversion
		/// back and forth is normally ok, but certain characters such as single quotes must always be paired. Remove any such characters here.
		/// </summary>
		/// <param name="Value">The macro value</param>
		/// <returns>The sanitized value</returns>
		static string SanitizeMacroValue(string Value)
		{
			StringBuilder Result = new StringBuilder(Value.Length);
			for (int Idx = 0; Idx < Value.Length; Idx++)
			{
				if (Value[Idx] != '\'' && Value[Idx] != '\"')
				{
					Result.Append(Value[Idx]);
				}
			}
			return Result.ToString();
		}

		public override FileItem[] LinkImportLibrary(LinkEnvironment LinkEnvironment, IActionGraphBuilder Graph)
		{
			if (LinkEnvironment.bIsCrossReferenced)
			{
				// Windows Resource files can be excluded from import libraries
				LinkEnvironment CrossReferencedLinkEnvironment = new(LinkEnvironment);
				CrossReferencedLinkEnvironment.InputFiles.RemoveAll(x => x.Name.EndsWith(".res", StringComparison.OrdinalIgnoreCase) || x.Name.EndsWith(".res.obj", StringComparison.OrdinalIgnoreCase));
				return LinkAllFiles(CrossReferencedLinkEnvironment, true, Graph);
			}
			// by default do nothing
			return [];
		}

		public override FileItem LinkFiles(LinkEnvironment LinkEnvironment, bool bBuildImportLibraryOnly, IActionGraphBuilder Graph)
		{
			CppRootPaths RootPaths = LinkEnvironment.RootPaths;

			if (LinkEnvironment.bIsBuildingDotNetAssembly)
			{
				return FileItem.GetItemByFileReference(LinkEnvironment.OutputFilePath);
			}

			bool bIsBuildingLibraryOrImportLibrary = LinkEnvironment.bIsBuildingLibrary || bBuildImportLibraryOnly;

			// Get link arguments.
			List<string> Arguments = new List<string>();
			if (bIsBuildingLibraryOrImportLibrary)
			{
				AppendLibArguments(LinkEnvironment, Arguments);
			}
			else
			{
				AppendLinkArguments(LinkEnvironment, Arguments);
			}

			if (Target.WindowsPlatform.Compiler.IsMSVC() && LinkEnvironment.bPrintTimingInfo)
			{
				Arguments.Add("/time+");
			}

			if (IsDynamicDebuggingEnabled)
			{
				Arguments.Add("/dynamicdeopt");
			}

			// Rad linker should not be used for import libs
			if (Target.WindowsPlatform.bAllowRadLinker && !bBuildImportLibraryOnly)
			{
				// Saves space in type table in pdb
				Arguments.Add("/RAD_PDB_HASH_TYPE_NAMES:lenient");
				Arguments.Add("/RAD_SHARED_THREAD_POOL");
				//Arguments.Add("/RAD_WRITE_TEMP_FILES");

				// Speeds up linking - TODO: disabled because of windows issues
				//Arguments.Add("/rad_large_pages");

				// Speeds up linking. We don't want this enabled on shipping since it will affect security
				if (!LinkEnvironment.bIsBuildingDLL && !LinkEnvironment.bIsBuildingLibrary && LinkEnvironment.Configuration != CppConfiguration.Shipping)
				{
					Arguments.Add("/fixed");
				}
			}

			// If we're only building an import library, add the '/DEF' option that tells the LIB utility
			// to simply create a .LIB file and .EXP file, and don't bother validating imports
			if (bBuildImportLibraryOnly)
			{
				Arguments.Add("/DEF");

				// Ensure that the import library references the correct filename for the linked binary.
				Arguments.Add($"/NAME:\"{LinkEnvironment.OutputFilePath.GetFileName()}\"");

				// Ignore warnings about object files with no public symbols.
				Arguments.Add("/IGNORE:4221");
			}

			if (Target.WindowsPlatform.Compiler.IsClang() && Target.WindowsPlatform.bAllowClangLinker)
			{
				Arguments.Add("/IGNORE:importeddllmain");
			}

			if (!bIsBuildingLibraryOrImportLibrary)
			{
				// Delay-load these DLLs.
				foreach (string DelayLoadDLL in LinkEnvironment.DelayLoadDLLs.Distinct())
				{
					Arguments.Add($"/DELAYLOAD:\"{DelayLoadDLL}\"");
				}

				// Pass the module definition file to the linker if we have one
				if (LinkEnvironment.ModuleDefinitionFile != null && LinkEnvironment.ModuleDefinitionFile.Length > 0)
				{
					Arguments.Add($"/DEF:\"{LinkEnvironment.ModuleDefinitionFile}\"");
				}
			}

			// Set up the library paths for linking this binary
			if (bBuildImportLibraryOnly)
			{
				// When building an import library, ignore all the libraries included via embedded #pragma lib declarations.
				// We shouldn't need them to generate exports.
				Arguments.Add("/NODEFAULTLIB");
			}
			else if (!LinkEnvironment.bIsBuildingLibrary)
			{
				// Add the library paths to the argument list.
				foreach (DirectoryReference LibraryPath in LinkEnvironment.SystemLibraryPaths)
				{
					Arguments.Add($"/LIBPATH:\"{NormalizeCommandLinePath(LibraryPath, RootPaths)}\"");
				}
				foreach (DirectoryReference LibraryPath in EnvVars.LibraryPaths)
				{
					Arguments.Add($"/LIBPATH:\"{NormalizeCommandLinePath(LibraryPath, RootPaths)}\"");
				}

				// Add the excluded default libraries to the argument list.
				foreach (string ExcludedLibrary in LinkEnvironment.ExcludedLibraries)
				{
					Arguments.Add($"/NODEFAULTLIB:\"{ExcludedLibrary}\"");
				}
			}

			// Enable function level hot-patching
			if (!bBuildImportLibraryOnly && Target.WindowsPlatform.bCreateHotpatchableImage)
			{
				Arguments.Add("/FUNCTIONPADMIN:6"); // For some reason, not providing the number causes full linking to happen all the time
			}

			// For targets that are cross-referenced, we don't want to write a LIB file during the link step as that
			// file will clobber the import library we went out of our way to generate during an earlier step.  This
			// file is not needed for our builds, but there is no way to prevent MSVC from generating it when
			// linking targets that have exports.  We don't want this to clobber our LIB file and invalidate the
			// existing timstamp, so instead we simply emit it with a different name
			FileReference? ImportLibraryFilePath = null;
			if (LinkEnvironment.bIsCrossReferenced && !bBuildImportLibraryOnly)
			{
				Arguments.Add("/NOIMPLIB");
				if (!Target.WindowsPlatform.Compiler.IsClang())
				{
					Arguments.Add("/NOEXP"); // This compiler flag does not exist on lld-link.exe.. it skips the writing of the .exp file
				}
			}
			else if (Target.bShouldCompileAsDLL)
			{
				ImportLibraryFilePath = FileReference.Combine(LinkEnvironment.OutputDirectory!, LinkEnvironment.OutputFilePath.GetFileNameWithoutExtension() + ".lib");
			}
			else
			{
				ImportLibraryFilePath = FileReference.Combine(LinkEnvironment.IntermediateDirectory!, LinkEnvironment.OutputFilePath.GetFileNameWithoutExtension() + ".lib");
			}

			FileItem OutputFile;
			if (bBuildImportLibraryOnly)
			{
				OutputFile = FileItem.GetItemByFileReference(ImportLibraryFilePath!);
			}
			else
			{
				OutputFile = FileItem.GetItemByFileReference(LinkEnvironment.OutputFilePath);
			}

			List<FileItem> ProducedItems = new List<FileItem>();
			ProducedItems.Add(OutputFile);

			List<FileItem> PrerequisiteItems = new List<FileItem>();

			// Add the input files to a response file, and pass the response file on the command-line.
			List<string> InputFileNames = new List<string>();
			foreach (FileItem InputFile in LinkEnvironment.InputFiles)
			{
				if (InputFile.HasExtension(".toml"))
				{
					// We don't want the toml file for import lib creation
					if (!LinkEnvironment.bIsBuildingDLL || !bIsBuildingLibraryOrImportLibrary)
					{
						Arguments.Add("/ifcDebugRecords");
						Arguments.Add($"/ifcMap:\"{NormalizeCommandLinePath(InputFile, RootPaths)}\"");
						PrerequisiteItems.Add(InputFile);
					}
				}
				else
				{
					InputFileNames.Add($"\"{NormalizeCommandLinePath(InputFile, RootPaths)}\"");
					PrerequisiteItems.Add(InputFile);
				}
			}

			if (!bIsBuildingLibraryOrImportLibrary)
			{
				foreach (FileReference Library in LinkEnvironment.Libraries)
				{
					InputFileNames.Add($"\"{NormalizeCommandLinePath(Library, RootPaths)}\"");
					PrerequisiteItems.Add(FileItem.GetItemByFileReference(Library));
				}
				foreach (string SystemLibrary in LinkEnvironment.SystemLibraries)
				{
					if (Path.IsPathFullyQualified(SystemLibrary))
					{
						FileItem SystemLibraryItem = FileItem.GetItemByPath(SystemLibrary);
						InputFileNames.Add($"\"{NormalizeCommandLinePath(SystemLibraryItem, RootPaths)}\"");
						PrerequisiteItems.Add(SystemLibraryItem);
					}
					else
					{
						InputFileNames.Add($"\"{SystemLibrary}\"");
					}
				}

				foreach (FileItem NatvisFile in LinkEnvironment.DebuggerVisualizerFiles)
				{
					PrerequisiteItems.Add(NatvisFile);
					Arguments.Add($"/NATVIS:\"{NormalizeCommandLinePath(NatvisFile, RootPaths)}\"");
				}

				if (Target.WindowsPlatform.ManifestFile != null && !LinkEnvironment.bIsBuildingDLL)
				{
					PrerequisiteItems.Add(FileItem.GetItemByPath(Target.WindowsPlatform.ManifestFile));
				}
			}

			Arguments.AddRange(InputFileNames);

			// Add the output file to the command-line.
			Arguments.Add($"/OUT:\"{NormalizeCommandLinePath(OutputFile, RootPaths)}\"");

			// For import libraries and exports generated by cross-referenced builds, we don't track output files. VS 15.3+ doesn't touch timestamps for libs
			// and exp files with no modifications, breaking our dependency checking, but incremental linking will fall back to a full link if we delete it.
			// Since all DLLs are typically marked as cross referenced now anyway, we can just ignore this file to allow incremental linking to work.
			if (LinkEnvironment.bHasExports && !LinkEnvironment.bIsBuildingLibrary && !LinkEnvironment.bIsCrossReferenced && !Target.WindowsPlatform.bAllowClangLinker)
			{
				FileReference ExportFilePath = ImportLibraryFilePath!.ChangeExtension(".exp");
				FileItem ExportFile = FileItem.GetItemByFileReference(ExportFilePath);
				ProducedItems.Add(ExportFile);
			}

			if (!bIsBuildingLibraryOrImportLibrary)
			{
				// There is anything to export
				if (LinkEnvironment.bHasExports && !LinkEnvironment.bIsBuildingLibrary && !LinkEnvironment.bIsCrossReferenced)
				{
					// Write the import library to the output directory for nFringe support.
					FileItem ImportLibraryFile = FileItem.GetItemByFileReference(ImportLibraryFilePath!);
					Arguments.Add($"/IMPLIB:\"{NormalizeCommandLinePath(ImportLibraryFilePath!, RootPaths!)}\"");

					// Like the export file above, don't add the import library as a produced item when it's cross referenced.
					if (!LinkEnvironment.bIsCrossReferenced)
					{
						ProducedItems.Add(ImportLibraryFile);
					}
				}

				if (LinkEnvironment.bCreateDebugInfo)
				{
					// Write the PDB file to the output directory.
					{
						FileReference? PDBFilePath = FileReference.Combine(LinkEnvironment.OutputDirectory!, Path.GetFileNameWithoutExtension(OutputFile.AbsolutePath) + ".pdb");

						// If stripping private symbols, write the full pdb as .full.pdb
						if (Target.WindowsPlatform.bStripPrivateSymbols)
						{
							FileItem StrippedPDBFile = FileItem.GetItemByFileReference(PDBFilePath);
							Arguments.Add($"/PDBSTRIPPED:\"{NormalizeCommandLinePath(StrippedPDBFile, RootPaths)}\"");
							ProducedItems.Add(StrippedPDBFile);

							PDBFilePath = FileReference.Combine(LinkEnvironment.OutputDirectory!, Path.GetFileNameWithoutExtension(OutputFile.AbsolutePath) + ".full.pdb");
						}

						FileItem PDBFile = FileItem.GetItemByFileReference(PDBFilePath);
						Arguments.Add($"/PDB:\"{NormalizeCommandLinePath(PDBFilePath, RootPaths)}\"");
						ProducedItems.Add(PDBFile);
					}

					// Write the MAP file to the output directory.
					if (LinkEnvironment.bCreateMapFile)
					{
						FileReference MAPFilePath = FileReference.Combine(LinkEnvironment.OutputDirectory!, Path.GetFileNameWithoutExtension(OutputFile.AbsolutePath) + ".map");
						FileItem MAPFile = FileItem.GetItemByFileReference(MAPFilePath);
						Arguments.Add($"/MAP:\"{NormalizeCommandLinePath(MAPFilePath, RootPaths)}\"");
						ProducedItems.Add(MAPFile);

						// Export a list of object file paths, so we can locate the object files referenced by the map file
						ExportObjectFilePaths(LinkEnvironment, Path.ChangeExtension(MAPFilePath.FullName, ".objpaths"), EnvVars);
					}
				}

				// Add the additional arguments specified by the environment.
				if (!String.IsNullOrEmpty(LinkEnvironment.AdditionalArguments))
				{
					Arguments.Add(LinkEnvironment.AdditionalArguments.Trim());
				}
			}

			// Add any forced references to functions
			foreach (string IncludeFunction in LinkEnvironment.IncludeFunctions)
			{
				Arguments.Add($"/INCLUDE:{IncludeFunction}");
			}

			// Allow the toolchain to adjust/process the link arguments
			ModifyFinalLinkArguments(LinkEnvironment, Arguments, bBuildImportLibraryOnly);

			// Add training data as a prerequisite for pgo optimize
			if (EnvVars.Compiler.IsMSVC() && LinkEnvironment.bPGOOptimize && LinkEnvironment.PGOFilenamePrefix != null && LinkEnvironment.OutputFilePath.FullName.EndsWith(".exe", StringComparison.Ordinal))
			{
				// training data files are copied to the output directory in ModifyFinalLinkArguments() -> PreparePGOFilesMsvc()
				PrerequisiteItems.AddRange(DirectoryReference.EnumerateFiles(LinkEnvironment.OutputDirectory!, $"{LinkEnvironment.PGOFilenamePrefix}*.pgd").Select(FileItem.GetItemByFileReference));
				PrerequisiteItems.AddRange(DirectoryReference.EnumerateFiles(LinkEnvironment.OutputDirectory!, $"{LinkEnvironment.PGOFilenamePrefix}*.pgc").Select(FileItem.GetItemByFileReference));
			}

			if (IsDynamicDebuggingEnabled)
			{
				HashSet<string> exts = [".dll", ".exe", ".exp", ".lib", ".map", ".pdb"];
				List<FileItem> additional = [.. ProducedItems.Where(x => exts.Contains(x.Location.GetExtension()))
					.Select(x => FileItem.GetItemByFileReference(x.Location.ChangeExtension($".alt{x.Location.GetExtension()}")))
				];
				ProducedItems.AddRange(additional);
			}

			// Create an action that invokes the linker.
			Action LinkAction = Graph.CreateAction(ActionType.Link);

			// Create a response file for the linker, unless we're generating IntelliSense data
			FileReference ResponseFileName = GetResponseFileName(LinkEnvironment, OutputFile);
			if (!ProjectFileGenerator.bGenerateProjectFiles)
			{
				LinkAction.ResponseFileContents = [.. Arguments];
				FileItem ResponseFile = Graph.CreateIntermediateTextFile(ResponseFileName, LinkAction.ResponseFileContents);
				PrerequisiteItems.Add(ResponseFile);
			}

			LinkAction.RootPaths = LinkEnvironment.RootPaths;
			string ReadableArch = UnrealArchitectureConfig.ForPlatform(LinkEnvironment.Platform).ConvertToReadableArchitecture(LinkEnvironment.Architecture);
			LinkAction.CommandDescription += $"Link [{ReadableArch}]";
			LinkAction.WorkingDirectory = Unreal.EngineSourceDirectory;
			if (bIsBuildingLibraryOrImportLibrary)
			{
				LinkAction.CommandPath = EnvVars.LibraryManagerPath;
				LinkAction.CommandArguments = $"/LIB @\"{NormalizeCommandLinePath(ResponseFileName, RootPaths)}\"";
			}
			else
			{
				LinkAction.CommandPath = EnvVars.LinkerPath;
				LinkAction.CommandArguments = $"@\"{NormalizeCommandLinePath(ResponseFileName, RootPaths)}\"";
			}

			LinkAction.CommandVersion = EnvVars.Compiler.IsMSVC() ? EnvVars.CompilerVersion.ToString() : $"{EnvVars.Compiler} {EnvVars.CompilerVersion} MSVC {EnvVars.ToolChainVersion}";
			LinkAction.ProducedItems.UnionWith(ProducedItems);
			LinkAction.PrerequisiteItems.UnionWith(PrerequisiteItems);
			LinkAction.StatusDescription = Path.GetFileName(OutputFile.AbsolutePath);
			LinkAction.ArtifactMode = ArtifactMode.Enabled;

			// VS 15.3+ does not touch lib files if they do not contain any modifications, but we need to ensure the timestamps are updated to avoid repeatedly building them.
			if (bBuildImportLibraryOnly || (LinkEnvironment.bHasExports && !bIsBuildingLibraryOrImportLibrary))
			{
				LinkAction.DeleteItems.UnionWith(LinkAction.ProducedItems.Where(x => x.Location.HasExtension(".lib") || x.Location.HasExtension(".exp")));
			}

			// Delete PDB files for all produced items, since incremental updates are slower than full ones.
			if (!LinkEnvironment.bUseIncrementalLinking)
			{
				LinkAction.DeleteItems.UnionWith(LinkAction.ProducedItems.Where(x => x.Location.HasExtension(".pdb") || x.Location.HasExtension(".full.pdb")));
			}

			// Tell the action that we're building an import library here and it should conditionally be
			// ignored as a prerequisite for other actions
			LinkAction.bProducesImportLibrary = bBuildImportLibraryOnly || LinkEnvironment.bIsBuildingDLL;

			// Allow remote linking. Note that this may be overriden by the executor (eg. XGE.bAllowRemoteLinking)
			if (LinkAction.bProducesImportLibrary || LinkEnvironment.bIsBuildingDLL)
			{
				LinkAction.bCanExecuteRemotely = true;
			}

			if (Target.WindowsPlatform.Compiler.IsClang() &&
				Target.WindowsPlatform.bAllowClangLinker &&
				(Target.LinkType == TargetLinkType.Monolithic) &&
				(Unreal.IsBuildMachine() || LinkEnvironment.bAllowLTCG || LinkEnvironment.bPGOOptimize || LinkEnvironment.bPGOProfile))
			{
				// Set the weight to number of logical cores as lld can max out the available cores
				LinkAction.Weight = SystemUtils.GetLogicalProcessorCount();

				// Disallow remote to prevent this long running action from running on an agent if remote linking is enabled
				LinkAction.bCanExecuteRemotely = false;
			}

			if (EnvVars.Compiler.IsClang() && Target.WindowsPlatform.bSampleBasedPGO && LinkEnvironment.bPGOOptimize)
			{
				LinkAction.PrerequisiteItems.Add(FileItem.GetItemByFileReference(GetClangLinkProfDataFilename(LinkEnvironment)));
			}

			// Create link repro if requested, this argument is intentionally not added to the response file
			if (Target.WindowsPlatform.LinkReproDir != null)
			{
				DirectoryReference LinkReproRoot = new DirectoryReference(Target.WindowsPlatform.LinkReproDir);
				DirectoryReference LinkReproPath = DirectoryReference.Combine(LinkReproRoot, OutputFile.Name);
				DirectoryReference.CreateDirectory(LinkReproPath);
				LinkAction.CommandArguments = $"{LinkAction.CommandArguments} /LINKREPRO:{Utils.MakePathSafeToUseWithCommandLine(NormalizeCommandLinePath(LinkReproPath, RootPaths))}";

				LinkAction.ArtifactMode = ArtifactMode.None;
				LinkAction.bCanExecuteRemotely = false;
			}

			if (Target.bAllowUbaCompression)
			{
				LinkAction.CommandVersion = $"{LinkAction.CommandVersion} Compressed";
			}

			LinkAction.CacheBucket = GetCacheBucket(Target, LinkEnvironment);

			return OutputFile;
		}

		protected bool PreparePGOFilesMsvc(LinkEnvironment LinkEnvironment)
		{
			if (LinkEnvironment.bPGOOptimize && LinkEnvironment.OutputFilePath.FullName.EndsWith(".exe", StringComparison.Ordinal))
			{
				if (!Directory.Exists(LinkEnvironment.PGODirectory))
				{
					Logger.LogWarning("\"{PGODir}\" does not exist", LinkEnvironment.PGODirectory);
					return false;
				}

				// The linker expects the .pgd and any .pgc files to be in the output directory.
				// Copy the files there and make them writable...
				Logger.LogInformation("...copying the profile guided optimization files to output directory...");

				// prefer a PGD file that matches the output file
				string PGDFile = Path.Combine(LinkEnvironment.PGODirectory!, LinkEnvironment.PGOFilenamePrefix + ".pgd");
				string[] PGCFiles = Array.Empty<string>();

				bool bUsingMergedPGD = false;

				// check if we are using a pre-merged pgd file, if so use it instead
				if (LinkEnvironment.PGOMergedFilenamePrefix != null)
				{
					string MergedPGDFile = Path.Combine(LinkEnvironment.PGODirectory!, LinkEnvironment.PGOMergedFilenamePrefix + ".pgd");
					if (File.Exists(MergedPGDFile))
					{
						// Only use the merged pgd file if it actually exists, otherwise keep the default behavior
						PGDFile = MergedPGDFile;
						bUsingMergedPGD = true;
						Environment.SetEnvironmentVariable("PGOMGR", "/nowarn:188", EnvironmentVariableTarget.Process);
					}
					else
					{
						Logger.LogWarning("The specified merged .pgd file \"{MergedPgdFile}\" was not found.", LinkEnvironment.PGOMergedFilenamePrefix);
					}
				}

				if (!bUsingMergedPGD)
				{
					if (!File.Exists(PGDFile))
					{
						string[] PGDFiles = Directory.GetFiles(LinkEnvironment.PGODirectory!, "*.pgd");
						if (PGDFiles.Length > 1)
						{
							throw new BuildException("More than one .pgd file found in \"{0}\" and \"{1}\" not found ", LinkEnvironment.PGODirectory,
								PGDFile);
						}
						else if (PGDFiles.Length == 0)
						{
							Logger.LogWarning("No .pgd files found in \"{PgoDir}\".", LinkEnvironment.PGODirectory);
							return false;
						}

						PGDFile = PGDFiles.First();
					}

					PGCFiles = Directory.GetFiles(LinkEnvironment.PGODirectory!, "*.pgc");
					if (PGCFiles.Length == 0)
					{
						Logger.LogWarning("No .pgc files found in \"{PgoDir}\".", LinkEnvironment.PGODirectory);
						return false;
					}
				}

				// Make sure the destination directory exists!
				Directory.CreateDirectory(LinkEnvironment.OutputDirectory!.FullName);

				// Copy the .pgd to the linker output directory, renaming it to match the PGO filename prefix.
				string DestPGDFile = Path.Combine(LinkEnvironment.OutputDirectory.FullName, LinkEnvironment.PGOFilenamePrefix + ".pgd");
				Logger.LogInformation("{Source} -> {Target}", PGDFile, DestPGDFile);
				File.Copy(PGDFile, DestPGDFile, true);
				File.SetAttributes(DestPGDFile, FileAttributes.Normal);

				// Copy the *!n.pgc files (where n is an integer), renaming them to match the PGO filename prefix and ensuring they are numbered sequentially
				int PGCFileIndex = 0;
				foreach (string SrcFilePath in PGCFiles)
				{
					string DestFileName = String.Format("{0}!{1}.pgc", LinkEnvironment.PGOFilenamePrefix, ++PGCFileIndex);
					string DestFilePath = Path.Combine(LinkEnvironment.OutputDirectory.FullName, DestFileName);

					Logger.LogInformation("{Source} -> {Target}", SrcFilePath, DestFilePath);
					File.Copy(SrcFilePath, DestFilePath, true);
					File.SetAttributes(DestFilePath, FileAttributes.Normal);
				}
			}

			return true;
		}

		static readonly string[] PGOIgnoredLinkerSwitch =
		{
			"/USEPROFILE",
			"/GENPROFILE",
			"/FASTGENPROFILE",
			"/OUT:",
			"/PDB:",
			"/PDBSTRIPPED:",
			"/NOLOGO",
			"/LIBPATH:",
			"/errorReport:",
			"/d2:",
			"/NATVIS:",
			"/LINKREPRO:",
			"/experimental:deterministic"
		};
		private static IEnumerable<string> RemovePGOIgnoredSwitches(IEnumerable<string> SourceArguments)
		{
			return SourceArguments.Where(Argument => Argument.StartsWith('/') && !PGOIgnoredLinkerSwitch.Any(Switch => Argument.StartsWith(Switch, StringComparison.OrdinalIgnoreCase)));
		}

		protected virtual void AddPGOLinkArguments(LinkEnvironment LinkEnvironment, List<string> Arguments)
		{
			bool bPGOOptimize = LinkEnvironment.bPGOOptimize;
			bool bPGOProfile = LinkEnvironment.bPGOProfile;

			// Write the compiler and version used to generate the PGO profile data
			// We may want to use this to ensure compatibility
			if (bPGOProfile)
			{
				VersionNumber CompilerVersion = EnvVars.Compiler.IsIntel() ? MicrosoftPlatformSDK.GetClangVersionForIntelCompiler(EnvVars.CompilerPath) : EnvVars.CompilerVersion;
				string[] CompilerVersionLines = { EnvVars.Compiler.ToString(), CompilerVersion.ToString() };
				string CompilerVersionFilename = Path.Combine(LinkEnvironment.PGODirectory!, "PGOProfileCompilerInfo.txt");
				Utils.WriteFileIfChanged(new FileReference(CompilerVersionFilename), CompilerVersionLines, Logger);
			}

			if (Target.WindowsPlatform.Compiler.IsMSVC())
			{
				if (bPGOOptimize || bPGOProfile)
				{
					// serialize the linker arguments for comparing /USEPROFILE and /GENPROFILE
					IEnumerable<string> PGOArguments = RemovePGOIgnoredSwitches(Arguments);
					string PGOProfileLinkArgsFile = Path.Combine(LinkEnvironment.PGODirectory!, "PGOProfileLinkArgs.txt");

					if (bPGOProfile)
					{
						// save the latest linker arguments for /GENPROFILE alongside the PGO data
						Utils.WriteFileIfChanged(new FileReference(PGOProfileLinkArgsFile), PGOArguments, Logger);
					}
					else if (bPGOOptimize && Target.WindowsPlatform.bIgnoreStalePGOData && File.Exists(PGOProfileLinkArgsFile))
					{
						// compare latest linker arguments for /USEPROFILE to the last known /GENPROFILE and disable /USEPROFILE if they are different to avoid LNK1268
						IEnumerable<string> PGOProfileLinkArgs = RemovePGOIgnoredSwitches(File.ReadAllLines(PGOProfileLinkArgsFile));
						IEnumerable<string> AddedPGOArgs = PGOArguments.Except(PGOProfileLinkArgs);
						IEnumerable<string> RemovedPGOArgs = PGOProfileLinkArgs.Except(PGOArguments);

						if (AddedPGOArgs.Any() || RemovedPGOArgs.Any())
						{
							Logger.LogWarning("PGO Profile and PGO Optimize linker arguments do not match. Profile Guided Optimization will be disabled for {Config} until new PGO Profile data is generated.", Target.Configuration.ToString());
							bPGOOptimize = false;

							if (AddedPGOArgs.Any())
							{
								Logger.LogInformation("Added PGO args:");
								foreach (string Arg in AddedPGOArgs)
								{
									Logger.LogInformation("  {Arg}", Arg);
								}
							}
							if (RemovedPGOArgs.Any())
							{
								Logger.LogInformation("Removed PGO args:");
								foreach (string Arg in RemovedPGOArgs)
								{
									Logger.LogInformation("  {Arg}", Arg);
								}
							}
						}
					}
				}

				// If PGO was requested, apply Link-time code generation even if PGO was disabled due to mismatched args
				if (LinkEnvironment.bPGOOptimize || LinkEnvironment.bPGOProfile)
				{
					if (!Arguments.Contains("/LTCG"))
					{
						Arguments.Add("/LTCG");
					}
					Log.TraceInformationOnce("Enabling Link-time code generation (LTCG) as Profile Guided Optimization (PGO) was requested. Linking will take a while.");
				}

				if (bPGOOptimize)
				{
					if (PreparePGOFilesMsvc(LinkEnvironment))
					{
						Arguments.Add("/USEPROFILE");
						Log.TraceInformationOnce("Enabling using Profile Guided Optimization (PGO). Linking will take a while.");
					}
					else
					{
						Logger.LogWarning("Unable to prepare Profile Guided Optimization (PGO) files. Using PGO Optimize build will be disabled.");
					}
				}
				else if (bPGOProfile)
				{
					if (Target.WindowsPlatform.bUseFastGenProfile)
					{
						Log.TraceInformationOnce("Enabling generating Fastgen Profile Guided Optimization (PGO). Linking will take a while.");
						Arguments.Add("/FASTGENPROFILE");
					}
					else
					{
						if (Target.WindowsPlatform.bPGONoExtraCounters)
						{
							Log.TraceInformationOnce("Enabling generating Profile Guided Optimization No Extra Counters (PGO). Linking will take a while.");
							Arguments.Add("/GENPROFILE:NOTRACKEH");
						}
						else
						{
							Log.TraceInformationOnce("Enabling generating Profile Guided Optimization (PGO). Linking will take a while.");
							Arguments.Add("/GENPROFILE");
						}
					}
				}
			}
			else // Clang
			{
				if (LinkEnvironment.bAllowLTCG || bPGOProfile || bPGOOptimize)
				{
					if (Target.WindowsPlatform.bAllowClangLinker)
					{
						if (bPGOOptimize || bPGOProfile)
						{
							Log.TraceInformationOnce("Enabling Link-time optimization as Profile Guided Optimization (PGO) was requested. Linking will take a while.");
						}
						else
						{
							Log.TraceInformationOnce("Enabling Link-time optimization. Linking will take a while.");
						}

						// lld should consider logical cores when determining how many threads to use
						Arguments.Add("/opt:lldltojobs=all");

						// we build everything with the same toolchain, so we don't want to upgrade debug info because it runs validation and is expensive
						Arguments.Add("-mllvm:-disable-auto-upgrade-debug-info");

						// ThinLTO incremental cache
						DirectoryReference? ThinLTOCacheDir = DirectoryReference.FromString(LinkEnvironment.ThinLTOCacheDirectory);
						if (ThinLTOCacheDir != null)
						{
							Arguments.Add($"/lldltocache:\"{ThinLTOCacheDir}\"");
							string? thinLTOCachePruningArgs = LinkEnvironment.ThinLTOCachePruningArguments;
							if (thinLTOCachePruningArgs != null)
							{
								Arguments.Add($"/lldltocachepolicy:{thinLTOCachePruningArgs}");
							}
						}
					}
					else
					{
						Log.TraceWarningOnce("Link-time optimization requires Clang linker.");
					}
				}

				// Link arguments used for PGO on Clang/Intel compiler
				if (bPGOOptimize)
				{
					if (Target.WindowsPlatform.bSampleBasedPGO)
					{
						if (Directory.Exists(LinkEnvironment.PGODirectory))
						{
							FileReference ProfDataFilename = GetClangLinkProfDataFilename(LinkEnvironment);

							Log.TraceInformationOnce("Enabling using Sample-Based Profile Guided Optimization (SPGO). Linking will take a while.");

							Arguments.Add($"/dwodir:\"{LinkEnvironment.PGODirectory!}\"");

							if (LinkEnvironment.bAllowLTCG && Target.WindowsPlatform.bAllowClangLinker)
							{
								// Use a merged profdata file.
								Arguments.Add($"/lto-sample-profile:\"{ProfDataFilename}\"");
							}

							if (Target.WindowsPlatform.Compiler.IsIntel())
							{
								Arguments.Add($"-fprofile-sample-use=\"{ProfDataFilename}\"");
							}
						}
						else
						{
							Logger.LogWarning("\"{PGODir}\" does not exist. Using PGO Optimize build will be disabled.", LinkEnvironment.PGODirectory);
						}
					}
					else
					{
						Log.TraceInformationOnce("Enabling using Profile Guided Optimization (PGO). Linking will take a while.");
					}
				}
				else if (bPGOProfile)
				{
					if (Target.WindowsPlatform.bSampleBasedPGO)
					{
						Log.TraceInformationOnce("Enabling generating Sample-based Profile Guided Optimization (SPGO). Linking will take a while.");

						if (Target.WindowsPlatform.Compiler.IsIntel())
						{
							Arguments.Add("-profile-sample-generate");
						}

						Arguments.Add($"/dwodir:\"{LinkEnvironment.PGODirectory!}\"");
					}
					else
					{
						Log.TraceInformationOnce("Enabling generating Profile Guided Optimization (PGO). Linking will take a while.");
					}
				}
			}
		}

		protected virtual void ModifyFinalLinkArguments(LinkEnvironment LinkEnvironment, List<string> Arguments, bool bBuildImportLibraryOnly)
		{
			// do not strip the PDB path from the executable if we want to remotely debug it (otherwise VS can't find the symbols)
			if (LinkEnvironment.bCreateDebugInfo &&
				Target.Type != TargetType.Editor &&
				Target.Configuration != UnrealTargetConfiguration.Shipping &&
				Target.Platform.IsInGroup(UnrealPlatformGroup.Windows))
			{
				Arguments.Remove("/PDBALTPATH:%_PDB%");
			}

			// IMPLEMENT_MODULE_ is not required - it only exists to ensure developers add an IMPLEMENT_MODULE() declaration in code. These are always removed for PGO so that adding/removing a module won't invalidate PGC data.
			Arguments.RemoveAll(Argument => Argument.StartsWith("/INCLUDE:IMPLEMENT_MODULE_", StringComparison.Ordinal));

			AddPGOLinkArguments(LinkEnvironment, Arguments);
		}

		private static void ExportObjectFilePaths(LinkEnvironment LinkEnvironment, string FileName, VCEnvironment EnvVars)
		{
			// Write the list of object file directories
			HashSet<DirectoryReference> ObjectFileDirectories = new HashSet<DirectoryReference>();
			foreach (FileItem InputFile in LinkEnvironment.InputFiles)
			{
				ObjectFileDirectories.Add(InputFile.Location.Directory);
			}
			foreach (FileReference Library in LinkEnvironment.Libraries)
			{
				ObjectFileDirectories.Add(Library.Directory);
			}
			foreach (DirectoryReference LibraryPath in LinkEnvironment.SystemLibraryPaths)
			{
				ObjectFileDirectories.Add(LibraryPath);
			}
			foreach (string LibraryPath in (Environment.GetEnvironmentVariable("LIB") ?? "").Split(new char[] { ';' }, StringSplitOptions.RemoveEmptyEntries))
			{
				ObjectFileDirectories.Add(new DirectoryReference(LibraryPath));
			}
			foreach (DirectoryReference LibraryPath in EnvVars.LibraryPaths)
			{
				ObjectFileDirectories.Add(LibraryPath);
			}
			Directory.CreateDirectory(Path.GetDirectoryName(FileName)!);
			File.WriteAllLines(FileName, ObjectFileDirectories.Select(x => x.FullName).OrderBy(x => x).ToArray());
		}

		/// <summary>
		/// Gets the default include paths for the given platform.
		/// </summary>
		[SupportedOSPlatform("windows")]
		public static string GetVCIncludePaths(UnrealTargetPlatform Platform, WindowsCompiler Compiler, WindowsCompiler ToolChain, string? CompilerVersion, string? ToolchainVersion, ILogger Logger)
		{
			// Make sure we've got the environment variables set up for this target
			VCEnvironment EnvVars = VCEnvironment.Create(Compiler, ToolChain, Platform, UnrealArch.X64, CompilerVersion, ToolchainVersion, null, null, false, false, false, Logger);

			// Also add any include paths from the INCLUDE environment variable.  MSVC is not necessarily running with an environment that
			// matches what UBT extracted from the vcvars*.bat using SetEnvironmentVariablesFromBatchFile().  We'll use the variables we
			// extracted to populate the project file's list of include paths
			// @todo projectfiles: Should we only do this for VC++ platforms?
			StringBuilder IncludePaths = new StringBuilder();
			foreach (DirectoryReference IncludePath in EnvVars.IncludePaths)
			{
				IncludePaths.AppendFormat("{0};", IncludePath);
			}
			return IncludePaths.ToString();
		}

		public override string GetExtraLinkFileExtension()
		{
			return "obj";
		}

		public override void SetUpGlobalEnvironment(ReadOnlyTargetRules Target, CppCompileEnvironment GlobalCompileEnvironment, LinkEnvironment GlobalLinkEnvironment)
		{
			base.SetUpGlobalEnvironment(Target, GlobalCompileEnvironment, GlobalLinkEnvironment);

			GlobalCompileEnvironment.RootPaths.AddFolderName(CppRootPathFolder.Compiler, EnvVars.Compiler.IsMSVC() ? "MSVC" : EnvVars.Compiler.ToString());
			GlobalLinkEnvironment.RootPaths.AddFolderName(CppRootPathFolder.Compiler, EnvVars.Compiler.IsMSVC() ? "MSVC" : EnvVars.Compiler.ToString());

			GlobalCompileEnvironment.RootPaths[CppRootPathFolder.Compiler] = EnvVars.CompilerDir;
			GlobalLinkEnvironment.RootPaths[CppRootPathFolder.Compiler] = EnvVars.CompilerDir;
			if (EnvVars.CompilerDir != EnvVars.ToolChainDir)
			{
				GlobalCompileEnvironment.RootPaths.AddFolderName(CppRootPathFolder.Toolchain, EnvVars.ToolChain.IsMSVC() ? "MSVC" : EnvVars.ToolChain.ToString());
				GlobalLinkEnvironment.RootPaths.AddFolderName(CppRootPathFolder.Toolchain, EnvVars.ToolChain.IsMSVC() ? "MSVC" : EnvVars.ToolChain.ToString());

				GlobalCompileEnvironment.RootPaths[CppRootPathFolder.Toolchain] = EnvVars.ToolChainDir;
				GlobalLinkEnvironment.RootPaths[CppRootPathFolder.Toolchain] = EnvVars.ToolChainDir;
			}
			GlobalCompileEnvironment.RootPaths[CppRootPathFolder.WinSDK] = EnvVars.WindowsSdkDir;
			GlobalLinkEnvironment.RootPaths[CppRootPathFolder.WinSDK] = EnvVars.WindowsSdkDir;

			if (Target.WindowsPlatform.LinkReproDir != null)
			{
				DirectoryReference LinkReproRoot = new(Target.WindowsPlatform.LinkReproDir);
				GlobalLinkEnvironment.RootPaths.AddExtraPath(("LinkRepro", LinkReproRoot.FullName));
			}

			// This is used by uba to create one input key for all files under dirs
			EpicGames.UBA.Utils.RegisterPathHash(EnvVars.WindowsSdkDir, EnvVars.WindowsSdkVersion.ToString());
			EpicGames.UBA.Utils.RegisterPathHash(EnvVars.CompilerDir, EnvVars.CompilerVersion.ToString());
			if (EnvVars.Compiler != EnvVars.ToolChain)
			{
				EpicGames.UBA.Utils.RegisterPathHash(EnvVars.ToolChainDir, EnvVars.ToolChainVersion.ToString());
			}

			if (!String.IsNullOrEmpty(Target.WindowsPlatform.DiaSdkDir))
			{
				EpicGames.UBA.Utils.RegisterPathHash(new(Target.WindowsPlatform.DiaSdkDir), EnvVars.ToolChainVersion.ToString());
			}

			// Validate PGO data
			if (EnvVars.Compiler.IsClang() && GlobalLinkEnvironment.bPGOOptimize && Target.WindowsPlatform.bIgnoreStalePGOData)
			{
				string CompilerVersionFilename = Path.Combine(GlobalLinkEnvironment.PGODirectory!, "PGOProfileCompilerInfo.txt");
				if (File.Exists(CompilerVersionFilename))
				{
					string[] ProfileCompilerVersionLines = File.ReadAllLines(CompilerVersionFilename);
					if (ProfileCompilerVersionLines.Length >= 2 & Enum.TryParse(ProfileCompilerVersionLines[0].Trim(), false, out WindowsCompiler ProfileCompiler) && VersionNumber.TryParse(ProfileCompilerVersionLines[1].Trim(), out VersionNumber? ProfileVersion))
					{
						VersionNumber CompilerVersion = EnvVars.Compiler.IsIntel() ? MicrosoftPlatformSDK.GetClangVersionForIntelCompiler(EnvVars.CompilerPath) : EnvVars.CompilerVersion;
						if (ProfileCompiler.IsClang() && ProfileVersion.Components[0] > CompilerVersion.Components[0])
						{
							Logger.LogWarning("PGO Profile major LLVM version is newer than PGO Optimize LLVM major version. Profile Guided Optimization will be disabled for {Config} until new PGO Profile data is generated.", Target.Configuration.ToString());
							Logger.LogInformation("Profile: {ProfileCompiler} ({ProfileVersion}) Optimize: {OptimizeCompiler} ({OptimizeVersion})", ProfileCompiler, ProfileVersion, EnvVars.Compiler, CompilerVersion);
							GlobalCompileEnvironment.bPGOOptimize = false;
							GlobalLinkEnvironment.bPGOOptimize = false;
						}
					}
				}
			}
		}

		public override void ModifyBuildProducts(ReadOnlyTargetRules Target, UEBuildBinary Binary, IEnumerable<string> Libraries, IEnumerable<UEBuildBundleResource> BundleResources, Dictionary<FileReference, BuildProductType> BuildProducts)
		{
			if (Binary.Type == UEBuildBinaryType.DynamicLinkLibrary && (Target.WindowsPlatform.Compiler.IsMSVC() || !Target.WindowsPlatform.bAllowClangLinker))
			{
				if (Target.bShouldCompileAsDLL)
				{
					BuildProducts.Add(FileReference.Combine(Binary.OutputDir, Binary.OutputFilePath.GetFileNameWithoutExtension() + ".lib"), BuildProductType.SymbolFile);
					BuildProducts.Add(FileReference.Combine(Binary.OutputDir, Binary.OutputFilePath.GetFileNameWithoutExtension() + ".exp"), BuildProductType.SymbolFile);
				}
				else
				{
					BuildProducts.Add(FileReference.Combine(Binary.IntermediateDirectory, Binary.OutputFilePath.GetFileNameWithoutExtension() + ".lib"), BuildProductType.BuildResource);
				}
			}
			if (Binary.Type == UEBuildBinaryType.Executable && Target.bCreateMapFile)
			{
				foreach (FileReference OutputFilePath in Binary.OutputFilePaths)
				{
					BuildProducts.Add(FileReference.Combine(OutputFilePath.Directory, OutputFilePath.GetFileNameWithoutExtension() + ".map"), BuildProductType.MapFile);
					BuildProducts.Add(FileReference.Combine(OutputFilePath.Directory, OutputFilePath.GetFileNameWithoutExtension() + ".objpaths"), BuildProductType.MapFile);
				}
			}

			if (IsDynamicDebuggingEnabled)
			{
				foreach (KeyValuePair<FileReference, BuildProductType> item in BuildProducts.ToList())
				{
					BuildProducts.Add(item.Key.ChangeExtension($".alt{item.Key.GetExtension()}"), item.Value);
				}
			}

			if (Target.WindowsPlatform.ManifestFile != null && Binary.Type == UEBuildBinaryType.Executable && !Target.Architecture.bIsX64)
			{
				FileReference IntermediateManifest = GetSxSIntermediateManifest(Target, Binary.OutputDir);
				BuildProducts.TryAdd(IntermediateManifest, BuildProductType.BuildResource);
			}
		}

		/// <summary>
		/// Get the intermediate location to store the side-by-side assembly manifest file for the given target.
		/// </summary>
		/// <see href="https://learn.microsoft.com/en-us/windows/win32/SbsCs/about-private-assemblies-" />
		/// <remarks>
		/// In theory, each target could have different dependencies, so ideally each target would have their own private
		/// assembly with its own copy of the dependencies. Unfortunately, the destination of the dependencies is scattered
		/// across various different build files (e.g. `Engine/Source/ThirdParty/Intel/TBB/IntelTBB.Build.cs`) which each
		/// specify that the dependencies should be output to a folder named for the target architecture. This means that
		/// it's impractical to output a unique assembly per target.
		/// 
		/// Instead, we can have binaries that are output to the same directory share a manifest, which ensures that all of
		/// all dependencies will be satisfied (though each application may get a few extra DLLs that they don't load).
		/// We only ever add to the manifests, because it's not safe to delete things - we can't force all targets sharing an
		/// output directory to build at the same time, and if targets A and B share an output directory, A has dep A' and
		/// B has dep B', then we would make the manifest unusable for B if we cleared it of B' when building A and not B.
		/// This does mean that we never remove dependencies that have been removed from the targets, but this is the safer
		/// end of the compromise.
		/// </remarks>
		private static FileReference GetSxSIntermediateManifest(ReadOnlyTargetRules Target, DirectoryReference ExeDirectory)
		{
			DirectoryReference BaseDirectory = Target.ProjectFile?.Directory ?? Unreal.EngineDirectory;
			DirectoryReference RootManifestDirectory = DirectoryReference.Combine(BaseDirectory, "Intermediate", "SxSManifest");

			string RelativeOutputDirectory = ExeDirectory.MakeRelativeTo(BaseDirectory);
			DirectoryReference ManifestDirectory = DirectoryReference.Combine(RootManifestDirectory, RelativeOutputDirectory);

			string ManifestName = $"{Target.Architecture}.manifest";
			return FileReference.Combine(ManifestDirectory, ManifestName);
		}
	}
}
