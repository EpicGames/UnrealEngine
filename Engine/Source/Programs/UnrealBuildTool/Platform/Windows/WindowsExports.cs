// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.Linq;
using System.Runtime.Versioning;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace UnrealBuildTool
{
	/// <summary>
	/// Public Linux functions exposed to UAT
	/// </summary>
	public static class WindowsExports
	{
		/// <summary>
		/// Tries to get the directory for an installed Visual Studio version
		/// </summary>
		/// <param name="Compiler">The compiler version</param>
		/// <returns>True if successful</returns>
		public static IEnumerable<DirectoryReference>? TryGetVSInstallDirs(WindowsCompiler Compiler)
		{
			return WindowsPlatform.TryGetVSInstallDirs(Compiler, Log.Logger);
		}

		/// <summary>
		/// Gets the path to MSBuild.exe
		/// </summary>
		/// <returns>Path to MSBuild.exe</returns>
		[SupportedOSPlatform("windows")]
		public static string GetMSBuildToolPath()
		{
			return MicrosoftPlatformSDK.GetMsBuildToolPath(Log.Logger).FullName;
		}

		/// <summary>
		/// Returns the architecture name of the current architecture.
		/// </summary>
		/// <param name="arch">The architecture enum</param>
		/// <returns>String with the name</returns>
		public static string GetArchitectureName(UnrealArch arch)
		{
			return WindowsPlatform.GetArchitectureName(arch);
		}

		/// <summary>
		/// Tries to get the directory for an installed Windows SDK
		/// </summary>
		/// <param name="DesiredVersion">Receives the desired version on success</param>
		/// <param name="OutSdkVersion">Version of SDK</param>
		/// <param name="OutSdkDir">Path to SDK root folder</param>
		/// <returns>String with the name</returns>
		[SupportedOSPlatform("windows")]
		public static bool TryGetWindowsSdkDir(string DesiredVersion, [NotNullWhen(true)] out Version? OutSdkVersion, [NotNullWhen(true)] out DirectoryReference? OutSdkDir)
		{
			VersionNumber? vn;
			if (WindowsPlatform.TryGetWindowsSdkDir(DesiredVersion, Log.Logger, out vn, out OutSdkDir))
			{
				OutSdkVersion = new Version(vn.ToString());
				return true;
			}
			OutSdkVersion = new Version();
			return false;
		}

		/// <summary>
		/// Gets a list of Windows Sdk installation directories, ordered by preference
		/// </summary>
		/// <returns>String with the name</returns>
		[SupportedOSPlatform("windows")]
		public static List<KeyValuePair<string, DirectoryReference>> GetWindowsSdkDirs()
		{
			List<KeyValuePair<string, DirectoryReference>> WindowsSdkDirs = new List<KeyValuePair<string, DirectoryReference>>();

			// Add the default directory first
			VersionNumber? Version;
			DirectoryReference? DefaultWindowsSdkDir;
			if (WindowsPlatform.TryGetWindowsSdkDir(null, Log.Logger, out Version, out DefaultWindowsSdkDir))
			{
				WindowsSdkDirs.Add(new KeyValuePair<string, DirectoryReference>(Version.ToString(), DefaultWindowsSdkDir));
			}

			// Add all the other directories sorted in reverse order
			IReadOnlyDictionary<VersionNumber, DirectoryReference> WindowsSdkDirPairs = MicrosoftPlatformSDK.FindWindowsSdkDirs(Log.Logger);
			foreach (KeyValuePair<VersionNumber, DirectoryReference> Pair in WindowsSdkDirPairs.OrderByDescending(x => x.Key))
			{
				if (!WindowsSdkDirs.Any(x => x.Value == Pair.Value))
				{
					WindowsSdkDirs.Add(new KeyValuePair<string, DirectoryReference>(Pair.Key.ToString(), Pair.Value));
				}
			}

			return WindowsSdkDirs;
		}

		/// <summary>
		/// Enumerates all the Windows 10 SDK root directories
		/// </summary>
		/// <param name="RootDirs">Receives all the Windows 10 sdk root directories</param>
		/// <param name="Logger">Logger for output</param>
		[SupportedOSPlatform("windows")]
		public static void EnumerateSdkRootDirs(List<DirectoryReference> RootDirs, ILogger Logger)
		{
			MicrosoftPlatformSDK.EnumerateSdkRootDirs(RootDirs, Logger);
		}

		/// <summary>
		/// Determines the directory containing the MSVC toolchain
		/// </summary>
		/// <param name="Compiler">Major version of the compiler to use</param>
		/// <param name="CompilerVersion">The minimum compiler version to use</param>
		/// <param name="Architecture">Architecture that is required</param>
		/// <param name="Logger">Logger for output</param>
		/// <param name="OutToolChainVersion">Receives the chosen toolchain version</param>
		/// <param name="OutToolChainDir">Receives the directory containing the toolchain</param>
		/// <param name="OutRedistDir">Receives the optional directory containing redistributable components</param>
		/// <returns>True if the toolchain directory was found correctly</returns>
		[SupportedOSPlatform("windows")]
		public static bool TryGetToolChainDir(WindowsCompiler Compiler, string? CompilerVersion, UnrealArch Architecture, ILogger Logger, [NotNullWhen(true)] out VersionNumber? OutToolChainVersion, [NotNullWhen(true)] out DirectoryReference? OutToolChainDir, out DirectoryReference? OutRedistDir)
		{
			return MicrosoftPlatformSDK.TryGetToolChainDir(Compiler, CompilerVersion, Architecture, Logger, out OutToolChainVersion, out OutToolChainDir, out OutRedistDir);
		}
	}
}
