// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using System.Text;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace UnrealBuildTool
{
	/// <summary>
	/// Base class for platform-specific project generators
	/// </summary>
	class WindowsProjectGenerator : PlatformProjectGenerator
	{
		/// <summary>
		/// This will configure the Visual Studio project settings for remote debugging based on the fields below.
		/// Refer to https://learn.microsoft.com/en-us/visualstudio/debugger/remote-debugging-cpp for details on setting up Visual Studio and the remote device for remote debugging.
		/// 
		/// The is also dependent on the new remote Windows dev tools. See https://aka.ms/GameRemoteDevtools and assumes the remote device has the game deployed already
		/// </summary>
		[XmlConfigFile(Category = "WindowsPlatform")]
		protected bool bConfigureVSRemoteDebugger = true;

		/// <summary>
		/// Default remote deployment path for Windows Remote Deployment. Must match wdendpoint
		/// </summary>
		[XmlConfigFile(Category = "WindowsPlatform")]
		protected string RemoteWinRoot = @"C:\ProgramData\Microsoft GDK\gameroot\";

		/// <summary>
		/// Default remote machine name for Windows X64 remote debugging. Must previously be paired
		/// </summary>
		[XmlConfigFile(Category = "WindowsPlatform")]
		protected string? RemoteDebugMachineX64;

		/// <summary>
		/// Default remote machine name for Windows Arm64 and Arm64ec remote debugging. Must previously be paired
		/// </summary>
		[XmlConfigFile(Category="WindowsPlatform")]
		protected string? RemoteDebugMachineArm64;

		/// <summary>
		/// Whether remote debugging should use authentication or not
		/// </summary>
		[XmlConfigFile(Category="WindowsPlatform")]
		protected bool bRemoteDebuggerAuthentication = false;

		/// <summary>
		/// Location of the windows remote deployment tools (not user-configurable)
		/// </summary>
		private static string WdRemotePath => System.Environment.ExpandEnvironmentVariables("%LocalAppData%\\Microsoft\\WinGet\\Links\\wdRemote.exe");

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="Arguments">Command line arguments passed to the project generator</param>
		/// <param name="Logger">Logger for output</param>
#pragma warning disable IDE0060 // Remove unused parameter - constructor is found by reflection in GenerateProjectFilesMode.ExecuteAsync
		public WindowsProjectGenerator(CommandLineArguments Arguments, ILogger Logger)
			: base(Logger)
#pragma warning restore IDE0060
		{
			XmlConfig.ApplyTo(this);
		}

		/// <inheritdoc/>
		public override IEnumerable<UnrealTargetPlatform> GetPlatforms()
		{
			yield return UnrealTargetPlatform.Win64;
		}

		/// <inheritdoc/>
		public override string GetVisualStudioPlatformName(VSSettings InVSSettings)
		{
			if (InVSSettings.Platform == UnrealTargetPlatform.Win64)
			{
				if (InVSSettings.Architecture == UnrealArch.Arm64)
				{
					return "arm64";
				}
				else if (InVSSettings.Architecture == UnrealArch.Arm64ec)
				{
					return "arm64ec";
				}
				return "x64";
			}
			return InVSSettings.Platform.ToString();
		}

		/// <inheritdoc/>
		public override string GetVisualStudioUserFileStrings(VisualStudioUserFileSettings VCUserFileSettings, VSSettings InVSSettings, string InConditionString, TargetRules InTargetRules, FileReference TargetRulesPath, FileReference ProjectFilePath, FileReference? NMakeOutputPath, string ProjectName, string UProjectPath, string? ForeignUProjectPath)
		{
			StringBuilder VCUserFileContent = new StringBuilder();

			string LocalOrRemoteString = InVSSettings.Architecture == null || (InVSSettings.Architecture.Value.bIsX64 == UnrealArch.Host.Value.bIsX64)
				? "Local" : "Remote";

			VCUserFileContent.AppendLine("  <PropertyGroup {0}>", InConditionString);
			if (InTargetRules.Type != TargetType.Game)
			{
				string DebugOptions = "";

				if (ForeignUProjectPath != null)
				{
					DebugOptions += ForeignUProjectPath;
					DebugOptions += " -skipcompile";
				}
				else if (InTargetRules.Type == TargetType.Editor && InTargetRules.ProjectFile != null)
				{
					DebugOptions += ProjectName;
				}

				VCUserFileContent.AppendLine($"    <{LocalOrRemoteString}DebuggerCommandArguments>{DebugOptions}</{LocalOrRemoteString}DebuggerCommandArguments>");
			}

			VCUserFileContent.AppendLine($"    <DebuggerFlavor>Windows{LocalOrRemoteString}Debugger</DebuggerFlavor>");
			VCUserFileContent.AppendLine("  </PropertyGroup>");

			return VCUserFileContent.ToString();
		}

		/// <inheritdoc/>
		public override void GetVisualStudioPathsEntries(VSSettings InVSSettings, TargetType TargetType, FileReference TargetRulesPath, FileReference ProjectFilePath, FileReference NMakeOutputPath, StringBuilder ProjectFileBuilder)
		{
			base.GetVisualStudioPathsEntries(InVSSettings, TargetType, TargetRulesPath, ProjectFilePath, NMakeOutputPath, ProjectFileBuilder);

			if (bConfigureVSRemoteDebugger && 
				(TargetType == TargetType.Game || TargetType == TargetType.Client || TargetType == TargetType.Server) &&
				GetRemoteDebugProperties(InVSSettings, out string? RemoteDebugMachine))
			{			
				string TargetName = TargetRulesPath.GetFileNameWithoutAnyExtensions();
				string RemoteDeployFolder = (InVSSettings.Platform == UnrealTargetPlatform.Win64) ? TargetName : $"{TargetName}_{InVSSettings.Platform}";
				DirectoryReference BaseDir = DirectoryReference.Combine(NMakeOutputPath.Directory, @"..\..\..");

				DirectoryReference RemoteWorkingDir = DirectoryReference.Combine( new DirectoryReference(RemoteWinRoot), RemoteDeployFolder);
				FileReference RemoteDebugCommand = FileReference.Combine(RemoteWorkingDir, NMakeOutputPath.Directory.MakeRelativeTo(BaseDir), NMakeOutputPath.GetFileName());
				string RemoteAuthentication = bRemoteDebuggerAuthentication ? "RemoteWithAuthentication" : "RemoteWithoutAuthentication";
				string OutDir = ProjectFile.NormalizeProjectPath(BaseDir); // need to override OutDir for remote debugging so that VS deploys the executable relative to the project root, not the engine root

				ProjectFileBuilder.AppendLine($"    <OutDir>{OutDir}</OutDir>");
				ProjectFileBuilder.AppendLine($"    <DeploymentDirectory>{RemoteWorkingDir}</DeploymentDirectory>");
				ProjectFileBuilder.AppendLine($"    <RemoteDebuggerCommand>{RemoteDebugCommand}</RemoteDebuggerCommand>");
				ProjectFileBuilder.AppendLine($"    <RemoteDebuggerWorkingDirectory>{RemoteDebugCommand.Directory}</RemoteDebuggerWorkingDirectory>");
				ProjectFileBuilder.AppendLine($"    <RemoteDebuggerServerName>{RemoteDebugMachine}</RemoteDebuggerServerName>");
				ProjectFileBuilder.AppendLine($"    <RemoteDebuggerDebuggerType>NativeOnly</RemoteDebuggerDebuggerType>");
				ProjectFileBuilder.AppendLine($"    <RemoteDebuggerDeployCppRuntime>false</RemoteDebuggerDeployCppRuntime>");
				ProjectFileBuilder.AppendLine($"    <RemoteDebuggerConnection>{RemoteAuthentication}</RemoteDebuggerConnection>");

				// default to remote debugging for other architectures
				if (InVSSettings.Architecture != null && InVSSettings.Architecture.Value.bIsX64 != UnrealArch.Host.Value.bIsX64)
				{
					ProjectFileBuilder.AppendLine($"    <DebuggerFlavor>WindowsRemoteDebugger</DebuggerFlavor>");
				}
			}
		}

		/// <inheritdoc/>
		public override bool GetVisualStudioDeploymentEnabled(VSSettings InVSSettings)
		{
			// remote debugging requires deployment
			if (bConfigureVSRemoteDebugger && GetRemoteDebugProperties(InVSSettings, out _))
			{
				return true;
			}

			return false;
		}

		/// <inheritdoc/>
		public override bool RequiresVSUserFileGeneration()
		{
			return true;
		}

		/// <inheritdoc/>
		public override IList<string> GetSystemIncludePaths(UEBuildTarget InTarget)
		{
			List<string> Result = new List<string>();
			foreach (DirectoryReference Path in InTarget.Rules.WindowsPlatform.Environment!.IncludePaths)
			{
				Result.Add(Path.FullName);
			}

			return Result;
		}

		protected virtual bool GetRemoteDebugProperties(VSSettings InVSSettings, [NotNullWhen(true)] out string? RemoteDebugMachine)
		{
			RemoteDebugMachine = null;

			// check that the wdremote tool exists
			if (!File.Exists(WdRemotePath))
			{
				return false;
			}

			// check for the default remote device for the current architecture
			if (InVSSettings.Architecture != null)
			{
				if (((UnrealArch)InVSSettings.Architecture).bIsX64)
				{
					RemoteDebugMachine = RemoteDebugMachineX64;
				}
				else
				{
					RemoteDebugMachine = RemoteDebugMachineArm64;
				}
			}

			// if no specific remote machine was specified, default to the first one in the list of paired devices
			if (RemoteDebugMachine == null)
			{
				// read device names from the engine ini. must already be paired
				// best place is %LOCALAPPDATA%\Unreal Engine\Engine\Config\UserEngine.ini
				ConfigHierarchy EngineConfig = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, null, InVSSettings.Platform);
				if (EngineConfig.GetArray("RemoteWin", "DeviceNames", out List<string>? DeviceNames) && DeviceNames != null && DeviceNames.Count > 0)
				{
					RemoteDebugMachine = DeviceNames[0];
				}
			}

			return !System.String.IsNullOrEmpty(RemoteDebugMachine);
		}
	}
}
