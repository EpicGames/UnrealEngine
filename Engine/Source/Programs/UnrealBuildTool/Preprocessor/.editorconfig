# NOTE: Requires **VS2019 16.3** or later

# EditorConfig is awesome:http://EditorConfig.org

# Dotnet code style settings:
[*.cs]

# CA1822: Make member static
dotnet_diagnostic.CA1822.severity = warning

csharp_prefer_simple_using_statement = true
csharp_style_prefer_range_operator = true
csharp_style_implicit_object_creation_when_type_is_apparent = true

csharp_style_inlined_variable_declaration = true:suggestion
