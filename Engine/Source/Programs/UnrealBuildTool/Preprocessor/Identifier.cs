// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using EpicGames.Core;

namespace UnrealBuildTool
{
	/// <summary>
	/// Contains the name of an identifier. Individual objects are unique.
	/// </summary>
	class Identifier : IComparable<Identifier>
	{
		/// <summary>
		/// The name of this identifier
		/// </summary>
		readonly string _name;

		/// <summary>
		/// Global map of name to identifier instance
		/// </summary>
		static readonly ConcurrentDictionary<string, Identifier> s_nameToIdentifier = new(StringComparer.Ordinal);

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="name">Name of this identifier</param>
		private Identifier(string name)
		{
			_name = name;
		}

		/// <summary>
		/// Finds or adds an identifier with the given name
		/// </summary>
		/// <param name="name">Name of the identifier</param>
		/// <returns>New Identifier instance</returns>
		public static Identifier FindOrAdd(string name)
		{
			if (!s_nameToIdentifier.TryGetValue(name, out Identifier? result))
			{
				Identifier newIdentifier = new(name);
				if (s_nameToIdentifier.TryAdd(name, newIdentifier))
				{
					result = newIdentifier;
				}
				else
				{
					result = s_nameToIdentifier[name];
				}
			}
			return result;
		}

		/// <summary>
		/// Compares this identifier to another identifier
		/// </summary>
		/// <param name="other">Identifier to compare to</param>
		/// <returns>Value indicating which identifier should sort first</returns>
		public int CompareTo(Identifier? other)
		{
			return other is null ? 1 : String.Compare(_name, other._name, StringComparison.Ordinal);
		}

		/// <summary>
		/// Formats this identifier as a string for debugging
		/// </summary>
		/// <returns>Name of this identifier</returns>
		public override string ToString()
		{
			return _name;
		}
	}

	/// <summary>
	/// Well known predefined identifiers
	/// </summary>
	[System.Diagnostics.CodeAnalysis.SuppressMessage("Style", "IDE1006:Naming Styles", Justification = "Keep the names closer to definitions")]
	class Identifiers
	{
		public static readonly Identifier Include = Identifier.FindOrAdd("include");
		public static readonly Identifier Define = Identifier.FindOrAdd("define");
		public static readonly Identifier Undef = Identifier.FindOrAdd("undef");
		public static readonly Identifier If = Identifier.FindOrAdd("if");
		public static readonly Identifier Ifdef = Identifier.FindOrAdd("ifdef");
		public static readonly Identifier Ifndef = Identifier.FindOrAdd("ifndef");
		public static readonly Identifier Elif = Identifier.FindOrAdd("elif");
		public static readonly Identifier Else = Identifier.FindOrAdd("else");
		public static readonly Identifier Endif = Identifier.FindOrAdd("endif");
		public static readonly Identifier Defined = Identifier.FindOrAdd("defined");
		public static readonly Identifier Pragma = Identifier.FindOrAdd("pragma");
		public static readonly Identifier Once = Identifier.FindOrAdd("once");
		public static readonly Identifier Error = Identifier.FindOrAdd("error");
		public static readonly Identifier Warning = Identifier.FindOrAdd("warning");
		public static readonly Identifier __VA_ARGS__ = Identifier.FindOrAdd("__VA_ARGS__");
		public static readonly Identifier __FILE__ = Identifier.FindOrAdd("__FILE__");
		public static readonly Identifier __LINE__ = Identifier.FindOrAdd("__LINE__");
		public static readonly Identifier __COUNTER__ = Identifier.FindOrAdd("__COUNTER__");
		public static readonly Identifier Sizeof = Identifier.FindOrAdd("sizeof");
		public static readonly Identifier Alignof = Identifier.FindOrAdd("alignof");
		public static readonly Identifier __has_builtin = Identifier.FindOrAdd("__has_builtin");
		public static readonly Identifier __has_feature = Identifier.FindOrAdd("__has_feature");
		public static readonly Identifier __has_warning = Identifier.FindOrAdd("__has_warning");
		public static readonly Identifier __building_module = Identifier.FindOrAdd("__building_module");
		public static readonly Identifier __pragma = Identifier.FindOrAdd("__pragma");
		public static readonly Identifier __builtin_return_address = Identifier.FindOrAdd("__builtin_return_address");
		public static readonly Identifier __builtin_frame_address = Identifier.FindOrAdd("__builtin_frame_address");
		public static readonly Identifier __has_attribute = Identifier.FindOrAdd("__has_attribute");
		public static readonly Identifier __has_c_attribute = Identifier.FindOrAdd("__has_c_attribute");
		public static readonly Identifier __has_cpp_attribute = Identifier.FindOrAdd("__has_cpp_attribute");
		public static readonly Identifier __has_declspec_attribute = Identifier.FindOrAdd("__has_declspec_attribute");
		public static readonly Identifier __has_keyword = Identifier.FindOrAdd("__has_keyword");
		public static readonly Identifier __has_extension = Identifier.FindOrAdd("__has_extension");
		public static readonly Identifier __has_include = Identifier.FindOrAdd("__has_include");
		public static readonly Identifier __has_include_next = Identifier.FindOrAdd("__has_include_next");
		public static readonly Identifier __is_identifier = Identifier.FindOrAdd("__is_identifier");
		public static readonly Identifier __is_target_arch = Identifier.FindOrAdd("__is_target_arch");
	}

	/// <summary>
	/// Helper functions for serialization
	/// </summary>
	static class IdentifierExtensionMethods
	{
		/// <summary>
		/// Read an identifier from a binary archive
		/// </summary>
		/// <param name="reader">Reader to serialize data from</param>
		/// <returns>Instance of the serialized identifier</returns>
		public static Identifier ReadIdentifier(this BinaryArchiveReader reader)
		{
			return reader.ReadObjectReference<Identifier>(() => Identifier.FindOrAdd(reader.ReadString()!))!;
		}

		/// <summary>
		/// Write an identifier to a binary archive
		/// </summary>
		/// <param name="writer">Writer to serialize data to</param>
		/// <param name="identifier">Identifier to write</param>
		public static void WriteIdentifier(this BinaryArchiveWriter writer, Identifier? identifier)
		{
			writer.WriteObjectReference<Identifier?>(identifier, () => writer.WriteString(identifier!.ToString()));
		}
	}
}
