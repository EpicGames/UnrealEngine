// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using System.Linq;
using System.Text;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// Exception class for the preprocessor, which contains the file and position of the code causing an error
	/// </summary>
	public class PreprocessorException : Exception
	{
		/// <summary>
		/// The context when the error was encountered
		/// </summary>
		public readonly PreprocessorContext? Context;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="context">The current preprocesor context</param>
		/// <param name="format">Format string, to be passed to String.Format</param>
		/// <param name="args">Optional argument list for the format string</param>
		public PreprocessorException(PreprocessorContext? context, string format, params object[] args)
			: base(String.Format(format, args))
		{
			Context = context;
		}
	}

	/// <summary>
	/// Implementation of a C++ preprocessor.
	/// </summary>
	class Preprocessor
	{
		/// <summary>
		/// Type of an include path
		/// </summary>
		public enum IncludePathType
		{
			/// <summary>
			/// Regular include path, enclosed by quotes
			/// </summary>
			Normal,

			/// <summary>
			/// System include path, enclosed by angle brackets
			/// </summary>
			System,
		}

		/// <summary>
		/// Include paths to look in
		/// </summary>
		readonly List<DirectoryItem> _includeDirectories = new();

		/// <summary>
		/// Framework paths to look in
		/// </summary>
		readonly List<DirectoryItem> _frameworkDirectories = new();

		/// <summary>
		/// Set of all included files with the #pragma once directive
		/// </summary>
		readonly HashSet<FileItem> _pragmaOnceFiles = new();

		/// <summary>
		/// Set of any files that has been processed
		/// </summary>
		readonly HashSet<FileItem> _processedFiles = new();

		/// <summary>
		/// The current state of the preprocessor
		/// </summary>
		readonly PreprocessorState _state = new();

		/// <summary>
		/// Predefined token containing the constant "0"
		/// </summary>
		static readonly byte[] s_zeroLiteral = Encoding.UTF8.GetBytes("0");

		/// <summary>
		/// Predefined token containing the constant "1"
		/// </summary>
		static readonly byte[] s_oneLiteral = Encoding.UTF8.GetBytes("1");

		/// <summary>
		/// Value of the __COUNTER__ variable
		/// </summary>
		int _counter;

		/// <summary>
		/// List of files included by the preprocessor
		/// </summary>
		/// <returns>Enumerable of processed files</returns>
		public IEnumerable<FileItem> GetProcessedFiles()
		{
			return _processedFiles.AsEnumerable();
		}

		/// <summary>
		/// Default constructor
		/// </summary>
		public Preprocessor()
		{
			DateTime now = DateTime.Now;
			AddLiteralMacro("__DATE__", TokenType.String, String.Format("\"{0} {1,2} {2}\"", now.ToString("MMM"), now.Day, now.Year));
			AddLiteralMacro("__TIME__", TokenType.String, "\"" + now.ToString("HH:mm:ss") + "\"");
			AddLiteralMacro("__FILE__", TokenType.String, "\"<unknown>\"");
			AddLiteralMacro("__LINE__", TokenType.Number, "-1");
			AddLiteralMacro("__COUNTER__", TokenType.Number, "-1");
			AddLiteralMacro("CHAR_BIT", TokenType.Number, "8"); // Workaround for #include_next not being supported on Linux for limit.h
		}

		/// <summary>
		/// Determines whether the current preprocessor branch is active
		/// </summary>
		/// <returns>True if the current branch is active</returns>
		public bool IsCurrentBranchActive()
		{
			return _state.IsCurrentBranchActive();
		}

		/// <summary>
		/// Defines a macro. May have an optional '=Value' suffix.
		/// </summary>
		/// <param name="definition">Macro to define</param>
		public void AddDefinition(string definition)
		{
			List<Token> tokens = TokenReader.Tokenize(definition);
			if (tokens.Count == 0)
			{
				throw new PreprocessorException(null, "Missing macro name");
			}
			if (tokens[0].Type != TokenType.Identifier)
			{
				throw new PreprocessorException(null, "'{0}' is not a valid macro name", tokens[0].ToString()!);
			}

			List<Token> valueTokens = new();
			if (tokens.Count == 1)
			{
				valueTokens.Add(new Token(TokenType.Number, TokenFlags.None, s_oneLiteral));
			}
			else if (tokens[1].Type != TokenType.Equals)
			{
				throw new PreprocessorException(null, "Unable to parse macro definition '{0}'", definition);
			}
			else
			{
				valueTokens.AddRange(tokens.Skip(2));
			}

			PreprocessorMacro macro = new(tokens[0].Identifier!, null, valueTokens);
			_state.DefineMacro(macro);
		}

		/// <summary>
		/// Defines a macro
		/// </summary>
		/// <param name="name">Name of the macro</param>
		/// <param name="value">String to be parsed for the macro's value</param>
		public void AddDefinition(string name, string value)
		{
			List<Token> tokens = new();

			TokenReader reader = new(value);
			while (reader.MoveNext())
			{
				tokens.Add(reader.Current);
			}

			PreprocessorMacro macro = new(Identifier.FindOrAdd(name), null, tokens);
			_state.DefineMacro(macro);
		}

		/// <summary>
		/// Defines a macro
		/// </summary>
		/// <param name="macro">The macro definition</param>
		public void AddDefinition(PreprocessorMacro macro)
		{
			_state.DefineMacro(macro);
		}

		/// <summary>
		/// Adds an include path to the preprocessor
		/// </summary>
		/// <param name="directory">The include path</param>
		public void AddIncludePath(DirectoryItem directory)
		{
			if (!_includeDirectories.Contains(directory))
			{
				_includeDirectories.Add(directory);
			}
		}

		/// <summary>
		/// Adds an include path to the preprocessor
		/// </summary>
		/// <param name="location">The include path</param>
		public void AddIncludePath(DirectoryReference location)
		{
			DirectoryItem directory = DirectoryItem.GetItemByDirectoryReference(location);
			if (!directory.Exists)
			{
				throw new FileNotFoundException("Unable to find " + location.FullName);
			}
			AddIncludePath(directory);
		}

		/// <summary>
		/// Adds an include path to the preprocessor
		/// </summary>
		/// <param name="directoryName">The include path</param>
		public void AddIncludePath(string directoryName)
		{
			AddIncludePath(new DirectoryReference(directoryName));
		}

		/// <summary>
		/// Adds a framework path to the preprocessor
		/// </summary>
		/// <param name="directory">The framework path</param>
		public void AddFrameworkPath(DirectoryItem directory)
		{
			if (!_frameworkDirectories.Contains(directory))
			{
				_frameworkDirectories.Add(directory);
			}
		}

		/// <summary>
		/// Adds a framework path to the preprocessor
		/// </summary>
		/// <param name="location">The framework path</param>
		public void AddFrameworkPath(DirectoryReference location)
		{
			DirectoryItem directory = DirectoryItem.GetItemByDirectoryReference(location);
			if (!directory.Exists)
			{
				throw new FileNotFoundException("Unable to find " + location.FullName);
			}
			AddFrameworkPath(directory);
		}

		/// <summary>
		/// Adds a framework path to the preprocessor
		/// </summary>
		/// <param name="directoryName">The framework path</param>
		public void AddFrameworkPath(string directoryName)
		{
			AddFrameworkPath(new DirectoryReference(directoryName));
		}

		/// <summary>
		/// Try to resolve an quoted include against the list of include directories. Uses search order described by https://msdn.microsoft.com/en-us/library/36k2cdd4.aspx.
		/// </summary>
		/// <param name="context">The current preprocessor context</param>
		/// <param name="includePath">The path appearing in an #include directive</param>
		/// <param name="type">Specifies rules for how to resolve the include path (normal/system)</param>
		/// <param name="file">If found, receives the resolved file</param>
		/// <returns>True if the The resolved file</returns>
		public bool TryResolveIncludePath(PreprocessorContext context, string includePath, IncludePathType type, [NotNullWhen(true)] out FileItem? file)
		{
			// From MSDN (https://msdn.microsoft.com/en-us/library/36k2cdd4.aspx?f=255&MSPPError=-2147217396)
			//
			// The preprocessor searches for include files in this order:
			//
			// Quoted form:
			//   1) In the same directory as the file that contains the #include statement.
			//   2) In the directories of the currently opened include files, in the reverse order in which they were opened. 
			//      The search begins in the directory of the parent include file and continues upward through the directories of any grandparent include files.
			//   3) Along the path that's specified by each /I compiler option.
			//   4) Along the paths that are specified by the INCLUDE environment variable.
			//
			// Angle-bracket form:
			//   1) Along the path that's specified by each /I compiler option.
			//   2) Along the paths that are specified by the INCLUDE environment variable.

			// If it's an absolute path, return it immediately
			if (Path.IsPathRooted(includePath))
			{
				FileItem fileItem = FileItem.GetItemByPath(includePath);
				if (fileItem.Exists)
				{
					file = fileItem;
					return true;
				}
				else
				{
					file = null;
					return false;
				}
			}

			// Split the path into fragments
			string[] fragments = includePath.Split(Path.DirectorySeparatorChar, Path.AltDirectorySeparatorChar);

			// Try to match the include path against any of the included directories
			if (type == IncludePathType.Normal)
			{
				for (PreprocessorContext? outerContext = context; outerContext != null; outerContext = outerContext.Outer)
				{
					if (outerContext is PreprocessorFileContext outerFileContext)
					{
						if (TryResolveRelativeIncludePath(outerFileContext.Directory, fragments, out FileItem? resolvedFile))
						{
							file = resolvedFile;
							return true;
						}
					}
				}
			}

			// Try to match the include path against any of the system directories
			foreach (DirectoryItem baseDirectory in _includeDirectories)
			{
				if (TryResolveRelativeIncludePath(baseDirectory, fragments, out FileItem? resolvedFile))
				{
					file = resolvedFile;
					return true;
				}
			}

			// Try to match the include path against any of the MacOS framework Header paths
			if (fragments.Length > 1)
			{
				foreach (DirectoryItem baseDirectory in _frameworkDirectories)
				{
					if (baseDirectory.TryGetDirectory($"{fragments[0]}.framework", out DirectoryItem? frameworkBaseDirectory) &&
						frameworkBaseDirectory.TryGetDirectory("Headers", out DirectoryItem? headerDirectory) &&
						TryResolveRelativeIncludePath(headerDirectory, fragments.Skip(1).ToArray(), out FileItem? resolvedFile))
					{
						file = resolvedFile;
						return true;
					}
				}
			}

			// Failed to find the file
			file = null;
			return false;
		}

		/// <summary>
		/// Try to resolve an quoted include against the list of include directories. Uses search order described by https://msdn.microsoft.com/en-us/library/36k2cdd4.aspx.
		/// </summary>
		/// <param name="baseDirectory">The base directory to search from</param>
		/// <param name="fragments">Fragments of the relative path to follow</param>
		/// <param name="file">The file that was found, if successful</param>
		/// <returns>True if the The resolved file</returns>
		public static bool TryResolveRelativeIncludePath(DirectoryItem baseDirectory, string[] fragments, [NotNullWhen(true)] out FileItem? file)
		{
			DirectoryItem? directory = baseDirectory;
			for (int idx = 0; idx < fragments.Length - 1; idx++)
			{
				if (!directory.TryGetDirectory(fragments[idx], out directory))
				{
					file = null;
					return false;
				}
			}
			return directory.TryGetFile(fragments[^1], out file);
		}

		/// <summary>
		/// Parses a file recursively 
		/// </summary>
		/// <param name="file">File to parse</param>
		/// <param name="fragments">Lists of fragments that are parsed</param>
		/// <param name="outerContext">Outer context information, for error messages</param>
		/// <param name="sourceFileCache">Cache for source files</param>
		/// <param name="logger">Logger for output</param>
		/// <param name="showIncludes">Show all the included files, in order</param>
		/// <param name="ignoreMissingIncludes">Suppress exceptions if an include path can not be resolved</param>
		public void ParseFile(FileItem file, List<SourceFileFragment> fragments, PreprocessorContext? outerContext, SourceFileMetadataCache sourceFileCache, ILogger logger, bool showIncludes = false, bool ignoreMissingIncludes = false)
		{
			// If the file has already been included and had a #pragma once directive, don't include it again
			if (_pragmaOnceFiles.Contains(file))
			{
				return;
			}

			_processedFiles.Add(file);

			// Output a trace of the included files
			if (showIncludes)
			{
				logger.LogInformation("Note: including file: {FileLocation}", file.Location);
			}

			// If the file had a header guard, and the macro is still defined, don't include it again
			SourceFile sourceFile = sourceFileCache.GetSourceFile(file);
			if (sourceFile.HeaderGuardMacro != null && _state.IsMacroDefined(sourceFile.HeaderGuardMacro))
			{
				return;
			}

			// Create a context for this file
			PreprocessorFileContext context = new(sourceFile, outerContext);

			// Parse the markup for this file
			while (context.MarkupIdx < sourceFile.Markup.Length)
			{
				SourceFileMarkup markup = sourceFile.Markup[context.MarkupIdx];
				if (markup.Type == SourceFileMarkupType.Include)
				{
					if (_state.IsCurrentBranchActive())
					{
						// Parse the directive
						FileItem? includedFile = ParseIncludeDirective(markup, context, ignoreMissingIncludes);

						// Parse the included file
						if (includedFile != null)
						{
							ParseFile(includedFile, fragments, context, sourceFileCache, logger, showIncludes, ignoreMissingIncludes);
						}
					}
					context.MarkupIdx++;
				}
				else
				{
					// Get the next fragment
					SourceFileFragment fragment = sourceFile.Fragments[context.FragmentIdx];
					Debug.Assert(fragment.MarkupMin == context.MarkupIdx);

					// Parse this fragment
					ParseFragment(sourceFile, fragment, context);

					// Add this fragment to the list
					fragments.Add(fragment);
					context.FragmentIdx++;
				}
			}
		}

		/// <summary>
		/// Parse an include directive and resolve the file it references
		/// </summary>
		/// <param name="markup">Markup for the include directive</param>
		/// <param name="context">Current preprocessor context</param>
		/// <param name="ignoreMissingIncludes">Suppress exceptions if an include path can not be resolved</param>
		/// <returns>Included file</returns>
		FileItem? ParseIncludeDirective(SourceFileMarkup markup, PreprocessorFileContext context, bool ignoreMissingIncludes = false)
		{
			// Expand macros in the given tokens
			List<Token> expandedTokens = new();
			ExpandMacros(markup.Tokens!, expandedTokens, false, context);

			// Convert the string to a single token
			string includeToken = Token.Format(expandedTokens);

			// Expand any macros in them and resolve it
			IncludePathType type;
			if (includeToken.Length >= 2 && includeToken[0] == '\"' && includeToken[^1] == '\"')
			{
				type = IncludePathType.Normal;
			}
			else if (includeToken.Length >= 2 && includeToken[0] == '<' && includeToken[^1] == '>')
			{
				type = IncludePathType.System;
			}
			else
			{
				throw new PreprocessorException(context, "Couldn't resolve include '{0}'", includeToken);
			}

			// Get the include path
			string includePath = includeToken[1..^1];

			// Resolve the included file
			if (!TryResolveIncludePath(context, includePath, type, out FileItem? includedFile))
			{
				if (ignoreMissingIncludes)
				{
					Log.TraceWarningOnce("Couldn't resolve include '{0}' ({1})", includePath, context.SourceFile.Location);
				}
				else
				{
					throw new PreprocessorException(context, "Couldn't resolve include '{0}' ({1})", includePath, context.SourceFile.Location);
				}
			}
			return includedFile;
		}

		/// <summary>
		/// Parse a source file fragment, using cached transforms if possible
		/// </summary>
		/// <param name="sourceFile">The source file being parsed</param>
		/// <param name="fragment">Fragment to parse</param>
		/// <param name="context">Current preprocessor context</param>
		void ParseFragment(SourceFile sourceFile, SourceFileFragment fragment, PreprocessorFileContext context)
		{
			// Check if there's a valid transform that matches the current state
			int transformIdx = 0;
			for (; ; )
			{
				PreprocessorTransform[] transforms;
				lock (fragment)
				{
					transforms = fragment.Transforms;
					if (transformIdx == transforms.Length)
					{
						// Attach a new transform to the current state
						PreprocessorTransform transform = _state.BeginCapture();
						for (; context.MarkupIdx < fragment.MarkupMax; context.MarkupIdx++)
						{
							SourceFileMarkup markup = sourceFile.Markup[context.MarkupIdx];
							ParseMarkup(markup.Type, markup.Tokens!, context);
						}
						transform = _state.EndCapture()!;

						// Add it to the fragment for future fragments
						PreprocessorTransform[] newTransforms = new PreprocessorTransform[fragment.Transforms.Length + 1];
						for (int idx = 0; idx < transforms.Length; idx++)
						{
							newTransforms[idx] = transforms[idx];
						}
						newTransforms[transforms.Length] = transform;

						// Assign it to the fragment
						fragment.Transforms = newTransforms;
						return;
					}
				}

				for (; transformIdx < transforms.Length; transformIdx++)
				{
					PreprocessorTransform transform = transforms[transformIdx];
					if (_state.TryToApply(transform))
					{
						// Update the pragma once state
						if (transform.HasPragmaOnce)
						{
							_pragmaOnceFiles.Add(sourceFile.File);
						}

						// Move to the end of the fragment
						context.MarkupIdx = fragment.MarkupMax;
						return;
					}
				}
			}
		}

		/// <summary>
		/// Validate and add a macro using the given parameter and token list
		/// </summary>
		/// <param name="context">The current preprocessor context</param>
		/// <param name="name">Name of the macro</param>
		/// <param name="parameters">Parameter list for the macro</param>
		/// <param name="tokens">List of tokens</param>
		void AddMacro(PreprocessorContext context, Identifier name, List<Identifier>? parameters, List<Token> tokens)
		{
			if (tokens.Count == 0)
			{
				tokens.Add(new Token(TokenType.Placemarker, TokenFlags.None));
			}
			else
			{
				if (tokens[0].HasLeadingSpace)
				{
					tokens[0] = tokens[0].RemoveFlags(TokenFlags.HasLeadingSpace);
				}
				if (tokens[0].Type == TokenType.HashHash || tokens[^1].Type == TokenType.HashHash)
				{
					throw new PreprocessorException(context, "Invalid use of concatenation at start or end of token sequence");
				}
				if (parameters == null || parameters.Count == 0 || parameters[^1] != Identifiers.__VA_ARGS__)
				{
					if (tokens.Any(x => x.Identifier == Identifiers.__VA_ARGS__))
					{
						throw new PreprocessorException(context, "Invalid reference to {0}", Identifiers.__VA_ARGS__);
					}
				}
			}
			_state.DefineMacro(new PreprocessorMacro(name, parameters, tokens));
		}

		/// <summary>
		/// Set a predefined macro to a given value
		/// </summary>
		/// <param name="name">Name of the macro</param>
		/// <param name="type">Type of the macro token</param>
		/// <param name="value">Value of the macro</param>
		/// <returns>The created macro</returns>
		void AddLiteralMacro(string name, TokenType type, string value)
		{
			Token token = new(type, TokenFlags.None, value);
			PreprocessorMacro macro = new(Identifier.FindOrAdd(name), null, new List<Token> { token });
			_state.DefineMacro(macro);
		}

		/// <summary>
		/// Parse a marked up directive from a file
		/// </summary>
		/// <param name="type">The markup type</param>
		/// <param name="tokens">Tokens for the directive</param>
		/// <param name="context">The context that this markup is being parsed in</param>
		public void ParseMarkup(SourceFileMarkupType type, List<Token> tokens, PreprocessorContext context)
		{
			switch (type)
			{
				case SourceFileMarkupType.Include:
					throw new PreprocessorException(context, "Include directives should be handled by the caller.");
				case SourceFileMarkupType.Define:
					ParseDefineDirective(tokens, context);
					break;
				case SourceFileMarkupType.Undef:
					ParseUndefDirective(tokens, context);
					break;
				case SourceFileMarkupType.If:
					ParseIfDirective(tokens, context);
					break;
				case SourceFileMarkupType.Ifdef:
					ParseIfdefDirective(tokens, context);
					break;
				case SourceFileMarkupType.Ifndef:
					ParseIfndefDirective(tokens, context);
					break;
				case SourceFileMarkupType.Elif:
					ParseElifDirective(tokens, context);
					break;
				case SourceFileMarkupType.Else:
					ParseElseDirective(tokens, context);
					break;
				case SourceFileMarkupType.Endif:
					ParseEndifDirective(tokens, context);
					break;
				case SourceFileMarkupType.Pragma:
					ParsePragmaDirective(tokens, context);
					break;
			}
		}

		/// <summary>
		/// Read a macro definition
		/// </summary>
		/// <param name="tokens">List of tokens in the directive</param>
		/// <param name="context">The context that this directive is being parsed in</param>
		public void ParseDefineDirective(List<Token> tokens, PreprocessorContext context)
		{
			if (_state.IsCurrentBranchActive())
			{
				// Check there's a name token
				if (tokens.Count < 1 || tokens[0].Type != TokenType.Identifier || tokens[0].Identifier == Identifiers.Defined)
				{
					throw new PreprocessorException(context, "Invalid macro name");
				}

				// Read the macro name
				Identifier name = tokens[0].Identifier!;
				int tokenIdx = 1;

				// Read the macro parameter list, if there is one
				List<Identifier>? parameters = null;
				if (tokenIdx < tokens.Count && !tokens[tokenIdx].HasLeadingSpace && tokens[tokenIdx].Type == TokenType.LeftParen)
				{
					parameters = new List<Identifier>();
					if (++tokenIdx == tokens.Count)
					{
						throw new PreprocessorException(context, "Unexpected end of macro parameter list");
					}
					if (tokens[tokenIdx].Type != TokenType.RightParen)
					{
						for (; ; tokenIdx++)
						{
							// Check there's enough tokens left for a parameter name, plus ',' or ')'
							if (tokenIdx + 2 > tokens.Count)
							{
								throw new PreprocessorException(context, "Unexpected end of macro parameter list");
							}

							// Check it's a valid name, and add it to the list
							Token nameToken = tokens[tokenIdx++];
							if (nameToken.Type == TokenType.Ellipsis)
							{
								if (tokens[tokenIdx].Type != TokenType.RightParen)
								{
									throw new PreprocessorException(context, "Variadic macro arguments must be last in list");
								}
								else
								{
									nameToken = new Token(Identifiers.__VA_ARGS__, nameToken.Flags & TokenFlags.HasLeadingSpace);
								}
							}
							else
							{
								if (nameToken.Type != TokenType.Identifier || nameToken.Identifier == Identifiers.__VA_ARGS__)
								{
									throw new PreprocessorException(context, "Invalid preprocessor token: {0}", nameToken);
								}
								if (parameters.Contains(nameToken.Identifier!))
								{
									throw new PreprocessorException(context, "'{0}' has already been used as an argument name", nameToken);
								}
							}
							parameters.Add(nameToken.Identifier!);

							// Read the separator
							Token separatorToken = tokens[tokenIdx];
							if (separatorToken.Type == TokenType.RightParen)
							{
								break;
							}
							if (separatorToken.Type != TokenType.Comma)
							{
								throw new PreprocessorException(context, "Expected ',' or ')'");
							}
						}
					}
					tokenIdx++;
				}

				// Read the macro tokens
				AddMacro(context, name, parameters, tokens.Skip(tokenIdx).ToList());
			}
		}

		/// <summary>
		/// Parse an #undef directive
		/// </summary>
		/// <param name="tokens">List of tokens in the directive</param>
		/// <param name="context">The context that this directive is being parsed in</param>
		public void ParseUndefDirective(List<Token> tokens, PreprocessorContext context)
		{
			if (_state.IsCurrentBranchActive())
			{
				// Check there's a name token
				if (tokens.Count != 1)
				{
					throw new PreprocessorException(context, "Expected a single token after #undef");
				}
				if (tokens[0].Type != TokenType.Identifier)
				{
					throw new PreprocessorException(context, "Invalid macro name '{0}'", tokens[0]);
				}

				// Remove the macro from the list of definitions
				_state.UndefMacro(tokens[0].Identifier!);
			}
		}

		/// <summary>
		/// Parse an #if directive
		/// </summary>
		/// <param name="tokens">List of tokens in the directive</param>
		/// <param name="context">The context that this directive is being parsed in</param>
		public void ParseIfDirective(List<Token> tokens, PreprocessorContext context)
		{
			PreprocessorBranch branch = PreprocessorBranch.HasIfDirective;
			if (_state.IsCurrentBranchActive())
			{
				// Read a line into the buffer and expand the macros in it
				List<Token> expandedTokens = new();
				ExpandMacros(tokens, expandedTokens, true, context);

				// Evaluate the condition
				long result = PreprocessorExpression.Evaluate(context, expandedTokens);
				if (result != 0)
				{
					branch |= PreprocessorBranch.Active | PreprocessorBranch.Taken;
				}
			}
			_state.PushBranch(branch);
		}

		/// <summary>
		/// Parse an #ifdef directive
		/// </summary>
		/// <param name="tokens">List of tokens in the directive</param>
		/// <param name="context">The context that this directive is being parsed in</param>
		public void ParseIfdefDirective(List<Token> tokens, PreprocessorContext context)
		{
			PreprocessorBranch branch = PreprocessorBranch.HasIfdefDirective;
			if (_state.IsCurrentBranchActive())
			{
				// Make sure there's only one token
				if (tokens.Count != 1 || tokens[0].Type != TokenType.Identifier)
				{
					throw new PreprocessorException(context, "Missing or invalid macro name for #ifdef directive");
				}

				// Check if the macro is defined
				if (_state.IsMacroDefined(tokens[0].Identifier!))
				{
					branch |= PreprocessorBranch.Active | PreprocessorBranch.Taken;
				}
			}
			_state.PushBranch(branch);
		}

		/// <summary>
		/// Parse an #ifndef directive
		/// </summary>
		/// <param name="tokens">List of tokens for this directive</param>
		/// <param name="context">The context that this directive is being parsed in</param>
		public void ParseIfndefDirective(List<Token> tokens, PreprocessorContext context)
		{
			PreprocessorBranch branch = PreprocessorBranch.HasIfndefDirective;
			if (_state.IsCurrentBranchActive())
			{
				// Make sure there's only one token
				if (tokens.Count != 1 || tokens[0].Type != TokenType.Identifier)
				{
					throw new PreprocessorException(context, "Missing or invalid macro name for #ifndef directive");
				}

				// Check if the macro is defined
				if (!_state.IsMacroDefined(tokens[0].Identifier!))
				{
					branch |= PreprocessorBranch.Active | PreprocessorBranch.Taken;
				}
			}
			_state.PushBranch(branch);
		}

		/// <summary>
		/// Parse an #elif directive
		/// </summary>
		/// <param name="tokens">List of tokens for this directive</param>
		/// <param name="context">The context that this directive is being parsed in</param>
		public void ParseElifDirective(List<Token> tokens, PreprocessorContext context)
		{
			// Check we're in a branch, and haven't already read an #else directive
			if (!_state.TryPopBranch(out PreprocessorBranch branch))
			{
				throw new PreprocessorException(context, "#elif directive outside conditional block");
			}
			if (branch.HasFlag(PreprocessorBranch.Complete))
			{
				throw new PreprocessorException(context, "#elif directive cannot appear after #else");
			}

			// Pop the current branch state at this depth, so we can test against whether the parent state is enabled
			branch = (branch | PreprocessorBranch.HasElifDirective) & ~PreprocessorBranch.Active;
			if (_state.IsCurrentBranchActive())
			{
				// Read a line into the buffer and expand the macros in it
				List<Token> expandedTokens = new();
				ExpandMacros(tokens, expandedTokens, true, context);

				// Check we're at the end of a conditional block
				if (!branch.HasFlag(PreprocessorBranch.Taken))
				{
					long result = PreprocessorExpression.Evaluate(context, expandedTokens);
					if (result != 0)
					{
						branch |= PreprocessorBranch.Active | PreprocessorBranch.Taken;
					}
				}
			}
			_state.PushBranch(branch);
		}

		/// <summary>
		/// Parse an #else directive
		/// </summary>
		/// <param name="tokens">List of tokens in the directive</param>
		/// <param name="context">The context that this directive is being parsed in</param>
		public void ParseElseDirective(List<Token> tokens, PreprocessorContext context)
		{
			// Make sure there's nothing else on the line
			if (tokens.Count > 0)
			{
				throw new PreprocessorException(context, "Garbage after #else directive");
			}

			// Check we're in a branch, and haven't already read an #else directive
			if (!_state.TryPopBranch(out PreprocessorBranch branch))
			{
				throw new PreprocessorException(context, "#else directive without matching #if directive");
			}
			if ((branch & PreprocessorBranch.Complete) != 0)
			{
				throw new PreprocessorException(context, "Only one #else directive can appear in a conditional block");
			}

			// Check whether to take this branch, but only allow activating if the parent state is active.
			branch &= ~PreprocessorBranch.Active;
			if (_state.IsCurrentBranchActive() && !branch.HasFlag(PreprocessorBranch.Taken))
			{
				branch |= PreprocessorBranch.Active | PreprocessorBranch.Taken;
			}
			_state.PushBranch(branch | PreprocessorBranch.Complete);
		}

		/// <summary>
		/// Parse an #endif directive
		/// </summary>
		/// <param name="tokens">List of tokens in the directive</param>
		/// <param name="context">The context that this directive is being parsed in</param>
		[SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "<Pending>")]
		public void ParseEndifDirective(List<Token> tokens, PreprocessorContext context)
		{
			// Pop the branch off the stack
			if (!_state.TryPopBranch(out _))
			{
				throw new PreprocessorException(context, "#endif directive without matching #if/#ifdef/#ifndef directive");
			}
		}

		/// <summary>
		/// Parse a #pragma directive
		/// </summary>
		/// <param name="tokens">List of tokens in the directive</param>
		/// <param name="context">The context that this directive is being parsed in</param>
		public void ParsePragmaDirective(List<Token> tokens, PreprocessorContext context)
		{
			if (_state.IsCurrentBranchActive())
			{
				if (tokens.Count == 1 && tokens[0].Identifier == Identifiers.Once)
				{
					SourceFile sourceFile = GetCurrentSourceFile(context)!;
					_pragmaOnceFiles.Add(sourceFile.File);
					_state.MarkPragmaOnce();
				}
			}
		}

		/// <summary>
		/// Expand macros in the given sequence.
		/// </summary>
		/// <param name="inputTokens">Sequence of input tokens</param>
		/// <param name="outputTokens">List to receive the expanded tokens</param>
		/// <param name="isConditional">Whether a conditional expression is being evaluated (and 'defined' expressions are valid)</param>
		/// <param name="context">The context that this directive is being parsed in</param>
		public void ExpandMacros(IEnumerable<Token> inputTokens, List<Token> outputTokens, bool isConditional, PreprocessorContext context)
		{
			List<PreprocessorMacro> ignoreMacros = new();
			ExpandMacrosRecursively(inputTokens, outputTokens, isConditional, ignoreMacros, context);
		}

		/// <summary>
		/// Expand macros in the given sequence, ignoring previously expanded macro names from a list.
		/// </summary>
		/// <param name="inputTokens">Sequence of input tokens</param>
		/// <param name="outputTokens">List to receive the expanded tokens</param>
		/// <param name="isConditional">Whether a conditional expression is being evaluated (and 'defined' expressions are valid)</param>
		/// <param name="ignoreMacros">List of macros to ignore</param>
		/// <param name="context">The context that this directive is being parsed in</param>
		void ExpandMacrosRecursively(IEnumerable<Token> inputTokens, List<Token> outputTokens, bool isConditional, List<PreprocessorMacro> ignoreMacros, PreprocessorContext context)
		{
			IEnumerator<Token> inputEnumerator = inputTokens.GetEnumerator();
			if (inputEnumerator.MoveNext())
			{
				for (; ; )
				{
					if (!ReadExpandedToken(inputEnumerator, outputTokens, isConditional, ignoreMacros, context))
					{
						break;
					}
				}
			}
		}

		/// <summary>
		/// Merges an optional leading space flag into the given token (recycling the original token if possible).
		/// </summary>
		/// <param name="token">The token to merge a leading space into</param>
		/// <param name="hasLeadingSpace">The leading space flag</param>
		/// <returns>New token with the leading space flag set, or the existing token</returns>
		static Token MergeLeadingSpace(Token token, bool hasLeadingSpace)
		{
			Token result = token;
			if (hasLeadingSpace && !result.HasLeadingSpace)
			{
				result = result.AddFlags(TokenFlags.HasLeadingSpace);
			}
			return result;
		}

		/// <summary>
		/// Read a token from an enumerator and substitute it if it's a macro or 'defined' expression (reading more tokens as necessary to complete the expression).
		/// </summary>
		/// <param name="inputEnumerator">The enumerator of input tokens</param>
		/// <param name="outputTokens">List to receive the expanded tokens</param>
		/// <param name="isConditional">Whether a conditional expression is being evaluated (and 'defined' expressions are valid)</param>
		/// <param name="ignoreMacros">List of macros to ignore</param>
		/// <param name="context">The context that this directive is being parsed in</param>
		/// <returns>Result from calling the enumerator's MoveNext() method</returns>
		bool ReadExpandedToken(IEnumerator<Token> inputEnumerator, List<Token> outputTokens, bool isConditional, List<PreprocessorMacro> ignoreMacros, PreprocessorContext context)
		{
			// Capture the first token, then move to the next
			outputTokens.Add(inputEnumerator.Current);
			bool moveNext = inputEnumerator.MoveNext();

			// If it's an identifier, try to resolve it as a macro
			if (outputTokens[^1].Identifier == Identifiers.Defined && isConditional)
			{
				// Remove the 'defined' keyword
				outputTokens.RemoveAt(outputTokens.Count - 1);

				// Make sure there's another token
				if (!moveNext)
				{
					throw new PreprocessorException(context, "Invalid syntax for 'defined' expression");
				}

				// Check for the form 'defined identifier'
				Token nameToken;
				if (inputEnumerator.Current.Type == TokenType.Identifier)
				{
					nameToken = inputEnumerator.Current;
				}
				else
				{
					// Otherwise assume the form 'defined ( identifier )'
					if (inputEnumerator.Current.Type != TokenType.LeftParen || !inputEnumerator.MoveNext() || inputEnumerator.Current.Type != TokenType.Identifier)
					{
						throw new PreprocessorException(context, "Invalid syntax for 'defined' expression");
					}
					nameToken = inputEnumerator.Current;
					if (!inputEnumerator.MoveNext() || inputEnumerator.Current.Type != TokenType.RightParen)
					{
						throw new PreprocessorException(context, "Invalid syntax for 'defined' expression");
					}
				}

				// Insert a token for whether it's defined or not
				outputTokens.Add(new Token(TokenType.Number, TokenFlags.None, _state.IsMacroDefined(nameToken.Identifier!) ? s_oneLiteral : s_zeroLiteral));
				moveNext = inputEnumerator.MoveNext();
			}
			else
			{
				// Repeatedly try to expand the last token into the list
				while (outputTokens[^1].Type == TokenType.Identifier && !outputTokens[^1].Flags.HasFlag(TokenFlags.DisableExpansion))
				{
					// Try to get a macro for the current token
					if (!_state.TryGetMacro(outputTokens[^1].Identifier!, out PreprocessorMacro? macro) || ignoreMacros.Contains(macro))
					{
						break;
					}
					if (macro.IsFunctionMacro && (!moveNext || inputEnumerator.Current.Type != TokenType.LeftParen))
					{
						break;
					}

					// Remove the macro name from the output list
					bool hasLeadingSpace = outputTokens[^1].HasLeadingSpace;
					outputTokens.RemoveAt(outputTokens.Count - 1);

					// Save the initial number of tokens in the list, so we can tell if it expanded
					int numTokens = outputTokens.Count;

					// If it's an object macro, expand it immediately into the output buffer
					if (macro.IsObjectMacro)
					{
						// Expand the macro tokens into the output buffer
						ExpandObjectMacro(macro, outputTokens, isConditional, ignoreMacros, context);
					}
					else
					{
						// Read balanced token for argument list
						List<Token> argumentTokens = new();
						moveNext = ReadBalancedToken(inputEnumerator, argumentTokens, context);

						// Expand the macro tokens into the output buffer
						ExpandFunctionMacro(macro, argumentTokens, outputTokens, isConditional, ignoreMacros, context);
					}

					// If the macro expanded to nothing, quit
					if (outputTokens.Count <= numTokens)
					{
						break;
					}

					// Make sure the space is propagated to the expanded macro
					outputTokens[numTokens] = MergeLeadingSpace(outputTokens[numTokens], hasLeadingSpace);

					// Mark any tokens matching the macro name as not to be expanded again. This can happen with recursive object macros, eg. #define DWORD ::DWORD
					for (int idx = numTokens; idx < outputTokens.Count; idx++)
					{
						if (outputTokens[idx].Type == TokenType.Identifier && outputTokens[idx].Identifier == macro.Name)
						{
							outputTokens[idx] = outputTokens[idx].AddFlags(TokenFlags.DisableExpansion);
						}
					}
				}
			}
			return moveNext;
		}

		/// <summary>
		/// Gets a string for the __FILE__ macro
		/// </summary>
		/// <param name="context">Context to scan to find the current file</param>
		/// <returns>String representing the current context</returns>
		static string GetCurrentFileMacroValue(PreprocessorContext context)
		{
			SourceFile? sourceFile = GetCurrentSourceFile(context);
			if (sourceFile == null)
			{
				return "<unknown>";
			}
			else
			{
				return sourceFile.Location.FullName;
			}
		}

		/// <summary>
		/// Gets a string for the current file
		/// </summary>
		/// <param name="context">Context to scan to find the current file</param>
		/// <returns>Current source file being parsed</returns>
		static SourceFile? GetCurrentSourceFile(PreprocessorContext context)
		{
			SourceFile? sourceFile = null;
			for (PreprocessorContext? outerContext = context; outerContext != null; outerContext = outerContext.Outer)
			{
				if (outerContext is PreprocessorFileContext outerFileContext)
				{
					sourceFile = outerFileContext.SourceFile;
					break;
				}
			}
			return sourceFile;
		}

		/// <summary>
		/// Gets the current line number
		/// </summary>
		/// <param name="context">Context to scan to find the current file</param>
		/// <returns>Line number in the first file encountered</returns>
		static int GetCurrentLine(PreprocessorContext context)
		{
			for (PreprocessorContext? outerContext = context; outerContext != null; outerContext = outerContext.Outer)
			{
				if (outerContext is PreprocessorFileContext outerFileContext)
				{
					return outerFileContext.SourceFile.Markup[outerFileContext.MarkupIdx].LineNumber;
				}
			}
			return 0;
		}

		/// <summary>
		/// Expand an object macro
		/// </summary>
		/// <param name="macro">The functional macro</param>
		/// <param name="outputTokens">The list to receive the output tokens</param>
		/// <param name="isConditional">Whether the macro is being expanded in a conditional context, allowing use of the 'defined' keyword</param>
		/// <param name="ignoreMacros">List of macros currently being expanded, which should be ignored for recursion</param>
		/// <param name="context">The context that this directive is being parsed in</param>
		void ExpandObjectMacro(PreprocessorMacro macro, List<Token> outputTokens, bool isConditional, List<PreprocessorMacro> ignoreMacros, PreprocessorContext context)
		{
			// Special handling for the __LINE__ directive, since we need an updated line number for the current token
			if (macro.Name == Identifiers.__FILE__)
			{
				Token token = new(TokenType.String, TokenFlags.None, String.Format("\"{0}\"", GetCurrentFileMacroValue(context).Replace("\\", "\\\\", StringComparison.Ordinal)));
				outputTokens.Add(token);
			}
			else if (macro.Name == Identifiers.__LINE__)
			{
				Token token = new(TokenType.Number, TokenFlags.None, GetCurrentLine(context).ToString());
				outputTokens.Add(token);
			}
			else if (macro.Name == Identifiers.__COUNTER__)
			{
				Token token = new(TokenType.Number, TokenFlags.None, (_counter++).ToString());
				outputTokens.Add(token);
			}
			else
			{
				int outputTokenCount = outputTokens.Count;

				// Expand all the macros
				ignoreMacros.Add(macro);
				ExpandMacrosRecursively(macro.Tokens, outputTokens, isConditional, ignoreMacros, context);
				ignoreMacros.RemoveAt(ignoreMacros.Count - 1);

				// Concatenate any adjacent tokens
				for (int idx = outputTokenCount + 1; idx < outputTokens.Count - 1; idx++)
				{
					if (outputTokens[idx].Type == TokenType.HashHash)
					{
						outputTokens[idx - 1] = Token.Concatenate(outputTokens[idx - 1], outputTokens[idx + 1], context);
						outputTokens.RemoveRange(idx, 2);
						idx--;
					}
				}
			}
		}

		/// <summary>
		/// Expand a function macro
		/// </summary>
		/// <param name="macro">The functional macro</param>
		/// <param name="argumentListTokens">Identifiers for each argument token</param>
		/// <param name="outputTokens">The list to receive the output tokens</param>
		/// <param name="isConditional">Whether the macro is being expanded in a conditional context, allowing use of the 'defined' keyword</param>
		/// <param name="ignoreMacros">List of macros currently being expanded, which should be ignored for recursion</param>
		/// <param name="context">The context that this macro is being expanded</param>
		void ExpandFunctionMacro(PreprocessorMacro macro, List<Token> argumentListTokens, List<Token> outputTokens, bool isConditional, List<PreprocessorMacro> ignoreMacros, PreprocessorContext context)
		{
			// Replace any newlines with spaces, and merge them with the following token
			for (int idx = 0; idx < argumentListTokens.Count; idx++)
			{
				if (argumentListTokens[idx].Type == TokenType.Newline)
				{
					if (idx + 1 < argumentListTokens.Count)
					{
						argumentListTokens[idx + 1] = MergeLeadingSpace(argumentListTokens[idx + 1], true);
					}
					argumentListTokens.RemoveAt(idx--);
				}
			}

			// Split the arguments out into separate lists
			List<List<Token>> arguments = new();
			if (argumentListTokens.Count > 2)
			{
				for (int idx = 1; ; idx++)
				{
					if (!macro.HasVariableArgumentList || arguments.Count < macro.Parameters!.Count)
					{
						arguments.Add(new List<Token>());
					}

					List<Token> argument = arguments[^1];

					int initialIdx = idx;
					while (idx < argumentListTokens.Count - 1 && argumentListTokens[idx].Type != TokenType.Comma)
					{
						if (!ReadBalancedToken(argumentListTokens, ref idx, argument))
						{
							throw new PreprocessorException(context, "Invalid argument");
						}
					}

					if (argument.Count > 0 && arguments[^1][0].HasLeadingSpace)
					{
						argument[0] = argument[0].RemoveFlags(TokenFlags.HasLeadingSpace);
					}

					bool hasLeadingSpace = false;
					for (int tokenIdx = 0; tokenIdx < argument.Count; tokenIdx++)
					{
						if (argument[tokenIdx].Text.Length == 0)
						{
							hasLeadingSpace |= argument[tokenIdx].HasLeadingSpace;
							argument.RemoveAt(tokenIdx--);
						}
						else
						{
							argument[tokenIdx] = MergeLeadingSpace(argument[tokenIdx], hasLeadingSpace);
							hasLeadingSpace = false;
						}
					}

					if (argument.Count == 0)
					{
						argument.Add(new Token(TokenType.Placemarker, TokenFlags.None));
						argument.Add(new Token(TokenType.Placemarker, hasLeadingSpace ? TokenFlags.HasLeadingSpace : TokenFlags.None));
					}

					if (idx == argumentListTokens.Count - 1)
					{
						break;
					}

					if (argumentListTokens[idx].Type != TokenType.Comma)
					{
						throw new PreprocessorException(context, "Expected ',' between arguments");
					}

					if (macro.HasVariableArgumentList && arguments.Count == macro.Parameters!.Count && idx < argumentListTokens.Count - 1)
					{
						arguments[^1].Add(argumentListTokens[idx]);
					}
				}
			}

			// Add an empty variable argument if one was not specified
			if (macro.HasVariableArgumentList && arguments.Count == macro.Parameters!.Count - 1)
			{
				arguments.Add(new List<Token> { new(TokenType.Placemarker, TokenFlags.None) });
			}

			// Validate the argument list
			if (arguments.Count != macro.Parameters!.Count)
			{
				throw new PreprocessorException(context, "Incorrect number of arguments to macro");
			}

			// Expand each one of the arguments
			List<List<Token>> expandedArguments = new();
			for (int idx = 0; idx < arguments.Count; idx++)
			{
				List<Token> newArguments = new();
				ExpandMacrosRecursively(arguments[idx], newArguments, isConditional, ignoreMacros, context);
				expandedArguments.Add(newArguments);
			}

			// Substitute all the argument tokens
			List<Token> expandedTokens = new();
			for (int idx = 0; idx < macro.Tokens.Count; idx++)
			{
				Token token = macro.Tokens[idx];
				if (token.Type == TokenType.Hash && idx + 1 < macro.Tokens.Count)
				{
					// Stringizing operator
					int paramIdx = macro.FindParameterIndex(macro.Tokens[++idx].Identifier!);
					if (paramIdx == -1)
					{
						throw new PreprocessorException(context, "{0} is not an argument name", macro.Tokens[idx].Text);
					}
					expandedTokens.Add(new Token(TokenType.String, token.Flags & TokenFlags.HasLeadingSpace, String.Format("\"{0}\"", Token.Format(arguments[paramIdx]).Replace("\\", "\\\\", StringComparison.Ordinal).Replace("\"", "\\\"", StringComparison.Ordinal))));
				}
				else if (macro.HasVariableArgumentList && idx + 2 < macro.Tokens.Count && token.Type == TokenType.Comma && macro.Tokens[idx + 1].Type == TokenType.HashHash && macro.Tokens[idx + 2].Identifier == Identifiers.__VA_ARGS__)
				{
					// Special MSVC/GCC extension: ', ## __VA_ARGS__' removes the comma if __VA_ARGS__ is empty. MSVC seems to format the result with a forced space.
					List<Token> expandedArgument = expandedArguments[^1];
					if (expandedArgument.Any(x => x.Text.Length > 0))
					{
						expandedTokens.Add(token);
						AppendTokensWithWhitespace(expandedTokens, expandedArgument, false);
						idx += 2;
					}
					else
					{
						expandedTokens.Add(new Token(TokenType.Placemarker, token.Flags & TokenFlags.HasLeadingSpace));
						expandedTokens.Add(new Token(TokenType.Placemarker, TokenFlags.HasLeadingSpace));
						idx += 2;
					}
				}
				else if (token.Type == TokenType.Identifier)
				{
					// Expand a parameter
					int paramIdx = macro.FindParameterIndex(token.Identifier!);
					if (paramIdx == -1)
					{
						expandedTokens.Add(token);
					}
					else if (idx > 0 && macro.Tokens[idx - 1].Type == TokenType.HashHash)
					{
						AppendTokensWithWhitespace(expandedTokens, arguments[paramIdx], token.HasLeadingSpace);
					}
					else if (idx + 1 < macro.Tokens.Count && macro.Tokens[idx + 1].Type == TokenType.HashHash)
					{
						AppendTokensWithWhitespace(expandedTokens, arguments[paramIdx], token.HasLeadingSpace);
					}
					else
					{
						AppendTokensWithWhitespace(expandedTokens, expandedArguments[paramIdx], token.HasLeadingSpace);
					}
				}
				else
				{
					expandedTokens.Add(token);
				}
			}

			// Concatenate adjacent tokens
			for (int idx = 1; idx < expandedTokens.Count - 1; idx++)
			{
				if (expandedTokens[idx].Type == TokenType.HashHash)
				{
					Token concatenatedToken = Token.Concatenate(expandedTokens[idx - 1], expandedTokens[idx + 1], context);
					expandedTokens.RemoveRange(idx, 2);
					expandedTokens[--idx] = concatenatedToken;
				}
			}

			// Finally, return the expansion of this
			ignoreMacros.Add(macro);
			ExpandMacrosRecursively(expandedTokens, outputTokens, isConditional, ignoreMacros, context);
			ignoreMacros.RemoveAt(ignoreMacros.Count - 1);
		}

		/// <summary>
		/// Appends a list of tokens to another list, setting the leading whitespace flag to the given value
		/// </summary>
		/// <param name="outputTokens">List to receive the appended tokens</param>
		/// <param name="inputTokens">List of tokens to append</param>
		/// <param name="hasLeadingSpace">Whether there is space before the first token</param>
		static void AppendTokensWithWhitespace(List<Token> outputTokens, List<Token> inputTokens, bool hasLeadingSpace)
		{
			if (inputTokens.Count > 0)
			{
				outputTokens.Add(MergeLeadingSpace(inputTokens[0], hasLeadingSpace));
				outputTokens.AddRange(inputTokens.Skip(1));
			}
		}

		/// <summary>
		/// Copies a single token from one list of tokens to another, or if it's an opening parenthesis, the entire subexpression until the closing parenthesis.
		/// </summary>
		/// <param name="inputTokens">The input token list</param>
		/// <param name="inputIdx">First token index in the input token list. Set to the last uncopied token index on return.</param>
		/// <param name="outputTokens">List to receive the output tokens</param>
		/// <returns>True if a balanced expression was read, or false if the end of the list was encountered before finding a matching token</returns>
		static bool ReadBalancedToken(List<Token> inputTokens, ref int inputIdx, List<Token> outputTokens)
		{
			// Copy a single token to the output list
			Token token = inputTokens[inputIdx++];
			outputTokens.Add(token);

			// If it was the start of a subexpression, copy until the closing parenthesis
			if (token.Type == TokenType.LeftParen)
			{
				// Copy the contents of the subexpression
				for (; ; )
				{
					if (inputIdx == inputTokens.Count)
					{
						return false;
					}
					if (inputTokens[inputIdx].Type == TokenType.RightParen)
					{
						break;
					}
					if (!ReadBalancedToken(inputTokens, ref inputIdx, outputTokens))
					{
						return false;
					}
				}

				// Copy the closing parenthesis
				token = inputTokens[inputIdx++];
				outputTokens.Add(token);
			}
			return true;
		}

		/// <summary>
		/// Copies a single token from one list of tokens to another, or if it's an opening parenthesis, the entire subexpression until the closing parenthesis.
		/// </summary>
		/// <param name="inputEnumerator">The input token list</param>
		/// <param name="outputTokens">List to receive the output tokens</param>
		/// <param name="context">The context that the parser is in</param>
		/// <returns>True if a balanced expression was read, or false if the end of the list was encountered before finding a matching token</returns>
		static bool ReadBalancedToken(IEnumerator<Token> inputEnumerator, List<Token> outputTokens, PreprocessorContext context)
		{
			// Copy a single token to the output list
			Token token = inputEnumerator.Current;
			bool moveNext = inputEnumerator.MoveNext();
			outputTokens.Add(token);

			// If it was the start of a subexpression, copy until the closing parenthesis
			if (token.Type == TokenType.LeftParen)
			{
				// Copy the contents of the subexpression
				for (; ; )
				{
					if (!moveNext)
					{
						throw new PreprocessorException(context, "Unbalanced token sequence");
					}
					if (inputEnumerator.Current.Type == TokenType.RightParen)
					{
						outputTokens.Add(inputEnumerator.Current);
						moveNext = inputEnumerator.MoveNext();
						break;
					}
					moveNext = ReadBalancedToken(inputEnumerator, outputTokens, context);
				}
			}
			return moveNext;
		}
	}
}
