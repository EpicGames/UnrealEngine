// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// Represents a context that the preprocessor is working in. Used to form error messages, and determine things like the __FILE__ and __LINE__ directives.
	/// </summary>
	public abstract class PreprocessorContext
	{
		/// <summary>
		/// The outer context
		/// </summary>
		public readonly PreprocessorContext? Outer;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="outer">The outer context</param>
		protected PreprocessorContext(PreprocessorContext? outer)
		{
			Outer = outer;
		}
	}

	/// <summary>
	/// Context for a command line argument
	/// </summary>
	class PreprocessorCommandLineContext : PreprocessorContext
	{
		/// <summary>
		/// Constructor
		/// </summary>
		public PreprocessorCommandLineContext()
			: base(null)
		{
		}

		/// <summary>
		/// Formats this context for error messages
		/// </summary>
		/// <returns>String describing this context</returns>
		public override string ToString()
		{
			return "From command line";
		}
	}

	/// <summary>
	/// Represents a context that the preprocessor is reading from
	/// </summary>
	class PreprocessorFileContext : PreprocessorContext
	{
		/// <summary>
		/// The source file being read
		/// </summary>
		public readonly SourceFile SourceFile;

		/// <summary>
		/// The directory containing this file. When searching for included files, MSVC will check this directory.
		/// </summary>
		public readonly DirectoryItem Directory;

		/// <summary>
		/// Index of the current markup object being processed
		/// </summary>
		public int MarkupIdx { get; set; }

		/// <summary>
		/// Index of the next fragment to be read
		/// </summary>
		public int FragmentIdx { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="sourceFile">The source file being parsed</param>
		/// <param name="outer">The outer context</param>
		public PreprocessorFileContext(SourceFile sourceFile, PreprocessorContext? outer)
			: base(outer)
		{
			SourceFile = sourceFile;
			Directory = DirectoryItem.GetItemByDirectoryReference(sourceFile.Location.Directory);
			MarkupIdx = 0;
			FragmentIdx = 0;
		}

		/// <summary>
		/// Format this file for the debugger, and error messages
		/// </summary>
		/// <returns></returns>
		public override string ToString()
		{
			return String.Format("{0}({1})", SourceFile.Location, SourceFile.Markup[MarkupIdx].LineNumber);
		}
	}
}
