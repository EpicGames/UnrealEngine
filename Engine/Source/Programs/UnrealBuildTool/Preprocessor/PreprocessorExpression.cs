// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text;

namespace UnrealBuildTool
{
	/// <summary>
	/// Class to allow evaluating a preprocessor expression
	/// </summary>
	static class PreprocessorExpression
	{
		/// <summary>
		/// Precedence table for binary operators (indexed by TokenType). Lower values indicate higher precedence.
		/// </summary>
		static readonly byte[] s_precedence;

#pragma warning disable CA1810 // Initialize reference type static fields inline
		/// <summary>
		/// Static constructor. Initializes the Precedence table.
		/// </summary>
		static PreprocessorExpression()
		{
			s_precedence = new byte[(int)TokenType.Max];
			s_precedence[(int)TokenType.Multiply] = 10;
			s_precedence[(int)TokenType.Divide] = 10;
			s_precedence[(int)TokenType.Modulo] = 10;
			s_precedence[(int)TokenType.Plus] = 9;
			s_precedence[(int)TokenType.Minus] = 9;
			s_precedence[(int)TokenType.LeftShift] = 8;
			s_precedence[(int)TokenType.RightShift] = 8;
			s_precedence[(int)TokenType.CompareLess] = 7;
			s_precedence[(int)TokenType.CompareLessOrEqual] = 7;
			s_precedence[(int)TokenType.CompareGreater] = 7;
			s_precedence[(int)TokenType.CompareGreaterOrEqual] = 7;
			s_precedence[(int)TokenType.CompareEqual] = 6;
			s_precedence[(int)TokenType.CompareNotEqual] = 6;
			s_precedence[(int)TokenType.BitwiseAnd] = 5;
			s_precedence[(int)TokenType.BitwiseXor] = 4;
			s_precedence[(int)TokenType.BitwiseOr] = 3;
			s_precedence[(int)TokenType.LogicalAnd] = 2;
			s_precedence[(int)TokenType.LogicalOr] = 1;
		}
#pragma warning restore CA1810

		/// <summary>
		/// Evaluate a preprocessor expression
		/// </summary>
		/// <param name="context">The current preprocessor context</param>
		/// <param name="tokens">List of tokens in the expression</param>
		/// <returns>Value of the expression</returns>
		public static long Evaluate(PreprocessorContext context, List<Token> tokens)
		{
			int idx = 0;
			long result = EvaluateTernary(context, tokens, ref idx);
			if (idx != tokens.Count)
			{
				throw new PreprocessorException(context, $"Garbage after end of expression: {Token.Format(tokens)}");
			}
			return result;
		}

		/// <summary>
		/// Evaluates a ternary expression (a? b :c)
		/// </summary>
		/// <param name="context">The current preprocessor context</param>
		/// <param name="tokens">List of tokens in the expression</param>
		/// <param name="idx">Index into the token list of the next token to read</param>
		/// <returns>Value of the expression</returns>
		static long EvaluateTernary(PreprocessorContext context, List<Token> tokens, ref int idx)
		{
			long result = EvaluateUnary(context, tokens, ref idx);
			if (idx < tokens.Count && s_precedence[(int)tokens[idx].Type] != 0)
			{
				result = EvaluateBinary(context, tokens, ref idx, result, 1);
			}
			if (idx < tokens.Count && tokens[idx].Type == TokenType.QuestionMark)
			{
				// Read the left expression
				idx++;
				long lhs = EvaluateTernary(context, tokens, ref idx);

				// Check for the colon in the middle
				if (tokens[idx].Type != TokenType.Colon)
				{
					throw new PreprocessorException(context, $"Expected colon for conditional operator, found {tokens[idx].Text}");
				}

				// Read the right expression
				idx++;
				long rhs = EvaluateTernary(context, tokens, ref idx);

				// Evaluate it
				result = (result != 0) ? lhs : rhs;
			}
			return result;
		}

		/// <summary>
		/// Recursive path for evaluating a sequence of binary operators, given a known LHS. Implementation of the shunting yard algorithm.
		/// </summary>
		/// <param name="context">The current preprocessor context</param>
		/// <param name="tokens">List of tokens in the expression</param>
		/// <param name="idx">Index into the token list of the next token to read</param>
		/// <param name="lhs">The LHS value for the binary expression</param>
		/// <param name="minPrecedence">Minimum precedence value to consume</param>
		/// <returns>Value of the expression</returns>
		static long EvaluateBinary(PreprocessorContext context, List<Token> tokens, ref int idx, long lhs, byte minPrecedence)
		{
			while (idx < tokens.Count)
			{
				// Read the next operator token
				Token tokenOp = tokens[idx];

				// Check if this operator binds more tightly than the outer expression, and exit if it does.
				int operatorPrecedence = s_precedence[(int)tokenOp.Type];
				if (operatorPrecedence < minPrecedence)
				{
					break;
				}

				// Move to the next token
				idx++;

				// Evaluate the immediate RHS value, then recursively evaluate any subsequent binary operators that bind more tightly to it than this.
				long rhs = EvaluateUnary(context, tokens, ref idx);
				while (idx < tokens.Count)
				{
					byte nextOperatorPrecedence = s_precedence[(int)tokens[idx].Type];
					if (nextOperatorPrecedence <= operatorPrecedence)
					{
						break;
					}
					rhs = EvaluateBinary(context, tokens, ref idx, rhs, nextOperatorPrecedence);
				}

				// Evalute the result of applying the operator to the LHS and RHS values
				switch (tokenOp.Type)
				{
					case TokenType.LogicalOr:
						lhs = ((lhs != 0) || (rhs != 0)) ? 1 : 0;
						break;
					case TokenType.LogicalAnd:
						lhs = ((lhs != 0) && (rhs != 0)) ? 1 : 0;
						break;
					case TokenType.BitwiseOr:
						lhs |= rhs;
						break;
					case TokenType.BitwiseXor:
						lhs ^= rhs;
						break;
					case TokenType.BitwiseAnd:
						lhs &= rhs;
						break;
					case TokenType.CompareEqual:
						lhs = (lhs == rhs) ? 1 : 0;
						break;
					case TokenType.CompareNotEqual:
						lhs = (lhs != rhs) ? 1 : 0;
						break;
					case TokenType.CompareLess:
						lhs = (lhs < rhs) ? 1 : 0;
						break;
					case TokenType.CompareGreater:
						lhs = (lhs > rhs) ? 1 : 0;
						break;
					case TokenType.CompareLessOrEqual:
						lhs = (lhs <= rhs) ? 1 : 0;
						break;
					case TokenType.CompareGreaterOrEqual:
						lhs = (lhs >= rhs) ? 1 : 0;
						break;
					case TokenType.LeftShift:
						lhs <<= (int)rhs;
						break;
					case TokenType.RightShift:
						lhs >>= (int)rhs;
						break;
					case TokenType.Plus:
						lhs += rhs;
						break;
					case TokenType.Minus:
						lhs -= rhs;
						break;
					case TokenType.Multiply:
						lhs *= rhs;
						break;
					case TokenType.Divide:
						lhs /= rhs;
						break;
					case TokenType.Modulo:
						lhs %= rhs;
						break;
					default:
						throw new NotImplementedException($"Binary operator '{tokenOp.Type}' has not been implemented");
				}
			}

			return lhs;
		}

		/// <summary>
		/// Evaluates a unary expression (+, -, !, ~)
		/// </summary>
		/// <param name="context">The current preprocessor context</param>
		/// <param name="tokens">List of tokens in the expression</param>
		/// <param name="idx">Index into the token list of the next token to read</param>
		/// <returns>Value of the expression</returns>
		static long EvaluateUnary(PreprocessorContext context, List<Token> tokens, ref int idx)
		{
			if (idx == tokens.Count)
			{
				throw new PreprocessorException(context, "Early end of expression");
			}

			switch (tokens[idx].Type)
			{
				case TokenType.Plus:
					idx++;
					return +EvaluateUnary(context, tokens, ref idx);
				case TokenType.Minus:
					idx++;
					return -EvaluateUnary(context, tokens, ref idx);
				case TokenType.LogicalNot:
					idx++;
					return (EvaluateUnary(context, tokens, ref idx) == 0) ? 1 : 0;
				case TokenType.BitwiseNot:
					idx++;
					return ~EvaluateUnary(context, tokens, ref idx);
				case TokenType.Identifier:
					Identifier text = tokens[idx].Identifier!;
					if (text == Identifiers.Sizeof)
					{
						throw new NotImplementedException(text.ToString());
					}
					else if (text == Identifiers.Alignof)
					{
						throw new NotImplementedException(text.ToString());
					}
					else if (text == Identifiers.__has_builtin || text == Identifiers.__has_feature
						|| text == Identifiers.__has_warning || text == Identifiers.__building_module
						|| text == Identifiers.__pragma || text == Identifiers.__builtin_return_address
						|| text == Identifiers.__builtin_frame_address || text == Identifiers.__has_keyword
						|| text == Identifiers.__has_extension || text == Identifiers.__is_target_arch
						|| text == Identifiers.__is_identifier)
					{
						if (tokens[idx + 1].Type != TokenType.LeftParen || tokens[idx + 3].Type != TokenType.RightParen)
						{
							throw new NotImplementedException(text.ToString());
						}
						idx += 4;
						return 0;
					}
					else if (text == Identifiers.__has_attribute || text == Identifiers.__has_declspec_attribute
						|| text == Identifiers.__has_c_attribute || text == Identifiers.__has_cpp_attribute
						|| text == Identifiers.__has_include || text == Identifiers.__has_include_next)
					{
						idx += tokens.Count - idx;
						return 0;
					}

					else
					{
						return EvaluatePrimary(context, tokens, ref idx);
					}
				default:
					return EvaluatePrimary(context, tokens, ref idx);
			}
		}

		/// <summary>
		/// Evaluates a primary expression
		/// </summary>
		/// <param name="context">The current preprocessor context</param>
		/// <param name="tokens">List of tokens in the expression</param>
		/// <param name="idx">Index into the token list of the next token to read</param>
		/// <returns>Value of the expression</returns>
		static long EvaluatePrimary(PreprocessorContext context, List<Token> tokens, ref int idx)
		{
			if (tokens[idx].Type == TokenType.Identifier)
			{
				idx++;
				return 0;
			}
			else if (tokens[idx].Type == TokenType.LeftParen)
			{
				// Read the expression
				idx++;
				long result = EvaluateTernary(context, tokens, ref idx);

				// Check for a closing parenthesis
				if (tokens[idx].Type != TokenType.RightParen)
				{
					throw new PreprocessorException(context, "Missing closing parenthesis");
				}

				// Return the value
				idx++;
				return result;
			}
			else if (tokens[idx].Type == TokenType.Number)
			{
				return ParseNumericLiteral(context, tokens[idx++].Literal!);
			}
			else
			{
				throw new PreprocessorException(context, "Unexpected token in expression: {0}", tokens[idx]);
			}
		}

		/// <summary>
		/// Parse a numeric literal from the given token
		/// </summary>
		/// <param name="context">The current preprocessor context</param>
		/// <param name="literal">The utf-8 literal characters</param>
		/// <returns>The literal value of the token</returns>
		static long ParseNumericLiteral(PreprocessorContext context, byte[] literal)
		{
			// Parse the token
			long value = 0;
			if (literal.Length >= 2 && literal[0] == '0' && (literal[1] == 'x' || literal[1] == 'X'))
			{
				for (int idx = 2; idx < literal.Length; idx++)
				{
					if (literal[idx] >= '0' && literal[idx] <= '9')
					{
						value = (value << 4) + (literal[idx] - '0');
					}
					else if (literal[idx] >= 'a' && literal[idx] <= 'f')
					{
						value = (value << 4) + (literal[idx] - 'a') + 10;
					}
					else if (literal[idx] >= 'A' && literal[idx] <= 'F')
					{
						value = (value << 4) + (literal[idx] - 'A') + 10;
					}
					else if (IsValidIntegerSuffix(literal, idx))
					{
						break;
					}
					else
					{
						throw new PreprocessorException(context, "Invalid hexadecimal literal: '{0}'", Encoding.UTF8.GetString(literal));
					}
				}
			}
			else if (literal[0] == '0')
			{
				for (int idx = 1; idx < literal.Length; idx++)
				{
					if (literal[idx] >= '0' && literal[idx] <= '7')
					{
						value = (value << 3) + (literal[idx] - '0');
					}
					else if (IsValidIntegerSuffix(literal, idx))
					{
						break;
					}
					else
					{
						throw new PreprocessorException(context, "Invalid octal literal: '{0}'", Encoding.UTF8.GetString(literal));
					}
				}
			}
			else
			{
				for (int idx = 0; idx < literal.Length; idx++)
				{
					if (literal[idx] >= '0' && literal[idx] <= '9')
					{
						value = (value * 10) + (literal[idx] - '0');
					}
					else if (IsValidIntegerSuffix(literal, idx))
					{
						break;
					}
					else
					{
						throw new PreprocessorException(context, "Invalid decimal literal: '{0}'", Encoding.UTF8.GetString(literal));
					}
				}
			}
			return value;
		}

		/// <summary>
		/// Checks whether the given literal has a valid integer suffix, starting at the given position
		/// </summary>
		/// <param name="literal">The literal token</param>
		/// <param name="index">Index within the literal to start reading</param>
		/// <returns>The literal value of the token</returns>
		static bool IsValidIntegerSuffix(byte[] literal, int index)
		{
			bool allowTrailingUnsigned = true;

			if (literal[index] == 'u' || literal[index] == 'U')
			{
				// 'U'
				index++;
				allowTrailingUnsigned = false;
			}

			if (index < literal.Length && (literal[index] == 'l' || literal[index] == 'L'))
			{
				// 'LL' or 'L'
				if (index + 1 < literal.Length && literal[index + 1] == literal[index])
				{
					index += 2;
				}
				else
				{
					index++;
				}
			}
			else if (index + 2 < literal.Length && (literal[index] == 'i' || literal[index] == 'I') && literal[index + 1] == '3' && literal[index + 2] == '2')
			{
				// 'I32' (Microsoft extension)
				index += 3;
				allowTrailingUnsigned = false;
			}
			else if (index + 2 < literal.Length && (literal[index] == 'i' || literal[index] == 'I') && literal[index + 1] == '6' && literal[index + 2] == '4')
			{
				// 'I64' (Microsoft extension)
				index += 3;
				allowTrailingUnsigned = false;
			}

			if (allowTrailingUnsigned && index < literal.Length && (literal[index] == 'u' || literal[index] == 'U'))
			{
				// 'U'
				index++;
			}

			return index == literal.Length;
		}
	}
}
