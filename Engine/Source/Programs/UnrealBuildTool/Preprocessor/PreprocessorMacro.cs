// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using EpicGames.Core;

namespace UnrealBuildTool
{
	/// <summary>
	/// Stores information about a defined preprocessor macro
	/// </summary>
	class PreprocessorMacro
	{
		/// <summary>
		/// Name of the macro
		/// </summary>
		public readonly Identifier Name;

		/// <summary>
		/// Parameter names for the macro. The '...' placeholder is represented by the __VA_ARGS__ string.
		/// </summary>
		public readonly List<Identifier>? Parameters;

		/// <summary>
		/// Raw list of tokens for this macro
		/// </summary>
		public readonly List<Token> Tokens;

		/// <summary>
		/// Construct a preprocessor macro
		/// </summary>
		/// <param name="name">Name of the macro</param>
		/// <param name="parameters">Parameter list for the macro. Should be null for object macros. Ownership of this list is transferred.</param>
		/// <param name="tokens">Tokens for the macro. Ownership of this list is transferred.</param>
		public PreprocessorMacro(Identifier name, List<Identifier>? parameters, List<Token> tokens)
		{
			Name = name;
			Parameters = parameters;
			Tokens = tokens;
		}

		/// <summary>
		/// Read a macro from a binary archive
		/// </summary>
		/// <param name="reader">Reader to serialize from</param>
		public PreprocessorMacro(BinaryArchiveReader reader)
		{
			Name = reader.ReadIdentifier();
			Parameters = reader.ReadList(() => reader.ReadIdentifier());
			Tokens = reader.ReadList(() => reader.ReadToken())!;
		}

		/// <summary>
		/// Write a macro to a binary archive
		/// </summary>
		/// <param name="writer">Writer to serialize to</param>
		public void Write(BinaryArchiveWriter writer)
		{
			writer.WriteIdentifier(Name);
			writer.WriteList(Parameters, x => writer.WriteIdentifier(x));
			writer.WriteList(Tokens, x => writer.WriteToken(x));
		}

		/// <summary>
		/// Finds the index of a parameter in the parameter list
		/// </summary>
		/// <param name="parameter">Parameter name to look for</param>
		/// <returns>Index of the parameter, or -1 if it's not found.</returns>
		public int FindParameterIndex(Identifier parameter)
		{
			if (Parameters != null)
			{
				for (int idx = 0; idx < Parameters.Count; idx++)
				{
					if (Parameters[idx] == parameter)
					{
						return idx;
					}
				}
			}
			return -1;
		}

		/// <summary>
		/// Checks whether this macro definition is equivalent to another macro definition
		/// </summary>
		/// <param name="other">The macro definition to compare against</param>
		/// <returns>True if the macro definitions are equivalent</returns>
		public bool IsEquivalentTo(PreprocessorMacro other)
		{
			if (this != other)
			{
				if (Name != other.Name || Tokens.Count != other.Tokens.Count)
				{
					return false;
				}
				if (Parameters != null)
				{
					if (other.Parameters == null || other.Parameters.Count != Parameters.Count || !Enumerable.SequenceEqual(Parameters, other.Parameters))
					{
						return false;
					}
				}
				else
				{
					if (other.Parameters != null)
					{
						return false;
					}
				}
				if (!Enumerable.SequenceEqual(Tokens, other.Tokens))
				{
					return false;
				}
			}
			return true;
		}

		/// <summary>
		/// True if the macro is an object macro
		/// </summary>
		public bool IsObjectMacro => Parameters == null;

		/// <summary>
		/// True if the macro is a function macro
		/// </summary>
		public bool IsFunctionMacro => Parameters != null;

		/// <summary>
		/// The number of required parameters. For variadic macros, the last parameter is optional.
		/// </summary>
		public int MinRequiredParameters => HasVariableArgumentList ? (Parameters!.Count - 1) : Parameters!.Count;

		/// <summary>
		/// True if the macro has a variable argument list
		/// </summary>
		public bool HasVariableArgumentList => Parameters!.Count > 0 && Parameters[^1] == Identifiers.__VA_ARGS__;

		/// <summary>
		/// Converts this macro to a string for debugging
		/// </summary>
		/// <returns>The tokens in this macro</returns>
		public override string ToString()
		{
			StringBuilder result = new(Name.ToString());
			if (Parameters != null)
			{
				result.AppendFormat("({0})", String.Join(", ", Parameters));
			}
			result.Append('=');
			if (Tokens.Count > 0)
			{
				result.Append(Tokens[0].Text);
				for (int idx = 1; idx < Tokens.Count; idx++)
				{
					if (Tokens[idx].HasLeadingSpace)
					{
						result.Append(' ');
					}
					result.Append(Tokens[idx].Text);
				}
			}
			return result.ToString();
		}
	}
}
