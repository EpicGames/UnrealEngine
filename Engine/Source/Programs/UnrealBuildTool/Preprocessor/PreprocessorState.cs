// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;

namespace UnrealBuildTool
{
	/// <summary>
	/// Flags indicating the state of each nested conditional block in a stack.
	/// </summary>
	[Flags]
	enum PreprocessorBranch
	{
		/// <summary>
		/// The current conditional branch is active.
		/// </summary>
		Active = 0x01,

		/// <summary>
		/// A branch within the current conditional block has been taken. Any #else/#elif blocks will not be taken.
		/// </summary>
		Taken = 0x02,

		/// <summary>
		/// The #else directive for this conditional block has been encountered. An #endif directive is expected next.
		/// </summary>
		Complete = 0x04,

		/// <summary>
		/// The first condition in this branch was an #if directive (as opposed to an #ifdef or #ifndef directive)
		/// </summary>
		HasIfDirective = 0x10,

		/// <summary>
		/// The first condition in this branch was an #if directive (as opposed to an #ifdef or #ifndef directive)
		/// </summary>
		HasIfdefDirective = 0x20,

		/// <summary>
		/// The first condition in this branch was an #ifndef directive (as opposed to an #ifdef or #if directive)
		/// </summary>
		HasIfndefDirective = 0x40,

		/// <summary>
		/// The branch has an #elif directive
		/// </summary>
		HasElifDirective = 0x80,
	}

	/// <summary>
	/// Marshals access to the preprocessor state, ensuring that any changes are tracked within the active transform
	/// </summary>
	class PreprocessorState
	{
		/// <summary>
		/// Stack of conditional branch states
		/// </summary>
		readonly List<PreprocessorBranch> _branches = new();

		/// <summary>
		/// Mapping of name to macro definition
		/// </summary>
		readonly Dictionary<Identifier, PreprocessorMacro> _nameToMacro = new();

		/// <summary>
		/// The current transform. Any queries or state modifications will be recorded in this.
		/// </summary>
		PreprocessorTransform? _transform;

		/// <summary>
		/// Enumerates the current branches. Do not use this for decision logic; any dependencies will not be tracked.
		/// </summary>
		public IEnumerable<PreprocessorBranch> CurrentBranches => _branches;

		/// <summary>
		/// Enumerates the current macros. Do not use this for decision logic; any dependencies will not be tracked.
		/// </summary>
		public IEnumerable<PreprocessorMacro> CurrentMacros => _nameToMacro.Values;

		/// <summary>
		/// Initialize an empty preprocessor state
		/// </summary>
		public PreprocessorState()
		{
		}

		/// <summary>
		/// Duplicates another preprocessor state. Throws an exception if a transform is currently being built in the other state.
		/// </summary>
		/// <param name="other">The preprocessor state to copy</param>
		public PreprocessorState(PreprocessorState other)
		{
			if (other._transform != null)
			{
				throw new Exception("Unable to copy another preprocessor state while a transform is being built.");
			}

			_branches.AddRange(other._branches);

			foreach (KeyValuePair<Identifier, PreprocessorMacro> pair in other._nameToMacro)
			{
				_nameToMacro[pair.Key] = pair.Value;
			}
		}

		/// <summary>
		/// Create a new transform object, and assign it to be current
		/// </summary>
		/// <returns>The new transform object</returns>
		public PreprocessorTransform BeginCapture()
		{
			_transform = new PreprocessorTransform();
			return _transform;
		}

		/// <summary>
		/// Detach the current transform
		/// </summary>
		/// <returns>The transform object</returns>
		public PreprocessorTransform? EndCapture()
		{
			PreprocessorTransform? prevTransform = _transform;
			_transform = null;
			return prevTransform;
		}

		/// <summary>
		/// Sets a flag indicating that a #pragma once directive was encountered
		/// </summary>
		public void MarkPragmaOnce()
		{
			if (_transform != null)
			{
				_transform.HasPragmaOnce = true;
			}
		}

		/// <summary>
		/// Set a macro to a specific definition
		/// </summary>
		/// <param name="macro">Macro definition</param>
		public void DefineMacro(PreprocessorMacro macro)
		{
			_nameToMacro[macro.Name] = macro;

			if (_transform != null)
			{
				_transform.NewMacros[macro.Name] = macro;
			}
		}

		/// <summary>
		/// Checks if a macro with the given name is defined
		/// </summary>
		/// <param name="name">The macro name</param>
		/// <returns>True if the macro is defined</returns>
		public bool IsMacroDefined(Identifier name)
		{
			// Could account for the fact that we don't need the full definition later...
			return TryGetMacro(name, out _);
		}

		/// <summary>
		/// Removes a macro definition
		/// </summary>
		/// <param name="name">Name of the macro</param>
		public void UndefMacro(Identifier name)
		{
			_nameToMacro.Remove(name);

			if (_transform != null)
			{
				_transform.NewMacros[name] = null;
			}
		}

		/// <summary>
		/// Attemps to get the definition for a macro
		/// </summary>
		/// <param name="name">Name of the macro</param>
		/// <param name="macro">Receives the macro definition, or null if it's not defined</param>
		/// <returns>True if the macro is defined, otherwise false</returns>
		public bool TryGetMacro(Identifier name, [NotNullWhen(true)] out PreprocessorMacro? macro)
		{
			_nameToMacro.TryGetValue(name, out macro);

			if (_transform != null && !_transform.NewMacros.ContainsKey(name))
			{
				_transform.RequiredMacros[name] = macro;
			}

			return macro != null;
		}

		/// <summary>
		/// Pushes the preprocessor branch onto the stack
		/// </summary>
		/// <param name="branch">The branch state</param>
		public void PushBranch(PreprocessorBranch branch)
		{
			_branches.Add(branch);

			_transform?.NewBranches.Add(branch);
		}

		/// <summary>
		/// Pops a preprocessor branch from the stack
		/// </summary>
		/// <returns>The popped branch state</returns>
		public PreprocessorBranch PopBranch()
		{
			if (!TryPopBranch(out PreprocessorBranch branch))
			{
				throw new InvalidOperationException("Branch stack is empty");
			}
			return branch;
		}

		/// <summary>
		/// Attempts to pops a preprocessor branch from the stack
		/// </summary>
		/// <param name="branch">On success, receives the preprocessor branch state</param>
		/// <returns>True if a branch was active, else false</returns>
		public bool TryPopBranch(out PreprocessorBranch branch)
		{
			if (_branches.Count == 0)
			{
				branch = 0;
				return false;
			}
			else
			{
				branch = _branches[^1];
				_branches.RemoveAt(_branches.Count - 1);

				if (_transform != null)
				{
					if (_transform.NewBranches.Count > 0)
					{
						_transform.NewBranches.RemoveAt(_transform.NewBranches.Count - 1);
					}
					else
					{
						_transform.RequiredBranches.Add(branch);
						_transform.RequireTopmostActive = null;
					}
				}

				return true;
			}
		}

		/// <summary>
		/// Determines if the branch that the preprocessor is in is active
		/// </summary>
		/// <returns>True if the branch is active, false otherwise</returns>
		public bool IsCurrentBranchActive()
		{
			bool active = (_branches.Count == 0 || _branches[^1].HasFlag(PreprocessorBranch.Active));
			if (_transform != null && _transform.NewBranches.Count == 0)
			{
				_transform.RequireTopmostActive = active;
			}
			return active;
		}

		/// <summary>
		/// Determines if the given transform can apply to the current preprocessor state
		/// </summary>
		/// <param name="transform">The transform to test</param>
		/// <returns>True if the transform can be applied to the current state</returns>
		public bool CanApply(PreprocessorTransform transform)
		{
			// Check all the required branches match
			for (int idx = 0; idx < transform.RequiredBranches.Count; idx++)
			{
				if (_branches[_branches.Count - idx - 1] != transform.RequiredBranches[idx])
				{
					return false;
				}
			}

			// Check the topmost branch is active
			if (transform.RequireTopmostActive.HasValue)
			{
				bool topmostActive = _branches.Count == transform.RequiredBranches.Count || _branches[_branches.Count - transform.RequiredBranches.Count - 1].HasFlag(PreprocessorBranch.Active);
				if (transform.RequireTopmostActive.Value != topmostActive)
				{
					return false;
				}
			}

			// Check all the required macros match
			foreach (KeyValuePair<Identifier, PreprocessorMacro?> requiredPair in transform.RequiredMacros)
			{
				if (_nameToMacro.TryGetValue(requiredPair.Key, out PreprocessorMacro? macro))
				{
					if (requiredPair.Value == null || !macro.IsEquivalentTo(requiredPair.Value))
					{
						return false;
					}
				}
				else
				{
					if (requiredPair.Value != null)
					{
						return false;
					}
				}
			}

			return true;
		}

		/// <summary>
		/// Apply the given transform to the current preprocessor state
		/// </summary>
		/// <param name="transform">The transform to apply</param>
		/// <returns>True if the transform was applied to the current state</returns>
		public bool TryToApply(PreprocessorTransform transform)
		{
			if (!CanApply(transform))
			{
				return false;
			}

			// Update the branch state
			_branches.RemoveRange(_branches.Count - transform.RequiredBranches.Count, transform.RequiredBranches.Count);
			_branches.AddRange(transform.NewBranches);

			// Update the macro definitions
			foreach (KeyValuePair<Identifier, PreprocessorMacro?> newMacro in transform.NewMacros)
			{
				if (newMacro.Value == null)
				{
					_nameToMacro.Remove(newMacro.Key);
				}
				else
				{
					_nameToMacro[newMacro.Key] = newMacro.Value;
				}
			}

			return true;
		}
	}
}
