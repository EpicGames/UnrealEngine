// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using EpicGames.Core;

namespace UnrealBuildTool
{
	/// <summary>
	/// Tracks dependencies for transforming the current preprocessor state
	/// </summary>
	class PreprocessorTransform
	{
		/// <summary>
		/// Whether the preprocessor must be in an active state after the branches below are popped. This cannot be treated as a 'pop', because it might be the global scope.
		/// </summary>
		public bool? RequireTopmostActive { get; set; } = null;

		/// <summary>
		/// List of branch states that will be popped from the stack
		/// </summary>
		public readonly List<PreprocessorBranch> RequiredBranches;

		/// <summary>
		/// Map of macro name to their required value
		/// </summary>
		public readonly Dictionary<Identifier, PreprocessorMacro?> RequiredMacros;

		/// <summary>
		/// List of new branches that will be pushed onto the stack
		/// </summary>
		public readonly List<PreprocessorBranch> NewBranches;

		/// <summary>
		/// List of new macros that will be defined
		/// </summary>
		public readonly Dictionary<Identifier, PreprocessorMacro?> NewMacros;

		/// <summary>
		/// This fragment contains a pragma once directive
		/// </summary>
		public bool HasPragmaOnce { get; set; } = false;

		/// <summary>
		/// Constructor
		/// </summary>
		public PreprocessorTransform()
		{
			RequiredBranches = new List<PreprocessorBranch>();
			RequiredMacros = new Dictionary<Identifier, PreprocessorMacro?>();
			NewBranches = new List<PreprocessorBranch>();
			NewMacros = new Dictionary<Identifier, PreprocessorMacro?>();
		}

		/// <summary>
		/// Reads a transform from disk
		/// </summary>
		/// <param name="reader">Archive to serialize from</param>
		public PreprocessorTransform(BinaryArchiveReader reader)
		{
			if (reader.ReadBool())
			{
				RequireTopmostActive = reader.ReadBool();
			}

			RequiredBranches = reader.ReadList(() => (PreprocessorBranch)reader.ReadByte())!;
			RequiredMacros = reader.ReadDictionary(() => reader.ReadIdentifier(), () => (PreprocessorMacro?)reader.ReadObjectReference(() => new PreprocessorMacro(reader)))!;
			NewBranches = reader.ReadList(() => (PreprocessorBranch)reader.ReadByte())!;
			NewMacros = reader.ReadDictionary(() => reader.ReadIdentifier(), () => (PreprocessorMacro?)reader.ReadObjectReference(() => new PreprocessorMacro(reader)))!;
			HasPragmaOnce = reader.ReadBool();
		}

		/// <summary>
		/// Writes a transform to disk
		/// </summary>
		/// <param name="writer">Archive to serialize to</param>
		public void Write(BinaryArchiveWriter writer)
		{
			writer.WriteBool(RequireTopmostActive.HasValue);
			if (RequireTopmostActive.HasValue)
			{
				writer.WriteBool(RequireTopmostActive.Value);
			}

			writer.WriteList(RequiredBranches, x => writer.WriteByte((byte)x));
			writer.WriteDictionary(RequiredMacros, k => writer.WriteIdentifier(k), v => writer.WriteObjectReference(v, () => v!.Write(writer)));
			writer.WriteList(NewBranches, x => writer.WriteByte((byte)x));
			writer.WriteDictionary(NewMacros, k => writer.WriteIdentifier(k), v => writer.WriteObjectReference(v, () => v!.Write(writer)));
			writer.WriteBool(HasPragmaOnce);
		}
	}
}
