// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.IO;
using System.Text;
using EpicGames.Core;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// Encapsulates metadata for a C++ source file
	/// </summary>
	class SourceFile
	{
		/// <summary>
		/// The corresponding file item
		/// </summary>
		public readonly FileItem File;

		/// <summary>
		/// Hash of the source file data
		/// </summary>
		public readonly ContentHash Hash;

		/// <summary>
		/// Last write time of this file when constructed
		/// </summary>
		public readonly long LastWriteTimeUtc;

		/// <summary>
		/// Markup dividing up this source file
		/// </summary>
		public readonly SourceFileMarkup[] Markup;

		/// <summary>
		/// Macro which is defined as part of the header guard. If this macro is defined, we don't need to include the header more than once.
		/// </summary>
		public Identifier? HeaderGuardMacro { get; private set; }

		/// <summary>
		/// Parsed fragments in this file. Only set for files which are being optimized.
		/// </summary>
		public readonly SourceFileFragment[] Fragments;

		/// <summary>
		/// Gets the location of this file
		/// </summary>
		public FileReference Location => File.Location;

		/// <summary>
		/// Construct a SourceFile from a file on disk
		/// </summary>
		/// <param name="file">Location of the file</param>
		public SourceFile(FileItem file)
			: this(file, ReadFileWithNullTerminator(file.Location))
		{
			LastWriteTimeUtc = file.LastWriteTimeUtc.Ticks;
		}

		/// <summary>
		/// Construct a SourceFile from a raw byte array
		/// </summary>
		/// <param name="file">Location of the file</param>
		/// <param name="data">Contents of the file</param>
		public SourceFile(FileItem file, byte[] data)
		{
			File = file;

			// Compute the hash
			Hash = ContentHash.MD5(data);

			// Read the preprocessor markup
			TokenReader reader = new(data);
			Markup = SourceFileMarkup.Parse(reader);

			// Try to parse a header guard from this file
			ParseHeaderGuard();

			// Create the fragments
			List<SourceFileFragment> fragmentsList = new();
			for (int maxIdx = 0; maxIdx < Markup.Length; maxIdx++)
			{
				if (Markup[maxIdx].Type != SourceFileMarkupType.Include)
				{
					int minIdx = maxIdx++;
					while (maxIdx < Markup.Length && Markup[maxIdx].Type != SourceFileMarkupType.Include)
					{
						maxIdx++;
					}
					fragmentsList.Add(new SourceFileFragment(this, minIdx, maxIdx));
				}
			}
			Fragments = fragmentsList.ToArray();
		}

		/// <summary>
		/// Read a source file from a binary reader
		/// </summary>
		/// <param name="reader">The reader to serialize from</param>
		public SourceFile(BinaryArchiveReader reader)
		{
			File = reader.ReadFileItem()!;
			Hash = reader.ReadContentHash()!;
			LastWriteTimeUtc = reader.ReadLong();
			Markup = reader.ReadArray(() => new SourceFileMarkup(reader))!;
			HeaderGuardMacro = reader.ReadIdentifier();
			Fragments = reader.ReadArray(() => new SourceFileFragment(this, reader))!;
		}

		/// <summary>
		/// Read a source file from a binary reader
		/// </summary>
		/// <param name="writer">The reader to serialize to</param>
		public void Write(BinaryArchiveWriter writer)
		{
			writer.WriteFileItem(File);
			writer.WriteContentHash(Hash);
			writer.WriteLong(LastWriteTimeUtc);
			writer.WriteArray(Markup, x => x.Write(writer));
			writer.WriteIdentifier(HeaderGuardMacro);
			writer.WriteArray(Fragments, x => x.Write(writer));
		}

		/// <summary>
		/// Reads the contents of a file into a buffer, with a null terminator
		/// </summary>
		/// <param name="location">Location of the file</param>
		/// <returns>Array of bytes, with a null terminator</returns>
		public static byte[] ReadFileWithNullTerminator(FileReference location)
		{
			using FileStream stream = FileReference.Open(location, FileMode.Open, FileAccess.Read, FileShare.Read);
			int length = (int)stream.Length;

			byte[] data = new byte[length + 1];
			stream.ReadExactly(data, 0, length);

			return ConvertToNullTerminatedUtf8(data, length);
		}

		/// <summary>
		/// Converts a buffer from a given encoding to UTF8, appending a null terminator
		/// </summary>
		/// <param name="data">The source data</param>
		/// <param name="length">Length of the data in the source buffer</param>
		/// <returns>Array of utf-8 encoded characters, with a null terminator</returns>
		public static byte[] ConvertToNullTerminatedUtf8(byte[] data, int length)
		{
			if (length >= 2)
			{
				if (data[0] == 0xfe && data[1] == 0xff)
				{
					return ConvertToNullTerminatedUtf8(Encoding.BigEndianUnicode, data, length);
				}
				else if (data[0] == 0xff && data[1] == 0xfe)
				{
					return ConvertToNullTerminatedUtf8(Encoding.Unicode, data, length);
				}
			}
			return data;
		}

		/// <summary>
		/// Converts a buffer from a given encoding to UTF8, appending a null terminator
		/// </summary>
		/// <param name="sourceEncoding">The encoding to convert from</param>
		/// <param name="data">The source data</param>
		/// <param name="length">Length of the data in the source buffer</param>
		/// <returns>Array of utf-8 encoded characters, with a null terminator</returns>
		public static byte[] ConvertToNullTerminatedUtf8(Encoding sourceEncoding, byte[] data, int length)
		{
			byte[] newBytes = data;
			if (sourceEncoding is not UTF8Encoding)
			{
				char[] chars = sourceEncoding.GetChars(data, 0, length);

				int newLength = Encoding.UTF8.GetByteCount(chars);
				newBytes = new byte[newLength + 1];

				Encoding.UTF8.GetBytes(chars, 0, chars.Length, newBytes, 0);
			}
			return newBytes;
		}

		/// <summary>
		/// Checks whether the markup matches a C-style header guard
		/// </summary>
		void ParseHeaderGuard()
		{
			// Make sure there are enough markup entries in this list
			if (Markup.Length >= 3)
			{
				int minIdx = 0;

				// Get the define used for the header guard
				Identifier? possibleHeaderGuardMacro;
				if (Markup[minIdx].Type == SourceFileMarkupType.Ifndef && Markup[minIdx].Tokens!.Count == 1 && Markup[minIdx].Tokens![0].Type == TokenType.Identifier)
				{
					possibleHeaderGuardMacro = Markup[minIdx].Tokens![0].Identifier!;
				}
				else if (Markup[minIdx].Type == SourceFileMarkupType.If && Markup[minIdx].Tokens!.Count == 3 && Markup[minIdx].Tokens![0].Type == TokenType.LogicalNot && Markup[minIdx].Tokens![1].Identifier == Identifiers.Defined && Markup[minIdx].Tokens![2].Type == TokenType.Identifier)
				{
					possibleHeaderGuardMacro = Markup[minIdx].Tokens![2].Identifier!;
				}
				else
				{
					possibleHeaderGuardMacro = null;
				}

				// Check it starts with an #if or #ifdef
				if (Markup[minIdx + 1].Type == SourceFileMarkupType.Define && Markup[minIdx + 1].Tokens!.Count == 1 && Markup[minIdx + 1].Tokens![0].Identifier == possibleHeaderGuardMacro)
				{
					// Find the point at which the include depth goes back to zero
					int maxIdx = minIdx + 1;
					for (int depth = 1; maxIdx < Markup.Length; maxIdx++)
					{
						depth += Markup[maxIdx].GetConditionDepthDelta();
						if (depth == 0)
						{
							break;
						}
					}

					// Check it matched the end of the file
					if (maxIdx == Markup.Length - 1)
					{
						HeaderGuardMacro = possibleHeaderGuardMacro;
					}
				}
			}
		}

		/// <summary>
		/// Converts a source file to a string for debugging
		/// </summary>
		/// <returns>The full path to the file</returns>
		public override string ToString()
		{
			return File.ToString();
		}
	}
}
