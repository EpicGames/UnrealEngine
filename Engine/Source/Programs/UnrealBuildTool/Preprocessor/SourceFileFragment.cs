// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using EpicGames.Core;

namespace UnrealBuildTool
{
	/// <summary>
	/// Represents an fragment of a source file consisting of a sequence of includes followed by some arbitrary directives or source code.
	/// </summary>
	class SourceFileFragment
	{
		/// <summary>
		/// The file that this fragment is part of
		/// </summary>
		public readonly SourceFile File;

		/// <summary>
		/// Index into the file's markup array of the start of this fragment (inclusive)
		/// </summary>
		public readonly int MarkupMin;

		/// <summary>
		/// Index into the file's markup array of the end of this fragment (exclusive). Set to zero for external files.
		/// </summary>
		public readonly int MarkupMax;

		/// <summary>
		/// List of known transforms for this fragment
		/// </summary>
		public PreprocessorTransform[] Transforms { get; set; } = Array.Empty<PreprocessorTransform>();

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="file">The source file that this fragment is part of</param>
		/// <param name="markupMin">Index into the file's markup array of the start of this fragment (inclusive)</param>
		/// <param name="markupMax">Index into the file's markup array of the end of this fragment (exclusive)</param>
		public SourceFileFragment(SourceFile file, int markupMin, int markupMax)
		{
			File = file;
			MarkupMin = markupMin;
			MarkupMax = markupMax;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="file">The source file that this fragment is part of</param>
		/// <param name="reader">Reader to deserialize this object from</param>
		public SourceFileFragment(SourceFile file, BinaryArchiveReader reader)
		{
			File = file;
			MarkupMin = reader.ReadInt();
			MarkupMax = reader.ReadInt();
			Transforms = reader.ReadArray(() => new PreprocessorTransform(reader))!;
		}

		/// <summary>
		/// Write this fragment to an archive
		/// </summary>
		/// <param name="writer">The writer to serialize to</param>
		public void Write(BinaryArchiveWriter writer)
		{
			writer.WriteInt(MarkupMin);
			writer.WriteInt(MarkupMax);
			writer.WriteArray(Transforms, x => x.Write(writer));
		}

		/// <summary>
		/// Summarize this fragment for the debugger
		/// </summary>
		/// <returns>String representation of this fragment for the debugger</returns>
		public override string ToString()
		{
			if (MarkupMax == 0)
			{
				return File.ToString();
			}
			else
			{
				return String.Format("{0}: {1}-{2}", File.ToString(), File.Markup[MarkupMin].LineNumber, File.Markup[MarkupMax - 1].LineNumber + 1);
			}
		}
	}
}
