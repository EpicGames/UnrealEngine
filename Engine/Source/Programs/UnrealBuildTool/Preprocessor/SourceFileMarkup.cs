// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text;
using EpicGames.Core;

namespace UnrealBuildTool
{
	/// <summary>
	/// Possible types of source file markup
	/// </summary>
	enum SourceFileMarkupType
	{
		/// <summary>
		/// A span of tokens which are not preprocessor directives
		/// </summary>
		Text,

		/// <summary>
		/// An #include directive
		/// </summary>
		Include,

		/// <summary>
		/// A #define directive
		/// </summary>
		Define,

		/// <summary>
		/// An #undef directive
		/// </summary>
		Undef,

		/// <summary>
		/// An #if directive
		/// </summary>
		If,

		/// <summary>
		/// An #ifdef directive
		/// </summary>
		Ifdef,

		/// <summary>
		/// An #ifndef directive
		/// </summary>
		Ifndef,

		/// <summary>
		/// An #elif directive
		/// </summary>
		Elif,

		/// <summary>
		/// An #else directive
		/// </summary>
		Else,

		/// <summary>
		/// An #endif directive
		/// </summary>
		Endif,

		/// <summary>
		/// A #pragma directive
		/// </summary>
		Pragma,

		/// <summary>
		/// An #error directive
		/// </summary>
		Error,

		/// <summary>
		/// A #warning directive
		/// </summary>
		Warning,

		/// <summary>
		/// An empty '#' on a line of its own
		/// </summary>
		Empty,

		/// <summary>
		/// Some other directive
		/// </summary>
		OtherDirective
	}

	/// <summary>
	/// Base class for an annotated section of a source file
	/// </summary>
	[Serializable]
	class SourceFileMarkup
	{
		/// <summary>
		/// The directive corresponding to this markup
		/// </summary>
		public readonly SourceFileMarkupType Type;

		/// <summary>
		/// The one-based line number of this markup
		/// </summary>
		public readonly int LineNumber;

		/// <summary>
		/// The tokens parsed for this markup. Set for directives.
		/// </summary>
		public readonly List<Token>? Tokens;

		/// <summary>
		/// Construct the annotation with the given range
		/// </summary>
		/// <param name="type">The type of this directive</param>
		/// <param name="lineNumber">The line number of this markup</param>
		/// <param name="tokens">List of tokens</param>
		public SourceFileMarkup(SourceFileMarkupType type, int lineNumber, List<Token>? tokens)
		{
			Type = type;
			LineNumber = lineNumber;
			Tokens = tokens;
		}

		/// <summary>
		/// Constructs a markup object using data read from an archive
		/// </summary>
		/// <param name="reader">The reader to deserialize from</param>
		public SourceFileMarkup(BinaryArchiveReader reader)
		{
			Type = (SourceFileMarkupType)reader.ReadByte();
			LineNumber = reader.ReadInt();
			Tokens = reader.ReadList(() => reader.ReadToken());
		}

		/// <summary>
		/// Serializes this object to a binary archive
		/// </summary>
		/// <param name="writer">Writer to serialize to</param>
		public void Write(BinaryArchiveWriter writer)
		{
			writer.WriteByte((byte)Type);
			writer.WriteInt(LineNumber);
			writer.WriteList(Tokens, x => writer.WriteToken(x));
		}

		/// <summary>
		/// Determines if this markup indicates a conditional preprocessor directive
		/// </summary>
		/// <returns>True if this object is a conditional preprocessor directive</returns>
		public bool IsConditionalPreprocessorDirective()
		{
			return Type switch
			{
				SourceFileMarkupType.If or SourceFileMarkupType.Ifdef or SourceFileMarkupType.Ifndef or SourceFileMarkupType.Elif or SourceFileMarkupType.Else or SourceFileMarkupType.Endif => true,
				_ => false,
			};
		}

		/// <summary>
		/// How this condition modifies the condition depth. Opening "if" statements have a value of +1, "endif" statements have a value of -1, and "else" statements have a value of 0.
		/// </summary>
		public int GetConditionDepthDelta()
		{
			if (Type == SourceFileMarkupType.If || Type == SourceFileMarkupType.Ifdef || Type == SourceFileMarkupType.Ifndef)
			{
				return +1;
			}
			else if (Type == SourceFileMarkupType.Endif)
			{
				return -1;
			}
			else
			{
				return 0;
			}
		}

		/// <summary>
		/// Generate a string describing this annotation
		/// </summary>
		/// <returns>String representation for debugging</returns>
		public override string ToString()
		{
			StringBuilder result = new();
			result.AppendFormat("[{0}] ", LineNumber);

			if (Type == SourceFileMarkupType.Text)
			{
				result.Append("...");
			}
			else
			{
				result.Append('#');
				if (Type != SourceFileMarkupType.OtherDirective)
				{
					result.Append(Type.ToString().ToUpperInvariant());
				}
				if (Tokens != null && Tokens.Count > 0)
				{
					result.Append(' ');
					Token.Format(Tokens, result);
				}
			}
			return result.ToString();
		}

		/// <summary>
		/// Create markup for the given file
		/// </summary>
		/// <param name="reader">Reader for tokens in the file</param>
		/// <returns>Array of markup objects which split up the given text buffer</returns>
		public static SourceFileMarkup[] Parse(TokenReader reader)
		{
			List<SourceFileMarkup> markup = new();
			if (reader.MoveNext())
			{
				bool moveNext = true;
				while (moveNext)
				{
					int startLineNumber = reader.LineNumber;
					if (reader.Current.Type == TokenType.Hash)
					{
						// Create the appropriate markup object for the directive
						SourceFileMarkupType type = SourceFileMarkupType.OtherDirective;
						if (reader.MoveNext())
						{
							if (reader.Current.Type == TokenType.Identifier)
							{
								Identifier directive = reader.Current.Identifier!;
								if (directive == Identifiers.Include)
								{
									type = SourceFileMarkupType.Include;
								}
								else if (directive == Identifiers.Define)
								{
									type = SourceFileMarkupType.Define;
								}
								else if (directive == Identifiers.Undef)
								{
									type = SourceFileMarkupType.Undef;
								}
								else if (directive == Identifiers.If)
								{
									type = SourceFileMarkupType.If;
								}
								else if (directive == Identifiers.Ifdef)
								{
									type = SourceFileMarkupType.Ifdef;
								}
								else if (directive == Identifiers.Ifndef)
								{
									type = SourceFileMarkupType.Ifndef;
								}
								else if (directive == Identifiers.Elif)
								{
									type = SourceFileMarkupType.Elif;
								}
								else if (directive == Identifiers.Else)
								{
									type = SourceFileMarkupType.Else;
								}
								else if (directive == Identifiers.Endif)
								{
									type = SourceFileMarkupType.Endif;
								}
								else if (directive == Identifiers.Pragma)
								{
									type = SourceFileMarkupType.Pragma;
								}
								else if (directive == Identifiers.Error)
								{
									type = SourceFileMarkupType.Error;
								}
								else if (directive == Identifiers.Warning)
								{
									type = SourceFileMarkupType.Warning;
								}
							}
							else if (reader.Current.Type == TokenType.Newline)
							{
								type = SourceFileMarkupType.Empty;
							}
						}

						// Create the token list
						List<Token> tokens = new();
						if (type == SourceFileMarkupType.OtherDirective)
						{
							tokens.Add(reader.Current);
						}

						// Read the first token
						if (type == SourceFileMarkupType.Empty)
						{
							moveNext = true;
						}
						else if (type == SourceFileMarkupType.Include)
						{
							moveNext = reader.MoveNextIncludePath();
						}
						else if (type == SourceFileMarkupType.Error || type == SourceFileMarkupType.Warning)
						{
							moveNext = reader.MoveNextTokenString();
						}
						else
						{
							moveNext = reader.MoveNext();
						}

						// Read the rest of the tokens
						while (moveNext && reader.Current.Type != TokenType.Newline)
						{
							tokens.Add(reader.Current);
							moveNext = reader.MoveNext();
						}

						// Create the markup
						markup.Add(new SourceFileMarkup(type, startLineNumber, tokens));

						// Move to the next token
						moveNext = reader.MoveNext();
					}
					else if (reader.Current.Type != TokenType.Newline)
					{
						// Create the new fragment
						markup.Add(new SourceFileMarkup(SourceFileMarkupType.Text, startLineNumber, null));

						// Move to the next directive
						moveNext = reader.MoveToNextDirective();
					}
					else
					{
						// Skip the empty line
						moveNext = reader.MoveNext();
					}
				}
			}
			return markup.ToArray();
		}
	}
}
