// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.Linq;
using System.Text;
using EpicGames.Core;

namespace UnrealBuildTool
{
	/// <summary>
	/// Flags for a single token
	/// </summary>
	[Flags]
	enum TokenFlags : byte
	{
		/// <summary>
		/// No flags
		/// </summary>
		None = 0x00,

		/// <summary>
		/// The token has space before it
		/// </summary>
		HasLeadingSpace = 0x01,

		/// <summary>
		/// Do not replace any instances of this token with the corresponding macro.
		/// </summary>
		DisableExpansion = 0x02,
	}

	/// <summary>
	/// Enumeration for token types
	/// </summary>
	enum TokenType : byte
	{
		End,
		LeftParen,
		RightParen,
		Comma,
		Identifier,
		Number,
		String,
		Character,
		Newline,
		Ellipsis,
		StringOfTokens,
		SystemInclude,
		Placemarker,
		Dot,
		QuestionMark,
		Colon,
		LogicalNot,
		LogicalAnd,
		LogicalOr,
		BitwiseXor,
		BitwiseAnd,
		BitwiseNot,
		BitwiseOr,
		Equals,
		LeftShift,
		RightShift,
		CompareEqual,
		CompareNotEqual,
		CompareLessOrEqual,
		CompareLess,
		CompareGreaterOrEqual,
		CompareGreater,
		Plus,
		Minus,
		Multiply,
		Divide,
		Modulo,
		Hash,
		HashHash,
		Unknown,
		Max
	}

	/// <summary>
	/// Single lexical token
	/// </summary>
	[DebuggerDisplay("{Text}")]
	sealed class Token : IEquatable<Token>
	{
		/// <summary>
		/// Static token names
		/// </summary>
		static readonly string[] s_staticTokens;

		/// <summary>
		/// Contains the type of the token
		/// </summary>
		public readonly TokenType Type;

		/// <summary>
		/// Properties for this token
		/// </summary>
		public readonly TokenFlags Flags;

		/// <summary>
		/// If this token is identifier, contains the identifier name
		/// </summary>
		public readonly Identifier? Identifier;

		/// <summary>
		/// If this token is a literal, contains the raw utf-8 representation of it.
		/// </summary>
		public readonly byte[]? Literal;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="type">The token type</param>
		/// <param name="flags">Flags for this token</param>
		public Token(TokenType type, TokenFlags flags)
		{
			Type = type;
			Flags = flags;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="identifier">The token identifier</param>
		/// <param name="flags">Flags for this token</param>
		public Token(Identifier identifier, TokenFlags flags)
		{
			Type = TokenType.Identifier;
			Flags = flags;
			Identifier = identifier;
		}

		/// <summary>
		/// Constructs a literal token
		/// </summary>
		public Token(TokenType type, TokenFlags flags, byte[] literal)
			: this(type, flags)
		{
			Debug.Assert(IsLiteral(type));
			Literal = literal;
		}

		/// <summary>
		/// Constructs a literal token
		/// </summary>
		public Token(TokenType type, TokenFlags flags, string literal)
			: this(type, flags)
		{
			Debug.Assert(IsLiteral(type));
			Literal = Encoding.UTF8.GetBytes(literal);
		}

		/// <summary>
		/// Reads a token from a binary archive
		/// </summary>
		/// <param name="reader">Archive to read from</param>
		public Token(BinaryArchiveReader reader)
		{
			Type = (TokenType)reader.ReadByte();
			Flags = (TokenFlags)reader.ReadByte();
			if (Type == TokenType.Identifier)
			{
				Identifier = reader.ReadIdentifier();
			}
			else if (IsLiteral(Type))
			{
				Literal = reader.ReadByteArray();
			}
		}

		/// <summary>
		/// Writes a token to a binary archive
		/// </summary>
		/// <param name="writer">The writer to serialize to</param>
		public void Write(BinaryArchiveWriter writer)
		{
			writer.WriteByte((byte)Type);
			writer.WriteByte((byte)Flags);
			if (Type == TokenType.Identifier)
			{
				writer.WriteIdentifier(Identifier!);
			}
			else if (IsLiteral(Type))
			{
				writer.WriteByteArray(Literal);
			}
		}

		/// <summary>
		/// Checks if a token is equal to another object
		/// </summary>
		/// <param name="other">The object to compare against</param>
		/// <returns>True if the objects are equal, false otherwise</returns>
		public override bool Equals(object? other)
		{
			return Equals(other as Token);
		}

		/// <summary>
		/// Checks if two tokens are equivalent
		/// </summary>
		/// <param name="other">The object to compare against</param>
		/// <returns>True if the tokens are equal, false otherwise</returns>
		public bool Equals(Token? other)
		{
			if (other is null)
			{
				return false;
			}
			if (Type != other.Type || Flags != other.Flags || Identifier != other.Identifier)
			{
				return false;
			}
			if (Literal != null)
			{
				if (other.Literal == null || Literal.Length != other.Literal.Length || !Enumerable.SequenceEqual(Literal, other.Literal))
				{
					return false;
				}
			}
			else
			{
				if (other.Literal != null)
				{
					return false;
				}
			}
			return true;
		}

		/// <summary>
		/// Compares two tokens for equality
		/// </summary>
		/// <param name="lhs">The first token to compare</param>
		/// <param name="rhs">The second token to compare</param>
		/// <returns>True if the objects are equal, false otherwise</returns>
		public static bool operator ==(Token lhs, Token rhs)
		{
			if (lhs is null)
			{
				return (rhs is null);
			}
			else
			{
				return lhs.Equals(rhs);
			}
		}

		/// <summary>
		/// Compares two tokens for inequality
		/// </summary>
		/// <param name="lhs">The first token to compare</param>
		/// <param name="rhs">The second token to compare</param>
		/// <returns>True if the objects are not equal, false otherwise</returns>
		public static bool operator !=(Token lhs, Token rhs)
		{
			return !(lhs == rhs);
		}

		/// <summary>
		/// Gets a hash code for this object
		/// </summary>
		/// <returns>Hash code for the object</returns>
		public override int GetHashCode()
		{
			int result = (int)Type + (int)Flags * 7;
			if (Identifier != null)
			{
				result = (result * 11) + Identifier.GetHashCode();
			}
			if (Literal != null)
			{
				for (int idx = 0; idx < Literal.Length; idx++)
				{
					result = (result * 13) + Literal[idx];
				}
			}
			return base.GetHashCode();
		}

		/// <summary>
		/// Text corresponding to this token
		/// </summary>
		public string Text
		{
			get
			{
				if (Identifier != null)
				{
					return Identifier.ToString();
				}
				else if (Literal != null)
				{
					return Encoding.UTF8.GetString(Literal);
				}
				else
				{
					return s_staticTokens[(int)Type];
				}
			}
		}

		/// <summary>
		/// Returns a new token with different flags
		/// </summary>
		/// <param name="flagsToAdd">Flags to add from the token</param>
		/// <returns>New token with updated flags</returns>
		public Token AddFlags(TokenFlags flagsToAdd)
		{
			if (Identifier != null)
			{
				return new Token(Identifier, Flags | flagsToAdd);
			}
			else if (Literal != null)
			{
				return new Token(Type, Flags | flagsToAdd, Literal);
			}
			else
			{
				return new Token(Type, Flags | flagsToAdd);
			}
		}

		/// <summary>
		/// Returns a new token with different flags
		/// </summary>
		/// <param name="flagsToRemove">Flags to remove from the token</param>
		/// <returns>New token with updated flags</returns>
		public Token RemoveFlags(TokenFlags flagsToRemove)
		{
			if (Identifier != null)
			{
				return new Token(Identifier, Flags & ~flagsToRemove);
			}
			else if (Literal != null)
			{
				return new Token(Type, Flags & ~flagsToRemove, Literal);
			}
			else
			{
				return new Token(Type, Flags & ~flagsToRemove);
			}
		}

		/// <summary>
		/// Accessor for whether this token has leading whitespace
		/// </summary>
		public bool HasLeadingSpace => (Flags & TokenFlags.HasLeadingSpace) != 0;

		/// <summary>
		/// Checks whether two tokens match
		/// </summary>
		/// <param name="other">The token to compare against</param>
		/// <returns>True if the tokens match, false otherwise</returns>
		public bool Matches(Token other)
		{
			if (Type != other.Type)
			{
				return false;
			}

			if (Literal == null)
			{
				return Identifier == other.Identifier;
			}
			else
			{
				return other.Literal != null && Enumerable.SequenceEqual(Literal, other.Literal);
			}
		}

		/// <summary>
		/// Determines whether the given token type is a literal
		/// </summary>
		/// <param name="type">The type to test</param>
		/// <returns>True if the given type is a literal</returns>
		public static bool IsLiteral(TokenType type)
		{
			return type == TokenType.Unknown || type == TokenType.Character || type == TokenType.String || type == TokenType.Number || type == TokenType.StringOfTokens || type == TokenType.SystemInclude;
		}

		/// <summary>
		/// Concatenate a sequence of tokens into a string
		/// </summary>
		/// <param name="tokens">The sequence of tokens to concatenate</param>
		/// <returns>String containing the concatenated tokens</returns>
		public static string Format(IEnumerable<Token> tokens)
		{
			StringBuilder result = new();
			Format(tokens, result);
			return result.ToString();
		}

		/// <summary>
		/// Concatenate a sequence of tokens into a string
		/// </summary>
		/// <param name="tokens">The sequence of tokens to concatenate</param>
		/// <param name="result">Receives the formatted string</param>
		public static void Format(IEnumerable<Token> tokens, StringBuilder result)
		{
			IEnumerator<Token> enumerator = tokens.GetEnumerator();
			if (enumerator.MoveNext())
			{
				result.Append(enumerator.Current.Text);

				Token lastToken = enumerator.Current;
				while (enumerator.MoveNext())
				{
					Token token = enumerator.Current;
					if (token.HasLeadingSpace && (token.Type != TokenType.LeftParen || lastToken.Type != TokenType.Identifier || lastToken.Identifier != Identifiers.__pragma))
					{
						result.Append(' ');
					}
					result.Append(token.Text);
					lastToken = token;
				}
			}
		}

		/// <summary>
		/// Concatenate two tokens together
		/// </summary>
		/// <param name="firstToken">The first token</param>
		/// <param name="secondToken">The second token</param>
		/// <param name="context">Current preprocessor context</param>
		/// <returns>The combined token</returns>
		[System.Diagnostics.CodeAnalysis.SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "<Pending>")]
		public static Token Concatenate(Token firstToken, Token secondToken, PreprocessorContext context)
		{
			string text = firstToken.Text.ToString() + secondToken.Text.ToString();

			List<Token> tokens = new();

			TokenReader reader = new(text);
			while (reader.MoveNext())
			{
				tokens.Add(reader.Current);
			}

			if (tokens.Count == 0)
			{
				return new Token(TokenType.Placemarker, TokenFlags.None);
			}
			else if (tokens.Count == 1)
			{
				return tokens[0];
			}
			else
			{
				return new Token(TokenType.Unknown, TokenFlags.None, text);
			}
		}

#pragma warning disable CA1810 // Initialize reference type static fields inline
		/// <summary>
		/// Static constructor. Initializes the StaticTokens array.
		/// </summary>
		static Token()
		{
			s_staticTokens = new string[(int)TokenType.Max];
			s_staticTokens[(int)TokenType.LeftParen] = "(";
			s_staticTokens[(int)TokenType.RightParen] = ")";
			s_staticTokens[(int)TokenType.Comma] = ",";
			s_staticTokens[(int)TokenType.Newline] = "\n";
			s_staticTokens[(int)TokenType.Ellipsis] = "...";
			s_staticTokens[(int)TokenType.Placemarker] = "";
			s_staticTokens[(int)TokenType.Dot] = ".";
			s_staticTokens[(int)TokenType.QuestionMark] = "?";
			s_staticTokens[(int)TokenType.Colon] = ":";
			s_staticTokens[(int)TokenType.LogicalNot] = "!";
			s_staticTokens[(int)TokenType.LogicalAnd] = "&&";
			s_staticTokens[(int)TokenType.LogicalOr] = "||";
			s_staticTokens[(int)TokenType.BitwiseXor] = "^";
			s_staticTokens[(int)TokenType.BitwiseAnd] = "&";
			s_staticTokens[(int)TokenType.BitwiseNot] = "~";
			s_staticTokens[(int)TokenType.BitwiseOr] = "|";
			s_staticTokens[(int)TokenType.Equals] = "=";
			s_staticTokens[(int)TokenType.LeftShift] = "<<";
			s_staticTokens[(int)TokenType.RightShift] = ">>";
			s_staticTokens[(int)TokenType.CompareEqual] = "==";
			s_staticTokens[(int)TokenType.CompareNotEqual] = "!=";
			s_staticTokens[(int)TokenType.CompareLessOrEqual] = "<=";
			s_staticTokens[(int)TokenType.CompareLess] = "<";
			s_staticTokens[(int)TokenType.CompareGreaterOrEqual] = ">=";
			s_staticTokens[(int)TokenType.CompareGreater] = ">";
			s_staticTokens[(int)TokenType.Plus] = "+";
			s_staticTokens[(int)TokenType.Minus] = "-";
			s_staticTokens[(int)TokenType.Multiply] = "*";
			s_staticTokens[(int)TokenType.Divide] = "/";
			s_staticTokens[(int)TokenType.Modulo] = "%";
			s_staticTokens[(int)TokenType.Hash] = "#";
			s_staticTokens[(int)TokenType.HashHash] = "##";
		}
	}
#pragma warning restore CA1810

	/// <summary>
	/// Helper functions for serialization
	/// </summary>
	static class TokenExtensionMethods
	{
		/// <summary>
		/// Read a token from a binary archive
		/// </summary>
		/// <param name="reader">Reader to serialize data from</param>
		/// <returns>Token read from the archive</returns>
		public static Token ReadToken(this BinaryArchiveReader reader)
		{
			return new Token(reader);
		}

		/// <summary>
		/// Write a token to a binary archive
		/// </summary>
		/// <param name="writer">Writer to serialize data to</param>
		/// <param name="token">Token to write</param>
		public static void WriteToken(this BinaryArchiveWriter writer, Token token)
		{
			token.Write(writer);
		}
	}
}
