// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections;
using System.Collections.Generic;
using System.Text;

namespace UnrealBuildTool
{
	/// <summary>
	/// Tokenizer for C++ source files. Provides functions for navigating a source file skipping whitespace, comments, and so on when required.
	/// </summary>
	class TokenReader : IEnumerator<Token>
	{
		/// <summary>
		/// Flags used to classify the first character in a token
		/// </summary>
		enum FirstCharacterClass : byte
		{
			Unknown,
			Terminator,
			Whitespace,
			Newline,
			Identifier,
			Number,
			Character,
			String,
			Dot,
			QuestionMark,
			Colon,
			ExclamationMark,
			Pipe,
			Ampersand,
			Caret,
			Equals,
			LeftTriangleBracket,
			RightTriangleBracket,
			Plus,
			Minus,
			Star,
			Slash,
			PercentSign,
			Tilde,
			LeftParen,
			RightParen,
			Hash,
			Comma,
		}

		/// <summary>
		/// Flags used to classify subsequent characters in a token
		/// </summary>
		[Flags]
		enum CharacterFlags : byte
		{
			Identifier = 1,
			Digit = 2,
			NumberTail = 4,
			Whitespace = 8,
		}

		/// <summary>
		/// Map of utf-8 leading bytes to their class
		/// </summary>
		static readonly FirstCharacterClass[] s_firstCharacters;

		/// <summary>
		/// Array of flags for different leading utf-8 sequences
		/// </summary>
		static readonly CharacterFlags[] s_characters;

		/// <summary>
		/// The current buffer being read from. Encoded as UTF-8 with a null terminator.
		/// </summary>
		readonly byte[] _data;

		/// <summary>
		/// Current offset within the buffer
		/// </summary>
		int _offset;

		/// <summary>
		/// The current token
		/// </summary>
		Token? _currentToken;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="data">The raw byte stream to read from, encoded as UTF-8</param>
		public TokenReader(byte[] data)
		{
			_data = data;
			LineNumber = 0;
			LineNumberAfterToken = 1;

			// Make sure the input data has a null terminator
			if (data.Length == 0 || data[^1] != 0)
			{
				throw new ArgumentException("Data parameter must be null terminated.");
			}

			// If the data contains a UTF-8 BOM, skip over it
			if (data[0] == 0xef && data[1] == 0xbb && data[2] == 0xbf)
			{
				_offset = 3;
			}
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="text">The text to read from</param>
		public TokenReader(string text)
			: this(GetNullTerminatedByteArray(text))
		{
		}

		/// <summary>
		/// Copy constructor
		/// </summary>
		/// <param name="other">Token reader to copy from</param>
		public TokenReader(TokenReader other)
		{
			_data = other._data;
			_offset = other._offset;
			LineNumber = other.LineNumber;
			LineNumberAfterToken = other.LineNumberAfterToken;
		}

		/// <summary>
		/// Gets a null terminated byte array from the given string
		/// </summary>
		/// <param name="text">String to convert into bytes</param>
		/// <returns>Array of bytes</returns>
		public static byte[] GetNullTerminatedByteArray(string text)
		{
			byte[] bytes = new byte[Encoding.UTF8.GetByteCount(text) + 1];
			Encoding.UTF8.GetBytes(text, 0, text.Length, bytes, 0);
			return bytes;
		}

		/// <summary>
		/// Tokenize an input string
		/// </summary>
		/// <param name="text">Text to tokenize</param>
		/// <returns>List of tokens parsed from the text</returns>
		public static List<Token> Tokenize(string text)
		{
			List<Token> tokens = new();

			TokenReader reader = new(text);
			while (reader.MoveNext())
			{
				tokens.Add(reader.Current);
			}

			return tokens;
		}

		/// <summary>
		/// Current line number (one-based)
		/// </summary>
		public int LineNumber
		{
			get;
			private set;
		}

		/// <summary>
		/// Line number at the end of this token (one-based)
		/// </summary>
		public int LineNumberAfterToken
		{
			get;
			private set;
		}

		/// <summary>
		/// Returns the current token
		/// </summary>
		public Token Current => _currentToken!;

		/// <summary>
		/// Untyped implementation of Current for IEnumerator.
		/// </summary>
		object IEnumerator.Current => Current;

		/// <summary>
		/// Override of IEnumerator.Dispose. Not required.
		/// </summary>
		void IDisposable.Dispose()
		{
		}

		/// <summary>
		/// Move to the next token
		/// </summary>
		/// <returns>True if the reader could move to the next token, false otherwise</returns>
		public bool MoveNext()
		{
			// Skip past the leading whitespace
			TokenFlags flags = SkipWhitespace();

			// Update the line number to the one after the current token
			LineNumber = LineNumberAfterToken;

			// Capture the start of the token
			int startOffset = _offset;

			// Initialize the flags for the next token
			byte firstByte = _data[_offset++];
			switch (s_firstCharacters[firstByte])
			{
				case FirstCharacterClass.Unknown:
					if ((firstByte & 0x80) != 0)
					{
						while ((_data[_offset] & 0x80) != 0)
						{
							_offset++;
						}
					}
					_currentToken = CreateLiteral(TokenType.Unknown, flags, startOffset, _offset - startOffset);
					return true;
				case FirstCharacterClass.Terminator:
					_offset = startOffset;
					_currentToken = new Token(TokenType.End, flags);
					return false;
				case FirstCharacterClass.Newline:
					LineNumberAfterToken++;
					_currentToken = new Token(TokenType.Newline, flags);
					return true;
				case FirstCharacterClass.Identifier:
					// Identifier (or text literal with prefix)
					while ((s_characters[_data[_offset]] & CharacterFlags.Identifier) != 0)
					{
						_offset++;
					}

					// Check if it's a prefixed text literal
					if (_data[_offset] == '\'')
					{
						_offset++;
						SkipTextLiteral('\'');
						_currentToken = CreateLiteral(TokenType.Character, flags, startOffset, _offset - startOffset);
						return true;
					}
					else if (_data[_offset] == '\"')
					{
						_offset++;
						SkipTextLiteral('\"');
						_currentToken = CreateLiteral(TokenType.String, flags, startOffset, _offset - startOffset);
						return true;
					}
					else
					{
						string name = Encoding.UTF8.GetString(_data, startOffset, _offset - startOffset);
						Identifier identifier = Identifier.FindOrAdd(name);
						_currentToken = new Token(identifier, flags);
						return true;
					}
				case FirstCharacterClass.Number:
					// Numeric literal
					SkipNumericLiteral();
					_currentToken = CreateLiteral(TokenType.Number, flags, startOffset, _offset - startOffset);
					return true;
				case FirstCharacterClass.Character:
					// Character literal
					SkipTextLiteral('\'');
					_currentToken = CreateLiteral(TokenType.Character, flags, startOffset, _offset - startOffset);
					return true;
				case FirstCharacterClass.String:
					// String literal
					SkipTextLiteral('\"');
					_currentToken = CreateLiteral(TokenType.String, flags, startOffset, _offset - startOffset);
					return true;
				case FirstCharacterClass.Dot:
					// Numeric literal, ellipsis, or dot
					if ((s_characters[_data[_offset]] & CharacterFlags.Digit) != 0)
					{
						_offset++;
						SkipNumericLiteral();
						_currentToken = CreateLiteral(TokenType.Number, flags, startOffset, _offset - startOffset);
						return true;
					}
					else if (_data[_offset] == '.' && _data[_offset + 1] == '.')
					{
						_offset += 2;
						_currentToken = new Token(TokenType.Ellipsis, flags);
						return true;
					}
					else
					{
						_currentToken = new Token(TokenType.Dot, flags);
						return true;
					}
				case FirstCharacterClass.QuestionMark:
					_currentToken = new Token(TokenType.QuestionMark, flags);
					return true;
				case FirstCharacterClass.Colon:
					_currentToken = new Token(TokenType.Colon, flags);
					return true;
				case FirstCharacterClass.ExclamationMark:
					if (ReadCharacter('='))
					{
						_currentToken = new Token(TokenType.CompareNotEqual, flags);
					}
					else
					{
						_currentToken = new Token(TokenType.LogicalNot, flags);
					}
					return true;
				case FirstCharacterClass.Pipe:
					if (ReadCharacter('|'))
					{
						_currentToken = new Token(TokenType.LogicalOr, flags);
					}
					else
					{
						_currentToken = new Token(TokenType.BitwiseOr, flags);
					}
					return true;
				case FirstCharacterClass.Ampersand:
					if (ReadCharacter('&'))
					{
						_currentToken = new Token(TokenType.LogicalAnd, flags);
					}
					else
					{
						_currentToken = new Token(TokenType.BitwiseAnd, flags);
					}
					return true;
				case FirstCharacterClass.Caret:
					_currentToken = new Token(TokenType.BitwiseXor, flags);
					return true;
				case FirstCharacterClass.Equals:
					if (ReadCharacter('='))
					{
						_currentToken = new Token(TokenType.CompareEqual, flags);
					}
					else
					{
						_currentToken = new Token(TokenType.Equals, flags);
					}
					return true;
				case FirstCharacterClass.LeftTriangleBracket:
					if (ReadCharacter('<'))
					{
						_currentToken = new Token(TokenType.LeftShift, flags);
					}
					else if (ReadCharacter('='))
					{
						_currentToken = new Token(TokenType.CompareLessOrEqual, flags);
					}
					else
					{
						_currentToken = new Token(TokenType.CompareLess, flags);
					}
					return true;
				case FirstCharacterClass.RightTriangleBracket:
					if (ReadCharacter('>'))
					{
						_currentToken = new Token(TokenType.RightShift, flags);
					}
					else if (ReadCharacter('='))
					{
						_currentToken = new Token(TokenType.CompareGreaterOrEqual, flags);
					}
					else
					{
						_currentToken = new Token(TokenType.CompareGreater, flags);
					}
					return true;
				case FirstCharacterClass.Plus:
					_currentToken = new Token(TokenType.Plus, flags);
					return true;
				case FirstCharacterClass.Minus:
					_currentToken = new Token(TokenType.Minus, flags);
					return true;
				case FirstCharacterClass.Star:
					_currentToken = new Token(TokenType.Multiply, flags);
					return true;
				case FirstCharacterClass.Slash:
					_currentToken = new Token(TokenType.Divide, flags);
					return true;
				case FirstCharacterClass.PercentSign:
					_currentToken = new Token(TokenType.Modulo, flags);
					return true;
				case FirstCharacterClass.Tilde:
					_currentToken = new Token(TokenType.BitwiseNot, flags);
					return true;
				case FirstCharacterClass.LeftParen:
					_currentToken = new Token(TokenType.LeftParen, flags);
					return true;
				case FirstCharacterClass.RightParen:
					_currentToken = new Token(TokenType.RightParen, flags);
					return true;
				case FirstCharacterClass.Hash:
					if (ReadCharacter('#'))
					{
						_currentToken = new Token(TokenType.HashHash, flags);
					}
					else
					{
						_currentToken = new Token(TokenType.Hash, flags);
					}
					return true;
				case FirstCharacterClass.Comma:
					_currentToken = new Token(TokenType.Comma, flags);
					return true;
				default:
					throw new NotImplementedException();
			}
		}

		/// <summary>
		/// Move to the next token, parsing a single literal token string until the end of the line (in the form used by error directives)
		/// </summary>
		/// <returns>True if a token was parsed</returns>
		public bool MoveNextTokenString()
		{
			SkipWhitespace();

			int startOffset = _offset;
			while (_data[_offset] != 0 && _data[_offset] != '\n')
			{
				_offset++;
			}

			_currentToken = CreateLiteral(TokenType.StringOfTokens, TokenFlags.None, startOffset, _offset - startOffset);
			return true;
		}

		/// <summary>
		/// Move to the next token, allowing include directives in triangle brackets in the style permitted by system include directives
		/// </summary>
		/// <returns>True if a token was read</returns>
		public bool MoveNextIncludePath()
		{
			bool result = MoveNext();
			if (result && Current.Type == TokenType.CompareLess)
			{
				int startOffset = _offset - 1;
				for (int endOffset = _offset; _data[endOffset] != 0 && _data[endOffset] != '\n'; endOffset++)
				{
					if (_data[endOffset] == '>')
					{
						_currentToken = CreateLiteral(TokenType.SystemInclude, TokenFlags.None, startOffset, endOffset + 1 - startOffset);
						_offset = endOffset + 1;
						break;
					}
				}
			}
			return result;
		}

		/// <summary>
		/// Creates a literal token
		/// </summary>
		/// <param name="type">Type of token to create</param>
		/// <param name="flags">Flags for the token</param>
		/// <param name="offset">Offset of the literal within the source data stream</param>
		/// <param name="length">Length of the literal</param>
		/// <returns>New token for the literal</returns>
		private Token CreateLiteral(TokenType type, TokenFlags flags, int offset, int length)
		{
			byte[] literal = new byte[length];
			for (int idx = 0; idx < length; idx++)
			{
				literal[idx] = _data[offset + idx];
			}
			return new Token(type, flags, literal);
		}

		/// <summary>
		/// Scan ahead until we reach the next directive (a hash token after a newline). Assumes that the current line does not contain a directive.
		/// </summary>
		/// <returns>True if we were able to find another token</returns>
		public bool MoveToNextDirective()
		{
			// Scan lines until we reach a directive
			for (; ; )
			{
				// Move to the next newline
				while (_data[_offset] != '\n')
				{
					if (_data[_offset] == 0)
					{
						return false;
					}
					else if (_data[_offset] == '\\' && _data[_offset + 1] == '\r' && _data[_offset + 2] == '\n')
					{
						LineNumberAfterToken++;
						_offset += 3;
					}
					else if (_data[_offset] == '\\' && _data[_offset + 1] == '\n')
					{
						LineNumberAfterToken++;
						_offset += 2;
					}
					else if (_data[_offset] == '/' && _data[_offset + 1] == '*')
					{
						_offset += 2;
						for (; _data[_offset] != 0; _offset++)
						{
							if (_data[_offset] == '\n')
							{
								LineNumberAfterToken++;
								continue;
							}
							if (_data[_offset] == '*' && _data[_offset + 1] == '/')
							{
								_offset += 2;
								break;
							}
						}
					}
					else
					{
						_offset++;
					}
				}

				// Move past the newline
				_offset++;
				LineNumberAfterToken++;

				// Skip any horizontal whitespace
				TokenFlags flags = SkipWhitespace();

				// Check if this is a line marker
				if (_data[_offset] == '#' && _data[_offset + 1] != '#')
				{
					_offset++;
					LineNumber = LineNumberAfterToken;
					_currentToken = new Token(TokenType.Hash, flags);
					return true;
				}
			}
		}

		/// <summary>
		/// Definition of IEnumerator.Reset(). Not supported.
		/// </summary>
		void IEnumerator.Reset()
		{
			throw new NotSupportedException();
		}

		/// <summary>
		/// Attempts to read a given character from the stream
		/// </summary>
		/// <param name="character">Character to read</param>
		/// <returns>True if the character was read (and the current position was updated)</returns>
		bool ReadCharacter(char character)
		{
			if (_data[_offset] == character)
			{
				_offset++;
				return true;
			}
			return false;
		}

		/// <summary>
		/// Advances the given position past any horizontal whitespace or comments
		/// </summary>
		/// <returns>Flags for the following token</returns>
		TokenFlags SkipWhitespace()
		{
			TokenFlags flags = TokenFlags.None;
			for (; ; )
			{
				// Quickly skip over trivial whitespace
				while ((s_characters[_data[_offset]] & CharacterFlags.Whitespace) != 0)
				{
					_offset++;
					flags |= TokenFlags.HasLeadingSpace;
				}

				// Look at what's next
				char character = (char)_data[_offset];
				if (character == '\\')
				{
					byte nextCharacter = _data[_offset + 1];
					if (nextCharacter == '\r' && _data[_offset + 2] == '\n')
					{
						LineNumberAfterToken++;
						_offset += 3;
					}
					else if (nextCharacter == '\n')
					{
						LineNumberAfterToken++;
						_offset += 2;
					}
					else
					{
						break;
					}
				}
				else if (character == '/')
				{
					byte nextCharacter = _data[_offset + 1];
					if (nextCharacter == '/')
					{
						_offset += 2;
						for (; _data[_offset] != 0; _offset++)
						{
							if (_data[_offset] == '\n')
							{
								break;
							}
						}
						flags |= TokenFlags.HasLeadingSpace;
					}
					else if (nextCharacter == '*')
					{
						_offset += 2;
						for (; _data[_offset] != 0; _offset++)
						{
							if (_data[_offset] == '\n')
							{
								LineNumberAfterToken++;
								continue;
							}
							if (_data[_offset] == '*' && _data[_offset + 1] == '/')
							{
								_offset += 2;
								break;
							}
						}
						flags |= TokenFlags.HasLeadingSpace;
					}
					else
					{
						break;
					}
				}
				else
				{
					if (character == '\r')
					{
						_offset++;
					}
					else
					{
						break;
					}
				}
			}
			return flags;
		}

		/// <summary>
		/// Skip past a text literal (a quoted character literal or string literal)
		/// </summary>
		/// <param name="lastCharacter">The terminating character to look for, ignoring escape sequences</param>
		void SkipTextLiteral(char lastCharacter)
		{
			for (; ; )
			{
				char character = (char)_data[_offset];
				if (character == '\0')
				{
					throw new Exception("Unexpected end of file in text literal");
				}

				_offset++;

				if (character == '\\' && _data[_offset] != 0)
				{
					_offset++;
				}
				else if (character == lastCharacter)
				{
					break;
				}
			}
		}

		/// <summary>
		/// Skips over a numeric literal after the initial digit or dot/digit pair.
		/// </summary>
		void SkipNumericLiteral()
		{
			for (; ; )
			{
				while ((s_characters[_data[_offset]] & CharacterFlags.NumberTail) != 0)
				{
					_offset++;
				}
				if ((_data[_offset] == '+' || _data[_offset] == '-') && (_data[_offset - 1] == 'e' || _data[_offset - 1] == 'E'))
				{
					_offset++;
				}
				else
				{
					break;
				}
			}
		}

#pragma warning disable CA1810 // Initialize reference type static fields inline
		/// <summary>
		/// Static constructor. Initializes the lookup tables used by the lexer.
		/// </summary>
		static TokenReader()
		{
			s_firstCharacters = new FirstCharacterClass[256];

			// End of file
			s_firstCharacters[0] = FirstCharacterClass.Terminator;

			// Horizontal whitespace
			s_firstCharacters[' '] = FirstCharacterClass.Whitespace;
			s_firstCharacters['\t'] = FirstCharacterClass.Whitespace;
			s_firstCharacters['\v'] = FirstCharacterClass.Whitespace;
			s_firstCharacters['\r'] = FirstCharacterClass.Whitespace;

			// Newline
			s_firstCharacters['\n'] = FirstCharacterClass.Newline;

			// Identifiers
			s_firstCharacters['_'] = FirstCharacterClass.Identifier;
			for (int idx = 'a'; idx <= 'z'; idx++)
			{
				s_firstCharacters[idx] = FirstCharacterClass.Identifier;
			}
			for (int idx = 'A'; idx <= 'Z'; idx++)
			{
				s_firstCharacters[idx] = FirstCharacterClass.Identifier;
			}

			// Numeric literals
			for (int idx = '0'; idx <= '9'; idx++)
			{
				s_firstCharacters[idx] = FirstCharacterClass.Number;
			}

			// Character literals
			s_firstCharacters['\''] = FirstCharacterClass.Character;

			// String literals
			s_firstCharacters['\"'] = FirstCharacterClass.String;

			// Other symbols
			s_firstCharacters['.'] = FirstCharacterClass.Dot;
			s_firstCharacters['?'] = FirstCharacterClass.QuestionMark;
			s_firstCharacters[':'] = FirstCharacterClass.Colon;
			s_firstCharacters['!'] = FirstCharacterClass.ExclamationMark;
			s_firstCharacters['|'] = FirstCharacterClass.Pipe;
			s_firstCharacters['&'] = FirstCharacterClass.Ampersand;
			s_firstCharacters['^'] = FirstCharacterClass.Caret;
			s_firstCharacters['='] = FirstCharacterClass.Equals;
			s_firstCharacters['<'] = FirstCharacterClass.LeftTriangleBracket;
			s_firstCharacters['>'] = FirstCharacterClass.RightTriangleBracket;
			s_firstCharacters['+'] = FirstCharacterClass.Plus;
			s_firstCharacters['-'] = FirstCharacterClass.Minus;
			s_firstCharacters['*'] = FirstCharacterClass.Star;
			s_firstCharacters['/'] = FirstCharacterClass.Slash;
			s_firstCharacters['%'] = FirstCharacterClass.PercentSign;
			s_firstCharacters['~'] = FirstCharacterClass.Tilde;
			s_firstCharacters['('] = FirstCharacterClass.LeftParen;
			s_firstCharacters[')'] = FirstCharacterClass.RightParen;
			s_firstCharacters['#'] = FirstCharacterClass.Hash;
			s_firstCharacters[','] = FirstCharacterClass.Comma;

			// Flags for secondary characters
			s_characters = new CharacterFlags[256];

			// Identifiers
			s_characters['_'] |= CharacterFlags.Identifier;
			for (int idx = 'a'; idx <= 'z'; idx++)
			{
				s_characters[idx] |= CharacterFlags.Identifier;
			}
			for (int idx = 'A'; idx <= 'Z'; idx++)
			{
				s_characters[idx] |= CharacterFlags.Identifier;
			}
			for (int idx = '0'; idx <= '9'; idx++)
			{
				s_characters[idx] |= CharacterFlags.Identifier;
			}

			// Numbers
			for (int idx = '0'; idx <= '9'; idx++)
			{
				s_characters[idx] |= CharacterFlags.Digit;
			}

			// Preprocessing number tail
			s_characters['.'] |= CharacterFlags.NumberTail;
			s_characters['_'] |= CharacterFlags.NumberTail;
			for (int idx = '0'; idx <= '9'; idx++)
			{
				s_characters[idx] |= CharacterFlags.NumberTail;
			}
			for (int idx = 'a'; idx <= 'z'; idx++)
			{
				s_characters[idx] |= CharacterFlags.NumberTail;
			}
			for (int idx = 'A'; idx <= 'Z'; idx++)
			{
				s_characters[idx] |= CharacterFlags.NumberTail;
			}

			// Whitespace
			s_characters[' '] |= CharacterFlags.Whitespace;
			s_characters['\t'] |= CharacterFlags.Whitespace;
			s_characters['\v'] |= CharacterFlags.Whitespace;
		}
#pragma warning restore CA1810
	}
}
