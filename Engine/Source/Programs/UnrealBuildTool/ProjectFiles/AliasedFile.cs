// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using EpicGames.Core;

namespace UnrealBuildTool
{
	/// <summary>
	/// Class that stores info about aliased file.
	/// </summary>
	struct AliasedFile : IComparable<AliasedFile>
	{
		public AliasedFile(FileReference Location, string FileSystemPath, string ProjectPath)
		{
			this.Location = Location;
			this.FileSystemPath = FileSystemPath;
			this.ProjectPath = ProjectPath;
		}

		public int CompareTo(AliasedFile other) => String.CompareOrdinal(Location.FullName, other.Location.FullName);

		// Full location on disk.
		public readonly FileReference Location;

		// File system path.
		public readonly string FileSystemPath;

		// Project path.
		public readonly string ProjectPath;
	}
}
