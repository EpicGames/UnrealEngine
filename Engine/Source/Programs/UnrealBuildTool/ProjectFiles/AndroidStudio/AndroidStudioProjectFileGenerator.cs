// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.IO;
using System.Linq;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	class AndroidStudioProjectFile : ProjectFile
	{
		public AndroidStudioProjectFile(FileReference InitFilePath, DirectoryReference BaseDir)
			: base(InitFilePath, BaseDir)
		{
		}
	}
	
	class AndroidStudioFileGenerator : ProjectFileGenerator
	{
		public override string ProjectFileExtension => ".gradle";

		public AndroidStudioFileGenerator(FileReference? InOnlyGameProject, CSharpProjectCache CSharpProjCache)
			: base(InOnlyGameProject, CSharpProjCache)
		{
		}

		public override bool ShouldGenerateIntelliSenseData() => false;

		protected override void ConfigureProjectFileGeneration(string[] Arguments, ref bool IncludeAllPlatforms, ILogger Logger)
		{
			bIncludeEnginePrograms = false;
			bGeneratingGameProjectFiles = true;
			
			base.ConfigureProjectFileGeneration(Arguments, ref IncludeAllPlatforms, Logger);
		}
		
		public override void CleanProjectFiles(DirectoryReference InPrimaryProjectDirectory, string InPrimaryProjectName, DirectoryReference InIntermediateProjectFilesDirectory, ILogger Logger)
		{
			string ProjectDir = Path.Combine(InPrimaryProjectDirectory.FullName, $"{InPrimaryProjectName}.gradle");
			
			UEDeployAndroid.DeleteDirectory(ProjectDir, Logger);
			try
			{
				Directory.Delete(ProjectDir);
			}
			catch (DirectoryNotFoundException)
			{
			}
		}

		protected override ProjectFile AllocateProjectFile(FileReference InitFilePath, DirectoryReference BaseDir)
		{
			return new AndroidStudioProjectFile(InitFilePath, BaseDir);
		}

		protected override bool WriteProjectFiles(PlatformProjectGeneratorCollection PlatformProjectGenerators, ILogger Logger)
		{
			TargetRules ClientTargetRules = (DefaultProject!.ProjectTargets.FirstOrDefault(Project => Project.TargetRules!.Type == TargetType.Client) ?? DefaultProject!.ProjectTargets.First(Project => Project.TargetRules!.Type == TargetType.Game)).TargetRules!;
			UnrealArchitectures Architectures = VCProjectFileGenerator.GetAndroidProjectArchitectures(OnlyGameProject);
			UnrealArch Architecture = Architectures.Architectures.First();

			string ProjectDir = Path.Combine(PrimaryProjectPath.FullName, $"{PrimaryProjectName}.gradle");
			string LLDBSymbolsLibsDir = Path.Combine(PrimaryProjectPath.FullName, "Intermediate", "Android", "LLDBSymbolsLibs");
			
			Dictionary<string, string> Replacements = new Dictionary<string, string>
			{
				{"${PY_VISUALIZER_PATH}", Path.Combine(Unreal.EngineDirectory.FullName, "Extras", "LLDBDataFormatters", "UEDataFormatters_2ByteChars.py")},
				{"${IDEA_RUN_CONFIGURATION_SYMBOL_PATHS}", $"\n      <symbol_dirs symbol_path=\"{Path.Combine(LLDBSymbolsLibsDir, Architecture.ToString())}\" />"}
			};
			
			UEDeployAndroid.CopyFileDirectory(Path.Combine(Unreal.EngineDirectory.FullName, "Build", "Android", "Java", "gradle"), ProjectDir, Replacements, [".idea", "runConfigurations", "app"]);
			UEDeployAndroid.CopyFileDirectory(Path.Combine(Unreal.EngineDirectory.FullName, "Build", "AndroidStudio"), ProjectDir, Replacements);
			{
				string GradlePropertiesContent = $"org.gradle.jvmargs=-XX:MaxHeapSize=4096m -Xmx9216m\nENGINELOCATION={Unreal.EngineDirectory.FullName.Replace('\\', '/')}\nPROJECT_NAME={PrimaryProjectName}\nTARGET_NAME={ClientTargetRules.Name}\nCONFIGURATION={ClientTargetRules.Configuration}\nUNREAL_ARCH={Architecture}\n";
				WriteFileIfChanged(Path.Combine(ProjectDir, "gradle.properties"), GradlePropertiesContent, Logger);
			}

			return true;
		}

		protected override bool WritePrimaryProjectFile(ProjectFile? UBTProject, PlatformProjectGeneratorCollection PlatformProjectGenerators, ILogger Logger)
		{
			return true;
		}
	}
}
