// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;

namespace UnrealBuildTool
{
	/// <summary>
	/// CLion project file generator which is just the CMakefileGenerator and only here for UBT to match against
	/// </summary>
	class CLionGenerator : CMakefileGenerator
	{
		/// <summary>
		/// Creates a new instance of the <see cref="CMakefileGenerator"/> class.
		/// </summary>
		public CLionGenerator(FileReference? InOnlyGameProject, CSharpProjectCache CSharpProjCache)
			: base(InOnlyGameProject, CSharpProjCache)
		{
		}
	}
}
