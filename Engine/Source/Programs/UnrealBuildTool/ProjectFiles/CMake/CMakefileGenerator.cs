// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	class CMakefileProjectFile : ProjectFile
	{
		public CMakefileProjectFile(FileReference InitFilePath, DirectoryReference BaseDir)
			: base(InitFilePath, BaseDir)
		{
		}
	}
	/// <summary>
	/// CMakefile project file generator implementation
	/// </summary>
	class CMakefileGenerator : ProjectFileGenerator
	{
		/// <summary>
		/// Creates a new instance of the <see cref="CMakefileGenerator"/> class.
		/// </summary>
		public CMakefileGenerator(FileReference? InOnlyGameProject, CSharpProjectCache CSharpProjCache)
			: base(InOnlyGameProject, CSharpProjCache)
		{
		}

		/// <summary>
		/// Determines whether or not we should generate code completion data whenever possible.
		/// </summary>
		/// <returns><value>true</value> if we should generate code completion data; <value>false</value> otherwise.</returns>
		public override bool ShouldGenerateIntelliSenseData()
		{
			return true;
		}

		/// <summary>
		/// Is this a project build?
		/// </summary>
		/// <remarks>
		/// This determines if engine files are included in the source lists.
		/// </remarks>
		/// <returns><value>true</value> if we should treat this as a project build; <value>false</value> otherwise.</returns>
		public bool IsProjectBuild => !String.IsNullOrEmpty(GameProjectName);

		/// <summary>
		/// The file extension for this project file.
		/// </summary>
		public override string ProjectFileExtension => ".txt";

		public string ProjectFileName => "CMakeLists" + ProjectFileExtension;

		/// <summary>
		/// The CMake helper file extension
		/// </summary>
		public static string CMakeExtension => ".cmake";

		/// <summary>
		/// The CMake file used to store the list of includes for the project.
		/// </summary>
		public static string CMakeIncludesFileName => "cmake-includes" + CMakeExtension;

		/// <summary>
		/// The CMake file used to store the configuration files (INI) for the engine.
		/// </summary>
		public static string CMakeEngineConfigsFileName => "cmake-config-engine" + CMakeExtension;

		/// <summary>
		/// The CMake file used to store the configuration files (INI) for the project.
		/// </summary>
		public static string CMakeProjectConfigsFileName => "cmake-config-project" + CMakeExtension;

		/// <summary>
		/// The CMake file used to store the additional build configuration files (CSharp) for the engine.
		/// </summary>
		public static string CMakeEngineCSFileName => "cmake-csharp-engine" + CMakeExtension;

		/// <summary>
		/// The CMake file used to store the additional configuration files (CSharp) for the project.
		/// </summary>
		public static string CMakeProjectCSFileName => "cmake-csharp-project" + CMakeExtension;

		/// <summary>
		/// The CMake file used to store the additional shader files (usf/ush) for the engine.
		/// </summary>
		public static string CMakeEngineShadersFileName => "cmake-shaders-engine" + CMakeExtension;

		/// <summary>
		/// The CMake file used to store the additional shader files (usf/ush) for the project.
		/// </summary>
		public static string CMakeProjectShadersFileName => "cmake-shaders-project" + CMakeExtension;

		/// <summary>
		/// The CMake file used to store the list of engine headers.
		/// </summary>
		public static string CMakeEngineHeadersFileName => "cmake-headers-ue" + CMakeExtension;
		/// <summary>
		/// The CMake file used to store the list of engine headers.
		/// </summary>
		public static string CMakeProjectHeadersFileName => "cmake-headers-project" + CMakeExtension;

		/// <summary>
		/// The CMake file used to store the list of sources for the engine.
		/// </summary>
		public static string CMakeEngineSourcesFileName => "cmake-sources-engine" + CMakeExtension;

		/// <summary>
		/// The CMake file used to store the list of sources for the project.
		/// </summary>
		public static string CMakeProjectSourcesFileName => "cmake-sources-project" + CMakeExtension;

		/// <summary>
		/// The CMake file used to store the list of definitions for the project.
		/// </summary>
		public static string CMakeDefinitionsFileName => "cmake-definitions" + CMakeExtension;

		/// <summary>
		/// Writes the primary project file (e.g. Visual Studio Solution file)
		/// </summary>
		/// <param name="UBTProject">The UnrealBuildTool project</param>
		/// <param name="PlatformProjectGenerators">The registered platform project generators</param>
		/// <returns>True if successful</returns>
		/// <param name="Logger"></param>
		protected override bool WritePrimaryProjectFile(ProjectFile? UBTProject, PlatformProjectGeneratorCollection PlatformProjectGenerators, ILogger Logger)
		{
			return true;
		}

		private void AppendCleanedPathToList(StringBuilder EngineFiles, StringBuilder ProjectFiles, string SourceFileRelativeToRoot, string FullName, string GameProjectPath, string UnrealRootPath, string GameRootPath)
		{
			if (!SourceFileRelativeToRoot.StartsWith("..", StringComparison.Ordinal) && !Path.IsPathRooted(SourceFileRelativeToRoot))
			{
				EngineFiles.Append("\t\"" + UnrealRootPath + "/Engine/" + Utils.CleanDirectorySeparators(SourceFileRelativeToRoot, '/') + "\"\n");
			}
			else
			{
				if (String.IsNullOrEmpty(GameProjectName))
				{
					EngineFiles.Append("\t\"" + Utils.CleanDirectorySeparators(SourceFileRelativeToRoot, '/').Substring(3) + "\"\n");
				}
				else
				{
					string RelativeGameSourcePath = Utils.MakePathRelativeTo(FullName, GameProjectPath);
					ProjectFiles.Append("\t\"" + GameRootPath + "/" + Utils.CleanDirectorySeparators(RelativeGameSourcePath, '/') + "\"\n");
				}
			}
		}

		private bool WriteCMakeLists(ILogger Logger)
		{
			string BuildCommand;
			const string CMakeSectionEnd = " )\n\n";
			StringBuilder CMakefileContent = new StringBuilder();

			// Create Engine/Project specific lists
			StringBuilder CMakeEngineSourceFilesList = new StringBuilder("set(ENGINE_SOURCE_FILES \n");
			StringBuilder CMakeProjectSourceFilesList = new StringBuilder("set(PROJECT_SOURCE_FILES \n");
			StringBuilder CMakeEngineHeaderFilesList = new StringBuilder("set(ENGINE_HEADER_FILES \n");
			StringBuilder CMakeProjectHeaderFilesList = new StringBuilder("set(PROJECT_HEADER_FILES \n");
			StringBuilder CMakeEngineCSFilesList = new StringBuilder("set(ENGINE_CSHARP_FILES \n");
			StringBuilder CMakeProjectCSFilesList = new StringBuilder("set(PROJECT_CSHARP_FILES \n");
			StringBuilder CMakeEngineConfigFilesList = new StringBuilder("set(ENGINE_CONFIG_FILES \n");
			StringBuilder CMakeProjectConfigFilesList = new StringBuilder("set(PROJECT_CONFIG_FILES \n");
			StringBuilder CMakeEngineShaderFilesList = new StringBuilder("set(ENGINE_SHADER_FILES \n");
			StringBuilder CMakeProjectShaderFilesList = new StringBuilder("set(PROJECT_SHADER_FILES \n");

			StringBuilder IncludeDirectoriesList = new StringBuilder("include_directories( \n");
			StringBuilder PreprocessorDefinitionsList = new StringBuilder("add_definitions( \n");

			string UnrealRootPath = Utils.CleanDirectorySeparators(Unreal.RootDirectory.FullName, '/');
			string CMakeGameRootPath = "";
			string GameProjectPath = "";
			string CMakeGameProjectFile = "";

			string HostArchitecture;
			string SetCompiler = "";
			if (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Win64)
			{
				HostArchitecture = "Win64";
				BuildCommand = "call \"" + UnrealRootPath + "/Engine/Build/BatchFiles/Build.bat\"";
			}
			else if (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Mac)
			{
				HostArchitecture = "Mac";
				BuildCommand = "cd \"" + UnrealRootPath + "\" && bash \"" + UnrealRootPath + "/Engine/Build/BatchFiles/" + HostArchitecture + "/Build.sh\"";
				bIncludeIOSTargets = true;
				bIncludeTVOSTargets = true;
			}
			else if (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Linux)
			{
				HostArchitecture = "Linux";
				BuildCommand = "cd \"" + UnrealRootPath + "\" && bash \"" + UnrealRootPath + "/Engine/Build/BatchFiles/" + HostArchitecture + "/Build.sh\"";

				string? CompilerPath = LinuxCommon.WhichClang(Logger);
				SetCompiler = "set(CMAKE_CXX_COMPILER " + CompilerPath + ")\n\n";
			}
			else
			{
				throw new BuildException("ERROR: CMakefileGenerator does not support this platform");
			}

			if (IsProjectBuild)
			{
				GameProjectPath = OnlyGameProject!.Directory.FullName;
				CMakeGameRootPath = Utils.CleanDirectorySeparators(OnlyGameProject.Directory.FullName, '/');
				CMakeGameProjectFile = Utils.CleanDirectorySeparators(OnlyGameProject.FullName, '/');
			}

			// Additional CMake file definitions
			string EngineHeadersFilePath = FileReference.Combine(IntermediateProjectFilesPath, CMakeEngineHeadersFileName).ToNormalizedPath();
			string ProjectHeadersFilePath = FileReference.Combine(IntermediateProjectFilesPath, CMakeProjectHeadersFileName).ToNormalizedPath();
			string EngineSourcesFilePath = FileReference.Combine(IntermediateProjectFilesPath, CMakeEngineSourcesFileName).ToNormalizedPath();
			string ProjectSourcesFilePath = FileReference.Combine(IntermediateProjectFilesPath, CMakeProjectSourcesFileName).ToNormalizedPath();
			//string ProjectFilePath = FileReference.Combine(IntermediateProjectFilesPath, CMakeProjectSourcesFileName).ToNormalizedPath();
			string IncludeFilePath = FileReference.Combine(IntermediateProjectFilesPath, CMakeIncludesFileName).ToNormalizedPath();
			string EngineConfigsFilePath = FileReference.Combine(IntermediateProjectFilesPath, CMakeEngineConfigsFileName).ToNormalizedPath();
			string ProjectConfigsFilePath = FileReference.Combine(IntermediateProjectFilesPath, CMakeProjectConfigsFileName).ToNormalizedPath();
			string EngineCSFilePath = FileReference.Combine(IntermediateProjectFilesPath, CMakeEngineCSFileName).ToNormalizedPath();
			string ProjectCSFilePath = FileReference.Combine(IntermediateProjectFilesPath, CMakeProjectCSFileName).ToNormalizedPath();
			string EngineShadersFilePath = FileReference.Combine(IntermediateProjectFilesPath, CMakeEngineShadersFileName).ToNormalizedPath();
			string ProjectShadersFilePath = FileReference.Combine(IntermediateProjectFilesPath, CMakeProjectShadersFileName).ToNormalizedPath();
			string DefinitionsFilePath = FileReference.Combine(IntermediateProjectFilesPath, CMakeDefinitionsFileName).ToNormalizedPath();

			CMakefileContent.Append(
				"# Makefile generated by CMakefileGenerator.cs (v1.2)\n" +
				"# *DO NOT EDIT*\n\n" +
				"cmake_minimum_required (VERSION 2.6)\n" +
				"project (Unreal)\n\n" +
				"# CMake Flags\n" +
				"set(CMAKE_CXX_STANDARD 14)\n" + // Need to keep this updated
				"set(CMAKE_CXX_USE_RESPONSE_FILE_FOR_OBJECTS 1 CACHE BOOL \"\" FORCE)\n" +
				"set(CMAKE_CXX_USE_RESPONSE_FILE_FOR_INCLUDES 1 CACHE BOOL \"\" FORCE)\n\n" +
				SetCompiler +
				"# Standard Includes\n" +
				"include(\"" + IncludeFilePath + "\")\n" +
				"include(\"" + DefinitionsFilePath + "\")\n" +
				"include(\"" + EngineHeadersFilePath + "\")\n" +
				"include(\"" + ProjectHeadersFilePath + "\")\n" +
				"include(\"" + EngineSourcesFilePath + "\")\n" +
				"include(\"" + ProjectSourcesFilePath + "\")\n" +
				"include(\"" + EngineCSFilePath + "\")\n" +
				"include(\"" + ProjectCSFilePath + "\")\n\n"
			);

			List<string> IncludeDirectories = new List<string>();
			List<string> PreprocessorDefinitions = new List<string>();

			foreach (ProjectFile CurProject in GeneratedProjectFiles)
			{
				foreach (string IncludeSearchPath in CurProject.IntelliSenseIncludeSearchPaths)
				{
					string IncludeDirectory = GetIncludeDirectory(IncludeSearchPath, Path.GetDirectoryName(CurProject.ProjectFilePath.FullName)!);
					if (IncludeDirectory != null && !IncludeDirectories.Contains(IncludeDirectory))
					{
						if (IncludeDirectory.Contains(Unreal.RootDirectory.FullName, StringComparison.Ordinal))
						{
							IncludeDirectories.Add(IncludeDirectory.Replace(Unreal.RootDirectory.FullName, UnrealRootPath, StringComparison.Ordinal));
						}
						else
						{
							// If the path isn't rooted, then it is relative to the game root
							if (!Path.IsPathRooted(IncludeDirectory))
							{
								IncludeDirectories.Add(CMakeGameRootPath + "/" + IncludeDirectory);
							}
							else
							{
								// This is a rooted path like /usr/local/sometool/include
								IncludeDirectories.Add(IncludeDirectory);
							}
						}
					}
				}

				foreach (string PreProcessorDefinition in CurProject.IntelliSensePreprocessorDefinitions)
				{
					string Definition = PreProcessorDefinition.Replace("TEXT(\"", "", StringComparison.Ordinal).Replace("\")", "", StringComparison.Ordinal).Replace("()=", "=", StringComparison.Ordinal);
					string AlternateDefinition = Definition.Contains("=0", StringComparison.Ordinal) ? Definition.Replace("=0", "=1", StringComparison.Ordinal) : Definition.Replace("=1", "=0", StringComparison.Ordinal);

					if (Definition.Equals("WITH_EDITORONLY_DATA=0", StringComparison.Ordinal))
					{
						Definition = AlternateDefinition;
					}

					if (!PreprocessorDefinitions.Contains(Definition) &&
						!PreprocessorDefinitions.Contains(AlternateDefinition) &&
						!Definition.StartsWith("UE_ENGINE_DIRECTORY", StringComparison.Ordinal) &&
						!Definition.StartsWith("ORIGINAL_FILE_NAME", StringComparison.Ordinal))
					{
						PreprocessorDefinitions.Add(Definition);
					}
				}
			}

			// Create SourceFiles, HeaderFiles, and ConfigFiles sections.
			List<FileReference> AllModuleFiles = DiscoverModules(FindGameProjects(Logger), null);
			foreach (FileReference CurModuleFile in AllModuleFiles)
			{
				List<FileReference> FoundFiles = SourceFileSearch.FindModuleSourceFiles(CurModuleFile);
				foreach (FileReference CurSourceFile in FoundFiles)
				{
					string SourceFileRelativeToRoot = CurSourceFile.MakeRelativeTo(Unreal.EngineDirectory);

					// Exclude files/folders on a per-platform basis.
					if (!IsPathExcludedOnPlatform(SourceFileRelativeToRoot))
					{
						if (SourceFileRelativeToRoot.EndsWith(".cpp", StringComparison.Ordinal))
						{
							AppendCleanedPathToList(CMakeEngineSourceFilesList, CMakeProjectSourceFilesList, SourceFileRelativeToRoot, CurSourceFile.FullName, GameProjectPath, UnrealRootPath, CMakeGameRootPath);
						}
						else if (SourceFileRelativeToRoot.EndsWith(".h", StringComparison.Ordinal))
						{
							AppendCleanedPathToList(CMakeEngineHeaderFilesList, CMakeProjectHeaderFilesList, SourceFileRelativeToRoot, CurSourceFile.FullName, GameProjectPath, UnrealRootPath, CMakeGameRootPath);
						}
						else if (SourceFileRelativeToRoot.EndsWith(".cs", StringComparison.Ordinal))
						{
							AppendCleanedPathToList(CMakeEngineCSFilesList, CMakeProjectCSFilesList, SourceFileRelativeToRoot, CurSourceFile.FullName, GameProjectPath, UnrealRootPath, CMakeGameRootPath);
						}
						else if (SourceFileRelativeToRoot.EndsWith(".usf", StringComparison.Ordinal) || SourceFileRelativeToRoot.EndsWith(".ush", StringComparison.Ordinal))
						{
							AppendCleanedPathToList(CMakeEngineShaderFilesList, CMakeProjectShaderFilesList, SourceFileRelativeToRoot, CurSourceFile.FullName, GameProjectPath, UnrealRootPath, CMakeGameRootPath);
						}
						else if (SourceFileRelativeToRoot.EndsWith(".ini", StringComparison.Ordinal))
						{
							AppendCleanedPathToList(CMakeEngineConfigFilesList, CMakeProjectConfigFilesList, SourceFileRelativeToRoot, CurSourceFile.FullName, GameProjectPath, UnrealRootPath, CMakeGameRootPath);
						}
					}
				}
			}

			foreach (string IncludeDirectory in IncludeDirectories)
			{
				IncludeDirectoriesList.Append("\t\"" + Utils.CleanDirectorySeparators(IncludeDirectory, '/') + "\"\n");
			}

			foreach (string PreprocessorDefinition in PreprocessorDefinitions)
			{
				int EqPos = PreprocessorDefinition.IndexOf('=', StringComparison.Ordinal);
				if (EqPos >= 0)
				{
					string Key = PreprocessorDefinition.Substring(0, EqPos);
					string Value = PreprocessorDefinition.Substring(EqPos).Replace("\"", "\\\"", StringComparison.Ordinal);
					PreprocessorDefinitionsList.Append("\t\"-D" + Key + Value + "\"\n");
				}
				else
				{
					PreprocessorDefinitionsList.Append("\t\"-D" + PreprocessorDefinition + "\"\n");
				}
			}

			// Add Engine/Shaders files (game are added via modules)
			List<FileReference> EngineShaderFiles = SourceFileSearch.FindFiles(DirectoryReference.Combine(Unreal.EngineDirectory, "Shaders"));
			foreach (FileReference CurSourceFile in EngineShaderFiles)
			{
				string SourceFileRelativeToRoot = CurSourceFile.MakeRelativeTo(Unreal.EngineDirectory);
				if (SourceFileRelativeToRoot.EndsWith(".usf", StringComparison.Ordinal) || SourceFileRelativeToRoot.EndsWith(".ush", StringComparison.Ordinal))
				{
					AppendCleanedPathToList(CMakeEngineShaderFilesList, CMakeProjectShaderFilesList, SourceFileRelativeToRoot, CurSourceFile.FullName, GameProjectPath, UnrealRootPath, CMakeGameRootPath);
				}
			}

			// Add Engine/Config ini files (game are added via modules)
			List<FileReference> EngineConfigFiles = SourceFileSearch.FindFiles(DirectoryReference.Combine(Unreal.EngineDirectory, "Config"));
			foreach (FileReference CurSourceFile in EngineConfigFiles)
			{
				string SourceFileRelativeToRoot = CurSourceFile.MakeRelativeTo(Unreal.EngineDirectory);
				if (SourceFileRelativeToRoot.EndsWith(".ini", StringComparison.Ordinal))
				{
					AppendCleanedPathToList(CMakeEngineConfigFilesList, CMakeProjectConfigFilesList, SourceFileRelativeToRoot, CurSourceFile.FullName, GameProjectPath, UnrealRootPath, CMakeGameRootPath);
				}
			}

			// Add section end to section strings;
			CMakeEngineSourceFilesList.Append(CMakeSectionEnd);
			CMakeEngineHeaderFilesList.Append(CMakeSectionEnd);
			CMakeEngineCSFilesList.Append(CMakeSectionEnd);
			CMakeEngineConfigFilesList.Append(CMakeSectionEnd);
			CMakeEngineShaderFilesList.Append(CMakeSectionEnd);

			CMakeProjectSourceFilesList.Append(CMakeSectionEnd);
			CMakeProjectHeaderFilesList.Append(CMakeSectionEnd);
			CMakeProjectCSFilesList.Append(CMakeSectionEnd);
			CMakeProjectConfigFilesList.Append(CMakeSectionEnd);
			CMakeProjectShaderFilesList.Append(CMakeSectionEnd);

			IncludeDirectoriesList.Append(CMakeSectionEnd);
			PreprocessorDefinitionsList.Append(CMakeSectionEnd);

			if (bIncludeShaderSource)
			{
				CMakefileContent.Append("# Optional Shader Include\n");
				if (!IsProjectBuild || bIncludeEngineSource)
				{
					CMakefileContent.Append("include(\"" + EngineShadersFilePath + "\")\n");
					CMakefileContent.Append("set_source_files_properties(${ENGINE_SHADER_FILES} PROPERTIES HEADER_FILE_ONLY TRUE)\n");
				}
				CMakefileContent.Append("include(\"" + ProjectShadersFilePath + "\")\n");
				CMakefileContent.Append("set_source_files_properties(${PROJECT_SHADER_FILES} PROPERTIES HEADER_FILE_ONLY TRUE)\n");
				CMakefileContent.Append("source_group(\"Shader Files\" REGULAR_EXPRESSION .*.usf)\n\n");
			}

			if (bIncludeConfigFiles)
			{
				CMakefileContent.Append("# Optional Config Include\n");
				if (!IsProjectBuild || bIncludeEngineSource)
				{
					CMakefileContent.Append("include(\"" + EngineConfigsFilePath + "\")\n");
					CMakefileContent.Append("set_source_files_properties(${ENGINE_CONFIG_FILES} PROPERTIES HEADER_FILE_ONLY TRUE)\n");
				}
				CMakefileContent.Append("include(\"" + ProjectConfigsFilePath + "\")\n");
				CMakefileContent.Append("set_source_files_properties(${PROJECT_CONFIG_FILES} PROPERTIES HEADER_FILE_ONLY TRUE)\n");
				CMakefileContent.Append("source_group(\"Config Files\" REGULAR_EXPRESSION .*.ini)\n\n");
			}

			string CMakeProjectCmdArg = "";
			string UBTArguements = "";

			if (bGeneratingGameProjectFiles)
			{
				UBTArguements += " -game";
			}
			// Should the builder output progress ticks
			if (ProgressWriter.bWriteMarkup)
			{
				UBTArguements += " -progress";
			}

			foreach (ProjectFile Project in GeneratedProjectFiles)
			{
				foreach (ProjectTarget TargetFile in Project.ProjectTargets.OfType<ProjectTarget>())
				{
					if (TargetFile.TargetFilePath == null)
					{
						continue;
					}

					string TargetName = TargetFile.TargetFilePath.GetFileNameWithoutAnyExtensions();       // Remove both ".cs" and ".

					foreach (UnrealTargetConfiguration CurConfiguration in (UnrealTargetConfiguration[])Enum.GetValues(typeof(UnrealTargetConfiguration)))
					{
						if (CurConfiguration != UnrealTargetConfiguration.Unknown && CurConfiguration != UnrealTargetConfiguration.Development)
						{
							if (InstalledPlatformInfo.IsValidConfiguration(CurConfiguration, EProjectType.Code) && !IsTargetExcluded(TargetName, BuildHostPlatform.Current.Platform, CurConfiguration))
							{
								if (TargetName == GameProjectName || TargetName == (GameProjectName + "Editor"))
								{
									CMakeProjectCmdArg = "\"-project=" + CMakeGameProjectFile + "\"";
								}

								string ConfName = Enum.GetName(typeof(UnrealTargetConfiguration), CurConfiguration)!;
								CMakefileContent.Append(String.Format("add_custom_target({0}-{3}-{1} {5} {0} {3} {1} {2}{4} -buildscw VERBATIM)\n", TargetName, ConfName, CMakeProjectCmdArg, HostArchitecture, UBTArguements, BuildCommand));

								// Add iOS and TVOS targets if valid
								if (bIncludeIOSTargets && !IsTargetExcluded(TargetName, UnrealTargetPlatform.IOS, CurConfiguration))
								{
									CMakefileContent.Append(String.Format("add_custom_target({0}-{3}-{1} {5} {0} {3} {1} {2}{4} VERBATIM)\n", TargetName, ConfName, CMakeProjectCmdArg, UnrealTargetPlatform.IOS, UBTArguements, BuildCommand));
								}
								if (bIncludeTVOSTargets && !IsTargetExcluded(TargetName, UnrealTargetPlatform.TVOS, CurConfiguration))
								{
									CMakefileContent.Append(String.Format("add_custom_target({0}-{3}-{1} {5} {0} {3} {1} {2}{4} VERBATIM)\n", TargetName, ConfName, CMakeProjectCmdArg, UnrealTargetPlatform.TVOS, UBTArguements, BuildCommand));
								}
							}
						}
					}
					if (!IsTargetExcluded(TargetName, BuildHostPlatform.Current.Platform, UnrealTargetConfiguration.Development))
					{
						if (TargetName == GameProjectName || TargetName == (GameProjectName + "Editor"))
						{
							CMakeProjectCmdArg = "\"-project=" + CMakeGameProjectFile + "\"";
						}

						CMakefileContent.Append(String.Format("add_custom_target({0} {4} {0} {2} Development {1}{3} -buildscw VERBATIM)\n\n", TargetName, CMakeProjectCmdArg, HostArchitecture, UBTArguements, BuildCommand));

						// Add iOS and TVOS targets if valid
						if (bIncludeIOSTargets && !IsTargetExcluded(TargetName, UnrealTargetPlatform.IOS, UnrealTargetConfiguration.Development))
						{
							CMakefileContent.Append(String.Format("add_custom_target({0}-{3} {5} {0} {3} {1} {2}{4} VERBATIM)\n", TargetName, UnrealTargetConfiguration.Development, CMakeProjectCmdArg, UnrealTargetPlatform.IOS, UBTArguements, BuildCommand));
						}
						if (bIncludeTVOSTargets && !IsTargetExcluded(TargetName, UnrealTargetPlatform.TVOS, UnrealTargetConfiguration.Development))
						{
							CMakefileContent.Append(String.Format("add_custom_target({0}-{3} {5} {0} {3} {1} {2}{4} VERBATIM)\n", TargetName, UnrealTargetConfiguration.Development, CMakeProjectCmdArg, UnrealTargetPlatform.TVOS, UBTArguements, BuildCommand));
						}
					}
				}
			}

			// Create Build Template
			if (IsProjectBuild && !bIncludeEngineSource)
			{
				CMakefileContent.AppendLine("add_executable(FakeTarget ${PROJECT_HEADER_FILES} ${PROJECT_SOURCE_FILES} ${PROJECT_CSHARP_FILES} ${PROJECT_SHADER_FILES} ${PROJECT_CONFIG_FILES})");
			}
			else
			{
				CMakefileContent.AppendLine("add_executable(FakeTarget ${ENGINE_HEADER_FILES} ${ENGINE_SOURCE_FILES} ${ENGINE_CSHARP_FILES} ${ENGINE_SHADER_FILES} ${ENGINE_CONFIG_FILES} ${PROJECT_HEADER_FILES} ${PROJECT_SOURCE_FILES} ${PROJECT_CSHARP_FILES} ${PROJECT_SHADER_FILES} ${PROJECT_CONFIG_FILES})");
			}

			string FullFileName = Path.Combine(PrimaryProjectPath.FullName, ProjectFileName);

			// Write out CMake files
			bool bWriteMakeList = WriteFileIfChanged(FullFileName, CMakefileContent.ToString(), Logger);
			bool bWriteEngineHeaders = WriteFileIfChanged(EngineHeadersFilePath, CMakeEngineHeaderFilesList.ToString(), Logger);
			bool bWriteProjectHeaders = WriteFileIfChanged(ProjectHeadersFilePath, CMakeProjectHeaderFilesList.ToString(), Logger);
			bool bWriteEngineSources = WriteFileIfChanged(EngineSourcesFilePath, CMakeEngineSourceFilesList.ToString(), Logger);
			bool bWriteProjectSources = WriteFileIfChanged(ProjectSourcesFilePath, CMakeProjectSourceFilesList.ToString(), Logger);
			bool bWriteIncludes = WriteFileIfChanged(IncludeFilePath, IncludeDirectoriesList.ToString(), Logger);
			bool bWriteDefinitions = WriteFileIfChanged(DefinitionsFilePath, PreprocessorDefinitionsList.ToString(), Logger);
			bool bWriteEngineConfigs = WriteFileIfChanged(EngineConfigsFilePath, CMakeEngineConfigFilesList.ToString(), Logger);
			bool bWriteProjectConfigs = WriteFileIfChanged(ProjectConfigsFilePath, CMakeProjectConfigFilesList.ToString(), Logger);
			bool bWriteEngineShaders = WriteFileIfChanged(EngineShadersFilePath, CMakeEngineShaderFilesList.ToString(), Logger);
			bool bWriteProjectShaders = WriteFileIfChanged(ProjectShadersFilePath, CMakeProjectShaderFilesList.ToString(), Logger);
			bool bWriteEngineCS = WriteFileIfChanged(EngineCSFilePath, CMakeEngineCSFilesList.ToString(), Logger);
			bool bWriteProjectCS = WriteFileIfChanged(ProjectCSFilePath, CMakeProjectCSFilesList.ToString(), Logger);

			// Return success flag if all files were written out successfully
			return bWriteMakeList &&
					bWriteEngineHeaders && bWriteProjectHeaders &&
					bWriteEngineSources && bWriteProjectSources &&
					bWriteEngineConfigs && bWriteProjectConfigs &&
					bWriteEngineCS && bWriteProjectCS &&
					bWriteEngineShaders && bWriteProjectShaders &&
					bWriteIncludes && bWriteDefinitions;
		}

		private static bool IsPathExcludedOnPlatform(string SourceFileRelativeToRoot)
		{
			if (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Linux)
			{
				return IsPathExcludedOnLinux(SourceFileRelativeToRoot);
			}
			else if (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Mac)
			{
				return IsPathExcludedOnMac(SourceFileRelativeToRoot);
			}
			else if (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Win64)
			{
				return IsPathExcludedOnWindows(SourceFileRelativeToRoot);
			}
			else
			{
				return false;
			}
		}

		private static bool IsPathExcludedOnLinux(string SourceFileRelativeToRoot)
		{
			// minimal filtering as it is helpful to be able to look up symbols from other platforms
			return SourceFileRelativeToRoot.Contains("Source/ThirdParty/", StringComparison.Ordinal);
		}

		private static bool IsPathExcludedOnMac(string SourceFileRelativeToRoot)
		{
			return SourceFileRelativeToRoot.Contains("Source/ThirdParty/", StringComparison.Ordinal) ||
				SourceFileRelativeToRoot.Contains("/Windows/", StringComparison.Ordinal) ||
				SourceFileRelativeToRoot.Contains("/Linux/", StringComparison.Ordinal) ||
				SourceFileRelativeToRoot.Contains("/VisualStudioSourceCodeAccess/", StringComparison.Ordinal) ||
				SourceFileRelativeToRoot.Contains("/WmfMedia/", StringComparison.Ordinal) ||
				SourceFileRelativeToRoot.Contains("/WindowsDeviceProfileSelector/", StringComparison.Ordinal) ||
				SourceFileRelativeToRoot.Contains("/WindowsMoviePlayer/", StringComparison.Ordinal) ||
				SourceFileRelativeToRoot.Contains("/WinRT/", StringComparison.Ordinal);
		}

		private static bool IsPathExcludedOnWindows(string SourceFileRelativeToRoot)
		{
			// minimal filtering as it is helpful to be able to look up symbols from other platforms
			return SourceFileRelativeToRoot.Contains("Source\\ThirdParty\\", StringComparison.Ordinal);
		}

		private bool IsTargetExcluded(string TargetName, UnrealTargetPlatform TargetPlatform, UnrealTargetConfiguration TargetConfig)
		{
			if (TargetPlatform == UnrealTargetPlatform.IOS || TargetPlatform == UnrealTargetPlatform.TVOS)
			{
				if (
					(
						TargetName.StartsWith("UnrealGame", StringComparison.Ordinal) ||
						(IsProjectBuild && TargetName.StartsWith(GameProjectName!, StringComparison.Ordinal)) || TargetName.StartsWith("QAGame", StringComparison.Ordinal)
					) && !TargetName.StartsWith("QAGameEditor", StringComparison.Ordinal))
				{
					return false;
				}
				return true;
			}
			// Only do this level of filtering if we are trying to speed things up tremendously
			if (bCmakeMinimalTargets)
			{
				// Editor or game builds get all target configs
				// The game project editor or game get all configs
				if ((TargetName.StartsWith("UnrealEditor", StringComparison.Ordinal) && !TargetName.StartsWith("UnrealEditorServices", StringComparison.Ordinal)) ||
					TargetName.StartsWith("UnrealGame", StringComparison.Ordinal) ||
					(IsProjectBuild && TargetName.StartsWith(GameProjectName!, StringComparison.Ordinal)))
				{
					return false;
				}
				// SCW & CRC are minimally included as just development builds
				else if (TargetConfig == UnrealTargetConfiguration.Development &&
					(TargetName.StartsWith("ShaderCompileWorker", StringComparison.Ordinal) || TargetName.StartsWith("CrashReportClient", StringComparison.Ordinal)))
				{
					return false;
				}
				else if ((TargetName.StartsWith("QAGameEditor", StringComparison.Ordinal) && !TargetName.StartsWith("QAGameEditorServices", StringComparison.Ordinal)) || TargetName.StartsWith("QAGame", StringComparison.Ordinal))
				{
					return false;
				}
				return true;
			}
			return false;
		}

		/// Adds the include directory to the list, after converting it to relative to Unreal root
		private static string GetIncludeDirectory(string IncludeDir, string ProjectDir)
		{
			string FullProjectPath = Path.GetFullPath(PrimaryProjectPath.FullName);
			string FullPath;
			// Check for paths outside of both the engine and the project
			if (Path.IsPathRooted(IncludeDir) &&
				!IncludeDir.StartsWith(FullProjectPath, StringComparison.Ordinal) &&
				!IncludeDir.StartsWith(Unreal.RootDirectory.FullName, StringComparison.Ordinal))
			{
				// Full path to a folder outside of project
				FullPath = IncludeDir;
			}
			else
			{
				FullPath = Path.GetFullPath(Path.Combine(ProjectDir, IncludeDir));
				if (!FullPath.StartsWith(Unreal.RootDirectory.FullName, StringComparison.Ordinal))
				{
					FullPath = Utils.MakePathRelativeTo(FullPath, FullProjectPath);
				}
				FullPath = FullPath.TrimEnd('/');
			}
			return FullPath;
		}

		#region ProjectFileGenerator implementation

		protected override bool WriteProjectFiles(PlatformProjectGeneratorCollection PlatformProjectGenerators, ILogger Logger)
		{
			return WriteCMakeLists(Logger);
		}

		/// <summary>
		/// This will filter out numerous targets to speed up cmake processing
		/// </summary>
		protected bool bCmakeMinimalTargets = false;

		/// <summary>
		/// Whether to include iOS targets or not
		/// </summary>
		protected bool bIncludeIOSTargets = false;

		/// <summary>
		/// Whether to include TVOS targets or not
		/// </summary>
		protected bool bIncludeTVOSTargets = false;

		protected override void ConfigureProjectFileGeneration(string[] Arguments, ref bool IncludeAllPlatforms, ILogger Logger)
		{
			base.ConfigureProjectFileGeneration(Arguments, ref IncludeAllPlatforms, Logger);
			// Check for minimal build targets to speed up cmake processing
			foreach (string CurArgument in Arguments)
			{
				switch (CurArgument.ToUpperInvariant())
				{
					case "-CMAKEMINIMALTARGETS":
						// To speed things up
						bIncludeDocumentation = false;
						bIncludeShaderSource = true;
						bIncludeTemplateFiles = false;
						bIncludeConfigFiles = true;
						// We want to filter out sets of targets to speed up builds via cmake
						bCmakeMinimalTargets = true;
						break;
				}
			}
		}

		/// <summary>
		/// Allocates a generator-specific project file object
		/// </summary>
		/// <param name="InitFilePath">Path to the project file</param>
		/// <param name="BaseDir">The base directory for files within this project</param>
		/// <returns>The newly allocated project file object</returns>
		protected override ProjectFile AllocateProjectFile(FileReference InitFilePath, DirectoryReference BaseDir)
		{
			return new CMakefileProjectFile(InitFilePath, BaseDir);
		}

		public override void CleanProjectFiles(DirectoryReference InPrimaryProjectDirectory, string InPrimaryProjectName, DirectoryReference InIntermediateProjectFilesDirectory, ILogger Logger)
		{
			// Remove Project File
			FileReference PrimaryProjectFile = FileReference.Combine(InPrimaryProjectDirectory, ProjectFileName);
			if (FileReference.Exists(PrimaryProjectFile))
			{
				FileReference.Delete(PrimaryProjectFile);
			}

			// Remove Headers Files
			FileReference EngineHeadersFile = FileReference.Combine(InIntermediateProjectFilesDirectory, CMakeEngineHeadersFileName);
			if (FileReference.Exists(EngineHeadersFile))
			{
				FileReference.Delete(EngineHeadersFile);
			}
			FileReference ProjectHeadersFile = FileReference.Combine(InIntermediateProjectFilesDirectory, CMakeProjectHeadersFileName);
			if (FileReference.Exists(ProjectHeadersFile))
			{
				FileReference.Delete(ProjectHeadersFile);
			}

			// Remove Sources Files
			FileReference EngineSourcesFile = FileReference.Combine(InIntermediateProjectFilesDirectory, CMakeEngineSourcesFileName);
			if (FileReference.Exists(EngineSourcesFile))
			{
				FileReference.Delete(EngineSourcesFile);
			}
			FileReference ProjectSourcesFile = FileReference.Combine(InIntermediateProjectFilesDirectory, CMakeProjectSourcesFileName);
			if (FileReference.Exists(ProjectSourcesFile))
			{
				FileReference.Delete(ProjectSourcesFile);
			}

			// Remove Includes File
			FileReference IncludeFile = FileReference.Combine(InIntermediateProjectFilesDirectory, CMakeIncludesFileName);
			if (FileReference.Exists(IncludeFile))
			{
				FileReference.Delete(IncludeFile);
			}

			// Remove CSharp Files
			FileReference EngineCSFile = FileReference.Combine(InIntermediateProjectFilesDirectory, CMakeEngineCSFileName);
			if (FileReference.Exists(EngineCSFile))
			{
				FileReference.Delete(EngineCSFile);
			}
			FileReference ProjectCSFile = FileReference.Combine(InIntermediateProjectFilesDirectory, CMakeProjectCSFileName);
			if (FileReference.Exists(ProjectCSFile))
			{
				FileReference.Delete(ProjectCSFile);
			}

			// Remove Config Files
			FileReference EngineConfigFile = FileReference.Combine(InIntermediateProjectFilesDirectory, CMakeEngineConfigsFileName);
			if (FileReference.Exists(EngineConfigFile))
			{
				FileReference.Delete(EngineConfigFile);
			}
			FileReference ProjectConfigsFile = FileReference.Combine(InIntermediateProjectFilesDirectory, CMakeProjectConfigsFileName);
			if (FileReference.Exists(ProjectConfigsFile))
			{
				FileReference.Delete(ProjectConfigsFile);
			}

			// Remove Config Files
			FileReference EngineShadersFile = FileReference.Combine(InIntermediateProjectFilesDirectory, CMakeEngineShadersFileName);
			if (FileReference.Exists(EngineShadersFile))
			{
				FileReference.Delete(EngineShadersFile);
			}
			FileReference ProjectShadersFile = FileReference.Combine(InIntermediateProjectFilesDirectory, CMakeProjectShadersFileName);
			if (FileReference.Exists(ProjectShadersFile))
			{
				FileReference.Delete(ProjectShadersFile);
			}

			// Remove Definitions File
			FileReference DefinitionsFile = FileReference.Combine(InIntermediateProjectFilesDirectory, CMakeDefinitionsFileName);
			if (FileReference.Exists(DefinitionsFile))
			{
				FileReference.Delete(DefinitionsFile);
			}
		}

		#endregion ProjectFileGenerator implementation
	}
}
