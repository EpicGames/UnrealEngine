// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace UnrealBuildTool;

internal class CSharpProjectCache(ILogger logger)
{
	/// <summary>
	/// Returns a partially cloned C# project so that properties like <see cref="ProjectFile.ShouldBuildForAllSolutionTargets"/> can be set.
	/// </summary>
	public VCSharpProjectFile this[FileReference projectPath]
		=> new(_cache.GetOrAdd(projectPath, pp => new Lazy<VCSharpProjectFile>(() => new VCSharpProjectFile(pp, _logger))).Value);

	private readonly ConcurrentDictionary<FileReference, Lazy<VCSharpProjectFile>> _cache = [];
	private readonly ILogger _logger = logger;
}
