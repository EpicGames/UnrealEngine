// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Xml;
using System.Xml.Linq;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace UnrealBuildTool
{
	enum CodeliteProjectFileFormat
	{
		CodeLite9,
		CodeLite10
	}

	class CodeLiteGenerator : ProjectFileGenerator
	{
		public string SolutionExtension = ".workspace";
		public string CodeCompletionFileName = "CodeCompletionFolders.txt";
		public string CodeCompletionPreProcessorFileName = "CodeLitePreProcessor.txt";
		readonly CodeliteProjectFileFormat ProjectFileFormat = CodeliteProjectFileFormat.CodeLite10;

		public CodeLiteGenerator(FileReference? InOnlyGameProject, CSharpProjectCache CSharpProjCache, CommandLineArguments CommandLine)
			: base(InOnlyGameProject, CSharpProjCache)
		{
			if (CommandLine.HasOption("-cl10"))
			{
				ProjectFileFormat = CodeliteProjectFileFormat.CodeLite10;
			}
			else if (CommandLine.HasOption("-cl9"))
			{
				ProjectFileFormat = CodeliteProjectFileFormat.CodeLite9;
			}
		}

		//
		// Returns CodeLite's project filename extension.
		//
		public override string ProjectFileExtension => ".project";
		protected override bool WritePrimaryProjectFile(ProjectFile? UBTProject, PlatformProjectGeneratorCollection PlatformProjectGenerators, ILogger Logger)
		{
			string SolutionFileName = PrimaryProjectName + SolutionExtension;
			string CodeCompletionFile = PrimaryProjectName + CodeCompletionFileName;
			string CodeCompletionPreProcessorFile = PrimaryProjectName + CodeCompletionPreProcessorFileName;

			string FullCodeLitePrimaryFile = Path.Combine(PrimaryProjectPath.FullName, SolutionFileName);
			string FullCodeLiteCodeCompletionFile = Path.Combine(PrimaryProjectPath.FullName, CodeCompletionFile);
			string FullCodeLiteCodeCompletionPreProcessorFile = Path.Combine(PrimaryProjectPath.FullName, CodeCompletionPreProcessorFile);

			//
			// HACK 
			// TODO This is for now a hack. According to the original submitter, Eranif (a CodeLite developer) will support code completion folders in *.workspace files.
			// We create a separate file with all the folder name in it to copy manually into the code completion
			// filed of CodeLite workspace. (Workspace Settings/Code Completion -> copy the content of the file threre.)
			List<string> IncludeDirectories = new List<string>();
			List<string> PreProcessor = new List<string>();

			foreach (ProjectFile CurProject in GeneratedProjectFiles)
			{
				CodeLiteProject? Project = CurProject as CodeLiteProject;
				if (Project == null)
				{
					continue;
				}

				foreach (string CurrentPath in Project.IntelliSenseIncludeSearchPaths)
				{
					// Convert relative path into absolute.
					DirectoryReference IntelliSenseIncludeSearchPath = DirectoryReference.Combine(Project.ProjectFilePath.Directory, CurrentPath);
					IncludeDirectories.Add(IntelliSenseIncludeSearchPath.FullName);
				}
				foreach (string CurrentPath in Project.IntelliSenseSystemIncludeSearchPaths)
				{
					// Convert relative path into absolute.
					DirectoryReference IntelliSenseSystemIncludeSearchPath = DirectoryReference.Combine(Project.ProjectFilePath.Directory, CurrentPath);
					IncludeDirectories.Add(IntelliSenseSystemIncludeSearchPath.FullName);
				}

				foreach (string CurDef in Project.IntelliSensePreprocessorDefinitions)
				{
					if (!PreProcessor.Contains(CurDef))
					{
						PreProcessor.Add(CurDef);
					}
				}
			}

			//
			// Write code completions data into files.
			//
			File.WriteAllLines(FullCodeLiteCodeCompletionFile, IncludeDirectories);
			File.WriteAllLines(FullCodeLiteCodeCompletionPreProcessorFile, PreProcessor);

			//
			// Write CodeLites Workspace
			//
			XmlWriterSettings settings = new XmlWriterSettings();
			settings.Indent = true;

			XElement CodeLiteWorkspace = new XElement("CodeLite_Workspace");
			XAttribute CodeLiteWorkspaceName = new XAttribute("Name", PrimaryProjectName);
			XAttribute CodeLiteWorkspaceSWTLW = new XAttribute("SWTLW", "Yes"); // This flag will only work in CodeLite version > 8.0. See below
			CodeLiteWorkspace.Add(CodeLiteWorkspaceName);
			CodeLiteWorkspace.Add(CodeLiteWorkspaceSWTLW);

			//
			// ATTN This part will work for the next release of CodeLite. That may
			// be CodeLite version > 8.0. CodeLite 8.0 does not have this functionality.
			// TODO Macros are ignored for now.
			//
			// Write Code Completion folders into the WorkspaceParserPaths section.
			//
			XElement CodeLiteWorkspaceParserPaths = new XElement("WorkspaceParserPaths");
			foreach (string CurrentPath in IncludeDirectories)
			{
				XElement CodeLiteWorkspaceParserPathInclude = new XElement("Include");
				XAttribute CodeLiteWorkspaceParserPath = new XAttribute("Path", CurrentPath);
				CodeLiteWorkspaceParserPathInclude.Add(CodeLiteWorkspaceParserPath);
				CodeLiteWorkspaceParserPaths.Add(CodeLiteWorkspaceParserPathInclude);

			}
			CodeLiteWorkspace.Add(CodeLiteWorkspaceParserPaths);

			//
			// Write project file information into CodeLite's workspace file.
			//
			XElement? CodeLiteWorkspaceTargetEngine = null;
			XElement? CodeLiteWorkspaceTargetPrograms = null;
			XElement? CodeLiteWorkspaceTargetGame = null;

			foreach (ProjectFile CurProject in AllProjectFiles)
			{
				string ProjectExtension = CurProject.ProjectFilePath.GetExtension();

				//
				// TODO For now ignore C# project files.
				//
				if (ProjectExtension == ".csproj")
				{
					continue;
				}

				//
				// Iterate through all targets.
				//
				foreach (ProjectTarget CurrentTarget in CurProject.ProjectTargets.OfType<ProjectTarget>())
				{
					string[] tmp = CurrentTarget.ToString().Split('.');
					string ProjectTargetFileName = CurProject.ProjectFilePath.Directory.MakeRelativeTo(PrimaryProjectPath) + "/" + tmp[0] + ProjectExtension;
					string ProjectName = tmp[0];

					XElement CodeLiteWorkspaceProject = new XElement("Project");
					XAttribute CodeLiteWorkspaceProjectName = new XAttribute("Name", ProjectName);
					XAttribute CodeLiteWorkspaceProjectPath = new XAttribute("Path", ProjectTargetFileName);
					XAttribute CodeLiteWorkspaceProjectActive = new XAttribute("Active", "No");
					CodeLiteWorkspaceProject.Add(CodeLiteWorkspaceProjectName);
					CodeLiteWorkspaceProject.Add(CodeLiteWorkspaceProjectPath);
					CodeLiteWorkspaceProject.Add(CodeLiteWorkspaceProjectActive);

					//
					// For CodeLite 10 we can use virtual folder to group projects.
					//
					if (ProjectFileFormat == CodeliteProjectFileFormat.CodeLite10)
					{
						if ((CurrentTarget.TargetRules!.Type == TargetType.Client) ||
							(CurrentTarget.TargetRules.Type == TargetType.Server) ||
							(CurrentTarget.TargetRules.Type == TargetType.Editor) ||
							(CurrentTarget.TargetRules.Type == TargetType.Game))
						{
							if (ProjectName.Equals("UnrealClient", StringComparison.Ordinal) ||
								ProjectName.Equals("UnrealServer", StringComparison.Ordinal) ||
								ProjectName.Equals("UnrealGame", StringComparison.Ordinal) ||
								ProjectName.Equals("UnrealEditor", StringComparison.Ordinal))
							{
								if (CodeLiteWorkspaceTargetEngine == null)
								{
									CodeLiteWorkspaceTargetEngine = new XElement("VirtualDirectory");
									XAttribute CodeLiteWorkspaceTargetEngineName = new XAttribute("Name", "Engine");
									CodeLiteWorkspaceTargetEngine.Add(CodeLiteWorkspaceTargetEngineName);
								}
								CodeLiteWorkspaceTargetEngine.Add(CodeLiteWorkspaceProject);
							}
							else
							{
								if (CodeLiteWorkspaceTargetGame == null)
								{
									CodeLiteWorkspaceTargetGame = new XElement("VirtualDirectory");
									XAttribute CodeLiteWorkspaceTargetGameName = new XAttribute("Name", "Game");
									CodeLiteWorkspaceTargetGame.Add(CodeLiteWorkspaceTargetGameName);
								}
								CodeLiteWorkspaceTargetGame.Add(CodeLiteWorkspaceProject);
							}
						}
						else if (CurrentTarget.TargetRules.Type == TargetType.Program)
						{
							if (CodeLiteWorkspaceTargetPrograms == null)
							{
								CodeLiteWorkspaceTargetPrograms = new XElement("VirtualDirectory");
								XAttribute CodeLiteWorkspaceTargetProgramsName = new XAttribute("Name", "Programs");
								CodeLiteWorkspaceTargetPrograms.Add(CodeLiteWorkspaceTargetProgramsName);
							}
							CodeLiteWorkspaceTargetPrograms.Add(CodeLiteWorkspaceProject);
						}
					}
					else if (ProjectFileFormat == CodeliteProjectFileFormat.CodeLite9)
					{
						CodeLiteWorkspace.Add(CodeLiteWorkspaceProject);
					}
				}
			}
			if (ProjectFileFormat == CodeliteProjectFileFormat.CodeLite10)
			{

				if (CodeLiteWorkspaceTargetEngine != null)
				{
					CodeLiteWorkspace.Add(CodeLiteWorkspaceTargetEngine);
				}
				if (CodeLiteWorkspaceTargetPrograms != null)
				{
					CodeLiteWorkspace.Add(CodeLiteWorkspaceTargetPrograms);
				}
				if (CodeLiteWorkspaceTargetGame != null)
				{
					CodeLiteWorkspace.Add(CodeLiteWorkspaceTargetGame);
				}
			}
			//
			// We need to create the configuration matrix. That will assign the project configuration to 
			// the samge workspace configuration.
			//
			XElement CodeLiteWorkspaceBuildMatrix = new XElement("BuildMatrix");
			foreach (UnrealTargetConfiguration CurConfiguration in SupportedConfigurations)
			{
				if (InstalledPlatformInfo.IsValidConfiguration(CurConfiguration, EProjectType.Code))
				{
					XElement CodeLiteWorkspaceBuildMatrixConfiguration = new XElement("WorkspaceConfiguration");
					XAttribute CodeLiteWorkspaceProjectName = new XAttribute("Name", CurConfiguration.ToString());
					XAttribute CodeLiteWorkspaceProjectSelected = new XAttribute("Selected", "no");
					CodeLiteWorkspaceBuildMatrixConfiguration.Add(CodeLiteWorkspaceProjectName);
					CodeLiteWorkspaceBuildMatrixConfiguration.Add(CodeLiteWorkspaceProjectSelected);

					foreach (ProjectFile CurProject in AllProjectFiles)
					{
						string ProjectExtension = CurProject.ProjectFilePath.GetExtension();

						//
						// TODO For now ignore C# project files.
						//
						if (ProjectExtension == ".csproj")
						{
							continue;
						}

						foreach (ProjectTarget target in CurProject.ProjectTargets.OfType<ProjectTarget>())
						{
							string[] tmp = target.ToString().Split('.');
							string ProjectName = tmp[0];

							XElement CodeLiteWorkspaceBuildMatrixConfigurationProject = new XElement("Project");
							XAttribute CodeLiteWorkspaceBuildMatrixConfigurationProjectName = new XAttribute("Name", ProjectName);
							XAttribute CodeLiteWorkspaceBuildMatrixConfigurationProjectConfigName = new XAttribute("ConfigName", CurConfiguration.ToString());
							CodeLiteWorkspaceBuildMatrixConfigurationProject.Add(CodeLiteWorkspaceBuildMatrixConfigurationProjectName);
							CodeLiteWorkspaceBuildMatrixConfigurationProject.Add(CodeLiteWorkspaceBuildMatrixConfigurationProjectConfigName);
							CodeLiteWorkspaceBuildMatrixConfiguration.Add(CodeLiteWorkspaceBuildMatrixConfigurationProject);
						}
					}
					CodeLiteWorkspaceBuildMatrix.Add(CodeLiteWorkspaceBuildMatrixConfiguration);
				}
			}

			CodeLiteWorkspace.Add(CodeLiteWorkspaceBuildMatrix);
			CodeLiteWorkspace.Save(FullCodeLitePrimaryFile);

			return true;
		}

		protected override ProjectFile AllocateProjectFile(FileReference InitFilePath, DirectoryReference BaseDir)
		{
			return new CodeLiteProject(InitFilePath, BaseDir, OnlyGameProject);
		}

		public override void CleanProjectFiles(DirectoryReference InPrimaryProjectDirectory, string InPrimaryProjectName, DirectoryReference InIntermediateProjectFilesDirectory, ILogger Logger)
		{
			// TODO Delete all files here. Not finished yet.
			string SolutionFileName = InPrimaryProjectName + SolutionExtension;
			string CodeCompletionFile = InPrimaryProjectName + CodeCompletionFileName;
			string CodeCompletionPreProcessorFile = InPrimaryProjectName + CodeCompletionPreProcessorFileName;

			FileReference FullCodeLitePrimaryFile = FileReference.Combine(InPrimaryProjectDirectory, SolutionFileName);
			FileReference FullCodeLiteCodeCompletionFile = FileReference.Combine(InPrimaryProjectDirectory, CodeCompletionFile);
			FileReference FullCodeLiteCodeCompletionPreProcessorFile = FileReference.Combine(InPrimaryProjectDirectory, CodeCompletionPreProcessorFile);

			if (FileReference.Exists(FullCodeLitePrimaryFile))
			{
				FileReference.Delete(FullCodeLitePrimaryFile);
			}
			if (FileReference.Exists(FullCodeLiteCodeCompletionFile))
			{
				FileReference.Delete(FullCodeLiteCodeCompletionFile);
			}
			if (FileReference.Exists(FullCodeLiteCodeCompletionPreProcessorFile))
			{
				FileReference.Delete(FullCodeLiteCodeCompletionPreProcessorFile);
			}

			// Delete the project files folder
			if (DirectoryReference.Exists(InIntermediateProjectFilesDirectory))
			{
				try
				{
					Directory.Delete(InIntermediateProjectFilesDirectory.FullName, true);
				}
				catch (Exception Ex)
				{
					Logger.LogInformation("Error while trying to clean project files path {InIntermediateProjectFilesDirectory}. Ignored.", InIntermediateProjectFilesDirectory);
					Logger.LogInformation("\t{Ex}", Ex.Message);
				}
			}
		}
	}
}
