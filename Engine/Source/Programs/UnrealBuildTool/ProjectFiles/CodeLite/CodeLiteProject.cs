// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Xml.Linq;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	class CodeLiteProject : ProjectFile
	{
		readonly FileReference? OnlyGameProject;

		public CodeLiteProject(FileReference InitFilePath, DirectoryReference BaseDir, FileReference? InOnlyGameProject) : base(InitFilePath, BaseDir)
		{
			OnlyGameProject = InOnlyGameProject;
		}

		public override bool WriteProjectFile(List<UnrealTargetPlatform> InPlatforms, List<UnrealTargetConfiguration> InConfigurations, PlatformProjectGeneratorCollection PlatformProjectGenerators, ILogger Logger)
		{
			bool bSuccess = false;
			string ProjectNameRaw = ProjectFilePath.GetFileNameWithoutExtension();
			//string ProjectPath = ProjectFilePath.FullName;
			string ProjectExtension = ProjectFilePath.GetExtension();
			string ProjectPlatformName = BuildHostPlatform.Current.Platform.ToString();

			// Get the output directory
			string EngineRootDirectory = Unreal.EngineDirectory.FullName;

			//
			// Build the working directory of the Game executable.
			//

			string GameWorkingDirectory = "";
			if (OnlyGameProject != null)
			{
				GameWorkingDirectory = Path.Combine(Path.GetDirectoryName(OnlyGameProject.FullName)!, "Binaries", ProjectPlatformName);
			}
			//
			// Build the working directory of the UnrealEditor executable.
			//
			string UnrealEngineEditorWorkingDirectory = Path.Combine(EngineRootDirectory, "Binaries", ProjectPlatformName);

			//
			// Create the folder where the project files goes if it does not exist
			//
			string FilePath = Path.GetDirectoryName(ProjectFilePath.FullName)!;
			if ((FilePath.Length > 0) && !Directory.Exists(FilePath))
			{
				Directory.CreateDirectory(FilePath);
			}

			string GameProjectFile = "";
			if (OnlyGameProject != null)
			{
				GameProjectFile = OnlyGameProject.FullName;
			}

			//
			// Write all targets which will be separate projects.
			//
			foreach (Project target in ProjectTargets)
			{
				string[] tmp = target.ToString()!.Split('.');
				string ProjectTargetFileName = Path.GetDirectoryName(ProjectFilePath.FullName) + "/" + tmp[0] + ProjectExtension;
				string TargetName = tmp[0];
				TargetType ProjectTargetType = target.TargetRules!.Type;

				//
				// Create the CodeLites root element.
				//
				XElement CodeLiteProject = new XElement("CodeLite_Project");
				XAttribute CodeLiteProjectAttributeName = new XAttribute("Name", TargetName);
				CodeLiteProject.Add(CodeLiteProjectAttributeName);

				//
				// Select only files we want to add.
				// TODO Maybe skipping those files directly in the following foreach loop is faster?
				//
				List<SourceFile> FilterSourceFile = SourceFiles.FindAll(s => (
					s.Reference.HasExtension(".h")
					|| s.Reference.HasExtension(".cpp")
					|| s.Reference.HasExtension(".c")
					|| s.Reference.HasExtension(".cs")
					|| s.Reference.HasExtension(".uproject")
					|| s.Reference.HasExtension(".uplugin")
					|| s.Reference.HasExtension(".ini")
					|| s.Reference.HasExtension(".usf")
					|| s.Reference.HasExtension(".ush")
				));

				//
				// Find/Create the correct virtual folder and place the file into it.
				//
				foreach (SourceFile CurrentFile in FilterSourceFile)
				{
					//
					// Try to get the correct relative folder representation for the project.
					//
					string CurrentFilePath = "";
					// TODO It seems that the full pathname doesn't work for some files like .ini, .usf, .ush
					if ((ProjectTargetType == TargetType.Client) ||
						(ProjectTargetType == TargetType.Editor) ||
						(ProjectTargetType == TargetType.Game) ||
						(ProjectTargetType == TargetType.Server))
					{
						if (TargetName.Equals("UnrealClient", StringComparison.Ordinal) ||
							TargetName.Equals("UnrealServer", StringComparison.Ordinal) ||
							TargetName.Equals("UnrealGame", StringComparison.Ordinal) ||
							TargetName.Equals("UnrealEditor", StringComparison.Ordinal))
						{
							int Idx = Unreal.EngineDirectory.FullName.Length;
							CurrentFilePath = Path.GetDirectoryName(Path.GetFullPath(CurrentFile.Reference.FullName))!.Substring(Idx);
						}
						else
						{
							int Idx = Path.GetDirectoryName(CurrentFile.Reference.FullName)!.IndexOf(ProjectNameRaw, StringComparison.Ordinal) + ProjectNameRaw.Length;
							CurrentFilePath = Path.GetDirectoryName(CurrentFile.Reference.FullName)!.Substring(Idx);
						}
					}
					else if (ProjectTargetType == TargetType.Program)
					{
						//
						// We do not need all the editors subfolders to show the content. Find the correct programs subfolder.
						//
						int Idx = Path.GetDirectoryName(CurrentFile.Reference.FullName)!.IndexOf(TargetName, StringComparison.Ordinal) + TargetName.Length;
						CurrentFilePath = Path.GetDirectoryName(CurrentFile.Reference.FullName)!.Substring(Idx);
					}

					char[] Delimiters = new char[] { '/', '\\' };
					string[] SplitFolders = CurrentFilePath.Split(Delimiters, StringSplitOptions.RemoveEmptyEntries);
					//
					// Set the CodeLite root folder again.
					//
					XElement root = CodeLiteProject;

					//
					// Iterate through all XElement virtual folders until we find the right place to put the file.
					// TODO this looks more like a hack to me.
					//
					foreach (string FolderName in SplitFolders)
					{
						if (String.IsNullOrEmpty(FolderName))
						{
							continue;
						}

						//
						// Let's look if there is a virtual folder withint the current XElement.
						//
						IEnumerable<XElement> tests = root.Elements("VirtualDirectory");
						if (!tests.Any())
						{
							//
							// No, then we have to create.
							//
							XElement vf = new XElement("VirtualDirectory");
							XAttribute vfn = new XAttribute("Name", FolderName);
							vf.Add(vfn);
							root.Add(vf);
							root = vf;
						}
						else
						{
							//
							// Yes, then let's find the correct sub XElement.
							//
							bool notfound = true;

							//
							// We have some virtual directories let's find the correct one.
							//
							foreach (XElement element in tests)
							{
								//
								// Look the the following folder
								XAttribute? attribute = element.Attribute("Name");
								if (attribute?.Value == FolderName)
								{
									// Ok, we found the folder as subfolder, let's use it.
									root = element;
									notfound = false;
									break;
								}
							}
							//
							// If we are here we didn't find any XElement with that subfolder, then we have to create.
							//
							if (notfound)
							{
								XElement vf = new XElement("VirtualDirectory");
								XAttribute vfn = new XAttribute("Name", FolderName);
								vf.Add(vfn);
								root.Add(vf);
								root = vf;
							}
						}
					}

					//
					// If we are at this point we found the correct XElement folder
					//
					XElement file = new XElement("File");
					XAttribute fileAttribute = new XAttribute("Name", CurrentFile.Reference.FullName);
					file.Add(fileAttribute);
					root.Add(file);
				}

				XElement CodeLiteSettings = new XElement("Settings");
				CodeLiteProject.Add(CodeLiteSettings);

				XElement CodeLiteGlobalSettings = new XElement("GlobalSettings");
				CodeLiteProject.Add(CodeLiteGlobalSettings);

				foreach (UnrealTargetConfiguration CurConf in InConfigurations)
				{
					XElement CodeLiteConfiguration = new XElement("Configuration");
					XAttribute CodeLiteConfigurationName = new XAttribute("Name", CurConf.ToString());
					CodeLiteConfiguration.Add(CodeLiteConfigurationName);

					//
					// Create Configuration General part. 
					//
					XElement CodeLiteConfigurationGeneral = new XElement("General");

					//
					// Create the executable filename.
					//
					string ExecutableToRun = "";
					string PlatformConfiguration = "-" + ProjectPlatformName + "-" + CurConf.ToString();
					if (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Linux)
					{
						ExecutableToRun = "./" + TargetName;
						if ((ProjectTargetType == TargetType.Game) ||
							(ProjectTargetType == TargetType.Program))
						{
							if (CurConf != UnrealTargetConfiguration.Development)
							{
								ExecutableToRun += PlatformConfiguration;
							}
						}
						else if (ProjectTargetType == TargetType.Editor)
						{
							ExecutableToRun = "./UnrealEditor";
							if ((CurConf == UnrealTargetConfiguration.Debug) ||
								(CurConf == UnrealTargetConfiguration.Shipping) ||
								(CurConf == UnrealTargetConfiguration.Test))
							{
								ExecutableToRun += PlatformConfiguration;
							}
						}
					}
					else if (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Mac)
					{
						ExecutableToRun = "./" + TargetName;
						if ((ProjectTargetType == TargetType.Game) || (ProjectTargetType == TargetType.Program))
						{
							if (CurConf != UnrealTargetConfiguration.Development)
							{
								ExecutableToRun += PlatformConfiguration;
							}
							ExecutableToRun += ".app/Contents/MacOS/" + TargetName;
							if (CurConf != UnrealTargetConfiguration.Development)
							{
								ExecutableToRun += PlatformConfiguration;
							}
						}
						else if (ProjectTargetType == TargetType.Editor)
						{
							ExecutableToRun = "./UnrealEditor";
							if ((CurConf == UnrealTargetConfiguration.Debug) ||
								(CurConf == UnrealTargetConfiguration.Shipping) ||
								(CurConf == UnrealTargetConfiguration.Test))
							{
								ExecutableToRun += PlatformConfiguration;
							}
							ExecutableToRun += ".app/Contents/MacOS/UnrealEditor";
							if ((CurConf != UnrealTargetConfiguration.Development) && (CurConf != UnrealTargetConfiguration.DebugGame))
							{
								ExecutableToRun += PlatformConfiguration;
							}
						}
					}
					else if (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Win64)
					{
						ExecutableToRun = TargetName;
						if ((ProjectTargetType == TargetType.Game) || (ProjectTargetType == TargetType.Program))
						{
							if (CurConf != UnrealTargetConfiguration.Development)
							{
								ExecutableToRun += PlatformConfiguration;
							}
						}
						else if (ProjectTargetType == TargetType.Editor)
						{
							ExecutableToRun = "UnrealEditor";
							if ((CurConf == UnrealTargetConfiguration.Debug) ||
								(CurConf == UnrealTargetConfiguration.Shipping) ||
								(CurConf == UnrealTargetConfiguration.Test))
							{
								ExecutableToRun += PlatformConfiguration;
							}
						}

						ExecutableToRun += ".exe";

					}
					else
					{
						throw new BuildException("Unsupported platform.");
					}

					// Is this project a Game type?
					XAttribute GeneralExecutableToRun = new XAttribute("Command", ExecutableToRun);
					if (ProjectTargetType == TargetType.Game)
					{
						if (CurConf.ToString().Contains("Debug", StringComparison.Ordinal))
						{
							string commandArguments = " -debug";
							XAttribute GeneralExecutableToRunArguments = new XAttribute("CommandArguments", commandArguments);
							CodeLiteConfigurationGeneral.Add(GeneralExecutableToRunArguments);
						}
						if (TargetName.Equals("UnrealGame", StringComparison.Ordinal))
						{
							XAttribute GeneralExecutableWorkingDirectory = new XAttribute("WorkingDirectory", UnrealEngineEditorWorkingDirectory);
							CodeLiteConfigurationGeneral.Add(GeneralExecutableWorkingDirectory);
						}
						else
						{
							XAttribute GeneralExecutableWorkingDirectory = new XAttribute("WorkingDirectory", GameWorkingDirectory);
							CodeLiteConfigurationGeneral.Add(GeneralExecutableWorkingDirectory);
						}
					}
					else if (ProjectTargetType == TargetType.Editor)
					{
						if (TargetName != "UnrealEditor" && !String.IsNullOrEmpty(GameProjectFile))
						{
							string commandArguments = "\"" + GameProjectFile + "\"" + " -game";
							XAttribute CommandArguments = new XAttribute("CommandArguments", commandArguments);
							CodeLiteConfigurationGeneral.Add(CommandArguments);
						}
						XAttribute WorkingDirectory = new XAttribute("WorkingDirectory", UnrealEngineEditorWorkingDirectory);
						CodeLiteConfigurationGeneral.Add(WorkingDirectory);
					}
					else if (ProjectTargetType == TargetType.Program)
					{
						XAttribute WorkingDirectory = new XAttribute("WorkingDirectory", UnrealEngineEditorWorkingDirectory);
						CodeLiteConfigurationGeneral.Add(WorkingDirectory);
					}
					else if (ProjectTargetType == TargetType.Client)
					{
						XAttribute WorkingDirectory = new XAttribute("WorkingDirectory", UnrealEngineEditorWorkingDirectory);
						CodeLiteConfigurationGeneral.Add(WorkingDirectory);
					}
					else if (ProjectTargetType == TargetType.Server)
					{
						XAttribute WorkingDirectory = new XAttribute("WorkingDirectory", UnrealEngineEditorWorkingDirectory);
						CodeLiteConfigurationGeneral.Add(WorkingDirectory);
					}
					CodeLiteConfigurationGeneral.Add(GeneralExecutableToRun);

					CodeLiteConfiguration.Add(CodeLiteConfigurationGeneral);

					//
					// End of Create Configuration General part. 
					//

					//
					// Create Configuration Custom Build part. 
					//
					XElement CodeLiteConfigurationCustomBuild = new XElement("CustomBuild");
					CodeLiteConfiguration.Add(CodeLiteConfigurationGeneral);
					XAttribute CodeLiteConfigurationCustomBuildEnabled = new XAttribute("Enabled", "yes");
					CodeLiteConfigurationCustomBuild.Add(CodeLiteConfigurationCustomBuildEnabled);

					//
					// Add the working directory for the custom build commands.
					//
					XElement CustomBuildWorkingDirectory = new XElement("WorkingDirectory");
					XText CustuomBuildWorkingDirectory = new XText(Unreal.UnrealBuildToolDllPath.Directory.FullName);
					CustomBuildWorkingDirectory.Add(CustuomBuildWorkingDirectory);
					CodeLiteConfigurationCustomBuild.Add(CustomBuildWorkingDirectory);

					//
					// End of Add the working directory for the custom build commands.
					//

					//
					// Make Build Target.
					//
					XElement CustomBuildCommand = new XElement("BuildCommand");
					CodeLiteConfigurationCustomBuild.Add(CustomBuildCommand);

					string BuildTarget = TargetName + " " + ProjectPlatformName + " " + CurConf.ToString();
					if (BuildHostPlatform.Current.Platform != UnrealTargetPlatform.Win64)
					{
						string PlatformName = "Linux";
						if (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Mac)
						{
							PlatformName = "Mac";
						}

						BuildTarget = Path.Combine(Unreal.EngineDirectory.FullName, "Build/BatchFiles", PlatformName, "Build.sh") + " " + BuildTarget;
					}
					else
					{
						BuildTarget = $"{Unreal.DotnetPath} \"{Unreal.UnrealBuildToolDllPath}\" {BuildTarget}";
					}

					if (GameProjectFile.Length > 0)
					{
						BuildTarget += " -project=" + "\"" + GameProjectFile + "\"";
					}

					XText commandLine = new XText(BuildTarget);
					CustomBuildCommand.Add(commandLine);

					//
					// End of Make Build Target
					//

					//
					// Clean Build Target.
					//
					XElement CustomCleanCommand = new XElement("CleanCommand");
					CodeLiteConfigurationCustomBuild.Add(CustomCleanCommand);

					string CleanTarget = BuildTarget + " -clean";
					XText CleanCommandLine = new XText(CleanTarget);

					CustomCleanCommand.Add(CleanCommandLine);

					//
					// End of Clean Build Target.
					//

					//
					// Rebuild Build Target.
					//
					XElement CustomRebuildCommand = new XElement("RebuildCommand");
					CodeLiteConfigurationCustomBuild.Add(CustomRebuildCommand);

					string RebuildTarget = CleanTarget + "\n" + BuildTarget;
					XText RebuildCommandLine = new XText(RebuildTarget);

					CustomRebuildCommand.Add(RebuildCommandLine);

					//
					// End of Clean Build Target.
					//

					//
					// Some other fun Custom Targets.
					//
					if (ProjectTargetType == TargetType.Game)
					{
						string CookGameCommandLine = "dotnet AutomationTool.dll BuildCookRun ";

						// Projects filename
						if (OnlyGameProject != null)
						{
							CookGameCommandLine += "-project=\"" + OnlyGameProject.FullName + "\" ";
						}

						// Disables Perforce functionality 
						CookGameCommandLine += "-noP4 ";

						// Do not kill any spawned processes on exit
						CookGameCommandLine += "-nokill ";
						CookGameCommandLine += "-clientconfig=" + CurConf.ToString() + " ";
						CookGameCommandLine += "-serverconfig=" + CurConf.ToString() + " ";
						CookGameCommandLine += "-platform=" + ProjectPlatformName + " ";
						CookGameCommandLine += "-targetplatform=" + ProjectPlatformName + " "; // TODO Maybe I can add all the supported one.
						CookGameCommandLine += "-nocompile ";
						CookGameCommandLine += "-compressed -stage -deploy";

						//
						// Cook Game.
						//
						XElement CookGame = new XElement("Target");
						XAttribute CookGameName = new XAttribute("Name", "Cook Game");
						XText CookGameCommand = new XText(CookGameCommandLine + " -cook");
						CookGame.Add(CookGameName);
						CookGame.Add(CookGameCommand);
						CodeLiteConfigurationCustomBuild.Add(CookGame);

						XElement CookGameOnTheFly = new XElement("Target");
						XAttribute CookGameNameOnTheFlyName = new XAttribute("Name", "Cook Game on the fly");
						XText CookGameOnTheFlyCommand = new XText(CookGameCommandLine + " -cookonthefly");
						CookGameOnTheFly.Add(CookGameNameOnTheFlyName);
						CookGameOnTheFly.Add(CookGameOnTheFlyCommand);
						CodeLiteConfigurationCustomBuild.Add(CookGameOnTheFly);

						XElement SkipCook = new XElement("Target");
						XAttribute SkipCookName = new XAttribute("Name", "Skip Cook Game");
						XText SkipCookCommand = new XText(CookGameCommandLine + " -skipcook");
						SkipCook.Add(SkipCookName);
						SkipCook.Add(SkipCookCommand);
						CodeLiteConfigurationCustomBuild.Add(SkipCook);

					}
					//
					// End of Some other fun Custom Targets.
					//
					CodeLiteConfiguration.Add(CodeLiteConfigurationCustomBuild);

					//
					// End of Create Configuration Custom Build part. 
					//

					CodeLiteSettings.Add(CodeLiteConfiguration);
				}

				CodeLiteSettings.Add(CodeLiteGlobalSettings);

				//
				// Save the XML file.
				//
				CodeLiteProject.Save(ProjectTargetFileName);

				bSuccess = true;
			}
			return bSuccess;
		}
	}
}
