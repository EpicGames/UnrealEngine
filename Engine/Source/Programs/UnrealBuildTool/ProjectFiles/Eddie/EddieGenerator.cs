// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Text;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace UnrealBuildTool
{
	class EddieProjectFileGenerator : ProjectFileGenerator
	{
		public EddieProjectFileGenerator(FileReference? InOnlyGameProject, CSharpProjectCache CSharpProjCache)
			: base(InOnlyGameProject, CSharpProjCache)
		{
		}

		public override string ProjectFileExtension => ".wkst";

		public override void CleanProjectFiles(DirectoryReference InPrimaryProjectDirectory, string InPrimaryProjectName, DirectoryReference InIntermediateProjectFilesPath, ILogger Logger)
		{
			FileReference PrimaryProjDeleteFilename = FileReference.Combine(InPrimaryProjectDirectory, InPrimaryProjectName + ".wkst");
			if (FileReference.Exists(PrimaryProjDeleteFilename))
			{
				File.Delete(PrimaryProjDeleteFilename.FullName);
			}

			// Delete the project files folder
			if (DirectoryReference.Exists(InIntermediateProjectFilesPath))
			{
				try
				{
					Directory.Delete(InIntermediateProjectFilesPath.FullName, true);
				}
				catch (Exception Ex)
				{
					Logger.LogInformation("Error while trying to clean project files path {InIntermediateProjectFilesPath}. Ignored.", InIntermediateProjectFilesPath);
					Logger.LogInformation("\t{Ex}", Ex.Message);
				}
			}
		}

		protected override ProjectFile AllocateProjectFile(FileReference InitFilePath, DirectoryReference BaseDir)
		{
			return new EddieProjectFile(InitFilePath, BaseDir);
		}

		private bool WriteEddieWorkset(ILogger Logger)
		{
			bool bSuccess = false;

			StringBuilder WorksetDataContent = new StringBuilder();
			WorksetDataContent.Append("# @Eddie Workset@" + ProjectFileGenerator.NewLine);
			WorksetDataContent.Append("AddWorkset \"" + PrimaryProjectName + ".wkst\" \"" + PrimaryProjectPath + "\"" + ProjectFileGenerator.NewLine);

#pragma warning disable IDE0059 // Remove unnecessary value assignment - this is a false positive, the null assignment is required as the lambda is recursive.
			System.Action<String /*Path*/, List<PrimaryProjectFolder> /* Folders */>? AddProjectsFunction = null;
			AddProjectsFunction = (Path, FolderList) =>
				{
					foreach (PrimaryProjectFolder CurFolder in FolderList)
					{
						string NewPath = Path + "/" + CurFolder.FolderName;
						WorksetDataContent.Append("AddFileGroup \"" + NewPath + "\" \"" + CurFolder.FolderName + "\"" + ProjectFileGenerator.NewLine);

						AddProjectsFunction!(NewPath, CurFolder.SubFolders);

						foreach (ProjectFile CurProject in CurFolder.ChildProjects)
						{
							EddieProjectFile? EddieProject = CurProject as EddieProjectFile;
							if (EddieProject != null)
							{
								WorksetDataContent.Append("AddFile \"" + EddieProject.ToString() + "\" \"" + EddieProject.ProjectFilePath + "\"" + ProjectFileGenerator.NewLine);
							}
						}

						WorksetDataContent.Append("EndFileGroup \"" + NewPath + "\"" + ProjectFileGenerator.NewLine);
					}
				};
#pragma warning restore IDE0059
			AddProjectsFunction(PrimaryProjectName, RootFolder.SubFolders);

			string ProjectName = PrimaryProjectName;
			string FilePath = PrimaryProjectPath + "/" + ProjectName + ".wkst";

			bSuccess = WriteFileIfChanged(FilePath, WorksetDataContent.ToString(), Logger, new UTF8Encoding());

			return bSuccess;
		}

		protected override bool WritePrimaryProjectFile(ProjectFile? UBTProject, PlatformProjectGeneratorCollection PlatformProjectGenerators, ILogger Logger)
		{
			return WriteEddieWorkset(Logger);
		}

		protected override void ConfigureProjectFileGeneration(string[] Arguments, ref bool IncludeAllPlatforms, ILogger Logger)
		{
			// Call parent implementation first
			base.ConfigureProjectFileGeneration(Arguments, ref IncludeAllPlatforms, Logger);

			if (bGeneratingGameProjectFiles)
			{
				bIncludeEngineSource = true;
			}
		}
	}
}