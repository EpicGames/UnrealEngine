// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.IO;
using System.Text;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace UnrealBuildTool
{

	class EddieSourceFile : ProjectFile.SourceFile
	{
		/// <summary>
		/// Constructor
		/// </summary>
		public EddieSourceFile(FileReference InitFilePath, DirectoryReference? InitRelativeBaseFolder)
			: base(InitFilePath, InitRelativeBaseFolder)
		{
		}
	}

	class EddieFolder
	{
		public EddieFolder(string InName, string InPath)
		{
			Name = InName;
			Path = InPath;
			FullPath = null;
			bIsModuleFolder = false;
		}

		public string Name;
		public string Path;
		public string? FullPath;
		public string? WorksetPath;
		public bool bIsModuleFolder;
		public Dictionary<string, EddieFolder> Folders = new Dictionary<string, EddieFolder>();
		public List<EddieSourceFile> Files = new List<EddieSourceFile>();
	}

	class EddieProjectFile : ProjectFile
	{
		Dictionary<string, EddieFolder> Folders = new Dictionary<string, EddieFolder>();

		public EddieProjectFile(FileReference InitFilePath, DirectoryReference BaseDir)
			: base(InitFilePath, BaseDir)
		{
		}

		public override string ToString()
		{
			return ProjectFilePath.GetFileNameWithoutExtension();
		}

		public override SourceFile? AllocSourceFile(FileReference InitFilePath, DirectoryReference? InitProjectSubFolder)
		{
			if (InitFilePath.GetFileName().StartsWith('.'))
			{
				return null;
			}
			return new EddieSourceFile(InitFilePath, InitProjectSubFolder);
		}

		public static EddieFolder? FindFolderByRelativePath(ref Dictionary<string, EddieFolder> Groups, string RelativePath)
		{
			string[] Parts = RelativePath.Split(Path.DirectorySeparatorChar);
			string CurrentPath = "";
			Dictionary<string, EddieFolder> CurrentParent = Groups;

			foreach (string Part in Parts)
			{
				EddieFolder CurrentGroup;

				if (!System.String.IsNullOrEmpty(CurrentPath))
				{
					CurrentPath += Path.DirectorySeparatorChar;
				}

				CurrentPath += Part;

				if (!CurrentParent.ContainsKey(CurrentPath))
				{
					CurrentGroup = new EddieFolder(Path.GetFileName(CurrentPath), CurrentPath);
					CurrentParent.Add(CurrentPath, CurrentGroup);
				}
				else
				{
					CurrentGroup = CurrentParent[CurrentPath];
				}

				if (CurrentPath == RelativePath)
				{
					return CurrentGroup;
				}

				CurrentParent = CurrentGroup.Folders;
			}

			return null;
		}

		private void ParseSourceFilesIntoGroups()
		{
			foreach (SourceFile CurSourceFile in SourceFiles)
			{
				EddieSourceFile SourceFile = (EddieSourceFile)CurSourceFile;
				string FileName = SourceFile.Reference.GetFileName();
				//string FileExtension = Path.GetExtension(FileName);
				//string FilePath = SourceFile.Reference.MakeRelativeTo(ProjectFilePath.Directory);
				//string FilePathMac = Utils.CleanDirectorySeparators(FilePath, '/');

				string ProjectRelativeSourceFile = CurSourceFile.Reference.MakeRelativeTo(ProjectFilePath.Directory);
				string RelativeSourceDirectory = Path.GetDirectoryName(ProjectRelativeSourceFile)!;
				// Use the specified relative base folder
				if (CurSourceFile.BaseFolder != null)   // NOTE: We are looking for null strings, not empty strings!
				{
					RelativeSourceDirectory = Path.GetDirectoryName(CurSourceFile.Reference.MakeRelativeTo(CurSourceFile.BaseFolder))!;
				}
				EddieFolder? Group = FindFolderByRelativePath(ref Folders, RelativeSourceDirectory);
				if (Group != null)
				{
					if (FileName.EndsWith(".build.cs", System.StringComparison.Ordinal) ||
						FileName.EndsWith(".Build.cs", System.StringComparison.Ordinal) ||
						FileName.EndsWith(".csproj", System.StringComparison.Ordinal))
					{
						Group.bIsModuleFolder = true;
						Group.WorksetPath = ProjectFilePath.FullName + "." + Group.Name + ".wkst";
					}

					Group.FullPath = Path.GetDirectoryName(SourceFile.Reference.FullName);
					Group.Files.Add(SourceFile);
				}
			}
		}

		private static void EmitProject(StringBuilder Content, Dictionary<string, EddieFolder> Folders, ILogger Logger)
		{
			foreach (KeyValuePair<string, EddieFolder> CurGroup in Folders)
			{
				if (Path.GetFileName(CurGroup.Key) != "Documentation")
				{
					Content.Append("AddFileGroup \"" + Path.GetFileName(CurGroup.Key) + "\" \"" + (CurGroup.Value.FullPath ?? CurGroup.Value.Path) + "\"" + ProjectFileGenerator.NewLine);

					if (CurGroup.Value.bIsModuleFolder)
					{
						StringBuilder ProjectFileContent = new StringBuilder();

						ProjectFileContent.Append("# @Eddie Workset@" + ProjectFileGenerator.NewLine);
						ProjectFileContent.Append("AddWorkset \"" + Path.GetFileName(CurGroup.Key) + ".wkst\" \"" + CurGroup.Value.WorksetPath + "\"" + ProjectFileGenerator.NewLine);

						ProjectFileContent.Append("AddFileGroup \"" + Path.GetFileName(CurGroup.Key) + "\" \"" + (CurGroup.Value.FullPath ?? CurGroup.Value.Path) + "\"" + ProjectFileGenerator.NewLine);

						EmitProject(ProjectFileContent, CurGroup.Value.Folders, Logger);

						foreach (EddieSourceFile File in CurGroup.Value.Files)
						{
							ProjectFileContent.Append("AddFile \"" + File.Reference.GetFileName() + "\" \"" + File.Reference.FullName + "\"" + ProjectFileGenerator.NewLine);
						}

						ProjectFileContent.Append("EndFileGroup \"" + Path.GetFileName(CurGroup.Key) + "\"" + ProjectFileGenerator.NewLine);

						ProjectFileGenerator.WriteFileIfChanged(CurGroup.Value.WorksetPath!, ProjectFileContent.ToString(), Logger, new UTF8Encoding());

						Content.Append("AddFile \"" + Path.GetFileName(CurGroup.Key) + "\" \"" + CurGroup.Value.WorksetPath + "\"" + ProjectFileGenerator.NewLine);
					}
					else
					{
						EmitProject(Content, CurGroup.Value.Folders, Logger);

						foreach (EddieSourceFile File in CurGroup.Value.Files)
						{
							Content.Append("AddFile \"" + File.Reference.GetFileName() + "\" \"" + File.Reference.FullName + "\"" + ProjectFileGenerator.NewLine);
						}
					}

					Content.Append("EndFileGroup \"" + Path.GetFileName(CurGroup.Key) + "\"" + ProjectFileGenerator.NewLine);
				}
			}
		}

		public override bool WriteProjectFile(List<UnrealTargetPlatform> InPlatforms, List<UnrealTargetConfiguration> InConfigurations, PlatformProjectGeneratorCollection PlatformProjectGenerators, ILogger Logger)
		{
			//string TargetName = ProjectFilePath.GetFileNameWithoutExtension();

			//FileReference GameProjectPath = null;
			//foreach(ProjectTarget Target in ProjectTargets)
			//{
			//	if(Target.UnrealProjectFilePath != null)
			//	{
			//		GameProjectPath = Target.UnrealProjectFilePath;
			//		break;
			//	}
			//}

			StringBuilder ProjectFileContent = new StringBuilder();

			ProjectFileContent.Append("# @Eddie Workset@" + ProjectFileGenerator.NewLine);
			ProjectFileContent.Append("AddWorkset \"" + ToString() + ".wkst\" \"" + ProjectFilePath.FullName + "\"" + ProjectFileGenerator.NewLine);

			ParseSourceFilesIntoGroups();
			EmitProject(ProjectFileContent, Folders, Logger);

			return ProjectFileGenerator.WriteFileIfChanged(ProjectFilePath.FullName, ProjectFileContent.ToString(), Logger, new UTF8Encoding());
		}
	}
}
