// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	class MakefileProjectFile : ProjectFile
	{
		public MakefileProjectFile(FileReference InitFilePath, DirectoryReference BaseDir)
			: base(InitFilePath, BaseDir)
		{
		}
	}

	/// <summary>
	/// Makefile project file generator implementation
	/// </summary>
	class MakefileGenerator : ProjectFileGenerator
	{
		/// True if intellisense data should be generated (takes a while longer)
		/// Now this is needed for project target generation.
		const bool bGenerateIntelliSenseData = true;

		/// Default constructor
		public MakefileGenerator(FileReference? InOnlyGameProject, CSharpProjectCache CSharpProjCache)
			: base(InOnlyGameProject, CSharpProjCache)
		{
		}

		/// True if we should include IntelliSense data in the generated project files when possible
		public override bool ShouldGenerateIntelliSenseData()
		{
			return bGenerateIntelliSenseData;
		}

		protected override void ConfigureProjectFileGeneration(string[] Arguments, ref bool IncludeAllPlatforms, ILogger Logger)
		{
			base.ConfigureProjectFileGeneration(Arguments, ref IncludeAllPlatforms, Logger);
			bIncludeEnginePrograms = true;
		}

		/// File extension for project files we'll be generating (e.g. ".vcxproj")
		public override string ProjectFileExtension => ".mk";

		protected override bool WritePrimaryProjectFile(ProjectFile? UBTProject, PlatformProjectGeneratorCollection PlatformProjectGenerators, ILogger Logger)
		{
			bool bSuccess = true;
			return bSuccess;
		}

		private bool WriteMakefile(ILogger Logger)
		{
			string EditorTarget = "UnrealEditor";
			string ScriptExtension = (OperatingSystem.IsWindows()) ? ".bat" : ".sh";
			string CurrentPlatform = BuildHostPlatform.Current.Platform.ToString();

			IEnumerable<ProjectTarget> AllProjects = GeneratedProjectFiles.SelectMany(x => x.ProjectTargets.OfType<ProjectTarget>()).Where(x => x.TargetFilePath != null);

			if (!String.IsNullOrEmpty(GameProjectName))
			{
				ProjectTarget? EditorProjectTarget = AllProjects.FirstOrDefault(x => x.UnrealProjectFilePath == OnlyGameProject && x.TargetRules?.Type == TargetType.Editor);
				if (EditorProjectTarget != null)
				{
					EditorTarget = EditorProjectTarget.Name;
				}
			}

			string BuildCommand = $"BUILD = \"$(UNREALROOTPATH)/Engine/Build/BatchFiles/RunUBT{ScriptExtension}\"\n";
			string FileName = "Makefile"; // PrimaryProjectName + ".mk";
			StringBuilder MakefileContent = new StringBuilder();
			MakefileContent.AppendLine("# Makefile generated by MakefileGenerator.cs");
			MakefileContent.AppendLine("# *DO NOT EDIT*");
			MakefileContent.AppendLine();
			MakefileContent.AppendLine($"UNREALROOTPATH = {Unreal.RootDirectory.FullName}");
			MakefileContent.AppendLine();

			MakefileContent.Append("TARGETS =");
			foreach (ProjectTarget TargetFile in AllProjects)
			{
				string TargetFileName = TargetFile.TargetFilePath.GetFileNameWithoutExtension();
				string Basename = TargetFileName.Substring(0, TargetFileName.LastIndexOf(".Target", StringComparison.InvariantCultureIgnoreCase));

				foreach (UnrealTargetPlatform Platform in TargetFile.SupportedPlatforms.Concat(TargetFile.ExtraSupportedPlatforms).Distinct().OrderBy(x => x.ToString()))
				{
					foreach (UnrealTargetConfiguration CurConfiguration in (UnrealTargetConfiguration[])Enum.GetValues(typeof(UnrealTargetConfiguration)))
					{
						if (CurConfiguration != UnrealTargetConfiguration.Unknown)
						{
							if (InstalledPlatformInfo.IsValid(TargetFile.TargetRules!.Type, Platform, CurConfiguration, EProjectType.Code, InstalledPlatformState.Supported))
							{
								string Confname = Enum.GetName(typeof(UnrealTargetConfiguration), CurConfiguration)!;
								MakefileContent.Append(String.Format(" \\\n\t{0}-{1}-{2} ", Basename, Platform, Confname));
							}
						}
					}
				}
				MakefileContent.Append(" \\\n\t" + Basename);
			}
			MakefileContent.Append("\\\n\tconfigure");

			MakefileContent.Append("\n\n" + BuildCommand + "\n" +
				"all: StandardSet\n\n" +
				$"RequiredTools: CrashReportClient-{CurrentPlatform}-Shipping CrashReportClientEditor-{CurrentPlatform}-Shipping ShaderCompileWorker UnrealLightmass EpicWebHelper-{CurrentPlatform}-Shipping\n\n" +
				$"StandardSet: RequiredTools UnrealFrontend {EditorTarget} UnrealInsights\n\n" +
				$"DebugSet: RequiredTools UnrealFrontend-{CurrentPlatform}-Debug {EditorTarget}-{CurrentPlatform}-Debug\n\n"
			);

			string MakeBuildCommand = "$(BUILD)";
			foreach (ProjectTarget TargetFile in AllProjects)
			{
				string TargetFileName = TargetFile.TargetFilePath.GetFileNameWithoutExtension();
				string Basename = TargetFileName.Substring(0, TargetFileName.LastIndexOf(".Target", StringComparison.InvariantCultureIgnoreCase));
				string MakeProjectCmdArg = "";

				if (TargetFile.UnrealProjectFilePath != null)
				{
					MakeProjectCmdArg = $" -Project=\"{TargetFile.UnrealProjectFilePath}\"";
				}

				foreach (UnrealTargetPlatform Platform in TargetFile.SupportedPlatforms.Concat(TargetFile.ExtraSupportedPlatforms).Distinct().OrderBy(x => x.ToString()))
				{
					foreach (UnrealTargetConfiguration CurConfiguration in (UnrealTargetConfiguration[])Enum.GetValues(typeof(UnrealTargetConfiguration)))
					{
						if (CurConfiguration != UnrealTargetConfiguration.Unknown)
						{
							if (InstalledPlatformInfo.IsValid(TargetFile.TargetRules!.Type, Platform, CurConfiguration, EProjectType.Code, InstalledPlatformState.Supported))
							{
								string Confname = Enum.GetName(typeof(UnrealTargetConfiguration), CurConfiguration)!;
								MakefileContent.Append(String.Format("\n{1}-{4}-{2}:\n\t {0} {1} {4} {2} {3} $(ARGS)\n", MakeBuildCommand, Basename, Confname, MakeProjectCmdArg, Platform));
							}
						}
					}
				}
				MakefileContent.Append(String.Format("\n{0}: {0}-{1}-Development\n", Basename, CurrentPlatform));
			}

			MakefileContent.Append("\nconfigure:\n");
			if (OnlyGameProject != null)
			{
				MakefileContent.Append($"\t{MakeBuildCommand} -ProjectFiles -Project=\"{OnlyGameProject.FullName}\" -Game \n");
			}
			else
			{
				MakefileContent.Append("\t{MakeBuildCommand} -ProjectFiles \n");
			}

			MakefileContent.Append("\n.PHONY: $(TARGETS)\n");
			FileReference FullFileName = FileReference.Combine(PrimaryProjectPath, FileName);
			return WriteFileIfChanged(FullFileName.FullName, MakefileContent.ToString(), Logger);
		}

		/// ProjectFileGenerator interface
		//protected override bool WritePrimaryProjectFile( ProjectFile UBTProject )
		protected override bool WriteProjectFiles(PlatformProjectGeneratorCollection PlatformProjectGenerators, ILogger Logger)
		{
			return WriteMakefile(Logger);
		}

		/// ProjectFileGenerator interface
		/// <summary>
		/// Allocates a generator-specific project file object
		/// </summary>
		/// <param name="InitFilePath">Path to the project file</param>
		/// <param name="BaseDir">The base directory for files within this project</param>
		/// <returns>The newly allocated project file object</returns>
		protected override ProjectFile AllocateProjectFile(FileReference InitFilePath, DirectoryReference BaseDir)
		{
			return new MakefileProjectFile(InitFilePath, BaseDir);
		}

		/// ProjectFileGenerator interface
		public override void CleanProjectFiles(DirectoryReference InPrimaryProjectDirectory, string InPrimaryProjectName, DirectoryReference InIntermediateProjectFilesDirectory, ILogger Logger)
		{
		}
	}
}
