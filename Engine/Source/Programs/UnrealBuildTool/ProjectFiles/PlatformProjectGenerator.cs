// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Text;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace UnrealBuildTool
{
	/// <summary>
	/// Base class for platform-specific project generators
	/// </summary>
	abstract class PlatformProjectGenerator
	{
		public const string DefaultPlatformConfigurationType = "Makefile";
		protected readonly ILogger Logger;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="Logger">Logger for output</param>
		public PlatformProjectGenerator(ILogger Logger)
		{
			this.Logger = Logger;
		}

		/// <summary>
		/// Register the platform with the UEPlatformProjectGenerator class
		/// </summary>
		public abstract IEnumerable<UnrealTargetPlatform> GetPlatforms();

		public virtual void GenerateGameProjectStub(ProjectFileGenerator InGenerator, string InTargetName, string InTargetFilepath, TargetRules InTargetRules,
			List<UnrealTargetPlatform> InPlatforms, List<UnrealTargetConfiguration> InConfigurations)
		{
			// Do nothing
		}

		public virtual void GenerateGameProperties(UnrealTargetConfiguration Configuration, StringBuilder VCProjectFileContent, TargetType TargetType, DirectoryReference RootDirectory, FileReference TargetFilePath)
		{
			// Do nothing
		}

		public virtual bool RequiresVSUserFileGeneration()
		{
			return false;
		}

		readonly ConcurrentDictionary<int, bool> FoundVSISupportDict = new();

		public struct VSSettings
		{
			public UnrealTargetPlatform Platform;
			public UnrealTargetConfiguration Configuration = UnrealTargetConfiguration.Development;
			public VCProjectFileFormat ProjectFileFormat = VCProjectFileFormat.Default;
			public UnrealArch? Architecture = null;

			public VSSettings(UnrealTargetPlatform InPlatform, UnrealTargetConfiguration InConfiguration, VCProjectFileFormat InProjectFileFormat, UnrealArch? InArchitecture)
			{
				Platform = InPlatform;
				Configuration = InConfiguration;
				ProjectFileFormat = InProjectFileFormat;
				Architecture = InArchitecture;
			}
		};

		/// <summary>
		/// Checks the local VS install directories to see if the platform is supported.
		/// </summary>
		/// <param name="InVSSettings">The ProjectFileSettings that contains the project platform/configuration/etc info being built</param>
		/// <returns>bool true if native VisualStudio support (or custom VSI) is available</returns>
		public virtual bool HasVisualStudioSupport(VSSettings InVSSettings)
		{
			int HashResult = 691;
			HashResult *= InVSSettings.Platform.ToString().GetHashCode(StringComparison.Ordinal);
			HashResult *= InVSSettings.Configuration.ToString().GetHashCode(StringComparison.Ordinal);
			HashResult *= InVSSettings.ProjectFileFormat.ToString().GetHashCode(StringComparison.Ordinal);
			HashResult *= InVSSettings.Architecture != null ? InVSSettings.Architecture.ToString()!.GetHashCode(StringComparison.Ordinal) : 1;

			return FoundVSISupportDict.GetOrAdd(HashResult, _ =>
			{
				WindowsCompiler VSCompiler;
				string VCVersion;

				switch (InVSSettings.ProjectFileFormat)
				{
					case VCProjectFileFormat.VisualStudio2022:
						VSCompiler = WindowsCompiler.VisualStudio2022;
						VCVersion = "v170";
						break;
					case VCProjectFileFormat.VisualStudio2026:
						VSCompiler = WindowsCompiler.VisualStudio2026;
						VCVersion = "v180";
						break;
					default:
						// Unknown VS Version
						return false;
				}

				IEnumerable<DirectoryReference>? InstallDirs = WindowsPlatform.TryGetVSInstallDirs(VSCompiler, Logger);
				if (InstallDirs != null)
				{
					foreach (DirectoryReference VSInstallDir in InstallDirs)
					{
						DirectoryReference PlatformsPath = new DirectoryReference(System.IO.Path.Combine(VSInstallDir.FullName, "MSBuild\\Microsoft\\VC\\", VCVersion, "Platforms", GetVisualStudioPlatformName(InVSSettings)));
						if (DirectoryReference.Exists(PlatformsPath))
						{
							return true;
						}
					}
				}
				return false;
			});
		}

		/// <summary>
		/// Return the VisualStudio platform name for this build platform
		/// </summary>
		/// <param name="InVSSettings">The ProjectFileSettings that contains the project platform/configuration/etc info being built</param>
		/// <returns>string    The name of the platform that VisualStudio recognizes</returns>
		public virtual string GetVisualStudioPlatformName(VSSettings InVSSettings)
		{
			// By default, return the platform string
			return InVSSettings.Platform.ToString();
		}

		/// <summary>
		/// Return whether we need a distinct platform name - for cases where there are two targets of the same VS solution platform
		/// </summary>
		/// <param name="InVSSettings">The ProjectFileSettings that contains the project platform/configuration/etc info being built</param>
		/// <returns>bool</returns>
		public virtual bool RequiresDistinctVisualStudioConfigurationName(VSSettings InVSSettings)
		{
			return false;
		}

		/// <summary>
		/// Return project configuration settings that must be included before the default props file for all configurations
		/// </summary>
		/// <param name="InPlatform">The UnrealTargetPlatform being built</param>
		/// <param name="ProjectFileBuilder">String builder for the project file</param>
		/// <returns>string    The custom configuration section for the project file; Empty string if it doesn't require one</returns>
		public virtual void GetVisualStudioPreDefaultString(UnrealTargetPlatform InPlatform, StringBuilder ProjectFileBuilder)
		{
		}

		/// <summary>
		/// Return project configuration settings that must be included before the default props file
		/// </summary>
		/// <param name="InPlatform">The UnrealTargetPlatform being built</param>
		/// <param name="InConfiguration">The UnrealTargetConfiguration being built</param>
		/// <param name="ProjectFileBuilder">String builder for the project file</param>
		/// <returns>string    The custom configuration section for the project file; Empty string if it doesn't require one</returns>
		public virtual void GetVisualStudioPreDefaultString(UnrealTargetPlatform InPlatform, UnrealTargetConfiguration InConfiguration, StringBuilder ProjectFileBuilder)
		{
		}

		/// <summary>
		/// Return the platform toolset string to write into the project configuration
		/// </summary>
		/// <param name="InVSSettings">The ProjectFileSettings that contains the project platform/configuration/etc info being built</param>
		/// <param name="ProjectFileBuilder">String builder for the project file</param>
		/// <returns>string    The custom configuration section for the project file; Empty string if it doesn't require one</returns>
		public virtual void GetVisualStudioPlatformToolsetString(VSSettings InVSSettings, StringBuilder ProjectFileBuilder)
		{
		}

		/// <summary>
		/// Return any custom property group lines
		/// </summary>
		/// <param name="InVSSettings">The ProjectFileSettings that contains the project platform/configuration/etc info being built</param>
		/// <param name="ProjectFileBuilder">String builder for the project file</param>
		/// <returns>string    The custom property import lines for the project file; Empty string if it doesn't require one</returns>
		public virtual void GetAdditionalVisualStudioPropertyGroups(VSSettings InVSSettings, StringBuilder ProjectFileBuilder)
		{
		}

		/// <summary>
		/// Return any custom property group lines
		/// </summary>
		/// <param name="InVSSettings">The ProjectFileSettings that contains the project platform/configuration/etc info being built</param>
		/// <returns>string    The platform configuration type.  Defaults to "Makefile" unless overridden</returns>
		public virtual string GetVisualStudioPlatformConfigurationType(VSSettings InVSSettings)
		{
			return DefaultPlatformConfigurationType;
		}

		/// <summary>
		/// Return any custom paths for VisualStudio this platform requires
		/// This include ReferencePath, LibraryPath, LibraryWPath, IncludePath and ExecutablePath.
		/// </summary>
		/// <param name="InVSSettings">The ProjectFileSettings that contains the project platform/configuration/etc info being built</param>
		/// <param name="TargetType">The type of target (game or program)</param>
		/// <param name="TargetRulesPath">Path to the .target.cs file</param>
		/// <param name="ProjectFilePath"></param>
		/// <param name="NMakeOutputPath"></param>
		/// <param name="ProjectFileBuilder">String builder for the project file</param>
		/// <returns>The custom path lines for the project file; Empty string if it doesn't require one</returns>
		public virtual void GetVisualStudioPathsEntries(VSSettings InVSSettings, TargetType TargetType, FileReference TargetRulesPath, FileReference ProjectFilePath, FileReference NMakeOutputPath, StringBuilder ProjectFileBuilder)
		{
		}

		/// <summary>
		/// Return any custom property settings. These will be included in the ImportGroup section
		/// </summary>
		/// <param name="InVSSettings">The ProjectFileSettings that contains the project platform/configuration/etc info being built</param>
		/// <param name="ProjectFileBuilder">String builder for the project file</param>
		/// <returns>string    The custom property import lines for the project file; Empty string if it doesn't require one</returns>
		public virtual void GetVisualStudioImportGroupProperties(VSSettings InVSSettings, StringBuilder ProjectFileBuilder)
		{
		}

		/// <summary>
		/// Return any custom property settings. These will be included right after Global properties to make values available to all other imports.
		/// </summary>
		/// <param name="InVSSettings">The ProjectFileSettings that contains the project platform/configuration/etc info being built</param>
		/// <param name="ProjectFileBuilder">String builder for the project file</param>
		/// <returns>string    The custom property import lines for the project file; Empty string if it doesn't require one</returns>
		public virtual void GetVisualStudioGlobalProperties(VSSettings InVSSettings, StringBuilder ProjectFileBuilder)
		{
		}

		/// <summary>
		/// Return any custom target overrides. These will be included last in the project file so they have the opportunity to override any existing settings.
		/// </summary>
		/// <param name="InVSSettings">The ProjectFileSettings that contains the project platform/configuration/etc info being built</param>
		/// <param name="ProjectFileBuilder">String builder for the project file</param>
		/// <returns>string    The custom property import lines for the project file; Empty string if it doesn't require one</returns>
		public virtual void GetVisualStudioTargetOverrides(VSSettings InVSSettings, StringBuilder ProjectFileBuilder)
		{
		}

		/// <summary>
		/// Return any custom layout directory sections
		/// </summary>
		/// <param name="InVSSettings">The ProjectFileSettings that contains the project platform/configuration/etc info being built</param>
		/// <param name="InConditionString"></param>
		/// <param name="TargetType">The type of target (game or program)</param>
		/// <param name="NMakeOutputPath"></param>
		/// <param name="ProjectFilePath"></param>
		/// <param name="TargetRulesPath"></param>
		/// <returns>string    The custom property import lines for the project file; Empty string if it doesn't require one</returns>
		public virtual string GetVisualStudioLayoutDirSection(VSSettings InVSSettings, string InConditionString, TargetType TargetType, FileReference TargetRulesPath, FileReference ProjectFilePath, FileReference NMakeOutputPath)
		{
			return "";
		}

		/// <summary>
		/// Get the output manifest section, if required
		/// </summary>
		/// <param name="InVSSettings">The ProjectFileSettings that contains the project platform/configuration/etc info being built</param>
		/// <param name="TargetType">The type of the target being built</param>
		/// <param name="TargetRulesPath">Path to the .target.cs file</param>
		/// <param name="ProjectFilePath">Path to the project file</param>
		/// <returns>The output manifest section for the project file; Empty string if it doesn't require one</returns>
		public virtual string GetVisualStudioOutputManifestSection(VSSettings InVSSettings, TargetType TargetType, FileReference TargetRulesPath, FileReference ProjectFilePath)
		{
			return "";
		}

		/// <summary>
		/// Get whether this platform deploys
		/// </summary>
		/// <returns>bool  true if the 'Deploy' option should be enabled</returns>
		public virtual bool GetVisualStudioDeploymentEnabled(VSSettings InVSSettings)
		{
			return false;
		}

		/// <summary>
		/// Additional configuration related to Visual Studio user file (.vcxproj.user). 
		/// </summary>
		public class VisualStudioUserFileSettings
		{
			public IReadOnlySet<string> PropertiesToPatch => PropertiesToPatchContainer;

			public IReadOnlySet<string> PropertiesToPatchOrderButPreserveValue => PropertiesToPatchOrderButPreserveValueContainer;

			/// <summary>
			/// Patch existing .vcxproj.user file based on the new file content and corresponding patching settings, by doing the following:
			/// - Go over every &lt;PropertyGroup&gt; in the new document.
			/// - Finds &lt;PropertyGroup&gt; in the current document with matching "Condition" attribute.
			/// - If no property group is found, appends property group from new document to the end of current document.
			/// - Otherwise, saves values of properties configured with bPreserveExistingValue == true.
			/// - Removes all properties flagged for patching from current document property group.
			/// - Adds all properties required for patching from new document property group as declared in new document order.
			///
			/// This makes possible to override some properties crucial for UE features while keeping rest of user configuration intact.
			/// 
			/// Otherwise, if .vcxproj.user doesn't exist, patching do nothing and instead creates a new file based on new file content.
			/// </summary>
			/// <param name="PropertyName">Name of the property to patch.</param>
			/// <param name="bPreserveExistingValue">If true, the value of the property will be taken from existing user file, but relative order will be taken from newly generated file.</param>
			public void PatchProperty(string PropertyName, bool bPreserveExistingValue = false)
			{
				lock (PropertiesToPatchContainer)
				{
					if (!PropertiesToPatchContainer.Contains(PropertyName))
					{
						PropertiesToPatchContainer.Add(PropertyName);
					}

					if (bPreserveExistingValue && !PropertiesToPatchOrderButPreserveValueContainer.Contains(PropertyName))
					{
						PropertiesToPatchOrderButPreserveValueContainer.Add(PropertyName);
					}
				}
			}

			private readonly HashSet<string> PropertiesToPatchContainer = new();
			private readonly HashSet<string> PropertiesToPatchOrderButPreserveValueContainer = new();
		}

		/// <summary>
		/// Get the text to insert into the user file for the given platform/configuration/target
		/// </summary>
		/// <param name="VCUserFileSettings">Configuration for user file creation/patching/etc.</param>
		/// <param name="InVSSettings">The ProjectFileSettings that contains the project platform/configuration/etc info being built</param>
		/// <param name="InConditionString">The condition string </param>
		/// <param name="InTargetRules">The target rules </param>
		/// <param name="TargetRulesPath">The target rules path</param>
		/// <param name="ProjectFilePath">The project file path</param>
		/// <param name="NMakeOutputPath">Output path for NMake</param>
		/// <param name="ProjectName">The name of the project</param>
		/// <param name="UProjectPath">Path to the project .uproject file</param>
		/// <param name="ForeignUProjectPath">Path to foreign .uproject file, if any</param>
		/// <returns>The string to append to the user file</returns>
		public virtual string GetVisualStudioUserFileStrings(VisualStudioUserFileSettings VCUserFileSettings, VSSettings InVSSettings, string InConditionString, TargetRules InTargetRules, FileReference TargetRulesPath, FileReference ProjectFilePath, FileReference? NMakeOutputPath, string ProjectName, string UProjectPath, string? ForeignUProjectPath)
		{
			return String.Empty;
		}

		/// <summary>
		/// For Additional Project Property files that need to be written out.  This is currently used only on Android. 
		/// </summary>
		public virtual void WriteAdditionalPropFile()
		{
		}

		/// <summary>
		/// For additional Project files (ex. *PROJECTNAME*-AndroidRun.androidproj.user) that needs to be written out.  This is currently used only on Android. 
		/// </summary>
		/// <param name="ProjectFile">Project file this will be related to</param>
		public virtual void WriteAdditionalProjUserFile(ProjectFile ProjectFile)
		{
		}

		/// <summary>
		/// For additional Project files (ex. *PROJECTNAME*-AndroidRun.androidproj) that needs to be written out.  This is currently used only on Android. 
		/// </summary>
		/// <param name="ProjectFile">Project file this will be related to</param>
		/// <returns>Project file written out, Solution folder it should be put in</returns>
		public virtual Tuple<ProjectFile, string>? WriteAdditionalProjFile(ProjectFile ProjectFile)
		{
			return null;
		}

		/// <summary>
		/// Gets the text to insert into the UnrealVS configuration file
		/// </summary>
		public virtual void GetUnrealVSConfigurationEntries(StringBuilder UnrealVSContent)
		{
		}

		/// <summary>
		/// Gets the text to insert into the Project files as additional build arguments for given platform/configuration
		/// </summary>
		/// <param name="InVSSettings">The ProjectFileSettings that contains the project platform/configuration/etc info being built</param>
		public virtual string GetExtraBuildArguments(VSSettings InVSSettings)
		{
			return "";
		}

		/// <summary>
		/// Indicates whether this generator will be emitting additional build targets.
		/// Functionally, it's fine to return true and not return any additional text in GetVisualStudioTargetsString, but doing
		/// so may incur additional cost of preparing data for generation.
		/// </summary>
		/// <param name="InPlatform">The platform being added</param>
		/// <returns>True if Targets will be generate, false otherwise. If true is returned, GetVisualStudioTargetsString will be called next.</returns>
		/// <seealso cref="GetVisualStudioTargetsString"/>
		public virtual bool HasVisualStudioTargets(UnrealTargetPlatform InPlatform)
		{
			return false;
		}

		/// <summary>
		/// Get the text to insert into the Project files as additional build targets for the given platform.
		/// </summary>
		/// <param name="InPlatform">The platform being added</param>
		/// <param name="InPlatformConfigurations">List of build configurations for the platform</param>
		/// <param name="ProjectFileBuilder">String builder for the project file</param>
		public virtual void GetVisualStudioTargetsString(UnrealTargetPlatform InPlatform, List<ProjectBuildConfiguration> InPlatformConfigurations, StringBuilder ProjectFileBuilder)
		{
		}

		/// <summary>
		/// Get include paths to system headers (STL, SDK, ...) needed for coding assistance features in IDE
		/// </summary>
		/// <param name="InTarget">Target for which include paths shall be returned</param>
		/// <returns>The list of include paths</returns>
		public virtual IList<string> GetSystemIncludePaths(UEBuildTarget InTarget)
		{
			return new List<string>(0);
		}
	}
}
