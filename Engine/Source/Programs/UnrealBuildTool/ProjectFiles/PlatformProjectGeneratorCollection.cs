// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Linq;
using System.Text;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace UnrealBuildTool
{
	/// <summary>
	/// Stores all the registered platform project generators
	/// </summary>
	class PlatformProjectGeneratorCollection
	{
		readonly Dictionary<UnrealTargetPlatform, PlatformProjectGenerator> ProjectGeneratorDictionary = new Dictionary<UnrealTargetPlatform, PlatformProjectGenerator>();

		/// <summary>
		/// Returns the list of platforms that have been registered in this collection
		/// </summary>
		/// <returns>Registered platforms</returns>
		public List<UnrealTargetPlatform> GetRegisteredPlatforms()
		{
			return ProjectGeneratorDictionary.Keys.ToList();
		}

		/// <summary>
		/// Register the given platforms UEPlatformProjectGenerator instance
		/// </summary>
		/// <param name="InPlatform">  The UnrealTargetPlatform to register with</param>
		/// <param name="InProjectGenerator">The UEPlatformProjectGenerator instance to use for the InPlatform</param>
		/// <param name="Logger">Logger for output</param>
		public void RegisterPlatformProjectGenerator(UnrealTargetPlatform InPlatform, PlatformProjectGenerator InProjectGenerator, ILogger Logger)
		{
			// Make sure the build platform is legal
			if (UEBuildPlatform.TryGetBuildPlatform(InPlatform, out _))
			{
				if (ProjectGeneratorDictionary.ContainsKey(InPlatform) == true)
				{
					Logger.LogInformation("RegisterPlatformProjectGenerator Warning: Registering project generator {Generator} for {Platform} when it is already set to {ExistingGenerator}",
						InProjectGenerator.ToString(), InPlatform.ToString(), ProjectGeneratorDictionary[InPlatform].ToString());
					ProjectGeneratorDictionary[InPlatform] = InProjectGenerator;
				}
				else
				{
					ProjectGeneratorDictionary.Add(InPlatform, InProjectGenerator);
				}
			}
			else
			{
				Logger.LogDebug("Skipping project file generator registration for {Platform} due to no valid BuildPlatform.", InPlatform.ToString());
			}
		}

		/// <summary>
		/// Retrieve the UEPlatformProjectGenerator instance for the given TargetPlatform
		/// </summary>
		/// <param name="InPlatform">    The UnrealTargetPlatform being built</param>
		/// <param name="bInAllowFailure">   If true, do not throw an exception and return null</param>
		/// <returns>UEPlatformProjectGenerator The instance of the project generator</returns>
		public PlatformProjectGenerator? GetPlatformProjectGenerator(UnrealTargetPlatform InPlatform, bool bInAllowFailure = false)
		{
			if (ProjectGeneratorDictionary.ContainsKey(InPlatform) == true)
			{
				return ProjectGeneratorDictionary[InPlatform];
			}
			if (bInAllowFailure == true)
			{
				return null;
			}
			throw new BuildException("GetPlatformProjectGenerator: No PlatformProjectGenerator found for {Platform}", InPlatform.ToString());
		}

		/// <summary>
		/// Allow various platform project generators to generate stub projects if required
		/// </summary>
		/// <param name="InGenerator"></param>
		/// <param name="InTargetName"></param>
		/// <param name="InTargetFilepath"></param>
		/// <param name="InTargetRules"></param>
		/// <param name="InPlatforms"></param>
		/// <param name="InConfigurations"></param>
		/// <returns></returns>
		public bool GenerateGameProjectStubs(ProjectFileGenerator InGenerator, string InTargetName, string InTargetFilepath, TargetRules InTargetRules,
			List<UnrealTargetPlatform> InPlatforms, List<UnrealTargetConfiguration> InConfigurations)
		{
			foreach (KeyValuePair<UnrealTargetPlatform, PlatformProjectGenerator> Entry in ProjectGeneratorDictionary)
			{
				PlatformProjectGenerator ProjGen = Entry.Value;
				ProjGen.GenerateGameProjectStub(InGenerator, InTargetName, InTargetFilepath, InTargetRules, InPlatforms, InConfigurations);
			}
			return true;
		}

		/// <summary>
		/// Allow various platform project generators to generate any special project properties if required
		/// </summary>
		/// <param name="InPlatform"></param>
		/// <param name="Configuration"></param>
		/// <param name="TargetType"></param>
		/// <param name="VCProjectFileContent"></param>
		/// <param name="RootDirectory"></param>
		/// <param name="TargetFilePath"></param>
		/// <returns></returns>
		public bool GenerateGamePlatformSpecificProperties(UnrealTargetPlatform InPlatform, UnrealTargetConfiguration Configuration, TargetType TargetType, StringBuilder VCProjectFileContent, DirectoryReference RootDirectory, FileReference TargetFilePath)
		{
			if (ProjectGeneratorDictionary.ContainsKey(InPlatform) == true)
			{
				ProjectGeneratorDictionary[InPlatform].GenerateGameProperties(Configuration, VCProjectFileContent, TargetType, RootDirectory, TargetFilePath);
			}
			return true;
		}

		public bool PlatformRequiresVSUserFileGeneration(List<UnrealTargetPlatform> InPlatforms)
		{
			bool bRequiresVSUserFileGeneration = false;
			foreach (KeyValuePair<UnrealTargetPlatform, PlatformProjectGenerator> Entry in ProjectGeneratorDictionary)
			{
				if (InPlatforms.Contains(Entry.Key))
				{
					PlatformProjectGenerator ProjGen = Entry.Value;
					bRequiresVSUserFileGeneration |= ProjGen.RequiresVSUserFileGeneration();
				}
			}
			return bRequiresVSUserFileGeneration;
		}
	}
}
