// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using EpicGames.Core;

namespace UnrealBuildTool
{
	/// <summary>
	/// Data common to all generated Project entries
	/// </summary>
	class Project
	{
		/// Platforms supported by the target
		public UnrealTargetPlatform[] SupportedPlatforms;

		/// Extra supported build configurations.  Normally the target rules determines these, but for synthetic targets we'll add them here.
		public List<UnrealTargetConfiguration> ExtraSupportedConfigurations = new List<UnrealTargetConfiguration>();

		/// Extra supported build platforms.  Normally the target rules determines these, but for synthetic targets we'll add them here.
		public List<UnrealTargetPlatform> ExtraSupportedPlatforms = new List<UnrealTargetPlatform>();

		/// If true, forces Development configuration regardless of which configuration is set as the Solution Configuration
		public bool ForceDevelopmentConfiguration = false;

		/// Whether the project requires 'Deploy' option set (VC projects)
		public bool ProjectDeploys = false;

		public Project(UnrealTargetPlatform[] SupportedPlatforms)
		{
			this.SupportedPlatforms = SupportedPlatforms;
		}

		/// Optional target rules for this target.  If the target came from a *.Target.cs file on disk, then it will have one of these.
		/// For targets that are synthetic (like UnrealBuildTool or other manually added project files) we won't have a rules object for those.
		/// Never set for an untargeted project
		public TargetRules? TargetRules
		{
			get;
			protected set;
		} = null;

		/// <summary>
		/// Path to the .uproject file on disk
		/// Never set for an untargeted project.
		/// </summary>
		public FileReference? UnrealProjectFilePath
		{
			get;
			protected set;
		} = null;
	}
}
