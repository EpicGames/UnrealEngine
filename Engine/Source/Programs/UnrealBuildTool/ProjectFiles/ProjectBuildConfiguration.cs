// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool
{
	/// <summary>
	/// Description of a project build configuration provided during project generation by ProjectFile.
	/// </summary>
	/// <seealso cref="PlatformProjectGenerator.GetVisualStudioTargetsString"/>
	public abstract class ProjectBuildConfiguration
	{
		/// <summary>
		/// Name of the build configuration as displayed in the IDE e.g. Debug_Client.
		/// </summary>
		public abstract string ConfigurationName { get; }

		/// <summary>
		/// Build command associated with this build configuration (as generated by classes derived from ProjectFile).
		/// </summary>
		public abstract string BuildCommand { get; }
	}
}
