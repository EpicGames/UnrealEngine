// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	abstract class ProjectFile
	{
		/// <summary>
		/// Represents a single source file (or other type of file) in a project
		/// </summary>
		public class SourceFile
		{
			/// <summary>
			/// Constructor
			/// </summary>
			/// <param name="InReference">Path to the source file on disk</param>
			/// <param name="InBaseFolder">The directory on this the path within the project will be relative to</param>
			public SourceFile(FileReference InReference, DirectoryReference? InBaseFolder)
			{
				Reference = InReference;
				BaseFolder = InBaseFolder;
			}

			/// <summary>
			/// File path to file on disk
			/// </summary>
			public FileReference Reference
			{
				get;
				private set;
			}

			/// <summary>
			/// Optional directory that overrides where files in this project are relative to when displayed in the IDE.  If null, will default to the project's BaseFolder.
			/// </summary>
			public DirectoryReference? BaseFolder
			{
				get;
				private set;
			}

			/// <summary>
			/// Define ToString() so the debugger can show the name in watch windows
			/// </summary>
			public override string ToString()
			{
				return Reference.ToString();
			}
		}

		/// <summary>
		/// Constructs a new project file object
		/// </summary>
		/// <param name="ProjectFilePath">The path to the project file, relative to the primary project file</param>
		/// <param name="BaseDir">The base directory for files within this project</param>
		protected ProjectFile(FileReference ProjectFilePath, DirectoryReference BaseDir)
		{
			this.ProjectFilePath = ProjectFilePath;
			this.BaseDir = BaseDir;

			ShouldBuildByDefaultForSolutionTargets = true;
			IntelliSenseCppVersion = CppStandardVersion.Default;
		}

		/// Project file path
		public FileReference ProjectFilePath
		{
			get;
			protected set;
		}

		/// <summary>
		/// The base directory for files within this project
		/// </summary>
		public DirectoryReference BaseDir
		{
			get;
			set;
		}

		/// Returns true if this is a generated project (as opposed to an imported project)
		public bool IsGeneratedProject
		{
			get;
			set;
		}

		/// Returns true if this is a "stub" project.  Stub projects function as simple containers for source files
		/// and are never actually "built" by the primary project.  Stub projects are always "generated" projects.
		public bool IsStubProject
		{
			get;
			set;
		}

		/// Returns true if this is a foreign project, and requires UBT to be passed the path to the .uproject file 
		/// on the command line.
		public bool IsForeignProject
		{
			get;
			set;
		}

		/// Returns true if this is a content only project, and will be re-using the engine targets
		public bool IsContentOnlyProject
		{
			get;
			set;
		}

		/// Returns true if this is a project without source code, but some build setting requires it to be compiled as code, which will make
		/// a unique executable, instead of using UnrealGame, etc
		/// @note: Currently unused by any generator, but a future generator may make use of it
		public bool IsHybridContentOnlyProject
		{
			get;
			set;
		}

		/// Whether this project should be built for all solution targets
		public bool ShouldBuildForAllSolutionTargets
		{
			get;
			set;
		}

		/// Whether this project should be built by default. Can still be built from the IDE through the context menu.
		public bool ShouldBuildByDefaultForSolutionTargets
		{
			get;
			set;
		}

		/// <summary>
		/// C++ version which is used in this project.
		/// </summary>
		public CppStandardVersion IntelliSenseCppVersion
		{
			get;
			protected set;
		}

		/// <summary>
		/// Information for IntelliSense: whether coroutines are enabled
		/// </summary>
		public bool IntelliSenseEnableCoroutines
		{
			get;
			protected set;
		}

		/// All of the targets in this project.  All non-stub projects must have at least one target.
		public readonly List<Project> ProjectTargets = new List<Project>();

		/// <summary>
		/// Checks if a file is already contained
		/// </summary>
		/// <param name="File">Files to check</param>
		public bool Contains(FileReference File)
		{
			return SourceFileMap.ContainsKey(File);
		}

		/// <summary>
		/// Adds a list of files to this project, ignoring dupes
		/// </summary>
		/// <param name="FilesToAdd">Files to add</param>
		/// <param name="BaseFolder">The directory the path within the project will be relative to</param>
		public void AddFilesToProject(List<FileReference> FilesToAdd, DirectoryReference BaseFolder)
		{
			foreach (FileReference CurFile in FilesToAdd)
			{
				AddFileToProject(CurFile, BaseFolder);
			}
		}

		/// <summary>
		/// Set of all files that have been added
		/// </summary>
		readonly Dictionary<FileReference, AliasedFile> AliasedFilesLookup = [];

		public Dictionary<FileReference, AliasedFile>.ValueCollection AliasedFiles => AliasedFilesLookup.Values;

		/// <summary>
		/// Adds aliased file to the project.
		/// </summary>
		/// <param name="File">Aliased file.</param>
		public void AddAliasedFileToProject(AliasedFile File)
		{
			AliasedFilesLookup.TryAdd(File.Location, File);
		}

		/// <summary>
		/// Adds a file to this project, ignoring dupes
		/// </summary>
		/// <param name="FilePath">Path to the file on disk</param>
		/// <param name="BaseFolder">The directory the path within the project will be relative to</param>
		public void AddFileToProject(FileReference FilePath, DirectoryReference BaseFolder)
		{
			// Check if hasn't already been added as an aliased file
			if (AliasedFilesLookup.ContainsKey(FilePath))
			{
				return;
			}

			// Don't add duplicates
			if (SourceFileMap.TryGetValue(FilePath, out SourceFile? ExistingFile))
			{
				if (ExistingFile.BaseFolder != BaseFolder)
				{
					throw new BuildException("Trying to add file '" + FilePath + "' to project '" + ProjectFilePath + "' when the file already exists, but with a different relative base folder '" + BaseFolder + "' is different than the current file's '" + ExistingFile.BaseFolder + "'!");
				}
			}
			else
			{
				SourceFile? File = AllocSourceFile(FilePath, BaseFolder);
				if (File != null)
				{
					SourceFileMap[FilePath] = File;
					SourceFiles.Add(File);
				}
			}
		}

		/// <summary>
		/// Splits the definition text into macro name and value (if any).
		/// </summary>
		/// <param name="Definition">Definition text</param>
		/// <param name="Key">Out: The definition name</param>
		/// <param name="Value">Out: The definition value or null if it has none</param>
		/// <returns>Pair representing macro name and value.</returns>
		private static void SplitDefinitionAndValue(string Definition, out string Key, out string Value)
		{
			int EqualsIndex = Definition.IndexOf('=', StringComparison.Ordinal);
			if (EqualsIndex >= 0)
			{
				Key = Definition.Substring(0, EqualsIndex);
				Value = Definition.Substring(EqualsIndex + 1);
			}
			else
			{
				Key = Definition;
				Value = "";
			}
		}

		/// <summary>
		/// Adds information about a module to this project file
		/// </summary>
		/// <param name="Module">The module to add</param>
		/// <param name="CompileEnvironment">Compile environment for this module</param>
		public virtual void AddModuleForIntelliSense(UEBuildModuleCPP Module, CppCompileEnvironment CompileEnvironment)
		{
			AddIntelliSensePreprocessorDefinitions(CompileEnvironment.Definitions);
			AddIntelliSenseIncludePaths(SystemIncludePaths, CompileEnvironment.SystemIncludePaths);
			AddIntelliSenseIncludePaths(UserIncludePaths, CompileEnvironment.UserIncludePaths);

			foreach (DirectoryItem BaseDir in Module.ModuleDirectories)
			{
				BuildEnvironment? BuildEnvironment;
				if (!BaseDirToBuildEnvironment.TryGetValue(BaseDir.Location, out BuildEnvironment))
				{
					BuildEnvironment = new BuildEnvironment();
					BaseDirToBuildEnvironment.Add(BaseDir.Location, BuildEnvironment);
				}

				AddIntelliSenseIncludePaths(BuildEnvironment.SystemIncludePaths, CompileEnvironment.SystemIncludePaths);
				AddIntelliSenseIncludePaths(BuildEnvironment.UserIncludePaths, CompileEnvironment.UserIncludePaths);
			}

			SetIntelliSenseEnableCoroutines(CompileEnvironment.bEnableCoroutines);
			SetIntelliSenseCppVersion(Module.Rules.CppStandard ?? CompileEnvironment.CppStandard);
		}

		/// <summary>
		/// Adds all of the specified preprocessor definitions to this VCProject's list of preprocessor definitions for all modules in the project
		/// </summary>
		/// <param name="NewPreprocessorDefinitions">List of preprocessor definitons to add</param>
		public void AddIntelliSensePreprocessorDefinitions(List<string> NewPreprocessorDefinitions)
		{
			foreach (string NewPreprocessorDefinition in NewPreprocessorDefinitions)
			{
				// Don't add definitions and value combinations that have already been added for this project
				string CurDef = NewPreprocessorDefinition;
				if (KnownIntelliSensePreprocessorDefinitions.Add(CurDef))
				{
					// Go ahead and check to see if the definition already exists, but the value is different
					bool AlreadyExists = false;

					SplitDefinitionAndValue(CurDef, out string Def, out _);

					// Ignore any API macros being import/export; we'll assume they're valid across the whole project
					if (Def.EndsWith("_API", StringComparison.Ordinal))
					{
						CurDef = Def + "=";
					}

					for (int DefineIndex = 0; DefineIndex < IntelliSensePreprocessorDefinitions.Count; ++DefineIndex)
					{
						SplitDefinitionAndValue(IntelliSensePreprocessorDefinitions[DefineIndex], out string ExistingDef, out _);
						if (ExistingDef == Def)
						{
							// Already exists, but the value is changing.  We don't bother clobbering values for existing defines for this project.
							AlreadyExists = true;
							break;
						}
					}

					if (!AlreadyExists)
					{
						IntelliSensePreprocessorDefinitions.Add(CurDef);
					}
				}
			}
		}

		/// <summary>
		/// Adds all of the specified include paths to this VCProject's list of include paths for all modules in the project
		/// </summary>
		/// <param name="Collection">The collection to add to</param>
		/// <param name="NewIncludePaths">List of include paths to add</param>
		public static void AddIntelliSenseIncludePaths(IncludePathsCollection Collection, IEnumerable<DirectoryReference> NewIncludePaths)
		{
			foreach (DirectoryReference CurPath in NewIncludePaths)
			{
				if (Collection.AbsolutePaths.Add(CurPath))
				{
					// Incoming include paths are relative to the solution directory, but we need these paths to be
					// relative to the project file's directory
					string PathRelativeToProjectFile = NormalizeProjectPath(CurPath);
					Collection.RelativePaths.Add(PathRelativeToProjectFile);
				}
			}
		}

		/// <summary>
		/// Sets highest C++ version which is used in this project
		/// </summary>
		/// <param name="CppVersion">Version</param>
		public void SetIntelliSenseCppVersion(CppStandardVersion CppVersion)
		{
			if (CppVersion != CppStandardVersion.Default)
			{
				if (CppVersion > IntelliSenseCppVersion)
				{
					IntelliSenseCppVersion = CppVersion;
				}
			}
		}

		/// <summary>
		/// Sets whether coroutines should be enabled or not
		/// </summary>
		/// <param name="Value">Should enable them or not</param>
		public void SetIntelliSenseEnableCoroutines(bool Value)
		{
			IntelliSenseEnableCoroutines |= Value;
		}

		/// <summary>
		/// Writes a project file to disk
		/// </summary>
		/// <param name="InPlatforms">The platforms to write the project files for</param>
		/// <param name="InConfigurations">The configurations to add to the project files</param>
		/// <param name="PlatformProjectGenerators">The registered platform project generators</param>
		/// <param name="Logger">Logger for output</param>
		/// <returns>True on success</returns>
		public virtual bool WriteProjectFile(List<UnrealTargetPlatform> InPlatforms, List<UnrealTargetConfiguration> InConfigurations, PlatformProjectGeneratorCollection PlatformProjectGenerators, ILogger Logger)
		{
			throw new BuildException("UnrealBuildTool cannot automatically generate this project type because WriteProjectFile() was not overridden.");
		}

		/// <summary>
		/// If found writes a debug project file to disk
		/// </summary>
		/// <param name="InPlatforms">The platforms to write the project files for</param>
		/// <param name="InConfigurations">The configurations to add to the project files</param>
		/// <param name="PlatformProjectGenerators">The registered platform project generators</param>
		/// <param name="Logger">Logger for output</param>
		/// <returns>List of project files written</returns>
		public virtual List<Tuple<ProjectFile, string>>? WriteDebugProjectFiles(List<UnrealTargetPlatform> InPlatforms, List<UnrealTargetConfiguration> InConfigurations, PlatformProjectGeneratorCollection PlatformProjectGenerators, ILogger Logger)
		{
			return null;
		}

		public virtual void LoadGUIDFromExistingProject()
		{
		}

		/// <summary>
		/// Allocates a generator-specific source file object
		/// </summary>
		/// <param name="InitFilePath">Path to the source file on disk</param>
		/// <param name="InitProjectSubFolder">Optional sub-folder to put the file in.  If empty, this will be determined automatically from the file's path relative to the project file</param>
		/// <returns>The newly allocated source file object</returns>
		public virtual SourceFile? AllocSourceFile(FileReference InitFilePath, DirectoryReference? InitProjectSubFolder = null)
		{
			return new SourceFile(InitFilePath, InitProjectSubFolder);
		}

		/// <summary>
		/// Takes the given path and tries to rebase it relative to the project or solution directory variables.
		/// </summary>
		public static string NormalizeProjectPath(string InputPath)
		{
			// If the path is rooted in an environment variable, leave it be.
			if (InputPath.StartsWith("$(", StringComparison.Ordinal))
			{
				return InputPath;
			}
			else if (InputPath.EndsWith('\\') || InputPath.EndsWith('/'))
			{
				return NormalizeProjectPath(new DirectoryReference(InputPath));
			}
			else
			{
				return NormalizeProjectPath(new FileReference(InputPath));
			}
		}

		/// <summary>
		/// Takes the given path and tries to rebase it relative to the project.
		/// </summary>
		public static string NormalizeProjectPath(FileSystemReference InputPath)
		{
			// Try to make it relative to the solution directory.
			if (InputPath.IsUnderDirectory(ProjectFileGenerator.PrimaryProjectPath))
			{
				return InputPath.MakeRelativeTo(ProjectFileGenerator.IntermediateProjectFilesPath);
			}
			else
			{
				return InputPath.FullName;
			}
		}

		/// <summary>
		/// Takes the given path and tries to rebase it relative to the project.
		/// </summary>
		public static void NormalizeProjectPath(ref StackStringBuilder outBuilder, FileSystemReference InputPath)
		{
			// Try to make it relative to the solution directory.
			if (InputPath.IsUnderDirectory(ProjectFileGenerator.PrimaryProjectPath))
			{
				InputPath.MakeRelativeTo(ref outBuilder, ProjectFileGenerator.IntermediateProjectFilesPath);
			}
			else
			{
				outBuilder.Append(InputPath.FullName);
			}
		}

		/// <summary>
		/// Takes the given path, normalizes it, and quotes it if necessary.
		/// </summary>
		public static string EscapePath(string InputPath)
		{
			string Result = InputPath;
			if (Result.Contains(' ', StringComparison.Ordinal))
			{
				Result = "\"" + Result + "\"";
			}
			return Result;
		}

		/// <summary>
		/// Visualizer for the debugger
		/// </summary>
		public override string ToString()
		{
			return ProjectFilePath.ToString();
		}

		/// <summary>
		/// Map of file paths to files in the project.
		/// </summary>
		private readonly Dictionary<FileReference, SourceFile> SourceFileMap = new Dictionary<FileReference, SourceFile>();

		/// <summary>
		/// Files in this project
		/// </summary>
		public readonly List<SourceFile> SourceFiles = new List<SourceFile>();

		/// <summary>
		/// Collection of include paths
		/// </summary>
		public class IncludePathsCollection
		{
			public List<string> RelativePaths = new List<string>();
			public HashSet<DirectoryReference> AbsolutePaths = new HashSet<DirectoryReference>();
		}

		/// <summary>
		/// Build environment for a particular module
		/// </summary>
		public class BuildEnvironment
		{
			public IncludePathsCollection UserIncludePaths = new IncludePathsCollection();
			public IncludePathsCollection SystemIncludePaths = new IncludePathsCollection();
		}

		/// <summary>
		/// Merged list of include paths for the project
		/// </summary>
		readonly IncludePathsCollection UserIncludePaths = new IncludePathsCollection();

		/// <summary>
		/// Merged list of include paths for the project
		/// </summary>
		readonly IncludePathsCollection SystemIncludePaths = new IncludePathsCollection();

		/// <summary>
		/// Map of base directory to user include paths
		/// </summary>
		protected Dictionary<DirectoryReference, BuildEnvironment> BaseDirToBuildEnvironment = new Dictionary<DirectoryReference, BuildEnvironment>();

		/// <summary>
		/// Legacy accessor for user search paths
		/// </summary>
		public List<string> IntelliSenseIncludeSearchPaths => UserIncludePaths.RelativePaths;

		/// <summary>
		/// Legacy accessor for system include paths
		/// </summary>
		public List<string> IntelliSenseSystemIncludeSearchPaths => SystemIncludePaths.RelativePaths;

		/// <summary>
		/// List of preprocessor definitions for the project
		/// </summary>
		public readonly List<string> IntelliSensePreprocessorDefinitions = new List<string>();

		/// <summary>
		/// Set of unique preprocessor definitions
		/// </summary>
		private readonly HashSet<string> KnownIntelliSensePreprocessorDefinitions = new HashSet<string>(StringComparer.InvariantCultureIgnoreCase);
	}
}
