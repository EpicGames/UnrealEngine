// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using EpicGames.Core;

namespace UnrealBuildTool
{
	/// <summary>
	/// A single targeted (with a TargetFilePath) project within a project.  A project may have any number of targets within it.
	/// in themselves that the project wraps up.
	/// </summary>
	class ProjectTarget : Project
	{
		/// The target rules file path on disk, if we have one
		public FileReference TargetFilePath { get; }

		/// The project file path on disk
		public FileReference ProjectFilePath { get; }

		/// Delegate for creating a rules instance for a given platform/configuration
		public Func<UnrealTargetPlatform, UnrealTargetConfiguration, TargetRules> CreateRulesDelegate { get; }

		/// Delegate for creating a rules instance for a given platform/configuration/architecture
		public Func<UnrealTargetPlatform, UnrealTargetConfiguration, UnrealArch, TargetRules> CreateRulesDelegateWithArch { get;  }

		public string Name => TargetFilePath.GetFileNameWithoutAnyExtensions();

		public override string ToString()
		{
			return TargetFilePath.GetFileNameWithoutExtension();
		}

		public ProjectTarget(
			FileReference TargetFilePath,
			FileReference ProjectFilePath,
			FileReference? UnrealProjectFilePath,
			TargetRules? TargetRules,
			UnrealTargetPlatform[] SupportedPlatforms,
			Func<UnrealTargetPlatform, UnrealTargetConfiguration, TargetRules> CreateRulesDelegate,
			Func<UnrealTargetPlatform, UnrealTargetConfiguration, UnrealArch, TargetRules> CreateRulesDelegateWithArch
			)
			: base(SupportedPlatforms)
		{
			this.TargetFilePath = TargetFilePath;
			this.ProjectFilePath = ProjectFilePath;
			this.UnrealProjectFilePath = UnrealProjectFilePath;
			this.TargetRules = TargetRules;
			this.CreateRulesDelegate = CreateRulesDelegate;
			this.CreateRulesDelegateWithArch = CreateRulesDelegateWithArch;
		}
	}
}
