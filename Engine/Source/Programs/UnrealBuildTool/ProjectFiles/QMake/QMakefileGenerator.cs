// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	class QMakefileProjectFile : ProjectFile
	{
		public QMakefileProjectFile(FileReference InitFilePath, DirectoryReference BaseDir)
			: base(InitFilePath, BaseDir)
		{
		}
	}

	/// <summary>
	/// QMakefile project file generator implementation
	/// </summary>
	class QMakefileGenerator : ProjectFileGenerator
	{
		/// Default constructor
		public QMakefileGenerator(FileReference? InOnlyGameProject, CSharpProjectCache CSharpProjCache)
			: base(InOnlyGameProject, CSharpProjCache)
		{
		}

		/// File extension for project files we'll be generating (e.g. ".vcxproj")
		public override string ProjectFileExtension => ".pro";

		protected override bool WritePrimaryProjectFile(ProjectFile? UBTProject, PlatformProjectGeneratorCollection PlatformProjectGenerators, ILogger Logger)
		{
			bool bSuccess = true;
			return bSuccess;
		}

		/// <summary>
		/// Splits the definition text into macro name and value (if any).
		/// </summary>
		/// <param name="Definition">Definition text</param>
		/// <param name="Key">Out: The definition name</param>
		/// <param name="Value">Out: The definition value or null if it has none</param>
		/// <returns>Pair representing macro name and value.</returns>
		private static void SplitDefinitionAndValue(string Definition, out string Key, out string Value)
		{
			int EqualsIndex = Definition.IndexOf('=', StringComparison.Ordinal);
			if (EqualsIndex >= 0)
			{
				Key = Definition.Substring(0, EqualsIndex);
				Value = Definition.Substring(EqualsIndex + 1);
			}
			else
			{
				Key = Definition;
				Value = "";
			}
		}

		/// Adds the include directory to the list, after converting it to relative to Unreal root
		private static void AddIncludeDirectory(ref List<string> IncludeDirectories, string IncludeDir, string ProjectDir)
		{
			string FullProjectPath = ProjectFileGenerator.PrimaryProjectPath.FullName;
			string FullPath;
			if (IncludeDir.StartsWith('/') && !IncludeDir.StartsWith(FullProjectPath, StringComparison.Ordinal))
			{
				// Full path to a fulder outside of project
				FullPath = IncludeDir;
			}
			else
			{
				FullPath = Path.GetFullPath(Path.Combine(ProjectDir, IncludeDir));
				FullPath = Utils.MakePathRelativeTo(FullPath, FullProjectPath);
				FullPath = FullPath.TrimEnd('/');
			}

			IncludeDirectories.Add(FullPath);
		}

		private bool WriteQMakePro(ILogger Logger)
		{
			// Some more stuff borrowed from Mac side of things.
			List<string> IncludeDirectories = new List<string>();
			List<string> SystemIncludeDirectories = new List<string>();
			List<string> DefinesAndValues = new List<string>();
			HashSet<string> DefinesOnly = new HashSet<string>();

			// DefineList.Add ("");

			string QMakeIncludesFileName = PrimaryProjectName + "Includes.pri";
			StringBuilder QMakeIncludesPriFileContent = new StringBuilder();

			string QMakeDefinesFileName = PrimaryProjectName + "Defines.pri";
			StringBuilder QMakeDefinesPriFileContent = new StringBuilder();

			string GameProjectPath = "";
			string GameProjectFile;
			string GameProjectRootPath = "";

			string BuildCommand;

			string QMakeGameProjectFile = "";
			string QMakeFilesDirectory = ".qmake";
			char Seperator = Path.DirectorySeparatorChar;
			string CurrentPlatform = "";
			string Console = "";
			if (OperatingSystem.IsWindows())
			{
				CurrentPlatform = "Win64";
			}
			else if (OperatingSystem.IsLinux())
			{
				CurrentPlatform = "Linux";
				Console = "bash";
			}
			else if (OperatingSystem.IsMacOS())
			{
				CurrentPlatform = "Mac";
				Console = "bash";
			}

			foreach (ProjectFile CurProject in GeneratedProjectFiles)
			{

				QMakefileProjectFile? QMakeProject = CurProject as QMakefileProjectFile;
				if (QMakeProject == null)
				{
					Logger.LogInformation("QMakeProject == null");
					continue;
				}

				foreach (string CurPath in QMakeProject.IntelliSenseIncludeSearchPaths)
				{
					AddIncludeDirectory(ref IncludeDirectories, CurPath, Path.GetDirectoryName(QMakeProject.ProjectFilePath.FullName)!);
					// System.Console.WriteLine ("Not empty now? CurPath == ", CurPath);
				}
				foreach (string CurPath in QMakeProject.IntelliSenseSystemIncludeSearchPaths)
				{
					AddIncludeDirectory(ref SystemIncludeDirectories, CurPath, Path.GetDirectoryName(QMakeProject.ProjectFilePath.FullName)!);
				}
			}

			// Remove duplicate paths from include dir and system include dir list
			IncludeDirectories = IncludeDirectories.Distinct().ToList();
			SystemIncludeDirectories = SystemIncludeDirectories.Distinct().ToList();

			// Iterate through all the defines for the projects that are generated by 
			// UnrealBuildTool
			// !RAKE: move to separate function
			QMakeDefinesPriFileContent.Append("DEFINES += \\\n");
			foreach (ProjectFile CurProject in GeneratedProjectFiles)
			{
				QMakefileProjectFile? QMakeProject = CurProject as QMakefileProjectFile;
				if (QMakeProject == null)
				{
					Logger.LogInformation("QMakeProject == null");
					continue;
				}

				foreach (string CurDefine in QMakeProject.IntelliSensePreprocessorDefinitions)
				{
					SplitDefinitionAndValue(CurDefine, out string define, out string value);

					if (!DefinesOnly.Contains(define))
					{
						// Logger.LogInformation (CurDefine);
						if (String.IsNullOrEmpty(value))
						{
							DefinesAndValues.Add("\t");
							DefinesAndValues.Add(String.Format("{0}=", define));
							DefinesAndValues.Add(" \\\n");
						}
						else
						{
							// Escape special characters
							// If the value contains spaces, we must put the whole value inside quotes
							value = value.Replace("\"", "\\\\\\\"", StringComparison.Ordinal);
							bool hasSpaces = false;
							if (value.Contains(' ', StringComparison.Ordinal))
							{
								value = value.Replace(" ", "\\ ", StringComparison.Ordinal);
								hasSpaces = true;
							}
							value = value.Replace("{", "\\{", StringComparison.Ordinal);
							value = value.Replace("}", "\\}", StringComparison.Ordinal);

							DefinesAndValues.Add("\t");
							DefinesAndValues.Add(define);
							DefinesAndValues.Add("=");
							if (hasSpaces)
							{
								DefinesAndValues.Add("\"");
							}

							DefinesAndValues.Add(value);
							if (hasSpaces)
							{
								DefinesAndValues.Add("\"");
							}

							DefinesAndValues.Add(" \\\n");
						}
						DefinesOnly.Add(define);
					}
				}
			}

			foreach (string Def in DefinesAndValues)
			{
				QMakeDefinesPriFileContent.Append(Def);
			}

			// Iterate through all the include paths that
			// UnrealBuildTool generates
			// !RAKE: Move to separate function
			QMakeIncludesPriFileContent.Append("INCLUDEPATH += \\\n");
			foreach (string CurPath in IncludeDirectories)
			{
				QMakeIncludesPriFileContent.Append('\t');
				QMakeIncludesPriFileContent.Append(CurPath);
				QMakeIncludesPriFileContent.Append(" \\\n");
			}

			foreach (string CurPath in SystemIncludeDirectories)
			{
				QMakeIncludesPriFileContent.Append('\t');
				QMakeIncludesPriFileContent.Append(CurPath);
				QMakeIncludesPriFileContent.Append(" \\\n");
			}
			QMakeIncludesPriFileContent.Append('\n');

			if (!String.IsNullOrEmpty(GameProjectName))
			{
				GameProjectPath = OnlyGameProject!.Directory.FullName;
				GameProjectFile = OnlyGameProject.FullName;
				QMakeGameProjectFile = "gameProjectFile=" + GameProjectFile + "\n";
				BuildCommand = $"build={Console} $$unrealRootPath{Seperator}Engine{Seperator}{Unreal.RelativeDotnetDirectory}{Seperator}dotnet $$unrealRootPath{Seperator}Engine{Seperator}Binaries{Seperator}DotNET{Seperator}UnrealBuildTool{Seperator}UnrealBuildTool.dll\n\n";
			}
			else
			{
				BuildCommand = String.Format("build=bash $$unrealRootPath{0}Engine{0}Build{0}BatchFiles{0}{1}{0}Build.sh\n", Seperator, CurrentPlatform);
			}

			string UnrealRootPath = Unreal.RootDirectory.FullName;

			string FileName = PrimaryProjectName + ".pro";

			string QMakeSourcePriFileName = PrimaryProjectName + "Source.pri";
			string QMakeHeaderPriFileName = PrimaryProjectName + "Header.pri";
			string QMakeConfigPriFileName = PrimaryProjectName + "Config.pri";

			StringBuilder QMakeFileContent = new StringBuilder();

			StringBuilder QMakeSourcePriFileContent = new StringBuilder();
			StringBuilder QMakeHeaderPriFileContent = new StringBuilder();
			StringBuilder QMakeConfigPriFileContent = new StringBuilder();

			string QMakeSectionEnd = " \n\n";

			StringBuilder QMakeSourceFilesListBuilder = new StringBuilder("SOURCES += \\ \n");
			StringBuilder QMakeHeaderFilesListBuilder = new StringBuilder("HEADERS += \\ \n");
			StringBuilder QMakeConfigFilesListBuilder = new StringBuilder("OTHER_FILES += \\ \n");
			string QMakeTargetList = "QMAKE_EXTRA_TARGETS += \\ \n";

			if (!String.IsNullOrEmpty(GameProjectName))
			{
				GameProjectRootPath = GameProjectName + "RootPath=" + GameProjectPath + "\n\n";
			}

			QMakeFileContent.Append(
				"# UnrealEngine.pro generated by QMakefileGenerator.cs\n" +
				"# *DO NOT EDIT*\n\n" +
				"TEMPLATE = aux\n" +
				"CONFIG += c++14\n" +
				"CONFIG -= console\n" +
				"CONFIG -= app_bundle\n" +
				"CONFIG -= qt\n\n" +
				"TARGET = UE5 \n\n" +
				"unrealRootPath=" + UnrealRootPath + "\n" +
				GameProjectRootPath +
				QMakeGameProjectFile +
				BuildCommand +
				"args=$(ARGS)\n\n" +
				"include(" + QMakeFilesDirectory + Seperator + QMakeSourcePriFileName + ")\n" +
				"include(" + QMakeFilesDirectory + Seperator + QMakeHeaderPriFileName + ")\n" +
				"include(" + QMakeFilesDirectory + Seperator + QMakeConfigPriFileName + ")\n" +
				"include(" + QMakeFilesDirectory + Seperator + QMakeIncludesFileName + ")\n" +
				"include(" + QMakeFilesDirectory + Seperator + QMakeDefinesFileName + ")\n\n"
			);

			// Create SourceFiles, HeaderFiles, and ConfigFiles sections.
			List<FileReference> AllModuleFiles = DiscoverModules(FindGameProjects(Logger), null);
			foreach (FileReference CurModuleFile in AllModuleFiles)
			{
				List<FileReference> FoundFiles = SourceFileSearch.FindModuleSourceFiles(CurModuleFile);
				foreach (FileReference CurSourceFile in FoundFiles)
				{
					string SourceFileRelativeToRoot = CurSourceFile.MakeRelativeTo(Unreal.EngineDirectory);
					// Exclude some directories that we don't compile (note that we still want Windows/Mac etc for code navigation)
					if (!SourceFileRelativeToRoot.Contains($"Source{Seperator}ThirdParty{Seperator}", StringComparison.Ordinal))
					{
						if (SourceFileRelativeToRoot.EndsWith(".cpp", StringComparison.Ordinal))
						{
							if (!SourceFileRelativeToRoot.StartsWith("..", StringComparison.Ordinal) && !Path.IsPathRooted(SourceFileRelativeToRoot))
							{
								QMakeSourceFilesListBuilder.Append("\t\"" + $"$$unrealRootPath{Seperator}Engine{Seperator}" + SourceFileRelativeToRoot + "\" \\\n");
							}
							else
							{
								if (String.IsNullOrEmpty(GameProjectName))
								{
									QMakeSourceFilesListBuilder.Append("\t\"" + SourceFileRelativeToRoot.Substring(3) + "\" \\\n");
								}
								else
								{
									QMakeSourceFilesListBuilder.Append("\t\"$$" + GameProjectName + $"RootPath{Seperator}" + Utils.MakePathRelativeTo(CurSourceFile.FullName, GameProjectPath) + "\" \\\n");
								}
							}
						}
						if (SourceFileRelativeToRoot.EndsWith(".h", StringComparison.Ordinal))
						{
							if (!SourceFileRelativeToRoot.StartsWith("..", StringComparison.Ordinal) && !Path.IsPathRooted(SourceFileRelativeToRoot))
							{
								// SourceFileRelativeToRoot = "Engine/" + SourceFileRelativeToRoot;
								QMakeHeaderFilesListBuilder.Append("\t\"" + $"$$unrealRootPath{Seperator}Engine{Seperator}" + SourceFileRelativeToRoot + "\" \\\n");
							}
							else
							{
								if (String.IsNullOrEmpty(GameProjectName))
								{
									// SourceFileRelativeToRoot = SourceFileRelativeToRoot.Substring (3);
									QMakeHeaderFilesListBuilder.Append("\t\"" + SourceFileRelativeToRoot.Substring(3) + "\" \\\n");
								}
								else
								{
									QMakeHeaderFilesListBuilder.Append("\t\"$$" + GameProjectName + $"RootPath{Seperator}" + Utils.MakePathRelativeTo(CurSourceFile.FullName, GameProjectPath) + "\" \\\n");
								}
							}
						}
						if (SourceFileRelativeToRoot.EndsWith(".cs", StringComparison.Ordinal))
						{
							if (!SourceFileRelativeToRoot.StartsWith("..", StringComparison.Ordinal) && !Path.IsPathRooted(SourceFileRelativeToRoot))
							{
								// SourceFileRelativeToRoot = "Engine/" + SourceFileRelativeToRoot;
								QMakeConfigFilesListBuilder.Append("\t\"" + $"$$unrealRootPath{Seperator}Engine{Seperator}" + SourceFileRelativeToRoot + "\" \\\n");
							}
							else
							{
								if (String.IsNullOrEmpty(GameProjectName))
								{
									// SourceFileRelativeToRoot = SourceFileRelativeToRoot.Substring (3);
									QMakeConfigFilesListBuilder.Append("\t\"" + SourceFileRelativeToRoot.Substring(3) + "\" \\\n");
								}
								else
								{
									QMakeConfigFilesListBuilder.Append("\t\"$$" + GameProjectName + $"RootPath{Seperator}" + Utils.MakePathRelativeTo(CurSourceFile.FullName, GameProjectPath) + "\" \\\n");
								}
							}
						}
					}
				}
			}

			// Add section end to section strings;
			QMakeSourceFilesListBuilder.Append(QMakeSectionEnd);
			QMakeHeaderFilesListBuilder.Append(QMakeSectionEnd);
			QMakeConfigFilesListBuilder.Append(QMakeSectionEnd);

			// Append sections to the QMakeLists.txt file
			QMakeSourcePriFileContent.Append(QMakeSourceFilesListBuilder);
			QMakeHeaderPriFileContent.Append(QMakeHeaderFilesListBuilder);
			QMakeConfigPriFileContent.Append(QMakeConfigFilesListBuilder);

			string QMakeProjectCmdArg = "";

			foreach (ProjectFile Project in GeneratedProjectFiles)
			{
				foreach (ProjectTarget TargetFile in Project.ProjectTargets.OfType<ProjectTarget>())
				{
					if (TargetFile.TargetFilePath == null)
					{
						continue;
					}

					string TargetName = TargetFile.TargetFilePath.GetFileNameWithoutAnyExtensions();        // Remove both ".cs" and ".

					foreach (UnrealTargetConfiguration CurConfiguration in (UnrealTargetConfiguration[])Enum.GetValues(typeof(UnrealTargetConfiguration)))
					{
						if (CurConfiguration != UnrealTargetConfiguration.Unknown && CurConfiguration != UnrealTargetConfiguration.Development)
						{
							if (InstalledPlatformInfo.IsValidConfiguration(CurConfiguration, EProjectType.Code))
							{

								if (TargetName == GameProjectName || TargetName == (GameProjectName + "Editor"))
								{
									QMakeProjectCmdArg = " -project=\"\\\"$$gameProjectFile\\\"\"";
								}
								string ConfName = Enum.GetName(typeof(UnrealTargetConfiguration), CurConfiguration)!;
								QMakeFileContent.Append(String.Format("{0}-{3}-{1}.commands = $$build {0} {3} {1} {2} $$args\n", TargetName, ConfName, QMakeProjectCmdArg, CurrentPlatform));
								QMakeTargetList += "\t" + TargetName + "-" + CurrentPlatform + "-" + ConfName + " \\\n"; // , TargetName, ConfName);
							}
						}
					}

					if (TargetName == GameProjectName || TargetName == (GameProjectName + "Editor"))
					{
						QMakeProjectCmdArg = " -project=\"\\\"$$gameProjectFile\\\"\"";
					}
					QMakeFileContent.Append(String.Format("{0}.commands = $$build {0} {2} Development {1} $$args\n\n", TargetName, QMakeProjectCmdArg, CurrentPlatform));
					QMakeTargetList += "\t" + TargetName + " \\\n";
				}
			}

			QMakeFileContent.Append(QMakeTargetList.TrimEnd('\\'));

			string FullFileName = Path.Combine(PrimaryProjectPath.FullName, FileName);
			string QMakeFilePath = Path.Combine(ProjectFileGenerator.PrimaryProjectPath.FullName, QMakeFilesDirectory);

			string FullQMakeDefinesFileName = Path.Combine(QMakeFilePath, QMakeDefinesFileName);
			string FullQMakeIncludesFileName = Path.Combine(QMakeFilePath, QMakeIncludesFileName);
			string FullQMakeSourcePriFileName = Path.Combine(QMakeFilePath, QMakeSourcePriFileName);
			string FullQMakeHeaderPriFileName = Path.Combine(QMakeFilePath, QMakeHeaderPriFileName);
			string FullQMakeConfigPriFileName = Path.Combine(QMakeFilePath, QMakeConfigPriFileName);

			WriteFileIfChanged(FullQMakeDefinesFileName, QMakeDefinesPriFileContent.ToString(), Logger);
			WriteFileIfChanged(FullQMakeIncludesFileName, QMakeIncludesPriFileContent.ToString(), Logger);
			WriteFileIfChanged(FullQMakeSourcePriFileName, QMakeSourcePriFileContent.ToString(), Logger);

			WriteFileIfChanged(FullQMakeHeaderPriFileName, QMakeHeaderPriFileContent.ToString(), Logger);
			WriteFileIfChanged(FullQMakeConfigPriFileName, QMakeConfigPriFileContent.ToString(), Logger);

			return WriteFileIfChanged(FullFileName, QMakeFileContent.ToString(), Logger);
		}

		/// ProjectFileGenerator interface
		//protected override bool WritePrimaryProjectFile( ProjectFile UBTProject )
		protected override bool WriteProjectFiles(PlatformProjectGeneratorCollection PlatformProjectGenerators, ILogger Logger)
		{
			return WriteQMakePro(Logger);
		}

		/// ProjectFileGenerator interface
		/// <summary>
		/// Allocates a generator-specific project file object
		/// </summary>
		/// <param name="InitFilePath">Path to the project file</param>
		/// <param name="BaseDir">The base directory for files within this project</param>
		/// <returns>The newly allocated project file object</returns>
		protected override ProjectFile AllocateProjectFile(FileReference InitFilePath, DirectoryReference BaseDir)
		{
			return new QMakefileProjectFile(InitFilePath, BaseDir);
		}

		/// ProjectFileGenerator interface
		public override void CleanProjectFiles(DirectoryReference InPrimaryProjectDirectory, string InPrimaryProjectName, DirectoryReference InIntermediateProjectFilesDirectory, ILogger Logger)
		{
		}
	}
}
