// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	internal class TargetEntry
	{
		public TargetEntry(FileReference OutputFile, UEBuildTarget BuildTarget, bool bBuildByDefault)
		{
			this.OutputFile = OutputFile;
			this.BuildTarget = BuildTarget;
			this.bBuildByDefault = bBuildByDefault;
		}

		public readonly FileReference OutputFile;
		public readonly UEBuildTarget BuildTarget;
		public readonly bool bBuildByDefault;
	}

	internal class RiderProjectFile : ProjectFile
	{
		private readonly DirectoryReference RootPath;
		private readonly HashSet<TargetType> TargetTypes;
		private readonly CommandLineArguments Arguments;

		private ToolchainInfo RootToolchainInfo = new ToolchainInfo();
		private UEBuildTarget? CurrentTarget;

		public RiderProjectFile(FileReference InProjectFilePath, DirectoryReference BaseDir,
			DirectoryReference RootPath, HashSet<TargetType> TargetTypes, CommandLineArguments Arguments)
			: base(InProjectFilePath, BaseDir)
		{
			this.RootPath = RootPath;
			this.TargetTypes = TargetTypes;
			this.Arguments = Arguments;
		}

		/// <summary>
		/// Write project file info in JSON file.
		/// For every combination of <c>UnrealTargetPlatform</c>, <c>UnrealTargetConfiguration</c> and <c>TargetType</c>
		/// will be generated separate JSON file.
		/// Project file will be stored:
		/// For UE:  {UnrealRoot}/Engine/Intermediate/ProjectFiles/.Rider/{Platform}/{Configuration}/{TargetType}/{ProjectName}.json
		/// For game: {GameRoot}/Intermediate/ProjectFiles/.Rider/{Platform}/{Configuration}/{TargetType}/{ProjectName}.json
		/// </summary>
		/// <remarks>
		/// * <c>TargetType.Editor</c> will be generated for current platform only and will ignore <c>UnrealTargetConfiguration.Test</c> and <c>UnrealTargetConfiguration.Shipping</c> configurations
		/// * <c>TargetType.Program</c>  will be generated for current platform only and <c>UnrealTargetConfiguration.Development</c> configuration only 
		/// </remarks>
		/// <param name="InPlatforms"></param>
		/// <param name="InConfigurations"></param>
		/// <param name="PlatformProjectGenerators"></param>
		/// <param name="Minimize"></param>
		/// <param name="Logger"></param>
		/// <returns></returns>
		public bool WriteProjectFile(List<UnrealTargetPlatform> InPlatforms,
			List<UnrealTargetConfiguration> InConfigurations,
			PlatformProjectGeneratorCollection PlatformProjectGenerators, JsonWriterStyle Minimize, ILogger Logger)
		{
			string ProjectName = ProjectFilePath.GetFileNameWithoutAnyExtensions();
			DirectoryReference ProjectRootFolder = RootPath;
			List<TargetEntry> FileToTarget = new List<TargetEntry>();

			foreach (UnrealTargetPlatform Platform in InPlatforms)
			{
				foreach (UnrealTargetConfiguration Configuration in InConfigurations)
				{
					foreach (ProjectTarget ProjectTarget in ProjectTargets.OfType<ProjectTarget>())
					{
						if (TargetTypes.Any() && !TargetTypes.Contains(ProjectTarget.TargetRules!.Type))
						{
							continue;
						}

						// Skip Programs for all configs except for current platform + Development & Debug configurations
						if (ProjectTarget.TargetRules!.Type == TargetType.Program &&
							(BuildHostPlatform.Current.Platform != Platform ||
							 !(Configuration == UnrealTargetConfiguration.Development || Configuration == UnrealTargetConfiguration.Debug)))
						{
							continue;
						}

						// Skip Editor for all platforms except for current platform
						if (ProjectTarget.TargetRules.Type == TargetType.Editor && (BuildHostPlatform.Current.Platform != Platform || (Configuration == UnrealTargetConfiguration.Test || Configuration == UnrealTargetConfiguration.Shipping)))
						{
							continue;
						}

						// Skip invalid platforms for the current TargetType
						if (!ProjectTarget.SupportedPlatforms.Contains(Platform))
						{
							continue;
						}

						bool bBuildByDefault = ShouldBuildByDefaultForSolutionTargets && ProjectTarget.SupportedPlatforms.Contains(Platform);

						DirectoryReference ConfigurationFolder = DirectoryReference.Combine(ProjectRootFolder, Platform.ToString(), Configuration.ToString());

						DirectoryReference TargetFolder =
							DirectoryReference.Combine(ConfigurationFolder, ProjectTarget.TargetRules.Type.ToString());

						UnrealArchitectures ProjectArchitectures = UEBuildPlatform
							.GetBuildPlatform(Platform)
							.ArchitectureConfig.ActiveArchitectures(ProjectTarget.UnrealProjectFilePath, ProjectTarget.Name);
						TargetDescriptor TargetDesc = new TargetDescriptor(ProjectTarget.UnrealProjectFilePath, ProjectTarget.Name,
							Platform, Configuration, ProjectArchitectures, Arguments);
						try
						{
							UEBuildTarget BuildTarget = UEBuildTarget.Create(TargetDesc, false, false, false, UnrealIntermediateEnvironment.GenerateProjectFiles, Logger);

							FileReference OutputFile = FileReference.Combine(TargetFolder, $"{ProjectName}.json");
							FileToTarget.Add(new TargetEntry(OutputFile, BuildTarget, bBuildByDefault));
						}
						catch (Exception Ex)
						{
							Logger.LogWarning("Exception while generating include data for Target:{Target}, Platform: {Platform}, Configuration: {Configuration}", TargetDesc.Name, Platform.ToString(), Configuration.ToString());
							Logger.LogWarning("{Ex}", Ex.ToString());
						}
					}
				}
			}
			foreach (TargetEntry TargetEntry in FileToTarget)
			{
				try
				{
					CurrentTarget = TargetEntry.BuildTarget;
					CurrentTarget.PreBuildSetup(Logger);
					SerializeTarget(TargetEntry.OutputFile, CurrentTarget, PlatformProjectGenerators, Minimize, TargetEntry.bBuildByDefault, Logger);
				}
				catch (Exception Ex)
				{
					Logger.LogWarning("Exception while generating include data for Target:{Target}, Platform: {Platform}, Configuration: {Configuration}",
						TargetEntry.BuildTarget.AppName, TargetEntry.BuildTarget.Platform.ToString(), TargetEntry.BuildTarget.Configuration.ToString());
					Logger.LogWarning("{Ex}", Ex.ToString());
				}
			}

			return true;
		}

		private void SerializeTarget(FileReference OutputFile, UEBuildTarget BuildTarget, PlatformProjectGeneratorCollection PlatformProjectGenerators, JsonWriterStyle Minimize, bool bBuildByDefault, ILogger Logger)
		{
			DirectoryReference.CreateDirectory(OutputFile.Directory);
			using (JsonWriter Writer = new JsonWriter(OutputFile, Minimize))
			{
				ExportTarget(BuildTarget, Writer, PlatformProjectGenerators, bBuildByDefault, Logger);
			}
		}

		/// <summary>
		/// Write a Target to a JSON writer. Is array is empty, don't write anything
		/// </summary>
		/// <param name="Target"></param>
		/// <param name="Writer">Writer for the array data</param>
		/// <param name="PlatformProjectGenerators"></param>
		/// <param name="bBuildByDefault"></param>
		/// <param name="Logger">Logger for output</param>
		private void ExportTarget(UEBuildTarget Target, JsonWriter Writer, PlatformProjectGeneratorCollection PlatformProjectGenerators, bool bBuildByDefault, ILogger Logger)
		{
			Writer.WriteObjectStart();

			Writer.WriteValue("Name", Target.TargetName);
			Writer.WriteValue("Configuration", Target.Configuration.ToString());
			Writer.WriteValue("Platform", Target.Platform.ToString());
			Writer.WriteValue("TargetFile", Target.TargetRulesFile.FullName);
			if (Target.ProjectFile != null)
			{
				Writer.WriteValue("ProjectFile", Target.ProjectFile.FullName);
			}

			ExportEnvironmentToJson(Target, Writer, PlatformProjectGenerators, bBuildByDefault, Logger);

			if (Target.Binaries.Any())
			{
				Writer.WriteArrayStart("Binaries");
				foreach (UEBuildBinary Binary in Target.Binaries)
				{
					Writer.WriteObjectStart();
					ExportBinary(Binary, Writer);
					Writer.WriteObjectEnd();
				}
				Writer.WriteArrayEnd();
			}

			CppCompileEnvironment GlobalCompileEnvironment = Target.CreateCompileEnvironmentForProjectFiles(Logger);
			HashSet<string> ModuleNames = new HashSet<string>();
			Writer.WriteObjectStart("Modules");
			foreach (UEBuildBinary Binary in Target.Binaries)
			{
				CppCompileEnvironment BinaryCompileEnvironment = Binary.CreateBinaryCompileEnvironment(GlobalCompileEnvironment);
				foreach (UEBuildModule Module in Binary.Modules)
				{
					if (ModuleNames.Add(Module.Name))
					{
						Writer.WriteObjectStart(Module.Name);
						UEBuildModuleCPP? ModuleCpp = Module as UEBuildModuleCPP;
						if (ModuleCpp != null)
						{
							CppCompileEnvironment ModuleCompileEnvironment = ModuleCpp.CreateCompileEnvironmentForIntellisense(Target.Rules, BinaryCompileEnvironment, Logger);
							ExportModuleCpp(ModuleCpp, ModuleCompileEnvironment, Writer, Logger);
							Module.PrivateIncludePaths.UnionWith(ModuleCompileEnvironment.UserIncludePaths);
						}
						ExportModule(Module, Binary.OutputDir, Target.GetExecutableDir(), Writer, Logger);
						Writer.WriteObjectEnd();
					}
				}
			}
			Writer.WriteObjectEnd();

			ExportPluginsFromTarget(Target, Writer, Logger);

			Writer.WriteObjectEnd();
		}

		private void ExportModuleCpp(UEBuildModuleCPP ModuleCPP, CppCompileEnvironment ModuleCompileEnvironment, JsonWriter Writer, ILogger Logger)
		{
			Writer.WriteValue("GeneratedCodeDirectory", ModuleCPP.GeneratedCodeDirectory != null ? ModuleCPP.GeneratedCodeDirectory.FullName : String.Empty);

			ToolchainInfo ModuleToolchainInfo = GenerateToolchainInfo(ModuleCompileEnvironment);
			if (!ModuleToolchainInfo.Equals(RootToolchainInfo))
			{
				Writer.WriteObjectStart("ToolchainInfo");
				foreach (Tuple<string, object?> Field in ModuleToolchainInfo.GetDiff(RootToolchainInfo))
				{
					WriteField(ModuleCPP.Name, Writer, Field, Logger);
				}
				Writer.WriteObjectEnd();
			}

			if (ModuleCompileEnvironment.PrecompiledHeaderIncludeFilename != null)
			{
				string CorrectFilePathPch;
				if (ExtractWrappedIncludeFile(ModuleCompileEnvironment.PrecompiledHeaderIncludeFilename, Logger, out CorrectFilePathPch))
				{
					Writer.WriteValue("SharedPCHFilePath", CorrectFilePathPch);
				}
			}
		}

		private static bool ExtractWrappedIncludeFile(FileSystemReference FileRef, ILogger Logger, out string CorrectFilePathPch)
		{
			const string Prefix = "// Generated by UnrealBuildTool (UEBuildModuleCPP.cs) : PCH for ";
			CorrectFilePathPch = "";
			try
			{
				using (StreamReader Reader = new StreamReader(FileRef.FullName))
				{
					string? Line = Reader.ReadLine();
					if (Line != null && Line.StartsWith(Prefix, StringComparison.Ordinal))
					{
						CorrectFilePathPch = Line.Substring(Prefix.Length).Trim();
						return true;
					}
				}
			}
			finally
			{
				Logger.LogDebug("Couldn't extract path to PCH from {FileRef}", FileRef);
			}
			return false;
		}

		/// <summary>
		/// Write a Module to a JSON writer. If array is empty, don't write anything
		/// </summary>
		/// <param name="BinaryOutputDir"></param>
		/// <param name="TargetOutputDir"></param>
		/// <param name="Writer">Writer for the array data</param>
		/// <param name="Module"></param>
		/// <param name="Logger"></param>
		private static void ExportModule(UEBuildModule Module, DirectoryReference BinaryOutputDir, DirectoryReference TargetOutputDir, JsonWriter Writer, ILogger Logger)
		{
			Writer.WriteValue("Name", Module.Name);
			Writer.WriteValue("Directory", Module.ModuleDirectory.FullName);
			Writer.WriteValue("Rules", Module.RulesFile.FullName);
			ExportJsonStringArray(Writer, "SubRules", Module.Rules.SubclassRules);
			Writer.WriteValue("PCHUsage", Module.Rules.PCHUsage.ToString());

			if (Module.Rules.PrivatePCHHeaderFile != null)
			{
				Writer.WriteValue("PrivatePCH", FileReference.Combine(Module.ModuleDirectory, Module.Rules.PrivatePCHHeaderFile).FullName);
			}

			if (Module.Rules.SharedPCHHeaderFile != null)
			{
				Writer.WriteValue("SharedPCH", FileReference.Combine(Module.ModuleDirectory, Module.Rules.SharedPCHHeaderFile).FullName);
			}

			ExportJsonModuleArray(Writer, "PublicDependencyModules", Module.PublicDependencyModules);
			ExportJsonModuleArray(Writer, "PublicIncludePathModules", Module.PublicIncludePathModules);
			ExportJsonModuleArray(Writer, "PrivateDependencyModules", Module.PrivateDependencyModules);
			ExportJsonModuleArray(Writer, "PrivateIncludePathModules", Module.PrivateIncludePathModules);
			ExportJsonModuleArray(Writer, "DynamicallyLoadedModules", Module.DynamicallyLoadedModules);

			ExportJsonStringArray(Writer, "PublicSystemIncludePaths", Module.PublicSystemIncludePaths.Select(x => x.FullName));
			ExportJsonStringArray(Writer, "PublicIncludePaths", Module.PublicIncludePaths.Select(x => x.FullName));
			ExportJsonStringArray(Writer, "InternalIncludePaths", Module.InternalIncludePaths.Select(x => x.FullName));

			ExportJsonStringArray(Writer, "LegacyPublicIncludePaths", Module.LegacyPublicIncludePaths.Select(x => x.FullName));
			ExportJsonStringArray(Writer, "LegacyParentIncludePaths", Module.LegacyParentIncludePaths.Select(x => x.FullName));

			ExportJsonStringArray(Writer, "PrivateIncludePaths", Module.PrivateIncludePaths.Select(x => x.FullName));
			ExportJsonStringArray(Writer, "PublicLibraryPaths", Module.PublicSystemLibraryPaths.Select(x => x.FullName));
			ExportJsonStringArray(Writer, "PublicAdditionalLibraries", Module.PublicSystemLibraries.Concat(Module.PublicLibraries.Select(x => x.FullName)));
			ExportJsonStringArray(Writer, "PublicFrameworks", Module.PublicFrameworks);
			ExportJsonStringArray(Writer, "PublicWeakFrameworks", Module.PublicWeakFrameworks);
			ExportJsonStringArray(Writer, "PublicDelayLoadDLLs", Module.PublicDelayLoadDLLs);
			ExportJsonStringArray(Writer, "PublicDefinitions", Module.PublicDefinitions);
			ExportJsonStringArray(Writer, "PrivateDefinitions", Module.Rules.PrivateDefinitions.Concat(EngineIncludeOrderHelper.GetDeprecationDefines(Module.Rules.IncludeOrderVersion)));
			ExportJsonStringArray(Writer, "ProjectDefinitions", /* TODO: Add method ShouldAddProjectDefinitions */ !Module.Rules.bTreatAsEngineModule ? Module.Rules.Target.ProjectDefinitions : Array.Empty<string>());
			ExportJsonStringArray(Writer, "ApiDefinitions", Module.GetEmptyApiMacros());
			Writer.WriteValue("ShouldAddLegacyPublicIncludePaths", Module.Rules.bLegacyPublicIncludePaths);
			Writer.WriteValue("ShouldAddLegacyParentIncludePaths", Module.Rules.bLegacyParentIncludePaths);

			if (Module.Rules.CircularlyReferencedDependentModules.Any())
			{
				Writer.WriteArrayStart("CircularlyReferencedModules");
				foreach (string ModuleName in Module.Rules.CircularlyReferencedDependentModules)
				{
					Writer.WriteValue(ModuleName);
				}
				Writer.WriteArrayEnd();
			}

			if (Module.Rules.RuntimeDependencies.Inner.Any())
			{
				// We don't use info from RuntimeDependencies for code analyzes (at the moment)
				// So we're OK with skipping some values if they are not presented
				Writer.WriteArrayStart("RuntimeDependencies");
				foreach (ModuleRules.RuntimeDependency RuntimeDependency in Module.Rules.RuntimeDependencies.Inner)
				{
					Writer.WriteObjectStart();

					try
					{
						Writer.WriteValue("Path",
							Module.ExpandPathVariables(RuntimeDependency.Path, BinaryOutputDir, TargetOutputDir));
					}
					catch (BuildException buildException)
					{
						Logger.LogDebug("Value {Value} for module {ModuleName} will not be stored. Reason: {Ex}", "Path", Module.Name, buildException);
					}

					if (RuntimeDependency.SourcePath != null)
					{
						try
						{
							Writer.WriteValue("SourcePath",
								Module.ExpandPathVariables(RuntimeDependency.SourcePath, BinaryOutputDir,
									TargetOutputDir));
						}
						catch (BuildException buildException)
						{
							Logger.LogDebug("Value {Value} for module {ModuleName} will not be stored. Reason: {Ex}", "SourcePath", Module.Name, buildException);
						}
					}

					Writer.WriteValue("Type", RuntimeDependency.Type.ToString());

					Writer.WriteObjectEnd();
				}
				Writer.WriteArrayEnd();
			}
		}

		/// <summary>
		/// Write an array of Modules to a JSON writer. If array is empty, don't write anything
		/// </summary>
		/// <param name="Writer">Writer for the array data</param>
		/// <param name="ArrayName">Name of the array property</param>
		/// <param name="Modules">Sequence of Modules to write. May be null.</param>
		private static void ExportJsonModuleArray(JsonWriter Writer, string ArrayName, IEnumerable<UEBuildModule>? Modules)
		{
			if (Modules == null || !Modules.Any())
			{
				return;
			}

			Writer.WriteArrayStart(ArrayName);
			foreach (UEBuildModule Module in Modules)
			{
				Writer.WriteValue(Module.Name);
			}
			Writer.WriteArrayEnd();
		}

		/// <summary>
		/// Write an array of strings to a JSON writer. Ifl array is empty, don't write anything
		/// </summary>
		/// <param name="Writer">Writer for the array data</param>
		/// <param name="ArrayName">Name of the array property</param>
		/// <param name="Strings">Sequence of strings to write. May be null.</param>
		private static void ExportJsonStringArray(JsonWriter Writer, string ArrayName, IEnumerable<string>? Strings)
		{
			if (Strings == null || !Strings.Any())
			{
				return;
			}

			Writer.WriteArrayStart(ArrayName);
			foreach (string String in Strings)
			{
				Writer.WriteValue(String);
			}
			Writer.WriteArrayEnd();
		}

		/// <summary>
		/// Write uplugin content to a JSON writer
		/// </summary>
		/// <param name="Plugin">Uplugin description</param>
		/// <param name="Writer">JSON writer</param>
		private static void ExportPlugin(UEBuildPlugin Plugin, JsonWriter Writer)
		{
			Writer.WriteObjectStart(Plugin.Name);

			Writer.WriteValue("File", Plugin.File.FullName);
			Writer.WriteValue("Type", Plugin.Type.ToString());
			if (Plugin.Dependencies != null && Plugin.Dependencies.Any())
			{
				Writer.WriteStringArrayField("Dependencies", Plugin.Dependencies.Select(it => it.Name));
			}
			if (Plugin.Modules.Any())
			{
				Writer.WriteStringArrayField("Modules", Plugin.Modules.Select(it => it.Name));
			}

			Writer.WriteObjectEnd();
		}

		/// <summary>
		/// Setup plugins for Target and write plugins to JSON writer. Don't write anything if there are no plugins 
		/// </summary>
		/// <param name="Target"></param>
		/// <param name="Writer"></param>
		/// <param name="Logger"></param>
		private static void ExportPluginsFromTarget(UEBuildTarget Target, JsonWriter Writer, ILogger Logger)
		{
			Target.SetupPlugins(Logger);
			if (Target.BuildPlugins == null || !Target.BuildPlugins.Any())
			{
				return;
			}

			Writer.WriteObjectStart("Plugins");
			foreach (UEBuildPlugin plugin in Target.BuildPlugins!)
			{
				ExportPlugin(plugin, Writer);
			}
			Writer.WriteObjectEnd();
		}

		/// <summary>
		/// Write information about this binary to a JSON file
		/// </summary>
		/// <param name="Binary"></param>
		/// <param name="Writer">Writer for this binary's data</param>
		private static void ExportBinary(UEBuildBinary Binary, JsonWriter Writer)
		{
			Writer.WriteValue("File", Binary.OutputFilePath.FullName);
			Writer.WriteValue("Type", Binary.Type.ToString());

			Writer.WriteArrayStart("Modules");
			foreach (UEBuildModule Module in Binary.Modules)
			{
				Writer.WriteValue(Module.Name);
			}
			Writer.WriteArrayEnd();
		}

		/// <summary>
		/// Write C++ toolchain information to JSON writer
		/// </summary>
		/// <param name="Target"></param>
		/// <param name="Writer"></param>
		/// <param name="PlatformProjectGenerators"></param>
		/// <param name="bBuildByDefault"></param>
		/// <param name="Logger"></param>
		private void ExportEnvironmentToJson(UEBuildTarget Target, JsonWriter Writer, PlatformProjectGeneratorCollection PlatformProjectGenerators, bool bBuildByDefault, ILogger Logger)
		{
			CppCompileEnvironment GlobalCompileEnvironment = Target.CreateCompileEnvironmentForProjectFiles(Logger);

			RootToolchainInfo = GenerateToolchainInfo(GlobalCompileEnvironment);

			Writer.WriteObjectStart("ToolchainInfo");
			foreach (Tuple<string, object?> Field in RootToolchainInfo.GetFields())
			{
				WriteField(Target.TargetName, Writer, Field, Logger);
			}
			Writer.WriteObjectEnd();

			ExportBuildInfo(Writer, Target, PlatformProjectGenerators, bBuildByDefault, Logger);

			Writer.WriteArrayStart("EnvironmentIncludePaths");
			foreach (DirectoryReference Path in GlobalCompileEnvironment.UserIncludePaths)
			{
				Writer.WriteValue(Path.FullName);
			}
			foreach (DirectoryReference Path in GlobalCompileEnvironment.SystemIncludePaths)
			{
				Writer.WriteValue(Path.FullName);
			}

			PlatformProjectGenerator? ProjGenerator = PlatformProjectGenerators.GetPlatformProjectGenerator(Target.Platform, true);
			if (ProjGenerator != null)
			{
				foreach (string Path in ProjGenerator.GetSystemIncludePaths(Target))
				{
					Writer.WriteValue(Path);
				}
			}

			Writer.WriteArrayEnd();

			Writer.WriteArrayStart("EnvironmentDefinitions");
			foreach (string Definition in GlobalCompileEnvironment.Definitions)
			{
				Writer.WriteValue(Definition);
			}
			Writer.WriteArrayEnd();
		}

		private void ExportBuildInfo(JsonWriter Writer, UEBuildTarget Target, PlatformProjectGeneratorCollection PlatformProjectGenerators,
			bool bBuildByDefault, ILogger Logger)
		{
			if (IsStubProject)
			{
				return;
			}

			try
			{
				string BuildScript;
				string RebuildScript;
				string CleanScript;
				string BuildArguments;
				string RebuildArguments;
				string CleanArguments;
				string Output = Target.Binaries[0].OutputFilePath.FullName;

				ProjectTarget ProjectTarget = ProjectTargets.OfType<ProjectTarget>().Single(It => Target.TargetRulesFile == It.TargetFilePath);
				string UProjectPath = IsForeignProject ? String.Format("\"{0}\"", ProjectTarget.UnrealProjectFilePath!.FullName) : "";
				UnrealTargetPlatform HostPlatform = BuildHostPlatform.Current.Platform;
				if (HostPlatform.IsInGroup(UnrealPlatformGroup.Windows))
				{
					PlatformProjectGenerator? ProjGenerator = PlatformProjectGenerators.GetPlatformProjectGenerator(Target.Platform, true);
					VCProjectFile.BuildCommandBuilder BuildCommandBuilder =
						new VCProjectFile.BuildCommandBuilder(
							new PlatformProjectGenerator.VSSettings(Target.Platform, Target.Configuration, VCProjectFileFormat.Default, null),
							ProjectTarget, UProjectPath)
						{
							ProjectGenerator = ProjGenerator,
							bIsForeignProject = IsForeignProject
						};

					BuildArguments = RebuildArguments = CleanArguments = BuildCommandBuilder.GetBuildArguments();
					BuildScript = EscapePath(BuildCommandBuilder.BuildScript.FullName);
					RebuildScript = EscapePath(BuildCommandBuilder.RebuildScript.FullName);
					CleanScript = EscapePath(BuildCommandBuilder.CleanScript.FullName);
				}
				else
				{
					BuildScript = CleanScript = GetBuildScript(HostPlatform);
					BuildArguments = GetBuildArguments(HostPlatform, ProjectTarget, Target, UProjectPath, false);
					CleanArguments = GetBuildArguments(HostPlatform, ProjectTarget, Target, UProjectPath, true);
					RebuildScript = RebuildArguments = "";
				}

				Writer.WriteObjectStart("BuildInfo");
				Writer.WriteValue("bBuildByDefault", bBuildByDefault);
				WriteCommand(Writer, "BuildCmd", BuildScript, BuildArguments);
				WriteCommand(Writer, "RebuildCmd", RebuildScript, RebuildArguments);
				WriteCommand(Writer, "CleanCmd", CleanScript, CleanArguments);
				Writer.WriteValue("Output", Output);
				Writer.WriteObjectEnd();
			}
			catch (Exception Ex)
			{
				Logger.LogWarning(Ex,
					"Exception while generating build info for Target: {Target}, Platform: {Platform}, Configuration: {Configuration}",
					Target.TargetName, Target.Platform.ToString(), Target.Configuration.ToString());
			}
		}

		private static string GetBuildScript(UnrealTargetPlatform HostPlatform)
		{
			DirectoryReference BatchFilesDirectory = DirectoryReference.Combine(Unreal.EngineDirectory, "Build", "BatchFiles");
			string ScriptExtension = HostPlatform.IsInGroup(UnrealPlatformGroup.Windows) ? ".bat" : ".sh";
			if (HostPlatform.IsInGroup(UnrealPlatformGroup.Linux))
			{
				BatchFilesDirectory = DirectoryReference.Combine(BatchFilesDirectory, "Linux");
			}
			else if (HostPlatform == UnrealTargetPlatform.Mac)
			{
				BatchFilesDirectory = DirectoryReference.Combine(BatchFilesDirectory, "Mac");
			}

			return EscapePath(FileReference.Combine(BatchFilesDirectory, "Build" + ScriptExtension).FullName);
		}

		private string GetBuildArguments(UnrealTargetPlatform HostPlatform, ProjectTarget ProjectTarget, UEBuildTarget Target, string UProjectPath, bool bIsClean)
		{
			UnrealTargetConfiguration Configuration = Target.Configuration;
			UnrealTargetPlatform Platform = Target.Platform;
			string TargetName = ProjectTarget.TargetFilePath.GetFileNameWithoutAnyExtensions();
			StringBuilder BuildArguments = new StringBuilder();
			BuildArguments.AppendFormat("{0} {1} {2}", TargetName, Platform.ToString(), Configuration.ToString());

			if (IsForeignProject)
			{
				BuildArguments.AppendFormat(" -Project={0}", UProjectPath);
			}

			if (Target.TargetType == TargetType.Editor)
			{
				BuildArguments.Append(" -buildscw");
			}

			if (bIsClean)
			{
				BuildArguments.Append(" -clean");
			}
			else if (HostPlatform.IsInGroup(UnrealPlatformGroup.Apple) &&
					 (Platform == UnrealTargetPlatform.TVOS || Platform == UnrealTargetPlatform.IOS))
			{
				BuildArguments.Append(" -deploy");
			}

			return BuildArguments.ToString();
		}

		private static void WriteCommand(JsonWriter Writer, string CommandName, string Command, string Arguments)
		{
			Writer.WriteObjectStart(CommandName);
			Writer.WriteValue("Command", Command);
			Writer.WriteValue("Args", Arguments);
			Writer.WriteObjectEnd();
		}

		private static void WriteField(string ModuleOrTargetName, JsonWriter Writer, Tuple<string, object?> Field, ILogger Logger)
		{
			if (Field.Item2 == null)
			{
				return;
			}

			string Name = Field.Item1;
			if (Field.Item2 is bool vbool)
			{
				Writer.WriteValue(Name, vbool);
			}
			else if (Field.Item2 is string vstring)
			{
				if (!String.IsNullOrEmpty(vstring))
				{
					Writer.WriteValue(Name, vstring);
				}
			}
			else if (Field.Item2 is int vint)
			{
				Writer.WriteValue(Name, vint);
			}
			else if (Field.Item2 is double vdouble)
			{
				Writer.WriteValue(Name, vdouble);
			}
			else if (Field.Item2 is CppStandardVersion version)
			{
				// Do not use version.ToString(). See: https://youtrack.jetbrains.com/issue/RIDER-68030
				switch (version)
				{
					case CppStandardVersion.Cpp20:
						Writer.WriteValue(Name, "Cpp20");
						break;
					case CppStandardVersion.Cpp23:
						Writer.WriteValue(Name, "Cpp23");
						break;
					case CppStandardVersion.Latest:
						Writer.WriteValue(Name, "Latest");
						break;
					default:
						Logger.LogError("Unsupported C++ standard type: {Type}", version);
						break;
				}
			}
			else if (Field.Item2 is Enum)
			{
				Writer.WriteValue(Name, Field.Item2.ToString());
			}
			else if (Field.Item2 is IEnumerable<string> FieldValue)
			{
				if (FieldValue.Any())
				{
					Writer.WriteStringArrayField(Name, FieldValue);
				}
			}
			else
			{
				Logger.LogWarning("Dumping incompatible ToolchainInfo field: {Name} with type: {Field} for: {ModuleOrTarget}",
					Name, Field.Item2, ModuleOrTargetName);
			}
		}

		private ToolchainInfo GenerateToolchainInfo(CppCompileEnvironment CompileEnvironment)
		{
			ToolchainInfo ToolchainInfo = new ToolchainInfo
			{
				CppStandard = CompileEnvironment.CppStandard,
				Configuration = CompileEnvironment.Configuration.ToString(),
				bEnableExceptions = CompileEnvironment.bEnableExceptions,
				bOptimizeCode = CompileEnvironment.bOptimizeCode,
				bUseInlining = CompileEnvironment.bUseInlining,
				bUseUnity = CompileEnvironment.bUseUnity,
				bCreateDebugInfo = CompileEnvironment.bCreateDebugInfo,
				bIsBuildingLibrary = CompileEnvironment.bIsBuildingLibrary,
				MinCpuArchX64 = CompileEnvironment.MinCpuArchX64,
				bIsBuildingDLL = CompileEnvironment.bIsBuildingDLL,
				bUseDebugCRT = CompileEnvironment.bUseDebugCRT,
				bUseRTTI = CompileEnvironment.bUseRTTI,
				bUseStaticCRT = CompileEnvironment.bUseStaticCRT,
				PrecompiledHeaderAction = CompileEnvironment.PrecompiledHeaderAction.ToString(),
				PrecompiledHeaderFile = CompileEnvironment.PrecompiledHeaderFile?.ToString(),
				ForceIncludeFiles = CompileEnvironment.ForceIncludeFiles.Select(Item => Item.ToString()).ToList(),
				bEnableCoroutines = CompileEnvironment.bEnableCoroutines
			};

			if (CompileEnvironment.Architectures.Architectures.Count >= 1)
			{
				ToolchainInfo.Architecture = CompileEnvironment.Architectures.Architectures[0].ToString();
			}

			if (CurrentTarget!.Platform.IsInGroup(UnrealPlatformGroup.Windows))
			{
				ToolchainInfo.bEnableAddressSanitizer = CurrentTarget.Rules.WindowsPlatform.bEnableAddressSanitizer;
				ToolchainInfo.bUpdatedCPPMacro = CurrentTarget.Rules.WindowsPlatform.bUpdatedCPPMacro;
				WindowsCompiler WindowsPlatformCompiler = CurrentTarget.Rules.WindowsPlatform.Compiler;
				ToolchainInfo.bStrictConformanceMode = WindowsPlatformCompiler.IsMSVC() && CurrentTarget.Rules.WindowsPlatform.bStrictConformanceMode;
				ToolchainInfo.bStrictPreprocessorConformanceMode = WindowsPlatformCompiler.IsMSVC();
				ToolchainInfo.Compiler = WindowsPlatformCompiler.ToString();
			}
			else
			{
				string PlatformName = $"{CurrentTarget.Platform}Platform";
				object? Value = typeof(ReadOnlyTargetRules).GetProperty(PlatformName)?.GetValue(CurrentTarget.Rules);
				object? CompilerField = Value?.GetType().GetProperty("Compiler")?.GetValue(Value);
				if (CompilerField != null)
				{
					ToolchainInfo.Compiler = CompilerField.ToString()!;
				}
			}

			return ToolchainInfo;
		}
	}
}
