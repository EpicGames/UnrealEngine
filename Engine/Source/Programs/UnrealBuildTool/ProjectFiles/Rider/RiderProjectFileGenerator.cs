// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;
using UnrealBuildTool.ProjectFiles.VisualStudio;

namespace UnrealBuildTool
{
	class RiderProjectFileGenerator : ProjectFileGenerator
	{
		public override string ProjectFileExtension => ".json";

		private readonly DirectoryReference EngineSourceProgramsDirectory =
			DirectoryReference.Combine(Unreal.EngineSourceDirectory, "Programs");

		private readonly CommandLineArguments Arguments;

		/// <summary>
		/// List of deprecated platforms.
		/// Don't generate project model for these platforms unless they are specified in "Platforms" console arguments. 
		/// </summary>
		/// <returns></returns>
		private readonly HashSet<UnrealTargetPlatform> DeprecatedPlatforms = new HashSet<UnrealTargetPlatform>();

#pragma warning disable IDE0044 // Make field readonly - these private static fields are set by command-line parsing.
		/// <summary>
		/// When specified only primary project file (root.json) will be generated.
		/// Normally project files for specific configurations are generated together with primary project file.
		/// </summary>
		[CommandLine("-OnlyPrimaryProjectFile")]
		bool bOnlyPrimaryProjectFile = false;

		/// <summary>
		/// Platforms to generate project files for
		/// </summary>
		[CommandLine("-Platforms=", ListSeparator = '+')]
		HashSet<UnrealTargetPlatform> Platforms = new HashSet<UnrealTargetPlatform>();

		/// <summary>
		/// Target types to generate project files for
		/// </summary>
		[CommandLine("-TargetTypes=", ListSeparator = '+')]
		HashSet<TargetType> TargetTypes = new HashSet<TargetType>();

		/// <summary>
		/// Target configurations to generate project files for
		/// </summary>
		[CommandLine("-TargetConfigurations=", ListSeparator = '+')]
		HashSet<UnrealTargetConfiguration> TargetConfigurations = new HashSet<UnrealTargetConfiguration>();

		/// <summary>
		/// Projects to generate project files for
		/// </summary>
		[CommandLine("-ProjectNames=", ListSeparator = '+')]
		HashSet<string> ProjectNames = new HashSet<string>();

		/// <summary>
		/// Should format JSON files in human readable form, or use packed one without indents
		/// </summary>
		[CommandLine("-Minimize", Value = "Compact")]
		private JsonWriterStyle Minimize = JsonWriterStyle.Readable;
#pragma warning restore IDE0044

		public RiderProjectFileGenerator(FileReference? InOnlyGameProject,
			CSharpProjectCache CSharpProjCache,
			CommandLineArguments InArguments)
			: base(InOnlyGameProject, CSharpProjCache)
		{
			Arguments = InArguments;
			Arguments.ApplyTo(this);
		}

		public override bool ShouldGenerateIntelliSenseData() => true;

		public override void CleanProjectFiles(DirectoryReference InPrimaryProjectDirectory, string InPrimaryProjectName,
			DirectoryReference InIntermediateProjectFilesDirectory, ILogger Logger)
		{
			DirectoryReference.Delete(InPrimaryProjectDirectory);
		}

		private void ConfigureProjectFileGeneration()
		{
			bIncludeConfigFiles = true;
			bIncludeEngineSource = true;
			bIncludeDocumentation = false;
			bIncludeShaderSource = true;
			bGenerateProjectFiles = true;
			bIncludeTemplateFiles = false;
			bIncludeTestAndShippingConfigs = true;
			bIncludeEnginePrograms = true; // It's true by default, but I like to have things explicit
		}

		protected override ProjectFile AllocateProjectFile(FileReference InitFilePath, DirectoryReference BaseDir)
		{
			RiderProjectFile projectFile = new RiderProjectFile(InitFilePath, BaseDir, RootPath: InitFilePath.Directory,
				Arguments: Arguments, TargetTypes: TargetTypes);
			return projectFile;
		}

		protected override bool WriteProjectFiles(PlatformProjectGeneratorCollection PlatformProjectGenerators, ILogger Logger)
		{
			if (bOnlyPrimaryProjectFile)
			{
				return true;
			}

			using (ProgressWriter Progress = new ProgressWriter("Writing project files...", true, Logger))
			{
				List<ProjectFile> ProjectsToGenerate = new List<ProjectFile>(GeneratedProjectFiles);
				if (ProjectNames.Any())
				{
					ProjectsToGenerate = ProjectsToGenerate.Where(it =>
						ProjectNames.Contains(it.ProjectFilePath.GetFileNameWithoutAnyExtensions())).ToList();
				}

				int TotalProjectFileCount = ProjectsToGenerate.Count;

				HashSet<UnrealTargetPlatform> PlatformsToGenerate = new HashSet<UnrealTargetPlatform>(SupportedPlatforms);
				if (Platforms.Any())
				{
					PlatformsToGenerate.IntersectWith(Platforms);
				}

				List<UnrealTargetPlatform> FilteredPlatforms = PlatformsToGenerate.Where(it =>
				{
					// Skip deprecated platforms if they are not specified in commandline arguments directly 
					if (DeprecatedPlatforms.Contains(it) && !Platforms.Contains(it))
					{
						return false;
					}

					if (UEBuildPlatform.IsPlatformAvailable(it))
					{
						return true;
					}

					Logger.LogWarning(
						"Platform {Platform} is not a valid platform to build. Check that the SDK is installed properly",
						it);
					Logger.LogWarning("Platform will be ignored in project file generation");
					return false;
				}).ToList();

				HashSet<UnrealTargetConfiguration> ConfigurationsToGenerate = new HashSet<UnrealTargetConfiguration>(SupportedConfigurations);
				if (TargetConfigurations.Any())
				{
					ConfigurationsToGenerate.IntersectWith(TargetConfigurations);
				}

				for (int ProjectFileIndex = 0; ProjectFileIndex < ProjectsToGenerate.Count; ++ProjectFileIndex)
				{
					RiderProjectFile? CurProject = ProjectsToGenerate[ProjectFileIndex] as RiderProjectFile;
					if (CurProject != null)
					{
						if (!CurProject.WriteProjectFile(FilteredPlatforms, ConfigurationsToGenerate.ToList(),
							PlatformProjectGenerators, Minimize, Logger))
						{
							return false;
						}
					}

					Progress.Write(ProjectFileIndex + 1, TotalProjectFileCount);
				}

				Progress.Write(TotalProjectFileCount, TotalProjectFileCount);
			}

			return true;
		}

		private FileReference GetRiderProjectLocation(string GeneratedProjectName)
		{
			if (OnlyGameProject != null && FileReference.Exists(OnlyGameProject))
			{
				return FileReference.Combine(OnlyGameProject.Directory, "Intermediate", "ProjectFiles", ".Rider", GeneratedProjectName);
			}
			else
			{
				return FileReference.Combine(Unreal.EngineDirectory, "Intermediate", "ProjectFiles", ".Rider", GeneratedProjectName);
			}
		}

		private void AddProjectsForAllTargets(
			PlatformProjectGeneratorCollection PlatformProjectGenerators,
			List<FileReference> AllGames,
			ILogger Logger,
			out ProjectFile? EngineProject,
			out List<ProjectFile> GameProjects,
			out Dictionary<FileReference, ProjectFile> ProgramProjects)
		{
			// As we're creating project files, we'll also keep track of whether we created an "engine" project and return that if we have one
			EngineProject = null;
			GameProjects = new List<ProjectFile>();
			ProgramProjects = new Dictionary<FileReference, ProjectFile>();

			// Find all of the target files.  This will filter out any modules or targets that don't
			// belong to platforms we're generating project files for.
			List<FileReference> AllTargetFiles = DiscoverTargets(AllGames, Logger, OnlyGameProject, SupportedPlatforms, bIncludeEngineSource: bIncludeEngineSource, bIncludeTempTargets: bIncludeTempTargets);

			// Sort the targets by name. When we have multiple targets of a given type for a project, we'll use the order to determine which goes in the primary project file (so that client names with a suffix will go into their own project).
			AllTargetFiles = AllTargetFiles.OrderBy(x => x.FullName, StringComparer.OrdinalIgnoreCase).ToList();

			// Separate the .target.cs files that are platform extension specializations, per target name. These will be added alongside their base target.cs
			HashSet<FileReference> AllSubTargetFiles;
			Dictionary<string, List<FileReference>> AllSubTargetFilesPerTarget;
			GetPlatformSpecializationsSubTargetsForAllTargets(AllTargetFiles, out AllSubTargetFiles, out AllSubTargetFilesPerTarget);

			foreach (FileReference TargetFilePath in AllTargetFiles.Except(AllSubTargetFiles))
			{
				string TargetName = TargetFilePath.GetFileNameWithoutAnyExtensions();

				// Check to see if this is an Engine target.  That is, the target is located under the "Engine" folder
				bool IsEngineTarget = false;
				bool WantProjectFileForTarget = true;
				if (TargetFilePath.IsUnderDirectory(Unreal.EngineDirectory))
				{
					// This is an engine target
					IsEngineTarget = true;

					if (TargetFilePath.IsUnderDirectory(EngineSourceProgramsDirectory))
					{
						WantProjectFileForTarget = bIncludeEnginePrograms;
					}
					else if (TargetFilePath.IsUnderDirectory(Unreal.EngineSourceDirectory))
					{
						WantProjectFileForTarget = bIncludeEngineSource;
					}
				}

				if (WantProjectFileForTarget)
				{
					RulesAssembly RulesAssembly;

					FileReference? CheckProjectFile =
						AllGames.FirstOrDefault(x => TargetFilePath.IsUnderDirectory(x.Directory));
					if (CheckProjectFile == null)
					{
						RulesAssembly = RulesCompiler.CreateEngineRulesAssembly(false, false, false, Logger);
					}
					else
					{
						RulesAssembly = RulesCompiler.CreateProjectRulesAssembly(CheckProjectFile, false, false, false, Logger);
					}

					// Create target rules for all of the platforms and configuration combinations that we want to enable support for.
					// Just use the current platform as we only need to recover the target type and both should be supported for all targets...
					TargetRules TargetRulesObject = RulesAssembly.CreateTargetRules(TargetName,
						BuildHostPlatform.Current.Platform, UnrealTargetConfiguration.Development, null, CheckProjectFile,
						null, Logger);

					bool IsProgramTarget = false;

					DirectoryReference? GameFolder = null;
					string ProjectFileNameBase;
					if (TargetRulesObject.Type == TargetType.Program)
					{
						IsProgramTarget = true;
						ProjectFileNameBase = TargetName;
					}
					else if (IsEngineTarget)
					{
						ProjectFileNameBase = EngineProjectFileNameBase;
					}
					else
					{
						// Figure out which game project this target belongs to
						FileReference? ProjectInfo = FindGameContainingFile(AllGames, TargetFilePath);
						if (ProjectInfo == null)
						{
							throw new BuildException("Found a non-engine target file (" + TargetFilePath +
													 ") that did not exist within any of the known game folders");
						}

						GameFolder = ProjectInfo.Directory;
						ProjectFileNameBase = ProjectInfo.GetFileNameWithoutExtension();
					}

					// Get the suffix to use for this project file. If we have multiple targets of the same type, we'll have to split them out into separate IDE project files.
					string? GeneratedProjectName = TargetRulesObject.GeneratedProjectName;
					if (GeneratedProjectName == null)
					{
						Lazy<ProjectFile>? ExistingProjectFile;
						if (ProjectFileMap.TryGetValue(GetRiderProjectLocation(ProjectFileNameBase), out ExistingProjectFile) &&
							ExistingProjectFile.Value.ProjectTargets.Any(x => x.TargetRules!.Type == TargetRulesObject.Type))
						{
							GeneratedProjectName = TargetRulesObject.Name;
						}
						else
						{
							GeneratedProjectName = ProjectFileNameBase;
						}
					}

					FileReference ProjectFilePath = GetRiderProjectLocation(GeneratedProjectName);
					if (TargetRulesObject.Type == TargetType.Game || TargetRulesObject.Type == TargetType.Client ||
						TargetRulesObject.Type == TargetType.Server)
					{
						// Allow platforms to generate stub projects here...
						PlatformProjectGenerators.GenerateGameProjectStubs(
							InGenerator: this,
							InTargetName: TargetName,
							InTargetFilepath: TargetFilePath.FullName,
							InTargetRules: TargetRulesObject,
							InPlatforms: SupportedPlatforms,
							InConfigurations: SupportedConfigurations);
					}

					DirectoryReference BaseFolder;
					if (IsProgramTarget)
					{
						BaseFolder = TargetFilePath.Directory;
					}
					else if (IsEngineTarget)
					{
						BaseFolder = Unreal.EngineDirectory;
					}
					else
					{
						BaseFolder = GameFolder!;
					}

					ProjectFile ProjectFile = FindOrAddProject(ProjectFilePath, BaseFolder);
					ProjectFile.IsForeignProject =
						CheckProjectFile != null && !NativeProjects.IsNativeProject(CheckProjectFile, Logger);
					ProjectFile.IsGeneratedProject = true;
					ProjectFile.IsStubProject = Unreal.IsProjectInstalled();
					if (TargetRulesObject.bBuildInSolutionByDefault.HasValue)
					{
						ProjectFile.ShouldBuildByDefaultForSolutionTargets =
							TargetRulesObject.bBuildInSolutionByDefault.Value;
					}

					// Add the project to the right output list
					if (IsProgramTarget)
					{
						ProgramProjects[TargetFilePath] = ProjectFile;
					}
					else if (IsEngineTarget)
					{
						EngineProject = ProjectFile;
						if (Unreal.IsEngineInstalled())
						{
							// Allow engine projects to be created but not built for Installed Engine builds
							EngineProject.IsForeignProject = false;
							EngineProject.IsGeneratedProject = true;
							EngineProject.IsStubProject = true;
						}
					}
					else
					{
						if (!GameProjects.Contains(ProjectFile))
						{
							GameProjects.Add(ProjectFile);

							// Add the .uproject file for this game/template
							FileReference UProjectFilePath =
								FileReference.Combine(BaseFolder, ProjectFileNameBase + ".uproject");
							if (FileReference.Exists(UProjectFilePath))
							{
								ProjectFile.AddFileToProject(UProjectFilePath, BaseFolder);
							}
							else
							{
								throw new BuildException(
									"Not expecting to find a game with no .uproject file.  File '{0}' doesn't exist",
									UProjectFilePath);
							}
						}
					}

					foreach (Project ExistingProjectTarget in ProjectFile.ProjectTargets)
					{
						if (ExistingProjectTarget.TargetRules!.Type == TargetRulesObject.Type)
						{
							throw new BuildException(
								"Not expecting project {0} to already have a target rules of with configuration name {1} ({2}) while trying to add: {3}",
								ProjectFilePath, TargetRulesObject.Type.ToString(),
								ExistingProjectTarget.TargetRules.ToString(), TargetRulesObject.ToString());
						}

						// Not expecting to have both a game and a program in the same project.  These would alias because we share the project and solution configuration names (just because it makes sense to)
						if ((ExistingProjectTarget.TargetRules.Type == TargetType.Game &&
							 TargetRulesObject.Type == TargetType.Program) ||
							(ExistingProjectTarget.TargetRules.Type == TargetType.Program &&
							 TargetRulesObject.Type == TargetType.Game))
						{
							throw new BuildException(
								"Not expecting project {0} to already have a Game/Program target ({1}) associated with it while trying to add: {2}",
								ProjectFilePath, ExistingProjectTarget.TargetRules.ToString(),
								TargetRulesObject.ToString());
						}
					}

					ProjectTarget ProjectTarget = new ProjectTarget
					(
						TargetFilePath: TargetFilePath,
						ProjectFilePath: ProjectFilePath,
						UnrealProjectFilePath: CheckProjectFile,
						TargetRules: TargetRulesObject,
						SupportedPlatforms: TargetRulesObject.GetSupportedPlatforms()
							.Where(x => UEBuildPlatform.TryGetBuildPlatform(x, out _)).ToArray(),
						CreateRulesDelegate: (Platform, Configuration) =>
							RulesAssembly.CreateTargetRules(TargetName, Platform, Configuration, null, CheckProjectFile,
								null, Logger),
						CreateRulesDelegateWithArch: (Platform, Configuration, Arch) =>
							RulesAssembly.CreateTargetRules(TargetName, Platform, Configuration, new(Arch), CheckProjectFile,
								null, Logger)
					);

					ProjectFile.ProjectTargets.Add(ProjectTarget);

					// Make sure the *.Target.cs file is in the project.
					ProjectFile.AddFileToProject(TargetFilePath, BaseFolder);

					// Add all matching *_<platform>.Target.cs to the same folder
					if (AllSubTargetFilesPerTarget.ContainsKey(TargetName))
					{
						ProjectFile.AddFilesToProject(AllSubTargetFilesPerTarget[TargetName], BaseFolder);
					}

					Logger.LogDebug("Generating target {Target} for {Project}", TargetRulesObject.Type.ToString(),
						ProjectFilePath);
				}
			}
		}

		private void SetupSupportedPlatformsAndConfigurations(ILogger Logger)
		{
			SetupSupportedPlatformsAndConfigurations(true, Logger, out _);
		}

		public override bool GenerateProjectFiles(PlatformProjectGeneratorCollection PlatformProjectGenerators, string[] arguments, ILogger Logger)
		{
			ConfigureProjectFileGeneration();

			if (bGeneratingGameProjectFiles)
			{
				PrimaryProjectPath = OnlyGameProject!.Directory;
				PrimaryProjectName = OnlyGameProject.GetFileNameWithoutExtension();

				if (!DirectoryReference.Exists(DirectoryReference.Combine(PrimaryProjectPath, "Source")))
				{
					if (!DirectoryReference.Exists(DirectoryReference.Combine(PrimaryProjectPath, "Intermediate",
						"Source")))
					{
						if (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Mac)
						{
							PrimaryProjectPath = Unreal.EngineDirectory;
							GameProjectName = "UnrealGame";
						}

						if (!DirectoryReference.Exists(DirectoryReference.Combine(PrimaryProjectPath, "Source")))
						{
							throw new BuildException("Directory '{0}' is missing 'Source' folder.", PrimaryProjectPath);
						}
					}
				}

				IntermediateProjectFilesPath =
					DirectoryReference.Combine(PrimaryProjectPath, "Intermediate", "ProjectFiles");
			}

			SetupSupportedPlatformsAndConfigurations(Logger);

			Logger.LogDebug("Detected supported platforms: {Platforms}", SupportedPlatforms);

			List<FileReference> AllGameProjects = FindGameProjects(Logger);

			GatherProjects(PlatformProjectGenerators, AllGameProjects, Logger);

			WritePrimaryProjectFile(UBTProject, PlatformProjectGenerators, Logger);
			if (!bOnlyPrimaryProjectFile)
			{
				WriteProjectFiles(PlatformProjectGenerators, Logger);
			}

			// Generate all necessary .props files for C# projects
			PrimaryProjectFolder ProgramsFolder = RootFolder.AddSubFolder("Programs");
			AddRulesModules(Rules.RulesFileType.AutomationModule, "Automation", AutomationProjectFiles, AllGameProjects, RootFolder, ProgramsFolder);
			AddRulesModules(Rules.RulesFileType.UbtPlugin, "UnrealBuildTool.Plugins", UbtPluginProjectFiles, AllGameProjects, RootFolder, ProgramsFolder);

			Logger.LogDebug("Project generation complete ({NumGenerated} generated, {NumImported} imported)", GeneratedProjectFiles.Count,
				OtherProjectFiles.Count);

			return true;
		}

		private void GatherProjects(PlatformProjectGeneratorCollection PlatformProjectGenerators,
			List<FileReference> AllGameProjects, ILogger Logger)
		{
			List<ProjectFile>? ModProjects = new List<ProjectFile>();

			// Setup buildable projects for all targets
			AddProjectsForAllTargets(PlatformProjectGenerators, AllGameProjects, Logger,
				out ProjectFile? EngineProject, out List<ProjectFile>? GameProjects, out _);

			AddProjectsForMods(GameProjects, ModProjects);
			AddAllGameProjects(GameProjects);

			// If we're still missing an engine project because we don't have any targets for it, make one up.
			if (EngineProject == null)
			{
				FileReference ProjectFilePath = FileReference.Combine(IntermediateProjectFilesPath,
					EngineProjectFileNameBase + ProjectFileExtension);

				EngineProject = FindOrAddProject(ProjectFilePath, Unreal.EngineDirectory);

				EngineProject.IsForeignProject = false;
				EngineProject.IsGeneratedProject = true;
				EngineProject.IsStubProject = true;
			}
		}

		protected override bool WritePrimaryProjectFile(ProjectFile? ProjectFile,
			PlatformProjectGeneratorCollection PlatformProjectGenerators, ILogger Logger)
		{
			try
			{
				HashSet<UnrealTargetPlatform> ValidPlatforms;
				List<VisualStudioPrimarySolutionProjectParams> VcxSolutionProjectParamList;
				VCProjectFileGenerator.CollectSolutionConfigurations(SupportedConfigurations,
					SupportedPlatforms, AllProjectFiles, bMakeProjectPerTarget, out ValidPlatforms, out VcxSolutionProjectParamList);
				FileReference OutputFile = GetRiderProjectLocation("root.json");
				DirectoryReference.CreateDirectory(OutputFile.Directory);
				using (JsonWriter Writer = new JsonWriter(OutputFile))
				{
					Writer.WriteObjectStart();
					Writer.WriteArrayStart("ConfigurationsAndPlatforms");
					foreach (VisualStudioPrimarySolutionProjectParams Params in VcxSolutionProjectParamList)
					{
						Writer.WriteObjectStart();
						Writer.WriteValue("Configuration", Params.ProjectConfiguration.ToString());
						Writer.WriteValue("Platform", Params.ProjectPlatform.ToString());
						Writer.WriteValue("Target", Params.ProjectTargetType.ToString());
						Writer.WriteObjectEnd();
					}

					Writer.WriteArrayEnd();
					Writer.WriteObjectEnd();
				}
			}
			catch (Exception Ex)
			{
				Logger.LogWarning("Exception while writing Rider root project file: {Ex}", Ex.ToString());
			}

			return true;
		}
	}
}
