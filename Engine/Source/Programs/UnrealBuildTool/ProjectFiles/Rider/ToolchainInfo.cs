// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Reflection;

namespace UnrealBuildTool
{
	sealed class ToolchainInfo : IEquatable<ToolchainInfo>
	{
		public CppStandardVersion CppStandard;
		public bool bUseRTTI;
		public bool bEnableExceptions;
		public bool bIsBuildingLibrary;
		public bool bIsBuildingDLL;
		public string? Architecture;
		public string? Configuration;
		public bool bOptimizeCode;
		public bool bUseInlining;
		public bool bUseUnity;
		public bool bCreateDebugInfo;
		public MinimumCpuArchitectureX64 MinCpuArchX64;
		public bool bUseDebugCRT;
		public bool bUseStaticCRT;
		public string? PrecompiledHeaderAction;
		public string? PrecompiledHeaderFile;
		public List<string>? ForceIncludeFiles;
		public string? Compiler;
		public bool bStrictConformanceMode;
		public bool bStrictPreprocessorConformanceMode;
		public bool bEnableCoroutines;
		public bool bEnableAddressSanitizer;
		public bool bUpdatedCPPMacro;

		public IEnumerable<Tuple<string, object?>> GetDiff(ToolchainInfo Other)
		{
			foreach (FieldInfo FieldInfo in typeof(ToolchainInfo).GetFields())
			{
				if (FieldInfo.GetValue(this) == null)
				{
					continue;
				}

				if (typeof(List<string>).IsAssignableFrom(FieldInfo.FieldType))
				{
					List<string> LocalField = (List<string>)FieldInfo.GetValue(this)!;
					HashSet<string> OtherField = new HashSet<string>((List<string>)FieldInfo.GetValue(Other)!);
					IEnumerable<string> Result = LocalField.Where(Item => OtherField.Contains(Item));
					if (Result.Any())
					{
						yield return new Tuple<string, object?>(FieldInfo.Name, Result);
					}
				}
				else if (!FieldInfo.GetValue(this)!.Equals(FieldInfo.GetValue(Other)))
				{
					yield return new Tuple<string, object?>(FieldInfo.Name, FieldInfo.GetValue(this));
				}
			}
		}

		public IEnumerable<Tuple<string, object?>> GetFields()
		{
			foreach (FieldInfo FieldInfo in typeof(ToolchainInfo).GetFields())
			{
				yield return new Tuple<string, object?>(FieldInfo.Name, FieldInfo.GetValue(this));
			}
		}

		public bool Equals(ToolchainInfo? Other)
		{
			if (Other is null)
			{
				return false;
			}

			if (ReferenceEquals(this, Other))
			{
				return true;
			}

			return CppStandard == Other.CppStandard && bUseRTTI == Other.bUseRTTI &&
				bEnableExceptions == Other.bEnableExceptions && bIsBuildingLibrary == Other.bIsBuildingLibrary &&
				bIsBuildingDLL == Other.bIsBuildingDLL && Architecture == Other.Architecture &&
				Configuration == Other.Configuration && bOptimizeCode == Other.bOptimizeCode &&
				bUseInlining == Other.bUseInlining && bUseUnity == Other.bUseUnity &&
				bCreateDebugInfo == Other.bCreateDebugInfo && MinCpuArchX64 == Other.MinCpuArchX64 &&
				bUseDebugCRT == Other.bUseDebugCRT && bUseStaticCRT == Other.bUseStaticCRT &&
				PrecompiledHeaderAction == Other.PrecompiledHeaderAction && PrecompiledHeaderFile == Other.PrecompiledHeaderFile &&
				Equals(ForceIncludeFiles, Other.ForceIncludeFiles) && Compiler == Other.Compiler &&
				bStrictConformanceMode == Other.bStrictConformanceMode && bStrictPreprocessorConformanceMode == Other.bStrictPreprocessorConformanceMode &&
				bEnableCoroutines == Other.bEnableCoroutines && bEnableAddressSanitizer == Other.bEnableAddressSanitizer &&
				bUpdatedCPPMacro == Other.bUpdatedCPPMacro;
		}

		public override bool Equals(object? Obj)
		{
			if (Obj is null)
			{
				return false;
			}

			if (ReferenceEquals(this, Obj))
			{
				return true;
			}

			return Obj is ToolchainInfo info && Equals(info);
		}

		public override int GetHashCode()
		{
			unchecked
			{
				int HashCode = (int)CppStandard;
				HashCode = (HashCode * 397) ^ bUseRTTI.GetHashCode();
				HashCode = (HashCode * 397) ^ bEnableExceptions.GetHashCode();
				HashCode = (HashCode * 397) ^ bIsBuildingLibrary.GetHashCode();
				HashCode = (HashCode * 397) ^ bIsBuildingDLL.GetHashCode();
				HashCode = (HashCode * 397) ^ (Architecture != null ? Architecture.GetHashCode(StringComparison.Ordinal) : 0);
				HashCode = (HashCode * 397) ^ (Configuration != null ? Configuration.GetHashCode(StringComparison.Ordinal) : 0);
				HashCode = (HashCode * 397) ^ bOptimizeCode.GetHashCode();
				HashCode = (HashCode * 397) ^ bUseInlining.GetHashCode();
				HashCode = (HashCode * 397) ^ bUseUnity.GetHashCode();
				HashCode = (HashCode * 397) ^ bCreateDebugInfo.GetHashCode();
				HashCode = (HashCode * 397) ^ MinCpuArchX64.GetHashCode();
				HashCode = (HashCode * 397) ^ bUseDebugCRT.GetHashCode();
				HashCode = (HashCode * 397) ^ bUseStaticCRT.GetHashCode();
				HashCode = (HashCode * 397) ^ (PrecompiledHeaderAction != null ? PrecompiledHeaderAction.GetHashCode(StringComparison.Ordinal) : 0);
				HashCode = (HashCode * 397) ^ (PrecompiledHeaderFile != null ? PrecompiledHeaderFile.GetHashCode(StringComparison.Ordinal) : 0);
				HashCode = (HashCode * 397) ^ (ForceIncludeFiles != null ? ForceIncludeFiles.GetHashCode() : 0);
				HashCode = (HashCode * 397) ^ (Compiler != null ? Compiler.GetHashCode(StringComparison.Ordinal) : 0);
				HashCode = (HashCode * 397) ^ bStrictConformanceMode.GetHashCode();
				HashCode = (HashCode * 397) ^ bStrictPreprocessorConformanceMode.GetHashCode();
				HashCode = (HashCode * 397) ^ bEnableCoroutines.GetHashCode();
				HashCode = (HashCode * 397) ^ bEnableAddressSanitizer.GetHashCode();
				HashCode = (HashCode * 397) ^ bUpdatedCPPMacro.GetHashCode();
				return HashCode;
			}
		}
	}
}