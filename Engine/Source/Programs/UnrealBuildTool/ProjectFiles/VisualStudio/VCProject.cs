// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using System.Linq;
using System.Security;
using System.Text;
using System.Xml;
using System.Xml.Linq;
using EpicGames.Core;
using Microsoft.CodeAnalysis;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;
using UnrealBuildTool.ProjectFiles.VisualStudio;
using static UnrealBuildTool.PlatformProjectGenerator;

namespace UnrealBuildTool
{
	/// <summary>
	/// Specifies the context for building an MSBuild project
	/// </summary>
	class MSBuildProjectContext
	{
		/// <summary>
		/// Name of the configuration
		/// </summary>
		public string ConfigurationName;

		/// <summary>
		/// Name of the platform
		/// </summary>
		public string PlatformName;

		/// <summary>
		/// Whether this context should be built by default
		/// </summary>
		public bool bBuildByDefault;

		/// <summary>
		/// Whether this context should be deployed by default
		/// </summary>
		public bool bDeployByDefault;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="ConfigurationName">Name of this configuration</param>
		/// <param name="PlatformName">Name of this platform</param>
		public MSBuildProjectContext(string ConfigurationName, string PlatformName)
		{
			this.ConfigurationName = ConfigurationName;
			this.PlatformName = PlatformName;
		}

		/// <summary>
		/// The name of this context
		/// </summary>
		public string Name => String.Format("{0}|{1}", ConfigurationName, PlatformName);

		/// <summary>
		/// Serializes this context to a string for debugging
		/// </summary>
		/// <returns>Name of this context</returns>
		public override string ToString()
		{
			return Name;
		}
	}

	/// <summary>
	/// Represents an arbitrary MSBuild project
	/// </summary>
	abstract class MSBuildProjectFile : ProjectFile
	{
		/// The project file version string
		public const string VCProjectFileVersionString = "10.0.30319.1";

		/// The build configuration name to use for stub project configurations.  These are projects whose purpose
		/// is to make it easier for developers to find source files and to provide IntelliSense data for the module
		/// to Visual Studio
		public const string StubProjectConfigurationName = "BuiltWithUnrealBuildTool";

		/// The name of the Visual C++ platform to use for stub project configurations
		/// NOTE: We always use Win32 for the stub project's platform, since that is guaranteed to be supported by Visual Studio
		public const string StubProjectPlatformName = "Win64";

		/// <summary>
		/// The Guid representing the project type e.g. C# or C++
		/// </summary>
		public virtual string ProjectTypeGUID => throw new BuildException("Unrecognized type of project file for Visual Studio solution");

		static Guid PathNamespaceGuid { get; } = new Guid("2D8570D5-7FFC-4E6D-A9D7-E860E117D717");

		/// <summary>
		/// Constructs a new project file object
		/// </summary>
		/// <param name="InitFilePath">The path to the project file on disk</param>
		/// <param name="BaseDir">The base directory for files within this project</param>
		/// <param name="TryReadProjectGuid">Try read the project guid from already existsing file</param>
		public MSBuildProjectFile(FileReference InitFilePath, DirectoryReference BaseDir, bool TryReadProjectGuid)
			: base(InitFilePath, BaseDir)
		{
			// Each project gets its own GUID.  This is stored in the project file and referenced in the solution file.

			_projectGuid = new Lazy<Guid>(() =>
			{
				Guid projectGuid = Guid.Empty;
				// First, check to see if we have an existing file on disk.  If we do, then we'll try to preserve the
				// GUID by loading it from the existing file.
				if (TryReadProjectGuid && FileReference.Exists(ProjectFilePath))
				{
					try
					{
						projectGuid = InternalLoadGUIDFromExistingProject();
					}
					catch (Exception)
					{
						// Failed to find GUID, so just create a new one
						return VCProjectFileGenerator.MakeMd5Guid(PathNamespaceGuid, ProjectFilePath.FullName);
					}
				}

				if (projectGuid == Guid.Empty)
				{
					// Generate a brand new GUID
					projectGuid = VCProjectFileGenerator.MakeMd5Guid(PathNamespaceGuid, ProjectFilePath.FullName);
				}

				return projectGuid;
			});
		}

		/// <summary>
		/// Attempts to load the project's GUID from an existing project file on disk
		/// </summary>
		public Guid InternalLoadGUIDFromExistingProject()
		{
			// Only load GUIDs if we're in project generation mode.  Regular builds don't need GUIDs for anything.
			if (ProjectFileGenerator.bGenerateProjectFiles)
			{
				XmlDocument Doc = new XmlDocument();
				Doc.Load(ProjectFilePath.FullName);

				// @todo projectfiles: Ideally we could do a better job about preserving GUIDs when only minor changes are made
				// to the project (such as adding a single new file.) It would make diffing changes much easier!

				// @todo projectfiles: Can we "seed" a GUID based off the project path and generate consistent GUIDs each time?

				XmlNodeList Elements = Doc.GetElementsByTagName("ProjectGuid");
				foreach (XmlElement? Element in Elements)
				{
					if (Element == null)
					{
						continue;
					}

					return Guid.ParseExact(Element.InnerText.Trim("{}".ToCharArray()), "D");
				}
			}

			return Guid.Empty;
		}

		public override void LoadGUIDFromExistingProject()
		{
			if (!FileReference.Exists(ProjectFilePath))
			{
				throw new Exception("Can't find project file " + ProjectFilePath);
			}
		}

		/// <summary>
		/// Get the project context for the given solution context
		/// </summary>
		/// <param name="SolutionProjectParams">The solution project params to get a context for</param>
		/// <param name="PlatformProjectGenerators">Set of platform project generators</param>
		/// <param name="Logger">Logger for output</param>
		/// <returns>Project context matching the given solution context</returns>
		public abstract MSBuildProjectContext? GetMatchingProjectContext(
			VisualStudioPrimarySolutionProjectParams SolutionProjectParams,
			PlatformProjectGeneratorCollection PlatformProjectGenerators,
			ILogger Logger);

		/// <summary>
		/// Checks to see if the specified solution platform and configuration is able to map to this project
		/// </summary>
		/// <param name="ProjectTarget">The target that we're checking for a valid platform/config combination</param>
		/// <param name="Platform">Platform</param>
		/// <param name="Configuration">Configuration</param>
		/// <returns>True if this is a valid combination for this project, otherwise false</returns>
		public static bool IsValidProjectPlatformAndConfiguration(Project ProjectTarget, UnrealTargetPlatform Platform, UnrealTargetConfiguration Configuration)
		{
			if (!ProjectFileGenerator.bIncludeTestAndShippingConfigs)
			{
				if (Configuration == UnrealTargetConfiguration.Test || Configuration == UnrealTargetConfiguration.Shipping)
				{
					return false;
				}
			}

			if (!ProjectFileGenerator.bIncludeDebugConfigs)
			{
				if (Configuration == UnrealTargetConfiguration.Debug || Configuration == UnrealTargetConfiguration.DebugGame)
				{
					return false;
				}
			}

			if (!ProjectFileGenerator.bIncludeDevelopmentConfigs)
			{
				if (Configuration == UnrealTargetConfiguration.Development)
				{
					return false;
				}
			}

			UEBuildPlatform? BuildPlatform;
			if (!UEBuildPlatform.TryGetBuildPlatform(Platform, out BuildPlatform))
			{
				return false;
			}

			if (BuildPlatform.HasRequiredSDKsInstalled() != SDKStatus.Valid)
			{
				return false;
			}

			List<UnrealTargetConfiguration> SupportedConfigurations = new List<UnrealTargetConfiguration>();
			List<UnrealTargetPlatform> SupportedPlatforms = new List<UnrealTargetPlatform>();
			if (ProjectTarget.TargetRules != null)
			{
				SupportedPlatforms.AddRange(ProjectTarget.SupportedPlatforms);
				SupportedConfigurations.AddRange(ProjectTarget.TargetRules.GetSupportedConfigurations());
			}

			// Add all of the extra platforms/configurations for this target
			{
				foreach (UnrealTargetPlatform ExtraPlatform in ProjectTarget.ExtraSupportedPlatforms)
				{
					if (!SupportedPlatforms.Contains(ExtraPlatform))
					{
						SupportedPlatforms.Add(ExtraPlatform);
					}
				}
				foreach (UnrealTargetConfiguration ExtraConfig in ProjectTarget.ExtraSupportedConfigurations)
				{
					if (!SupportedConfigurations.Contains(ExtraConfig))
					{
						SupportedConfigurations.Add(ExtraConfig);
					}
				}
			}

			// Only build for supported platforms
			if (SupportedPlatforms.Contains(Platform) == false)
			{
				return false;
			}

			// Only build for supported configurations
			if (SupportedConfigurations.Contains(Configuration) == false)
			{
				return false;
			}

			return true;
		}

		/// <summary>
		/// Escapes characters in a filename so they can be stored in an XML attribute
		/// </summary>
		/// <param name="FileName">The filename to escape</param>
		/// <returns>The escaped filename</returns>
		public static string EscapeFileName(string FileName)
		{
			return SecurityElement.Escape(FileName)!;
		}

		/// <summary>
		/// GUID for this Visual C++ project file
		/// </summary>
		public Guid ProjectGUID => _projectGuid.Value;

		readonly Lazy<Guid> _projectGuid;
	}

	class VCProjectFile : MSBuildProjectFile
	{
		readonly VCProjectFileFormat ProjectFileFormat;
		readonly bool bUsePrecompiled;
		readonly bool bMakeProjectPerTarget;
		readonly string? BuildToolOverride;
		readonly Dictionary<DirectoryReference, string> ModuleDirToForceIncludePaths = new Dictionary<DirectoryReference, string>();
		readonly Dictionary<DirectoryReference, string> ModuleDirToPchHeaderFile = new Dictionary<DirectoryReference, string>();
		readonly VCProjectFileSettings Settings;

		/// This is the platform name that Visual Studio is always guaranteed to support.  We'll use this as
		/// a platform for any project configurations where our actual platform is not supported by the
		/// installed version of Visual Studio (e.g, "iOS")
		public const string DefaultPlatformName = "x64";

		// This is the GUID that Visual Studio uses to identify a C++ project file in the solution
		public override string ProjectTypeGUID => "{8BC9CEB8-8B4A-11D0-8D11-00A0C91BC942}";

		/// <summary>
		/// Constructs a new project file object
		/// </summary>
		/// <param name="FilePath">The path to the project file on disk</param>
		/// <param name="BaseDir">The base directory for files within this project</param>
		/// <param name="ProjectFileFormat">Visual C++ project file version</param>
		/// <param name="bUsePrecompiled">Whether to add the -UsePrecompiled argumemnt when building targets</param>
		/// <param name="bMakeProjectPerTarget">Whether to add roll the target type into the config (ie "Development Editor")</param>
		/// <param name="BuildToolOverride">Optional arguments to pass to UBT when building</param>
		/// <param name="Settings">Other settings</param>
		/// <param name="TryReadProjectGuid">Try read the project guid from already existsing file</param>
		public VCProjectFile(FileReference FilePath, DirectoryReference BaseDir, VCProjectFileFormat ProjectFileFormat, bool bUsePrecompiled, bool bMakeProjectPerTarget, string? BuildToolOverride, VCProjectFileSettings Settings, bool TryReadProjectGuid)
			: base(FilePath, BaseDir, TryReadProjectGuid)
		{
			this.ProjectFileFormat = ProjectFileFormat;
			this.bUsePrecompiled = bUsePrecompiled;
			this.bMakeProjectPerTarget = bMakeProjectPerTarget;
			this.BuildToolOverride = BuildToolOverride;
			this.Settings = Settings;
		}

		/// <inheritdoc/>
		public override MSBuildProjectContext? GetMatchingProjectContext(
			VisualStudioPrimarySolutionProjectParams ProjectParams,
			PlatformProjectGeneratorCollection PlatformProjectGenerators,
			ILogger Logger)
		{
			// Stub projects always build in the same configuration
			if (IsStubProject)
			{
				return new MSBuildProjectContext(StubProjectConfigurationName, StubProjectPlatformName);
			}

			// Have to match every solution configuration combination to a project configuration (or use the invalid one)
			string ProjectConfigurationName = "Invalid";

			// Get the default platform. If there were not valid platforms for this project, just use one that will always be available in VS.
			string ProjectPlatformName = InvalidConfigPlatformNames!.First();

			// Whether the configuration should be built automatically as part of the solution
			bool bBuildByDefault = false;

			// Whether this configuration should deploy by default (requires bBuildByDefault)
			bool bDeployByDefault = false;

			// Programs are built in editor configurations (since the editor is like a desktop program too) and game configurations (since we omit the "game" qualification in the configuration name).
			bool IsProgramProject = ProjectTargets.Any() && ProjectTargets[0].TargetRules != null && ProjectTargets[0].TargetRules!.Type == TargetType.Program;
			{
				// Get the target type we expect to find for this project
				TargetType TargetConfigurationName = ProjectParams.ProjectTargetType;
				if (IsProgramProject)
				{
					TargetConfigurationName = TargetType.Program;
				}
				else if (ProjectParams.ProjectTargetType == TargetType.Program)
				{
					TargetConfigurationName = TargetType.Game;
				}

				// Now, we want to find a target in this project that maps to the current solution config combination.  Only up to one target should
				// and every solution config combination should map to at least one target in one project (otherwise we shouldn't have added it!).
				List<Project> MatchingProjectTargets = new List<Project>();
				foreach (Project ProjectTarget in ProjectTargets)
				{
					if (VCProjectFile.IsValidProjectPlatformAndConfiguration(ProjectTarget, ProjectParams.ProjectPlatform, ProjectParams.ProjectConfiguration))
					{
						if (ProjectTarget.TargetRules != null)
						{
							if (bMakeProjectPerTarget || TargetConfigurationName == ProjectTarget.TargetRules.Type)
							{
								MatchingProjectTargets.Add(ProjectTarget);
							}
						}
						else
						{
							if (ShouldBuildForAllSolutionTargets || TargetConfigurationName == TargetType.Game)
							{
								MatchingProjectTargets.Add(ProjectTarget);
							}
						}
					}
				}

				// Always allow SCW and UnrealLighmass to build in editor configurations
				if (MatchingProjectTargets.Count == 0 && ProjectParams.ProjectTargetType == TargetType.Editor && ProjectParams.ProjectPlatform == UnrealTargetPlatform.Win64)
				{
					foreach (Project ProjectTarget in ProjectTargets)
					{
						string TargetName = ProjectTargets[0].TargetRules!.Name;
						if (TargetName == "ShaderCompileWorker")
						{
							MatchingProjectTargets.Add(ProjectTarget);
							break;
						}
					}
				}

				// Make sure there's only one matching project target
				if (MatchingProjectTargets.Count > 1)
				{
					throw new BuildException(
						"Not expecting more than one target for project {0} to match solution configuration {1} {2} {3}",
						ProjectFilePath,
						ProjectParams.ProjectTargetType,
						ProjectParams.ProjectConfiguration,
						ProjectParams.ProjectPlatform);
				}

				// If we found a matching project, get matching configuration
				if (MatchingProjectTargets.Count == 1)
				{
					// Get the matching target
					Project MatchingProjectTarget = MatchingProjectTargets[0];

					// If the project wants to always build in "Development", regardless of what the solution configuration is set to, then we'll do that here.
					UnrealTargetConfiguration ProjectConfiguration = ProjectParams.ProjectConfiguration;
					if (MatchingProjectTarget.ForceDevelopmentConfiguration && ProjectParams.ProjectTargetType != TargetType.Game)
					{
						ProjectConfiguration = UnrealTargetConfiguration.Development;
					}

					// Get the matching project configuration
					UnrealTargetPlatform ProjectPlatform = ProjectParams.ProjectPlatform;
					if (IsStubProject)
					{
						if (ProjectConfiguration != UnrealTargetConfiguration.Unknown)
						{
							throw new BuildException("Stub project was expecting platform and configuration type to be set to Unknown");
						}
						ProjectConfigurationName = StubProjectConfigurationName;
						ProjectPlatformName = StubProjectPlatformName;
					}
					else
					{
						if (ProjectConfigAndTargetCombinations == null)
						{
							throw new BuildException("Project config and target combinations has not been populated.");
						}

						ProjectConfigAndTargetCombination? Combination = ProjectConfigAndTargetCombinations.FirstOrDefault(
							ProjectPlatform == UnrealTargetPlatform.Android && ProjectParams.ProjectArchitecture == null ?
								// Match first preferred Architecture if none is provided
								Combination =>
									Combination.Platform == ProjectPlatform &&
									Combination.Configuration == ProjectConfiguration &&
									Combination.ProjectTarget == MatchingProjectTarget :
								Combination =>
									Combination.Platform == ProjectPlatform &&
									Combination.Configuration == ProjectConfiguration &&
									Combination.ProjectTarget == MatchingProjectTarget &&
									Combination.Architecture == ProjectParams.ProjectArchitecture);

						if (Combination == null)
						{
							if (ProjectPlatform == UnrealTargetPlatform.Android)
							{
								// ok for Android not to find the architecture
								return null;
							}
							throw new BuildException("Could not find the project config/platform combination in the generated list.");
						}

						ProjectPlatformName = Combination.ProjectPlatformName;
						ProjectConfigurationName = Combination.ProjectConfigurationName;
					}

					// Set whether this project configuration should be built when the user initiates "build solution"
					if (ShouldBuildByDefaultForSolutionTargets)
					{
						// Some targets are "dummy targets"; they only exist to show user friendly errors in VS. Weed them out here, and don't set them to build by default.
						List<UnrealTargetPlatform>? SupportedPlatforms = null;
						if (MatchingProjectTarget.TargetRules != null)
						{
							SupportedPlatforms = new List<UnrealTargetPlatform>();
							SupportedPlatforms.AddRange(MatchingProjectTarget.SupportedPlatforms);
						}
						if (SupportedPlatforms == null || SupportedPlatforms.Contains(ProjectParams.ProjectPlatform))
						{
							bBuildByDefault = true;

							PlatformProjectGenerator? ProjGen = PlatformProjectGenerators.GetPlatformProjectGenerator(ProjectParams.ProjectPlatform, true);
							if (MatchingProjectTarget.ProjectDeploys ||
								((ProjGen != null) && (ProjGen.GetVisualStudioDeploymentEnabled(new VSSettings(ProjectPlatform, ProjectConfiguration, ProjectFileFormat, ProjectParams.ProjectArchitecture)) == true)))
							{
								bDeployByDefault = true;
							}
						}
					}
				}
			}

			return new MSBuildProjectContext(ProjectConfigurationName, ProjectPlatformName) { bBuildByDefault = bBuildByDefault, bDeployByDefault = bDeployByDefault };
		}

		class ProjectConfigAndTargetCombination
		{
			public readonly UnrealTargetPlatform? Platform;
			public readonly UnrealTargetConfiguration Configuration;
			public readonly string ProjectPlatformName;
			public readonly string ProjectConfigurationName;
			public readonly ProjectTarget? ProjectTarget;
			public readonly UnrealArch? Architecture;

			public ProjectConfigAndTargetCombination(UnrealTargetPlatform? InPlatform, UnrealTargetConfiguration InConfiguration, string InProjectPlatformName, string InProjectConfigurationName, ProjectTarget? InProjectTarget, UnrealArch? InArchitecture)
			{
				Platform = InPlatform;
				Configuration = InConfiguration;
				ProjectPlatformName = InProjectPlatformName;
				ProjectConfigurationName = InProjectConfigurationName;
				ProjectTarget = InProjectTarget;
				Architecture = InArchitecture;
			}

			public string? ProjectConfigurationAndPlatformName => (ProjectPlatformName == null) ? null : (ProjectConfigurationName + "|" + ProjectPlatformName);

			public override string ToString()
			{
				return $"{ProjectTarget} {Platform} {Configuration} {Architecture}".Trim();
			}
		}

		/// <inheritdoc/>
		public override void AddModuleForIntelliSense(UEBuildModuleCPP Module, CppCompileEnvironment CompileEnvironment)
		{
			base.AddModuleForIntelliSense(Module, CompileEnvironment);

			if (Settings.bUsePerFileIntellisense)
			{
				foreach (DirectoryItem ModuleDirectory in Module.ModuleDirectories)
				{
					List<string> ForceIncludePaths = new List<string>(CompileEnvironment.ForceIncludeFiles.Select(x => InsertPathVariables(x.Location)));
					if (CompileEnvironment.PrecompiledHeaderIncludeFilename != null)
					{
						string PchHeaderFile = InsertPathVariables(CompileEnvironment.PrecompiledHeaderIncludeFilename);
						ForceIncludePaths.Insert(0, PchHeaderFile);
						ModuleDirToPchHeaderFile[Module.ModuleDirectory] = PchHeaderFile;
					}
					ModuleDirToForceIncludePaths[Module.ModuleDirectory] = String.Join(";", ForceIncludePaths);
				}
			}
		}

		static string InsertPathVariables(FileReference Location)
		{
			if (Location.IsUnderDirectory(ProjectFileGenerator.PrimaryProjectPath))
			{
				return String.Format("$(SolutionDir){0}", Location.MakeRelativeTo(ProjectFileGenerator.PrimaryProjectPath));
			}
			else
			{
				return Location.FullName;
			}
		}

		/// <summary>
		/// Gets highest C++ version which is used in this project
		/// </summary>
		/// <returns>C++ standard version</returns>
		public CppStandardVersion GetIntelliSenseCppVersion()
		{
			if (IntelliSenseCppVersion != CppStandardVersion.Default)
			{
				return IntelliSenseCppVersion;
			}

			CppStandardVersion Version = CppStandardVersion.Default;
			foreach (ProjectConfigAndTargetCombination Combination in ProjectConfigAndTargetCombinations!)
			{
				if (Combination.ProjectTarget != null && Combination.ProjectTarget.TargetRules != null && Combination.ProjectTarget.TargetRules.CppStandard > Version)
				{
					Version = Combination.ProjectTarget.TargetRules.CppStandard;
				}
			}

			return Version;
		}

		/// <summary>
		/// Gets compiler switch for specifying in AdditionalOptions in .vcxproj file for specific C++ version
		/// </summary>
		private static string GetCppStandardCompileArgument(CppStandardVersion Version)
		{
			switch (Version)
			{
				case CppStandardVersion.Cpp20:
					return "/std:c++20";
				case CppStandardVersion.Cpp23:
				case CppStandardVersion.Latest:
					return "/std:c++latest";
				default:
					throw new BuildException($"Unsupported C++ standard type set: {Version}");
			}
		}

		private static string GetConformanceCompileArguments(TargetRules Target)
		{
			if (Target.Platform.IsInGroup(UnrealPlatformGroup.Microsoft))
			{
				List<string> Arguments = [];
				if (Target.WindowsPlatform.bStrictConformanceMode)
				{
					// This define is needed to ensure that MSVC static analysis mode doesn't declare attributes that are incompatible with strict conformance mode
					Arguments.Add("/DSAL_NO_ATTRIBUTE_DECLARATIONS=1");

					Arguments.Add("/permissive-");
					if (Target.WindowsPlatform.Compiler.IsMSVC())
					{
						Arguments.Add("/Zc:strictStrings-"); // Have to disable strict const char* semantics due to Windows headers not being compliant.
					}
				}
				else if (Target.WindowsPlatform.Compiler.IsMSVC())
				{
					Arguments.Add("/Zc:hiddenFriend");
				}

				if (Target.WindowsPlatform.bUpdatedCPPMacro)
				{
					Arguments.Add("/Zc:__cplusplus");
				}

				if (Target.WindowsPlatform.bStrictInlineConformance)
				{
					Arguments.Add("/Zc:inline");
				}

				if (Target.WindowsPlatform.Compiler.IsMSVC())
				{
					Arguments.Add("/Zc:preprocessor");
				}

				if (Target.WindowsPlatform.bStrictEnumTypesConformance && Target.WindowsPlatform.Compiler.IsMSVC())
				{
					Arguments.Add("/Zc:enumTypes");
				}

				if (Target.WindowsPlatform.bStrictODRViolationConformance && Target.WindowsPlatform.bOptimizeGlobalData && Target.WindowsPlatform.Compiler.IsMSVC())
				{
					Arguments.Add("/Zc:checkGwOdr");
				}
				return String.Join(' ', Arguments);
			}
			return String.Empty;
		}

		/// <summary>
		/// Gets compiler switch for specifying in AdditionalOptions in .vcxproj file for coroutines support
		/// </summary>
		private string GetEnableCoroutinesArgument()
		{
			if (IntelliSenseEnableCoroutines)
			{
				if (VCProjectFileGenerator.GetCompilerForIntellisense(ProjectFileFormat).IsMSVC())
				{
					return "/await:strict";
				}

				return "-fcoroutines-ts";
			}

			return String.Empty;
		}

		HashSet<string>? InvalidConfigPlatformNames;
		List<ProjectConfigAndTargetCombination>? ProjectConfigAndTargetCombinations;

		private void BuildProjectConfigAndTargetCombinations(List<UnrealTargetPlatform> InPlatforms, List<UnrealTargetConfiguration> InConfigurations, PlatformProjectGeneratorCollection PlatformProjectGenerators)
		{
			//no need to do this more than once
			if (ProjectConfigAndTargetCombinations == null)
			{
				HashSet<string> ProjectConfigAndTargets = new();
				// Build up a list of platforms and configurations this project will support.  In this list, Unknown simply
				// means that we should use the default "stub" project platform and configuration name.

				// If this is a "stub" project, then only add a single configuration to the project
				ProjectConfigAndTargetCombinations = new List<ProjectConfigAndTargetCombination>();
				if (IsStubProject)
				{
					ProjectConfigAndTargetCombination StubCombination = new ProjectConfigAndTargetCombination(UnrealTargetPlatform.Parse(StubProjectPlatformName), UnrealTargetConfiguration.Unknown, StubProjectPlatformName, StubProjectConfigurationName, null, null);
					ProjectConfigAndTargetCombinations.Add(StubCombination);
				}
				else
				{
					// Figure out all the desired configurations
					foreach (UnrealTargetConfiguration Configuration in InConfigurations)
					{
						//@todo.Rocket: Put this in a commonly accessible place?
						if (InstalledPlatformInfo.IsValidConfiguration(Configuration, EProjectType.Code) == false)
						{
							continue;
						}
						foreach (UnrealTargetPlatform Platform in InPlatforms)
						{
							if (InstalledPlatformInfo.IsValidPlatform(Platform, EProjectType.Code) == false)
							{
								continue;
							}
							UEBuildPlatform? BuildPlatform;
							if (UEBuildPlatform.TryGetBuildPlatform(Platform, out BuildPlatform) && (BuildPlatform.HasRequiredSDKsInstalled() == SDKStatus.Valid))
							{
								foreach (ProjectTarget ProjectTarget in ProjectTargets.OfType<ProjectTarget>())
								{
									if (!IsValidProjectPlatformAndConfiguration(ProjectTarget, Platform, Configuration))
									{
										continue;
									}

									Action<UnrealArch?> AddProjectAndTargetCombination = (UnrealArch? Arch) =>
									{
										PlatformProjectGenerator? PlatformProjectGenerator = PlatformProjectGenerators.GetPlatformProjectGenerator(Platform, bInAllowFailure: true);
										string ProjectPlatformName;
										string ProjectConfigurationName = Configuration.ToString();
										bool CreateDistinctConfigName = false;

										VSSettings VSSettings = new(Platform, Configuration, ProjectFileFormat, Arch);

										// Check to see if this platform is supported directly by Visual Studio projects.
										if (PlatformProjectGenerator != null && PlatformProjectGenerator.HasVisualStudioSupport(VSSettings))
										{
											// Allow the platform to specify the name used in VisualStudio.
											// Note that the actual name of the platform on the Visual Studio side may be different than what
											// UnrealBuildTool calls it (e.g. "Win64" -> "x64".) GetVisualStudioPlatformName() will figure this out.
											ProjectPlatformName = PlatformProjectGenerator.GetVisualStudioPlatformName(VSSettings);

											// The project generator may require a distinct configuration name - typically when two UnrealTargetPlatforms need the same ProjectPlatformName - otherwise the properties overwrite each oter.
											if (PlatformProjectGenerator.RequiresDistinctVisualStudioConfigurationName(VSSettings))
											{
												CreateDistinctConfigName = true;
											}
										}
										else
										{
											// Visual Studio doesn't natively support this platform, so we fake it by mapping it to
											// a project configuration that has the platform name in that configuration as a suffix,
											// and then using "x64" as the actual VS platform name
											CreateDistinctConfigName = true;
											ProjectPlatformName = DefaultPlatformName;
										}

										if (CreateDistinctConfigName)
										{
											ProjectConfigurationName = String.Format("{0}{1}_{2}", Platform.ToString(), Arch != null ? "_" + Arch.ToString() : String.Empty, Configuration.ToString());
										}

										TargetType TargetConfigurationType = ProjectTarget.TargetRules!.Type;
										if (!bMakeProjectPerTarget && TargetConfigurationType != TargetType.Game && TargetConfigurationType != TargetType.Program)
										{
											ProjectConfigurationName += "_" + TargetConfigurationType.ToString();
										}

										if (ProjectConfigAndTargetCombinations.Any(Combination => Combination.ProjectPlatformName == ProjectPlatformName && Combination.ProjectConfigurationName == ProjectConfigurationName))
										{
											throw new BuildException("'{0}' '{1} is already in the platform/config list. This means a platform generator is not marking that the config needs to be distinct.", ProjectPlatformName, ProjectConfigurationName);
										}

										ProjectConfigAndTargetCombinations.Add(new ProjectConfigAndTargetCombination(Platform, Configuration, ProjectPlatformName, ProjectConfigurationName, ProjectTarget, Arch));
									};

									UnrealArchitectures? Architectures = VCProjectFileGenerator.GetPlatformArchitecturesToGenerate(BuildPlatform, ProjectTarget);
									if (Architectures == null)
									{
										AddProjectAndTargetCombination(null);
									}
									else
									{
										IEnumerable<UnrealArch> Archs = Architectures.Architectures;
										if (BuildPlatform.ArchitectureConfig.Mode == UnrealArchitectureMode.SingleTargetLinkSeparately && BuildPlatform.Platform == UnrealTargetPlatform.Android)
										{
											// Order project Architectures first, as they are preferred
											Archs = Archs.Concat(BuildPlatform.ArchitectureConfig.AllSupportedArchitectures.Architectures.Except(Archs));
										}

										foreach (UnrealArch Arch in Archs)
										{
											AddProjectAndTargetCombination(Arch);
										}
									}
								}
							}
						}
					}
				}

				// Create a list of platforms for the "invalid" configuration. We always require at least one of these.
				if (ProjectConfigAndTargetCombinations.Count == 0)
				{
					InvalidConfigPlatformNames = new HashSet<string> { DefaultPlatformName };
				}
				else
				{
					InvalidConfigPlatformNames = new HashSet<string>(ProjectConfigAndTargetCombinations.Select(x => x.ProjectPlatformName));
				}
			}
		}

		/// <summary>
		/// If found writes a debug project file to disk
		/// </summary>
		/// <returns>True on success</returns>
		public override List<Tuple<ProjectFile, string>> WriteDebugProjectFiles(List<UnrealTargetPlatform> InPlatforms, List<UnrealTargetConfiguration> InConfigurations, PlatformProjectGeneratorCollection PlatformProjectGenerators, ILogger Logger)
		{
			//string ProjectName = ProjectFilePath.GetFileNameWithoutExtension();

			List<UnrealTargetPlatform> ProjectPlatforms = new List<UnrealTargetPlatform>();
			List<Tuple<ProjectFile, string>> ProjectFiles = new List<Tuple<ProjectFile, string>>();

			BuildProjectConfigAndTargetCombinations(InPlatforms, InConfigurations, PlatformProjectGenerators);

			foreach (ProjectConfigAndTargetCombination Combination in ProjectConfigAndTargetCombinations!)
			{
				if (Combination.Platform != null && !ProjectPlatforms.Contains(Combination.Platform.Value))
				{
					ProjectPlatforms.Add(Combination.Platform.Value);
				}
			}

			//write out any additional project files
			if (!IsStubProject && ProjectTargets.Any(x => x.TargetRules != null && x.TargetRules.Type != TargetType.Program))
			{
				foreach (UnrealTargetPlatform Platform in ProjectPlatforms)
				{
					PlatformProjectGenerator? ProjGenerator = PlatformProjectGenerators.GetPlatformProjectGenerator(Platform, true);
					if (ProjGenerator != null)
					{
						//write out additional prop file
						ProjGenerator.WriteAdditionalPropFile();

						//write out additional project user files
						ProjGenerator.WriteAdditionalProjUserFile(this);

						//write out additional project files
						Tuple<ProjectFile, string>? DebugProjectInfo = ProjGenerator.WriteAdditionalProjFile(this);
						if (DebugProjectInfo != null)
						{
							ProjectFiles.Add(DebugProjectInfo);
						}
					}
				}
			}

			return ProjectFiles;
		}

		private string[]? FilteredIncludeList = null;
		private string[]? FilteredPathsList = null;

		bool PathIsFilteredOut(DirectoryReference InPath, ref string[]? FilteredList)
		{
			// Turn the filter string into an array, remove whitespace, and normalize any path statements the first time
			// we are asked to check a path.
			if (FilteredList == null)
			{
				IEnumerable<string> CleanPaths = Settings.ExcludedFilePaths.Split(new[] { ';' }, StringSplitOptions.RemoveEmptyEntries)
					.Select(P => P.Trim())
					.Select(P => P.Replace('/', Path.DirectorySeparatorChar));

				FilteredList = CleanPaths.ToArray();
			}

			if (FilteredList.Length > 0)
			{
				// The user might have specified Foo, \Foo, Foo\, or \Foo\ as excludes.
				// Directory paths don't contain a trailing slash so add one to the string
				// we'll compare to our list so we can Contains and EndsWith to catch:
				// <Path>\Foo\Dir
				// <Path>\Foo
				Span<char> initial = stackalloc char[512];
				using StackStringBuilder sb = new(initial);
				sb.Append(InPath.FullName);
				sb.Append(Path.DirectorySeparatorChar);
				ReadOnlySpan<char> PathWithSeparator = sb.AsSpan();

				foreach (string Entry in FilteredList)
				{
					if (PathWithSeparator.Contains(Entry, StringComparison.OrdinalIgnoreCase)
						|| PathWithSeparator.EndsWith(Entry, StringComparison.OrdinalIgnoreCase))
					{
						return true;
					}
				}
			}

			return false;
		}

		bool IncludePathIsFilteredOut(DirectoryReference IncludePath)
		{
			return PathIsFilteredOut(IncludePath, ref FilteredIncludeList);
		}

		bool FilePathIsFilteredOut(DirectoryReference InPath)
		{
			return PathIsFilteredOut(InPath, ref FilteredPathsList);
		}

		bool TryGetBuildEnvironment(DirectoryReference BaseDir, [NotNullWhen(true)] out BuildEnvironment? OutBuildEnvironment)
		{
			for (DirectoryReference? CurrentDir = BaseDir; CurrentDir != null; CurrentDir = CurrentDir.ParentDirectory)
			{
				BuildEnvironment? BuildEnvironment;
				if (BaseDirToBuildEnvironment.TryGetValue(CurrentDir, out BuildEnvironment))
				{
					OutBuildEnvironment = BuildEnvironment;
					return true;
				}
			}

			OutBuildEnvironment = null;
			return false;
		}

		/// <summary>
		/// Append a list of include paths to a property list
		/// </summary>
		/// <param name="Builder">String builder for the property value</param>
		/// <param name="Collection">Collection of include paths</param>
		/// <param name="IgnorePaths">Set of paths to ignore</param>
		void AppendIncludePaths(ref StackStringBuilder Builder, IncludePathsCollection Collection, HashSet<DirectoryReference> IgnorePaths)
		{
			foreach (DirectoryReference IncludePath in Collection.AbsolutePaths)
			{
				if (!IgnorePaths.Contains(IncludePath) && !IncludePathIsFilteredOut(IncludePath))
				{
					NormalizeProjectPath(ref Builder, IncludePath);
					Builder.Append(';');
				}
			}
		}

		/// Implements Project interface
		public override bool WriteProjectFile(List<UnrealTargetPlatform> InPlatforms, List<UnrealTargetConfiguration> InConfigurations, PlatformProjectGeneratorCollection PlatformProjectGenerators, ILogger Logger)
		{
			string ProjectName = ProjectFilePath.GetFileNameWithoutExtension();

			bool bSuccess = true;

			// Setup VC project file content
			StringBuilder VCProjectFileContent = new StringBuilder();
			StringBuilder VCFiltersFileContent = new StringBuilder();
			StringBuilder VCUserFileContent = new StringBuilder();
			VisualStudioUserFileSettings VCUserFileSettings = new VisualStudioUserFileSettings();

			// Visual Studio doesn't require a *.vcxproj.filters file to even exist alongside the project unless
			// it actually has something of substance in it.  We'll avoid saving it out unless we need to.
			bool FiltersFileIsNeeded = false;

			// Project file header
			VCProjectFileContent.AppendLine("<?xml version=\"1.0\" encoding=\"utf-8\"?>");
			VCProjectFileContent.AppendLine("<Project DefaultTargets=\"Build\" ToolsVersion=\"{0}\" xmlns=\"http://schemas.microsoft.com/developer/msbuild/2003\">", VCProjectFileGenerator.GetProjectFileToolVersionString(ProjectFileFormat));

			bool bGenerateUserFileContent = PlatformProjectGenerators.PlatformRequiresVSUserFileGeneration(InPlatforms);
			if (bGenerateUserFileContent)
			{
				VCUserFileContent.AppendLine("<?xml version=\"1.0\" encoding=\"utf-8\"?>");
				VCUserFileContent.AppendLine("<Project ToolsVersion=\"{0}\" xmlns=\"http://schemas.microsoft.com/developer/msbuild/2003\">", VCProjectFileGenerator.GetProjectFileToolVersionString(ProjectFileFormat));
			}

			BuildProjectConfigAndTargetCombinations(InPlatforms, InConfigurations, PlatformProjectGenerators);

			VCProjectFileContent.AppendLine("  <ItemGroup Label=\"ProjectConfigurations\">");

			// Make a list of the platforms and configs as project-format names
			List<UnrealTargetPlatform> ProjectPlatforms = new List<UnrealTargetPlatform>();
			List<Tuple<string, UnrealTargetPlatform>> ProjectPlatformNameAndPlatforms = new List<Tuple<string, UnrealTargetPlatform>>();    // ProjectPlatformName, Platform
			List<Tuple<string, UnrealTargetConfiguration>> ProjectConfigurationNameAndConfigurations = new List<Tuple<string, UnrealTargetConfiguration>>();    // ProjectConfigurationName, Configuration
			List<ProjectConfigAndTargetCombination> ValidProjectConfigAndTargetCombinations = new List<ProjectConfigAndTargetCombination>();
			Dictionary<string, HashSet<UnrealTargetPlatform>> ProjectPlatformNameToPlatform = new Dictionary<string, HashSet<UnrealTargetPlatform>>();
			foreach (ProjectConfigAndTargetCombination Combination in ProjectConfigAndTargetCombinations!)
			{
				if (Combination.Platform == null)
				{
					continue;
				}
				if (Combination.ProjectTarget != null && !IsValidProjectPlatformAndConfiguration(Combination.ProjectTarget, (UnrealTargetPlatform)Combination.Platform, Combination.Configuration))
				{
					continue;
				}
				if (!ProjectPlatforms.Contains(Combination.Platform.Value))
				{
					ProjectPlatforms.Add(Combination.Platform.Value);
				}
				if (!ProjectPlatformNameAndPlatforms.Any(ProjectPlatformNameAndPlatformTuple => ProjectPlatformNameAndPlatformTuple.Item1 == Combination.ProjectPlatformName))
				{
					ProjectPlatformNameAndPlatforms.Add(Tuple.Create(Combination.ProjectPlatformName, Combination.Platform.Value));
				}
				if (!ProjectConfigurationNameAndConfigurations.Any(ProjectConfigurationNameAndConfigurationTuple => ProjectConfigurationNameAndConfigurationTuple.Item1 == Combination.ProjectConfigurationName))
				{
					ProjectConfigurationNameAndConfigurations.Add(Tuple.Create(Combination.ProjectConfigurationName, Combination.Configuration));
				}

				ValidProjectConfigAndTargetCombinations.Add(Combination);

				if (ProjectPlatformNameToPlatform.TryGetValue(Combination.ProjectPlatformName, out HashSet<UnrealTargetPlatform>? PlatformList))
				{
					PlatformList.Add(Combination.Platform.Value);
				}
				else
				{
					ProjectPlatformNameToPlatform.Add(Combination.ProjectPlatformName, new HashSet<UnrealTargetPlatform>() { Combination.Platform.Value });
				}
			}

			// Add the "invalid" configuration for each platform. We use this when the solution configuration does not match any project configuration.
			foreach (string InvalidConfigPlatformName in InvalidConfigPlatformNames!)
			{
				VCProjectFileContent.AppendLine("    <ProjectConfiguration Include=\"Invalid|{0}\">", InvalidConfigPlatformName);
				VCProjectFileContent.AppendLine("      <Configuration>Invalid</Configuration>");
				VCProjectFileContent.AppendLine("      <Platform>{0}</Platform>", InvalidConfigPlatformName);
				VCProjectFileContent.AppendLine("    </ProjectConfiguration>");
			}

			// Output ALL the project's config-platform permutations (project files MUST do this)
			foreach (Tuple<string, UnrealTargetConfiguration> ConfigurationTuple in ProjectConfigurationNameAndConfigurations)
			{
				string ProjectConfigurationName = ConfigurationTuple.Item1;
				foreach (Tuple<string, UnrealTargetPlatform> PlatformTuple in ProjectPlatformNameAndPlatforms)
				{
					string ProjectPlatformName = PlatformTuple.Item1;
					VCProjectFileContent.AppendLine("    <ProjectConfiguration Include=\"{0}|{1}\">", ProjectConfigurationName, ProjectPlatformName);
					VCProjectFileContent.AppendLine("      <Configuration>{0}</Configuration>", ProjectConfigurationName);
					VCProjectFileContent.AppendLine("      <Platform>{0}</Platform>", ProjectPlatformName);
					VCProjectFileContent.AppendLine("    </ProjectConfiguration>");
				}
			}

			VCProjectFileContent.AppendLine("  </ItemGroup>");

			VCFiltersFileContent.AppendLine("<?xml version=\"1.0\" encoding=\"utf-8\"?>");
			VCFiltersFileContent.AppendLine("<Project ToolsVersion=\"{0}\" xmlns=\"http://schemas.microsoft.com/developer/msbuild/2003\">", VCProjectFileGenerator.GetProjectFileToolVersionString(ProjectFileFormat));

			// Platform specific PropertyGroups, etc.
			if (!IsStubProject)
			{
				foreach (UnrealTargetPlatform Platform in ProjectPlatforms)
				{
					PlatformProjectGenerator? ProjGenerator = PlatformProjectGenerators.GetPlatformProjectGenerator(Platform, true);
					VSSettings VSSettings = new(Platform, UnrealTargetConfiguration.Development, ProjectFileFormat, null);
					if (ProjGenerator != null && ProjGenerator.HasVisualStudioSupport(VSSettings))
					{
						ProjGenerator.GetAdditionalVisualStudioPropertyGroups(VSSettings, VCProjectFileContent);
					}
				}
			}

			ProjectConfigAndTargetCombination? FoundCombo = ProjectConfigAndTargetCombinations.FirstOrDefault(combo => combo != null && combo.ProjectTarget != null && combo.ProjectTarget.TargetRules != null);
			TargetRules? DefaultRules = FoundCombo?.ProjectTarget?.TargetRules;
			bool IsTestTarget = (DefaultRules != null && DefaultRules.IsTestTarget);

			// Project globals (project GUID, project type, SCC bindings, etc)
			{
				VCProjectFileContent.AppendLine("  <PropertyGroup Label=\"Globals\">");
				VCProjectFileContent.AppendLine("    <ProjectGuid>{0}</ProjectGuid>", ProjectGUID.ToString("B").ToUpperInvariant());
				VCProjectFileContent.AppendLine("    <RootNamespace>{0}</RootNamespace>", ProjectName);
				if (IsTestTarget)
				{
					VCProjectFileContent.AppendLine("    <IsTestTarget>true</IsTestTarget>");
				}
				if (ProjectFileGenerator.bVisualStudioLinux)
				{
					VCProjectFileContent.AppendLine("      <Keyword>Linux</Keyword>");
					VCProjectFileContent.AppendLine("      <ApplicationType>Linux</ApplicationType>");
					VCProjectFileContent.AppendLine("      <TargetLinuxPlatform>Generic</TargetLinuxPlatform>");
					VCProjectFileContent.AppendLine("      <ApplicationTypeRevision>1.0</ApplicationTypeRevision>");
					VCProjectFileContent.AppendLine("      <LinuxProjectType>{D51BCBC9-82E9-4017-911E-C93873C4EA2B}</LinuxProjectType>");
				}
				VCProjectFileContent.AppendLine("  </PropertyGroup>");
			}

			// look for additional import lines for all platforms for non stub projects
			if (!IsStubProject)
			{
				foreach (UnrealTargetPlatform Platform in ProjectPlatforms)
				{
					VSSettings VSSettings = new(Platform, UnrealTargetConfiguration.Development, ProjectFileFormat, null);
					PlatformProjectGenerator? ProjGenerator = PlatformProjectGenerators.GetPlatformProjectGenerator(Platform, true);
					if (ProjGenerator != null && ProjGenerator.HasVisualStudioSupport(VSSettings))
					{
						ProjGenerator.GetVisualStudioGlobalProperties(VSSettings, VCProjectFileContent);
					}
				}
			}

			// Write each project configuration PreDefaultProps section
			HashSet<UnrealTargetPlatform> CommonPlatformsWritten = new HashSet<UnrealTargetPlatform>();
			foreach (ProjectConfigAndTargetCombination Combination in ValidProjectConfigAndTargetCombinations)
			{
				if (Combination.Platform != null)
				{
					UnrealTargetPlatform TargetPlatform = Combination.Platform.Value;
					PlatformProjectGenerator? ProjGenerator = PlatformProjectGenerators.GetPlatformProjectGenerator(TargetPlatform, true);
					if (ProjGenerator != null)
					{
						bool ProjectPlatformRequiresConfigurationName = false;
						if (ProjectPlatformNameToPlatform.TryGetValue(Combination.ProjectPlatformName, out HashSet<UnrealTargetPlatform>? PlatformList))
						{
							ProjectPlatformRequiresConfigurationName = PlatformList.Count > 1;
						}

						// Properties that are common for the unique platform
						// Note that if the platform name is not unique then the common properties will be added below
						if (!ProjectPlatformRequiresConfigurationName && !CommonPlatformsWritten.Contains(Combination.Platform.Value))
						{
							StringBuilder CommonPlatformToolsetString = new StringBuilder();
							ProjGenerator.GetVisualStudioPreDefaultString(TargetPlatform, CommonPlatformToolsetString);

							if (CommonPlatformToolsetString.Length > 0)
							{
								string ConditionString = "Condition=\"'$(Platform)'=='" + Combination.ProjectPlatformName + "'\"";
								VCProjectFileContent.AppendLine("  <PropertyGroup " + ConditionString + " Label=\"Configuration\">");
								VCProjectFileContent.Append(CommonPlatformToolsetString);
								VCProjectFileContent.AppendLine("  </PropertyGroup>");
							}
							CommonPlatformsWritten.Add(TargetPlatform);
						}

						// Properties that require the configuration and platform name
						StringBuilder PlatformToolsetString = new StringBuilder();
						if (ProjectPlatformRequiresConfigurationName)
						{
							ProjGenerator.GetVisualStudioPreDefaultString(TargetPlatform, PlatformToolsetString);
						}

						ProjGenerator.GetVisualStudioPreDefaultString(TargetPlatform, Combination.Configuration, PlatformToolsetString);
						if (PlatformToolsetString.Length > 0)
						{
							string ProjectConfigurationAndPlatformName = Combination.ProjectConfigurationName + "|" + Combination.ProjectPlatformName;
							string ConditionString = "Condition=\"'$(Configuration)|$(Platform)'=='" + ProjectConfigurationAndPlatformName + "'\"";
							VCProjectFileContent.AppendLine("  <PropertyGroup " + ConditionString + " Label=\"Configuration\">");
							VCProjectFileContent.Append(PlatformToolsetString);
							VCProjectFileContent.AppendLine("  </PropertyGroup>");
						}
					}
				}
			}

			// Write the per platform/config configuration info
			foreach (Tuple<string, UnrealTargetConfiguration> ConfigurationTuple in ProjectConfigurationNameAndConfigurations)
			{
				string ProjectConfigurationName = ConfigurationTuple.Item1;
				UnrealTargetConfiguration TargetConfiguration = ConfigurationTuple.Item2;
				foreach (Tuple<string, UnrealTargetPlatform> PlatformTuple in ProjectPlatformNameAndPlatforms)
				{
					string ProjectPlatformName = PlatformTuple.Item1;
					UnrealTargetPlatform TargetPlatform = PlatformTuple.Item2;

					PlatformProjectGenerator? ProjGenerator = PlatformProjectGenerators.GetPlatformProjectGenerator(TargetPlatform, true);
					if (ProjGenerator == null)
					{
						continue;
					}

					PlatformProjectGenerator.VSSettings VSSettings = new(TargetPlatform, TargetConfiguration, ProjectFileFormat, null);
					StringBuilder PlatformToolsetString = new StringBuilder();
					ProjGenerator.GetVisualStudioPlatformToolsetString(VSSettings, PlatformToolsetString);
					string PlatformConfigurationType = ProjGenerator.GetVisualStudioPlatformConfigurationType(VSSettings);

					// if we are using the defaults set earlier then skip writing this
					if (PlatformConfigurationType == PlatformProjectGenerator.DefaultPlatformConfigurationType && PlatformToolsetString.Length == 0)
					{
						continue;
					}

					string ProjectConfigurationAndPlatformName = ProjectConfigurationName + "|" + ProjectPlatformName;
					string ConditionString = "Condition=\"'$(Configuration)|$(Platform)'=='" + ProjectConfigurationAndPlatformName + "'\"";
					VCProjectFileContent.AppendLine("  <PropertyGroup {0} Label=\"Configuration\">", ConditionString);
					VCProjectFileContent.AppendLine("    <ConfigurationType>{0}</ConfigurationType>", PlatformConfigurationType);

					if (PlatformToolsetString.Length == 0)
					{
						VCProjectFileGenerator.AppendPlatformToolsetProperty(VCProjectFileContent, ProjectFileFormat);
					}
					else
					{
						VCProjectFileContent.Append(PlatformToolsetString);
					}

					VCProjectFileContent.AppendLine("  </PropertyGroup>");
				}
			}

			VCProjectFileContent.AppendLine("  <Import Project=\"UECommon.props\" />");
			VCProjectFileContent.AppendLine("  <ImportGroup Label=\"ExtensionSettings\" />");
			VCProjectFileContent.AppendLine("  <PropertyGroup Label=\"UserMacros\" />");

			// Merge as many include paths as possible into the shared list
			HashSet<DirectoryReference> SharedIncludeSearchPathsSet = new HashSet<DirectoryReference>();

			// Build up the new include search path string
			StringBuilder SharedIncludeSearchPaths = new StringBuilder();
			{
				// Find out how many source files there are in each directory
				ConcurrentDictionary<DirectoryReference, int> SourceDirToCount = new();
				Parallel2.ForEach(SourceFiles.Where(sf => sf.Reference.HasExtension(".cpp")), SourceFile =>
				{
					SourceDirToCount.AddOrUpdate(SourceFile.Reference.Directory, _ => 1, (k, v) => v + 1);
				});

				// Figure out the most common include paths
				ConcurrentDictionary<DirectoryReference, int> IncludePathToCount = new();
				Parallel2.ForEach(SourceDirToCount, Pair =>
				{
					if (TryGetBuildEnvironment(Pair.Key, out BuildEnvironment? OutBuildEnvironment))
					{
						foreach (DirectoryReference IncludePath in OutBuildEnvironment.UserIncludePaths.AbsolutePaths)
						{
							IncludePathToCount.AddOrUpdate(IncludePath, _ => Pair.Value, (k, v) => v + Pair.Value);
						}
						foreach (DirectoryReference IncludePath in OutBuildEnvironment.SystemIncludePaths.AbsolutePaths)
						{
							IncludePathToCount.AddOrUpdate(IncludePath, _ => Pair.Value, (k, v) => v + Pair.Value);
						}
						return;
					}
				});

				// Append the most common include paths to the search list.
				if (Settings.MaxSharedIncludePaths > 0)
				{
					foreach (DirectoryReference IncludePath in IncludePathToCount.OrderByDescending(x => x.Value).ThenBy(x => x.Key).Select(x => x.Key))
					{
						string RelativePath = NormalizeProjectPath(IncludePath);
						if (SharedIncludeSearchPaths.Length + RelativePath.Length >= Settings.MaxSharedIncludePaths)
						{
							break;
						}

						if (!IncludePathIsFilteredOut(IncludePath))
						{
							SharedIncludeSearchPathsSet.Add(IncludePath);
							SharedIncludeSearchPaths.AppendFormat("{0};", RelativePath);
						}
					}
				}

				SharedIncludeSearchPaths.AppendFormat("$(DefaultSystemIncludePaths);");
			}

			// Gather source folder and file info
			ConcurrentBag<AliasedFile> LocalAliasedFiles = new(AliasedFiles);
			ConcurrentDictionary<DirectoryReference, string?> DirectoryToPchFile = new();
			ConcurrentDictionary<DirectoryReference, string> DirectoryToForceIncludePaths = new();
			ConcurrentDictionary<DirectoryReference, string> DirectoryToIncludeSearchPaths = new();
			{
				Parallel2.ForEach(SourceFiles, CurFile =>
				{
					// get the filetype as represented to Visual Studio
					VCFileType FileType = GetVCFileType(CurFile.Reference.FullName);

					DirectoryReference FileSystemPathDir = CurFile.Reference.Directory;

					// if the filetype is an include and its path is filtered out, skip it entirely (should we do this for any type of
					// file? Possibly, but not today due to potential fallout)
					if (CanVCFileTypeBeIncluded(FileType) && IncludePathIsFilteredOut(FileSystemPathDir))
					{
						return;
					}

					// Allow filtering of any type of file
					if (FilePathIsFilteredOut(FileSystemPathDir))
					{
						return;
					}

					// We want all source file and directory paths in the project files to be relative to the project file's
					// location on the disk.  Convert the path to be relative to the project file directory
					string ProjectRelativeSourceFile = CurFile.Reference.MakeRelativeTo(ProjectFilePath.Directory);

					// By default, files will appear relative to the project file in the solution.  This is kind of the normal Visual
					// Studio way to do things, but because our generated project files are emitted to intermediate folders, if we always
					// did this it would yield really ugly paths int he solution explorer
					string FilterRelativeSourceDirectory;
					if (CurFile.BaseFolder == null)
					{
						FilterRelativeSourceDirectory = ProjectRelativeSourceFile;
					}
					else
					{
						FilterRelativeSourceDirectory = CurFile.Reference.MakeRelativeTo(CurFile.BaseFolder);
					}

					// Manually remove the filename for the filter. We run through this code path a lot, so just do it manually.
					int LastSeparatorIdx = FilterRelativeSourceDirectory.LastIndexOf(Path.DirectorySeparatorChar);
					if (LastSeparatorIdx == -1)
					{
						FilterRelativeSourceDirectory = "";
					}
					else
					{
						FilterRelativeSourceDirectory = FilterRelativeSourceDirectory.Substring(0, LastSeparatorIdx);
					}

					AliasedFile LocalAliasedFile = new(CurFile.Reference, ProjectRelativeSourceFile, FilterRelativeSourceDirectory);
					LocalAliasedFiles.Add(LocalAliasedFile);

					if (!CanVCFileTypeBeCompiled(FileType))
					{
						return;
					}

					DirectoryReference Directory = LocalAliasedFile.Location.Directory;

					// Find the PCH file
					DirectoryToPchFile.GetOrAdd(Directory, _ =>
					{
						string? PchHeaderFile = null;
						for (DirectoryReference? ParentDir = Directory; ParentDir != null; ParentDir = ParentDir.ParentDirectory)
						{
							if (ModuleDirToPchHeaderFile.TryGetValue(ParentDir, out PchHeaderFile))
							{
								break;
							}
						}
						return PchHeaderFile;
					});

					// Find the force-included headers
					DirectoryToForceIncludePaths.GetOrAdd(Directory, _ =>
					{
						string? ForceIncludePaths = null;
						for (DirectoryReference? ParentDir = Directory; ParentDir != null; ParentDir = ParentDir.ParentDirectory)
						{
							if (ModuleDirToForceIncludePaths.TryGetValue(ParentDir, out ForceIncludePaths))
							{
								break;
							}
						}

						// filter here. It's a little more graceful to do it where this info is built but easier to follow if we filter 
						// things our right before they're written.
						if (!String.IsNullOrEmpty(ForceIncludePaths))
						{
							IEnumerable<string> PathList = ForceIncludePaths.Split(new[] { ';' }, StringSplitOptions.RemoveEmptyEntries);
							ForceIncludePaths = String.Join(";", PathList.Where(P => !IncludePathIsFilteredOut(new DirectoryReference(P))));
						}

						ForceIncludePaths ??= String.Empty;

						return ForceIncludePaths;
					});

					if (TryGetBuildEnvironment(Directory, out BuildEnvironment? BuildEnvironment))
					{
						DirectoryToIncludeSearchPaths.GetOrAdd(Directory, _ =>
						{
							Span<char> initial = stackalloc char[32*1024];
							StackStringBuilder Builder = new(initial);
							AppendIncludePaths(ref Builder, BuildEnvironment.UserIncludePaths, SharedIncludeSearchPathsSet);
							AppendIncludePaths(ref Builder, BuildEnvironment.SystemIncludePaths, SharedIncludeSearchPathsSet);
							string res = Builder.ToString();
							Builder.Dispose();
							return res;
						});
					}
				});
			}

			// Check to see if all the source settings are the same
			string CommonForcedIncludes = String.Empty;
			string CommonAdditionalOptions = String.Empty;
			{
				if (!DirectoryToForceIncludePaths.IsEmpty)
				{
					string ForceIncludePathToCheck = DirectoryToForceIncludePaths.Values.First();
					if (DirectoryToForceIncludePaths.Values.All(x => x == ForceIncludePathToCheck))
					{
						CommonForcedIncludes = ForceIncludePathToCheck;
						DirectoryToForceIncludePaths.Clear();
					}
				}

				if (!DirectoryToPchFile.IsEmpty)
				{
					string? PchFileToCheck = DirectoryToPchFile.Values.FirstOrDefault();
					if (DirectoryToPchFile.Values.All(x => x == PchFileToCheck))
					{
						if (!String.IsNullOrEmpty(PchFileToCheck))
						{
							CommonAdditionalOptions = $"/Yu\"{PchFileToCheck}\"";
						}
						DirectoryToPchFile.Clear();
					}
				}
			}

			StringBuilder VCPreprocessorDefinitions = new StringBuilder();
			foreach (string CurDef in IntelliSensePreprocessorDefinitions)
			{
				if (VCPreprocessorDefinitions.Length > 0)
				{
					VCPreprocessorDefinitions.Append(';');
				}
				VCPreprocessorDefinitions.Append(CurDef);
			}

			Func<TargetRules?, string> GetAdditionalOptionsString = (TargetRules? TargetRules) =>
			{
				return String.Format("{0} {1}{2}{3}", GetCppStandardCompileArgument(GetIntelliSenseCppVersion()),
					GetEnableCoroutinesArgument(),
					DefaultRules != null ? (" " + GetConformanceCompileArguments(DefaultRules)) : String.Empty,
					CommonAdditionalOptions.Length > 0 ? (" " + CommonAdditionalOptions) : String.Empty);
			};

			string DefaultAdditionalOptions = GetAdditionalOptionsString(DefaultRules);

			// Write common IntelliSense info
			{
				// @todo projectfiles: Currently we are storing defines/include paths for ALL configurations rather than using ConditionString and storing
				//      this data uniquely for each target configuration.  IntelliSense may behave better if we did that, but it will result in a LOT more
				//      data being stored into the project file, and might make the IDE perform worse when switching configurations!
				VCProjectFileContent.AppendLine("  <PropertyGroup>");
				VCProjectFileContent.AppendLine("    <NMakePreprocessorDefinitions>$(NMakePreprocessorDefinitions){0}</NMakePreprocessorDefinitions>", (VCPreprocessorDefinitions.Length > 0 ? (";" + VCPreprocessorDefinitions) : String.Empty));
				// NOTE: Setting the IncludePath property rather than NMakeIncludeSearchPath results in significantly less
				// memory usage, because NMakeIncludeSearchPath metadata is duplicated to each output item. Functionality should be identical for
				// intellisense results.
				VCProjectFileContent.AppendLine("    <IncludePath>$(IncludePath){0}</IncludePath>", (SharedIncludeSearchPaths.Length > 0 ? (";" + SharedIncludeSearchPaths) : ""));
				VCProjectFileContent.AppendLine("    <NMakeForcedIncludes>$(NMakeForcedIncludes){0}</NMakeForcedIncludes>", (CommonForcedIncludes.Length > 0 ? (";" + CommonForcedIncludes) : ""));
				VCProjectFileContent.AppendLine("    <NMakeAssemblySearchPath>$(NMakeAssemblySearchPath)</NMakeAssemblySearchPath>");
				VCProjectFileContent.AppendLine("    <AdditionalOptions>{0}</AdditionalOptions>", DefaultAdditionalOptions);
				VCProjectFileContent.AppendLine("  </PropertyGroup>");
			}

			// Write platform properties
			HashSet<UnrealTargetPlatform?> WrittenPlatforms = new();
			foreach (ProjectConfigAndTargetCombination Combination in ProjectConfigAndTargetCombinations)
			{
				if (WrittenPlatforms.Add(Combination.Platform))
				{
					StringBuilder PlatformProperties = new();
					string PlatformAdditionalOptions = GetAdditionalOptionsString(Combination.ProjectTarget?.TargetRules);
					if (PlatformAdditionalOptions != DefaultAdditionalOptions)
					{
						PlatformProperties.AppendLine("    <AdditionalOptions>{0}</AdditionalOptions>", PlatformAdditionalOptions);
					}

					if (PlatformProperties.Length != 0)
					{
						VCProjectFileContent.AppendLine("  <PropertyGroup Condition=\"'$(Platform)'=='" + Combination.ProjectPlatformName + "'\">");
						VCProjectFileContent.Append(PlatformAdditionalOptions);
						VCProjectFileContent.AppendLine("  </PropertyGroup>");
					}
				}
			}

			// Write each configuration
			{
				StringBuilder[] TmpVCProjectFileContent = new StringBuilder[ProjectConfigAndTargetCombinations.Count];
				StringBuilder[] TmpVCUserFileContent = new StringBuilder[ProjectConfigAndTargetCombinations.Count];

				Parallel2.For(0, ProjectConfigAndTargetCombinations.Count, Index =>
				{
					TmpVCProjectFileContent[Index] = new StringBuilder();
					TmpVCUserFileContent[Index] = new StringBuilder();

					WriteConfiguration(ProjectName, ProjectConfigAndTargetCombinations[Index], TmpVCProjectFileContent[Index], PlatformProjectGenerators, bGenerateUserFileContent ? TmpVCUserFileContent[Index] : null, bGenerateUserFileContent ? VCUserFileSettings : null);
				});

				for (int Index = 0; Index < ProjectConfigAndTargetCombinations.Count; Index++)
				{
					if (TmpVCProjectFileContent[Index].Length > 0) 
					{
						VCProjectFileContent.Append(TmpVCProjectFileContent[Index]);
					}

					if (TmpVCUserFileContent[Index].Length > 0) 
					{
						VCUserFileContent.Append(TmpVCUserFileContent[Index]);
					}
				}
			}

			{
				// Collapse common values
				{
					StringBuilder CommonProjectFileContent = new StringBuilder();

					void AddProperties(IDictionary<DirectoryReference, string> DirectoryToStringDict, string CommonPropertyPrefix, string PropertyNamePrefix, string PropertyValuePrefix)
					{
						Dictionary<string, string> UpdatedValues = new(StringComparer.Ordinal);
						List<KeyValuePair<DirectoryReference, string>> KVPList = DirectoryToStringDict.ToList();
						KVPList.SortBy(kvp => kvp.Key);

						// Map each directory to a property
						{
							int PropertyIndex = 0;

							StringBuilder PropertyNameBuilder = new(PropertyNamePrefix);
							int PropertyNameBuilderLength = PropertyNameBuilder.Length;
							string NextPropertyName = PropertyNameBuilder.ToString();

							foreach (KeyValuePair<DirectoryReference, string> DirectoryKVP in KVPList)
							{
								string? Value = DirectoryKVP.Value;
								if (String.IsNullOrEmpty(Value))
								{
									continue;
								}

								if (UpdatedValues.TryAdd(Value, NextPropertyName))
								{
									string PropertyName = NextPropertyName;
									PropertyNameBuilder.Append('_').Append(++PropertyIndex);
									NextPropertyName = PropertyNameBuilder.ToString();
									PropertyNameBuilder.Length = PropertyNameBuilderLength;
									DirectoryToStringDict[DirectoryKVP.Key] = PropertyName;
								}
								else
								{
									DirectoryToStringDict[DirectoryKVP.Key] = UpdatedValues[Value];
								}
							}
						}

						// Find the common property values
						Dictionary<string, string> ValueToCommonPropertyDict = new(StringComparer.Ordinal);
						{
							Dictionary<string, int> ValueAndCount = new();
							foreach (string PropertyValue in UpdatedValues.Keys)
							{
								if (!String.IsNullOrEmpty(PropertyValue))
								{
									IEnumerable<string> PathList = PropertyValue.Split(new[] { ';' }, StringSplitOptions.RemoveEmptyEntries);
									foreach (string Path in PathList)
									{
										if (ValueAndCount.TryGetValue(Path, out int PathCount))
										{
											ValueAndCount[Path] = ++PathCount;
										}
										else
										{
											ValueAndCount[Path] = 1;
										}
									}
								}
							}

							List<string> CommonProperties = ValueAndCount.Where(kvp => kvp.Value > 1).Select(kvp => kvp.Key).ToList();
							CommonProperties.Sort(StringComparer.Ordinal);

							// Write out the common property values
							int CommonPropertyValueIndex = 0;
							foreach (string? CommonProperty in CommonProperties)
							{
								string PropertyName = CommonPropertyPrefix;
								if (CommonPropertyValueIndex > 0)
								{
									PropertyName += "_" + CommonPropertyValueIndex;
								}
								CommonPropertyValueIndex++;
								ValueToCommonPropertyDict.Add(CommonProperty, $"$({PropertyName})");
								CommonProjectFileContent.AppendLine($"    <{PropertyName}>{CommonProperty}</{PropertyName}>");
							}
						}

						// Write out the updated properties
						foreach (KeyValuePair<string, string> kvp in UpdatedValues)
						{
							string[] PathArray = kvp.Key.Split(new[] { ';' }, StringSplitOptions.RemoveEmptyEntries);
							for (int Index = 0; Index < PathArray.Length; Index++)
							{
								if (ValueToCommonPropertyDict.TryGetValue(PathArray[Index], out string? PropertyValue) && !String.IsNullOrEmpty(PropertyValue))
								{
									PathArray[Index] = PropertyValue;
								}
							}
							Array.Sort(PathArray, String.CompareOrdinal);
							string NewValue = String.Join(";", PathArray);
							CommonProjectFileContent.AppendLine($"    <{kvp.Value}>{PropertyValuePrefix}{NewValue}</{kvp.Value}>");
						}
					}

					// Use short names on these because there are _a ton_ of them in the big projects. the vcxproj of our biggest project went from 70mb to 31mb

					//AdditionalIncludeDirectories
					AddProperties(DirectoryToIncludeSearchPaths, "AID", "ClCompile_AdditionalIncludeDirectories", "$(NMakeIncludeSearchPath);");

					//ForcedIncludeFiles
					AddProperties(DirectoryToForceIncludePaths, "FIF", "ClCompile_ForcedIncludeFiles", String.Empty);

					// AdditionalOptions
					{
						int ValueIndex = 0;
						Dictionary<string, string> UpdatedValues = new();
						List<KeyValuePair<DirectoryReference, string?>> KVPList = DirectoryToPchFile.ToList();
						KVPList.SortBy(kvp => kvp.Key);

						StringBuilder PropertyNameBuilder = new("ClCompile_AdditionalOptions");
						int PropertyNameBuilderLength = PropertyNameBuilder.Length;
						string NextPropertyName = PropertyNameBuilder.ToString();

						foreach (KeyValuePair<DirectoryReference, string?> DirectoryKVP in KVPList)
						{
							string? Value = DirectoryKVP.Value;
							if (String.IsNullOrEmpty(Value))
							{
								continue;
							}
							if (UpdatedValues.TryAdd(Value, NextPropertyName))
							{
								string PropertyName = NextPropertyName;
								PropertyNameBuilder.Append('_').Append(++ValueIndex);
								NextPropertyName = PropertyNameBuilder.ToString();
								PropertyNameBuilder.Length = PropertyNameBuilderLength;

								CommonProjectFileContent.AppendLine($"    <{PropertyName}>$(AdditionalOptions) /Yu\"{Value}\"</{PropertyName}>");
								DirectoryToPchFile[DirectoryKVP.Key] = PropertyName;
							}
							else
							{
								DirectoryToPchFile[DirectoryKVP.Key] = UpdatedValues[Value];
							}
						}
					}

					if (CommonProjectFileContent.Length > 0)
					{
						VCProjectFileContent.AppendLine("  <PropertyGroup>");
						VCProjectFileContent.Append(CommonProjectFileContent);
						VCProjectFileContent.AppendLine("  </PropertyGroup>");
					}
				}

				VCFiltersFileContent.AppendLine("  <ItemGroup>");

				VCProjectFileContent.AppendLine("  <ItemGroup>");

				// Add all file directories to the filters file as solution filters
				HashSet<string> FilterDirectories = new HashSet<string>();

				AliasedFile[] LocalAliasedFilesArray = LocalAliasedFiles.ToArray();
				Array.Sort(LocalAliasedFilesArray);

				foreach (AliasedFile AliasedFile in LocalAliasedFilesArray)
				{
					// No need to add the root directory relative to the project (it would just be an empty string!)
					if (!String.IsNullOrWhiteSpace(AliasedFile.ProjectPath))
					{
						FiltersFileIsNeeded = EnsureFilterPathExists(AliasedFile.ProjectPath, VCFiltersFileContent, FilterDirectories);
					}

					// Get the filetype as represented to Visual Studio
					VCFileType FileType = GetVCFileType(AliasedFile.FileSystemPath);
					string VCFileType = GetVCFileTypeString(FileType);

					if (!CanVCFileTypeBeCompiled(FileType))
					{
						VCProjectFileContent.AppendLine("    <{0} Include=\"{1}\"/>", VCFileType, EscapeFileName(AliasedFile.FileSystemPath));
					}
					else
					{
						DirectoryReference Directory = AliasedFile.Location.Directory;

						// Find the include search paths
						if (TryGetBuildEnvironment(Directory, out BuildEnvironment? BuildEnvironment))
						{
							StringBuilder ClCompileInfo = new();
							if (DirectoryToIncludeSearchPaths.TryGetValue(Directory, out string? DirectoryToIncludeSearchPathValue) && !String.IsNullOrEmpty(DirectoryToIncludeSearchPathValue))
							{
								ClCompileInfo.AppendLine($"      <AdditionalIncludeDirectories>$({DirectoryToIncludeSearchPathValue})</AdditionalIncludeDirectories>");
							}

							if (DirectoryToForceIncludePaths.TryGetValue(Directory, out string? DirectoryToForceIncludePathValue) && !String.IsNullOrEmpty(DirectoryToForceIncludePathValue))
							{
								ClCompileInfo.AppendLine($"      <ForcedIncludeFiles>$({DirectoryToForceIncludePathValue})</ForcedIncludeFiles>");
							}

							if (!DirectoryToPchFile.IsEmpty)
							{
								string? PchHeaderFile = DirectoryToPchFile[Directory];
								if (PchHeaderFile != null && ProjectFileFormat >= VCProjectFileFormat.VisualStudio2022)
								{
									ClCompileInfo.AppendLine($"      <AdditionalOptions>$({DirectoryToPchFile[Directory]})</AdditionalOptions>");
								}
							}

							if (ClCompileInfo.Length == 0)
							{
								VCProjectFileContent.AppendLine("    <{0} Include=\"{1}\"/>", VCFileType, EscapeFileName(AliasedFile.FileSystemPath));
							}
							else
							{
								VCProjectFileContent.AppendLine("    <{0} Include=\"{1}\">", VCFileType, EscapeFileName(AliasedFile.FileSystemPath));
								VCProjectFileContent.Append(ClCompileInfo);
								VCProjectFileContent.AppendLine("    </{0}>", VCFileType);
							}
						}
						else
						{
							VCProjectFileContent.AppendLine("    <{0} Include=\"{1}\"/>", VCFileType, EscapeFileName(AliasedFile.FileSystemPath));
						}
					}

					if (!String.IsNullOrWhiteSpace(AliasedFile.ProjectPath))
					{
						VCFiltersFileContent.AppendLine("    <{0} Include=\"{1}\">", VCFileType, EscapeFileName(AliasedFile.FileSystemPath));
						VCFiltersFileContent.AppendLine("      <Filter>{0}</Filter>", Utils.CleanDirectorySeparators(EscapeFileName(AliasedFile.ProjectPath)));
						VCFiltersFileContent.AppendLine("    </{0}>", VCFileType);

						FiltersFileIsNeeded = true;
					}
					else
					{
						// No need to specify the root directory relative to the project (it would just be an empty string!)
						VCFiltersFileContent.AppendLine("    <{0} Include=\"{1}\" />", VCFileType, EscapeFileName(AliasedFile.FileSystemPath));
					}
				}

				VCProjectFileContent.AppendLine("  </ItemGroup>");

				VCFiltersFileContent.AppendLine("  </ItemGroup>");
			}

			// For Installed engine builds, include engine source in the source search paths if it exists. We never build it locally, so the debugger can't find it.
			if (Unreal.IsEngineInstalled() && !IsStubProject)
			{
				VCProjectFileContent.AppendLine("  <PropertyGroup>");
				VCProjectFileContent.Append("    <SourcePath>");
				foreach (string DirectoryName in Directory.EnumerateDirectories(Unreal.EngineSourceDirectory.FullName, "*", SearchOption.AllDirectories))
				{
					if (Directory.EnumerateFiles(DirectoryName, "*.cpp").Any())
					{
						VCProjectFileContent.Append(DirectoryName);
						VCProjectFileContent.Append(';');
					}
				}
				VCProjectFileContent.AppendLine("</SourcePath>");
				VCProjectFileContent.AppendLine("  </PropertyGroup>");
			}

			string OutputManifestString = "";
			if (!IsStubProject)
			{
				foreach (UnrealTargetPlatform Platform in ProjectPlatforms)
				{
					VSSettings VSSettings = new(Platform, UnrealTargetConfiguration.Development, ProjectFileFormat, null);
					PlatformProjectGenerator? ProjGenerator = PlatformProjectGenerators.GetPlatformProjectGenerator(Platform, true);
					if (ProjGenerator != null && ProjGenerator.HasVisualStudioSupport(VSSettings))
					{
						// @todo projectfiles: Serious hacks here because we are trying to emit one-time platform-specific sections that need information
						//    about a target type, but the project file may contain many types of targets!  Some of this logic will need to move into
						//    the per-target configuration writing code.
						TargetType HackTargetType = TargetType.Game;
						FileReference? HackTargetFilePath = null;
						foreach (ProjectConfigAndTargetCombination Combination in ProjectConfigAndTargetCombinations)
						{
							if (Combination.Platform == Platform &&
								Combination.ProjectTarget!.TargetRules != null &&
								Combination.ProjectTarget.TargetRules.Type == HackTargetType)
							{
								HackTargetFilePath = Combination.ProjectTarget.TargetFilePath;// ProjectConfigAndTargetCombinations[0].ProjectTarget.TargetFilePath;
								break;
							}
						}

						if (HackTargetFilePath != null)
						{
							OutputManifestString += ProjGenerator.GetVisualStudioOutputManifestSection(VSSettings, HackTargetType, HackTargetFilePath, ProjectFilePath);
						}
					}
				}
			}

			VCProjectFileContent.Append(OutputManifestString); // output manifest must come before the Cpp.targets file.
			VCProjectFileContent.AppendLine("  <ItemDefinitionGroup>");
			VCProjectFileContent.AppendLine("  </ItemDefinitionGroup>");
			VCProjectFileContent.AppendLine("  <Import Project=\"$(VCTargetsPath)\\Microsoft.Cpp.targets\" />");
			// Make sure CleanDependsOn is defined empty so the CppClean task isn't run when cleaning targets (use makefile instead)
			VCProjectFileContent.AppendLine("  <PropertyGroup>");
			VCProjectFileContent.AppendLine("    <CleanDependsOn> $(CleanDependsOn); </CleanDependsOn>");
			VCProjectFileContent.AppendLine("    <CppCleanDependsOn></CppCleanDependsOn>");
			VCProjectFileContent.AppendLine("  </PropertyGroup>");

			FileReference[] VSTestRunSettingsFiles = ProjectTargets.Select(Target => Target.TargetRules?.VSTestRunSettingsFile)
				.Where(File => File != null).Select(File => File!).Distinct().ToArray();
			if (VSTestRunSettingsFiles.Length == 1)
			{
				string RunSettingsRelativePath = VSTestRunSettingsFiles[0].MakeRelativeTo(ProjectFilePath.Directory);
				VCProjectFileContent.AppendLine("  <PropertyGroup>");
				VCProjectFileContent.AppendLine($"    <RunSettingsFilePath>$(ProjectDir){RunSettingsRelativePath}</RunSettingsFilePath>");
				VCProjectFileContent.AppendLine("  </PropertyGroup>");
			}
			else if (VSTestRunSettingsFiles.Length > 1)
			{
				string Files = String.Join(", ", VSTestRunSettingsFiles.Select(File => File!.FullName));
				Logger.LogWarning("Inconsistent VSTest run settings files for project '{ProjectFilePath}': {Files}", ProjectFilePath, Files);
			}

			if (!IsStubProject)
			{
				foreach (UnrealTargetPlatform Platform in ProjectPlatforms)
				{
					PlatformProjectGenerator? ProjGenerator = PlatformProjectGenerators.GetPlatformProjectGenerator(Platform, true);
					VSSettings VSSettings = new(Platform, UnrealTargetConfiguration.Development, ProjectFileFormat, null);
					if (ProjGenerator != null && ProjGenerator.HasVisualStudioSupport(VSSettings))
					{
						ProjGenerator.GetVisualStudioTargetOverrides(VSSettings, VCProjectFileContent);
					}
				}
			}
			VCProjectFileContent.AppendLine("  <ImportGroup Label=\"ExtensionTargets\">");
			VCProjectFileContent.AppendLine("  </ImportGroup>");

			WriteTargets(ProjectPlatforms, PlatformProjectGenerators, VCProjectFileContent);

			VCProjectFileContent.AppendLine("</Project>");

			VCFiltersFileContent.AppendLine("</Project>");

			if (bGenerateUserFileContent)
			{
				VCUserFileContent.AppendLine("</Project>");
			}

			// Save the project file
			if (bSuccess)
			{
				bSuccess = ProjectFileGenerator.WriteFileIfChanged(ProjectFilePath.FullName, VCProjectFileContent.ToString(), Logger);
			}

			// Save the filters file
			if (bSuccess)
			{
				// Create a path to the project file's filters file
				string VCFiltersFilePath = ProjectFilePath.FullName + ".filters";
				if (FiltersFileIsNeeded)
				{
					bSuccess = ProjectFileGenerator.WriteFileIfChanged(VCFiltersFilePath, VCFiltersFileContent.ToString(), Logger);
				}
				else
				{
					Logger.LogDebug("Deleting Visual C++ filters file which is no longer needed: {File}", VCFiltersFilePath);

					// Delete the filters file, if one exists.  We no longer need it
					try
					{
						File.Delete(VCFiltersFilePath);
					}
					catch (Exception)
					{
						Logger.LogInformation("Error deleting filters file (file may not be writable): {File}", VCFiltersFilePath);
					}
				}
			}

			// Save the user file, if required
			if (VCUserFileContent.Length > 0)
			{
				// Create a path to the project file's user file
				string VCUserFilePath = ProjectFilePath.FullName + ".user";
				// Never overwrite the existing user path as it will cause them to lose their settings
				if (ProjectFileGenerator.bForceUpdateAllFiles || File.Exists(VCUserFilePath) == false)
				{
					bSuccess = ProjectFileGenerator.WriteFileIfChanged(VCUserFilePath, VCUserFileContent.ToString(), Logger);
				}
				else
				{
					bSuccess = PatchVCUserFile(VCUserFilePath, VCUserFileContent.ToString(), VCUserFileSettings, Logger);
				}
			}

			return bSuccess;
		}

		private static bool PatchVCUserFile(string FileName, string NewFileContents, VisualStudioUserFileSettings UserFileSettings, ILogger Logger)
		{
			// Before we start, see if any relevant property is present in the new contents.
			if (UserFileSettings.PropertiesToPatch.Any(NewFileContents.Contains) == false)
			{
				return true;
			}

			XDocument CurrentContent;
			try
			{
				CurrentContent = XDocument.Load(FileName);
			}
			catch (Exception ex)
			{
				string Message = String.Format("Error while trying to parse XML file {0}.", FileName);
				Logger.LogError("{Message}", Message);
				throw new BuildException(ex, Message);
			}

			XDocument NewContent;
			try
			{
				NewContent = XDocument.Parse(NewFileContents);
			}
			catch (Exception ex)
			{
				string Message = String.Format("Error while trying to parse XML new data for file {0} ('{1}').", FileName, NewFileContents);
				Logger.LogError("{Message}", Message);
				throw new BuildException(ex, Message);
			}

			if (CurrentContent.Root == null || NewContent.Root == null)
			{
				return true;
			}

			XNamespace NS = NewContent.Root.Name.Namespace;
			// Skip patching if namespaces don't match (should never be the case?)
			if (NS != CurrentContent.Root.Name.Namespace)
			{
				return false;
			}

			// Create dictionaries with key == Condition of each <PropertyGroup> and value == <PropertyGroup> XElement itself for both current and new document.
			Dictionary<string, XElement> CurrentPropertyGroups = CurrentContent
				.Descendants(NS + "PropertyGroup")
				.Select(Element => (Attribute: Element.Attribute("Condition"), Element))
				.Where(Pair => Pair.Attribute != null)
				.ToDictionary(Pair => Pair.Attribute!.Value, Pair => Pair.Element);

			Dictionary<string, XElement> NewPropertyGroups = NewContent
				.Descendants(NS + "PropertyGroup")
				.Select(Element => (Attribute: Element.Attribute("Condition"), Element))
				.Where(Pair => Pair.Attribute != null)
				.ToDictionary(Pair => Pair.Attribute!.Value, Pair => Pair.Element);

			bool bNeedsSaving = false;

			// Go over every <PropertyGroup> in new document.
			foreach ((string Attribute, XElement NewPropertyGroup) in NewPropertyGroups)
			{
				// Check if new document <PropertyGroup> contains any properties that we need to patch in the current document.
				if (NewPropertyGroup.Elements().Any(Element => UserFileSettings.PropertiesToPatch.Contains(Element.Name.LocalName)) == false)
				{
					continue;
				}

				// Check if <PropertyGroup> with same "Condition" attribute already exist in the current document.
				// If yes, update required properties in existing <PropertyGroup> but preserve any other property in current document order.
				if (CurrentPropertyGroups.TryGetValue(Attribute, out XElement? CurrentPropertyGroup))
				{
					// Preserve values from current document for relevant properties by patching corresponding properties in new document. 
					IEnumerable<XElement> ElementsToPreserveValuesFrom = CurrentPropertyGroup
						.Elements()
						.Where(Element => UserFileSettings.PropertiesToPatchOrderButPreserveValue.Contains(Element.Name.LocalName));
					foreach (XElement CurrentElement in ElementsToPreserveValuesFrom)
					{
						XElement? NewElement = NewPropertyGroup.Element(CurrentElement.Name);
						if (NewElement != null)
						{
							NewElement.Value = CurrentElement.Value;
						}
					}

					XElement[] CurrentPropertyGroupElementsForPatch = CurrentPropertyGroup
						.Elements()
						.Where(Element => UserFileSettings.PropertiesToPatch.Contains(Element.Name.LocalName))
						.ToArray();

					XElement[] NewPropertyGroupElementsForPatch = NewPropertyGroup
						.Elements()
						.Where(Element => UserFileSettings.PropertiesToPatch.Contains(Element.Name.LocalName))
						.ToArray();

					// Check if all properties are already has the correct value and order, and skip patching if so.
					if (CurrentPropertyGroupElementsForPatch.Length == NewPropertyGroupElementsForPatch.Length &&
						!CurrentPropertyGroupElementsForPatch.Where((CurrentProperty, i) => CurrentProperty.Name != NewPropertyGroupElementsForPatch[i].Name || CurrentProperty.Value != NewPropertyGroupElementsForPatch[i].Value).Any())
					{
						continue;
					}

					// Remove all existing properties that we need to update from existing document, this simplifies logic of adding them,
					// because we need to update the values and ensure the order of properties is as declared in the new document,
					// because the order of properties is important for MSBuild when one property uses a value of another property.
					CurrentPropertyGroupElementsForPatch.Remove();

					// Add new properties to existing <PropertyGroup> in the order as they are defined in new document.
					CurrentPropertyGroup.Add(NewPropertyGroupElementsForPatch);

					bNeedsSaving = true;
				}
				else // Otherwise add new <PropertyGroup> as-is to the end of existing document.
				{
					CurrentContent.Root.Add(NewPropertyGroup);
					bNeedsSaving = true;
				}
			}

			if (bNeedsSaving)
			{
				try
				{
					CurrentContent.Save(FileName);
					Logger.LogDebug("Patching {Path}.", Path.GetFileName(FileName));
				}
				catch (Exception ex)
				{
					string Message = String.Format("Error while trying to write file {0}. The file is probably read-only.", FileName);
					Logger.LogError("{Message}", Message);
					throw new BuildException(ex, Message);
				}
			}
			else
			{
				Logger.LogDebug("{Path} doesn't require patching.", Path.GetFileName(FileName));
			}

			return true;
		}

		private class ProjectConfigurationForGenerator : ProjectBuildConfiguration
		{
			public override string ConfigurationName => Combination.ProjectConfigurationName;
			public override string BuildCommand => $"{EscapePath(NormalizeProjectPath(CommandBuilder.BuildScript))} {CommandBuilder.GetBuildArguments()}";

			private readonly ProjectConfigAndTargetCombination Combination;
			private readonly BuildCommandBuilder CommandBuilder;

			public ProjectConfigurationForGenerator(ProjectConfigAndTargetCombination InCombination, BuildCommandBuilder InCommandBuilder)
			{
				Combination = InCombination;
				CommandBuilder = InCommandBuilder;
			}
		}

		/// <summary>
		/// Write additional Target elements if needed by per-platform generators.
		/// </summary>
		/// <param name="ProjectPlatforms"></param>
		/// <param name="PlatformProjectGenerators"></param>
		/// <param name="VCProjectFileContent"></param>
		private void WriteTargets(List<UnrealTargetPlatform> ProjectPlatforms, PlatformProjectGeneratorCollection PlatformProjectGenerators, StringBuilder VCProjectFileContent)
		{
			foreach (UnrealTargetPlatform Platform in ProjectPlatforms)
			{
				PlatformProjectGenerator? ProjGenerator = PlatformProjectGenerators.GetPlatformProjectGenerator(Platform, true);
				if (ProjGenerator != null && ProjGenerator.HasVisualStudioTargets(Platform))
				{
					IEnumerable<ProjectConfigAndTargetCombination> PlatformCombinations = ProjectConfigAndTargetCombinations!.Where(Combination => Combination.Platform == Platform);

					if (PlatformCombinations.Any())
					{
						List<ProjectBuildConfiguration> PlatformConfigurations = PlatformCombinations.Select(Combination =>
							{
								string UProjectPath = "";
								if (IsForeignProject)
								{
									UProjectPath = String.Format("\"{0}\"", InsertPathVariables(Combination.ProjectTarget!.UnrealProjectFilePath!));
								}

								BuildCommandBuilder Builder = CreateArgumentsBuilder(Combination, UProjectPath, ProjGenerator);

								return new ProjectConfigurationForGenerator(Combination, Builder) as ProjectBuildConfiguration;
							}
						).ToList();

						ProjGenerator.GetVisualStudioTargetsString(Platform, PlatformConfigurations, VCProjectFileContent);
					}
				}
			}
		}

		private static bool EnsureFilterPathExists(string FilterRelativeSourceDirectory, StringBuilder VCFiltersFileContent, HashSet<string> FilterDirectories)
		{
			// We only want each directory to appear once in the filters file
			string PathRemaining = Utils.CleanDirectorySeparators(FilterRelativeSourceDirectory);
			bool FiltersFileIsNeeded = false;
			if (!FilterDirectories.Contains(PathRemaining))
			{
				// Make sure all subdirectories leading up to this directory each have their own filter, too!
				List<string> AllDirectoriesInPath = new List<string>();
				string PathSoFar = "";
				for (; ; )
				{
					if (PathRemaining.Length > 0)
					{
						int SlashIndex = PathRemaining.IndexOf(Path.DirectorySeparatorChar, StringComparison.Ordinal);
						string SplitDirectory;
						if (SlashIndex != -1)
						{
							SplitDirectory = PathRemaining.Substring(0, SlashIndex);
							PathRemaining = PathRemaining.Substring(SplitDirectory.Length + 1);
						}
						else
						{
							SplitDirectory = PathRemaining;
							PathRemaining = "";
						}
						if (!String.IsNullOrEmpty(PathSoFar))
						{
							PathSoFar += Path.DirectorySeparatorChar;
						}
						PathSoFar += SplitDirectory;

						AllDirectoriesInPath.Add(PathSoFar);
					}
					else
					{
						break;
					}
				}

				foreach (string LeadingDirectory in AllDirectoriesInPath)
				{
					if (FilterDirectories.Add(LeadingDirectory))
					{
						// Generate a unique stable GUID for this folder by hashing the LeadingDirectory string
						// NOTE: When saving generated project files, we ignore differences in GUIDs if every other part of the file
						//       matches identically with the pre-existing file
						string FilterGUID = VCProjectFileGenerator.MakeMd5Guid(LeadingDirectory).ToString("B").ToUpperInvariant();

						VCFiltersFileContent.AppendLine("    <Filter Include=\"{0}\">", EscapeFileName(LeadingDirectory));
						VCFiltersFileContent.AppendLine("      <UniqueIdentifier>{0}</UniqueIdentifier>", FilterGUID);
						VCFiltersFileContent.AppendLine("    </Filter>");

						FiltersFileIsNeeded = true;
					}
				}
			}

			return FiltersFileIsNeeded;
		}

		private enum VCFileType
		{
			None,
			CCode,
			Header,
			Inline,
			Resource,
			Manifest,
		};

		/// <summary>
		/// Returns the VCFileType string based on the VCFileType enum.
		/// </summary>
		/// <param name="FileType"></param>
		/// <returns>Name of the element in MSBuild project file for this file type</returns>
		private string GetVCFileTypeString(VCFileType FileType)
		{
			switch (FileType)
			{
				default:
				case VCFileType.None:
					return "None";
				case VCFileType.CCode:
					return "ClCompile";
				case VCFileType.Header:
					return Settings.bHeadersAsClCompile ? "ClCompile" : "ClInclude";
				case VCFileType.Inline:
					return "ClInclude";
				case VCFileType.Resource:
					return "ResourceCompile";
				case VCFileType.Manifest:
					return "Manifest";
			}
		}

		private static bool CanVCFileTypeBeIncluded(VCFileType FileType)
		{
			return FileType == VCFileType.Header || FileType == VCFileType.Inline;
		}

		private bool CanVCFileTypeBeCompiled(VCFileType FileType)
		{
			return FileType == VCFileType.CCode || (FileType == VCFileType.Header && Settings.bHeadersAsClCompile);
		}

		/// <summary>
		/// Returns the VCFileType enum value based on the file path.
		/// </summary>
		/// <param name="Path">The path of the file to return type for.</param>
		/// <returns>VCFileType enum value for this file.</returns>
		private static VCFileType GetVCFileType(string Path)
		{
			// What type of file is this?
			if (Path.EndsWith(".h", StringComparison.InvariantCultureIgnoreCase))
			{
				return VCFileType.Header;
			}
			if (Path.EndsWith(".inl", StringComparison.InvariantCultureIgnoreCase))
			{
				return VCFileType.Inline;
			}
			if (Path.EndsWith(".cpp", StringComparison.InvariantCultureIgnoreCase))
			{
				return VCFileType.CCode;
			}
			if (Path.EndsWith(".rc", StringComparison.InvariantCultureIgnoreCase))
			{
				return VCFileType.Resource;
			}
			if (Path.EndsWith(".manifest", StringComparison.InvariantCultureIgnoreCase))
			{
				return VCFileType.Manifest;
			}

			return VCFileType.None;
		}

		// Helper class to generate NMake build commands and arguments
		public class BuildCommandBuilder
		{
			public bool bEditorDependsOnShaderCompileWorker = true;
			public bool bBuildLiveCodingConsole;
			public bool bAddFastPDBToProjects;

			public bool bIsForeignProject;
			public bool bUsePrecompiled;
			public bool bIsFromMSBuild;

			public PlatformProjectGenerator? ProjectGenerator;

			public FileReference BuildScript { get; }
			public FileReference RebuildScript { get; }
			public FileReference CleanScript { get; }

			private readonly string? BuildToolOverride;

			private readonly string UProjectPath;

			private readonly VSSettings VSSettings;
			private readonly ProjectTarget ProjectTarget;

			public BuildCommandBuilder(VSSettings InVSSettings, ProjectTarget InProjectTarget, string InUProjectPath, string? InBuildToolOverride = null)
			{
				VSSettings = InVSSettings;
				ProjectTarget = InProjectTarget;
				UProjectPath = InUProjectPath;
				BuildToolOverride = InBuildToolOverride;

				DirectoryReference BatchFilesDirectory = DirectoryReference.Combine(Unreal.EngineDirectory, "Build", "BatchFiles");
				BuildScript = FileReference.Combine(BatchFilesDirectory, "Build.bat");
				RebuildScript = FileReference.Combine(BatchFilesDirectory, "Rebuild.bat");
				CleanScript = FileReference.Combine(BatchFilesDirectory, "Clean.bat");
			}

			public string GetBuildArguments(bool bAllowExtraTargets = true)
			{
				TargetRules TargetRulesObject = ProjectTarget.TargetRules!;
				string TargetName = ProjectTarget.TargetFilePath.GetFileNameWithoutAnyExtensions();

				StringBuilder BuildArguments = new StringBuilder();

				BuildArguments.AppendFormat("{0} {1} {2}", TargetName, VSSettings.Platform.ToString(), VSSettings.Configuration.ToString());
				if (UProjectPath.Length > 0)
				{
					BuildArguments.AppendFormat(" -Project={0}", UProjectPath);
				}

				List<string> ExtraTargets = new List<string>();
				if (!bUsePrecompiled && bAllowExtraTargets)
				{
					if (TargetRulesObject.Type == TargetType.Editor && bEditorDependsOnShaderCompileWorker && !Unreal.IsEngineInstalled())
					{
						string ProjParam = UProjectPath.Length > 0 ? $" -Project=\\\"{UProjectPath.Trim('"')}\\\"" : "";
						ExtraTargets.Add($"ShaderCompileWorker Win64 Development{ProjParam}");
					}
					if (TargetRulesObject.bWithLiveCoding && bBuildLiveCodingConsole && !Unreal.IsEngineInstalled() && TargetRulesObject.Name != "LiveCodingConsole")
					{
						ExtraTargets.Add(TargetRulesObject.bUseDebugLiveCodingConsole ? "LiveCodingConsole Win64 Debug" : "LiveCodingConsole Win64 Development");
					}
				}

				if (ExtraTargets.Count > 0)
				{
					BuildArguments.Replace("\"", "\\\"");
					BuildArguments.Insert(0, "-Target=\"");
					BuildArguments.Append('"');
					foreach (string ExtraTarget in ExtraTargets)
					{
						BuildArguments.AppendFormat(" -Target=\"{0} -Quiet\"", ExtraTarget);
					}
				}

				if (bUsePrecompiled)
				{
					BuildArguments.Append(" -UsePrecompiled");
				}

				// Always wait for the mutex between UBT invocations, so that building the whole solution doesn't fail.
				BuildArguments.Append(" -WaitMutex");

				if (bIsFromMSBuild)
				{
					BuildArguments.Append(" -FromMsBuild");
				}

				if (bAddFastPDBToProjects)
				{
					// Pass Fast PDB option to make use of Visual Studio's /DEBUG:FASTLINK option
					BuildArguments.Append(" -FastPDB");
				}

				if (BuildToolOverride != null)
				{
					BuildArguments.AppendFormat(" {0}", BuildToolOverride);
				}

				if (ProjectGenerator != null)
				{
					BuildArguments.Append(ProjectGenerator.GetExtraBuildArguments(VSSettings));
				}

				if (VSSettings.Architecture != null)
				{
					BuildArguments.AppendFormat(" -architecture={0}", VSSettings.Architecture);
				}

				return BuildArguments.ToString();
			}
		}

		private BuildCommandBuilder CreateArgumentsBuilder(ProjectConfigAndTargetCombination Combination, string UProjectPath, PlatformProjectGenerator? ProjGenerator)
		{
			BuildCommandBuilder Builder = new BuildCommandBuilder(new VSSettings(Combination.Platform!.Value, Combination.Configuration, ProjectFileFormat, Combination.Architecture),
				Combination.ProjectTarget!, UProjectPath, BuildToolOverride)
			{
				ProjectGenerator = ProjGenerator,
				bEditorDependsOnShaderCompileWorker = Settings.bEditorDependsOnShaderCompileWorker,
				bBuildLiveCodingConsole = Settings.bBuildLiveCodingConsole,
				bAddFastPDBToProjects = Settings.bAddFastPDBToProjects,
				bIsForeignProject = IsForeignProject,
				bUsePrecompiled = bUsePrecompiled,
				// Always include a flag to format log messages for MSBuild
				bIsFromMSBuild = true
			};

			return Builder;
		}

		private void WriteConfiguration(string ProjectName, ProjectConfigAndTargetCombination Combination, StringBuilder VCProjectFileContent, PlatformProjectGeneratorCollection PlatformProjectGenerators, StringBuilder? VCUserFileContent, VisualStudioUserFileSettings? VCUserFileSettings)
		{
			UnrealTargetConfiguration Configuration = Combination.Configuration;

			PlatformProjectGenerator? ProjGenerator = Combination.Platform != null ? PlatformProjectGenerators.GetPlatformProjectGenerator(Combination.Platform.Value, true) : null;

			FileReference? UProjectPathNullable = Combination.ProjectTarget?.UnrealProjectFilePath;
			string UProjectPath = "";
			if (UProjectPathNullable != null)
			{
				UProjectPath = String.Format("\"{0}\"", InsertPathVariables(UProjectPathNullable));
			}

			string ConditionString = "Condition=\"'$(Configuration)|$(Platform)'=='" + Combination.ProjectConfigurationAndPlatformName + "'\"";

			{
				// Add custom import info
				if (ProjGenerator != null)
				{
					StringBuilder CustomImportGroupInfo = new StringBuilder();
					ProjGenerator.GetVisualStudioImportGroupProperties(new(Combination.Platform!.Value, Configuration, ProjectFileFormat, null), CustomImportGroupInfo);
					if (CustomImportGroupInfo.Length != 0)
					{
						VCProjectFileContent.AppendLine("  <ImportGroup {0} Label=\"PropertySheets\">", ConditionString);
						VCProjectFileContent.Append(CustomImportGroupInfo);
						VCProjectFileContent.AppendLine("  </ImportGroup>");
					}
				}

				DirectoryReference ProjectDirectory = ProjectFilePath.Directory;
				FileReference? NMakePath = null;

				if (IsStubProject)
				{
					VCProjectFileContent.AppendLine("  <PropertyGroup {0}>", ConditionString);
					VCProjectFileContent.AppendLine("    <NMakeBuildCommandLine>@rem Nothing to do.</NMakeBuildCommandLine>");
					VCProjectFileContent.AppendLine("    <NMakeReBuildCommandLine>@rem Nothing to do.</NMakeReBuildCommandLine>");
					VCProjectFileContent.AppendLine("    <NMakeCleanCommandLine>@rem Nothing to do.</NMakeCleanCommandLine>");
					if (ProjectFileGenerator.bVisualStudioLinux)
					{
						VCProjectFileContent.AppendLine("    <BuildCommandLine>$(NMakeBuildCommandLine)</BuildCommandLine>");
						VCProjectFileContent.AppendLine("    <ReBuildCommandLine>$(NMakeReBuildCommandLine)</ReBuildCommandLine>");
						VCProjectFileContent.AppendLine("    <CleanCommandLine>$(NMakeCleanCommandLine)</CleanCommandLine>");
						VCProjectFileContent.AppendLine("    <LocalBuildOutputs />");
						VCProjectFileContent.AppendLine("    <SourcesToCopyRemotelyOverride />");
					}
					VCProjectFileContent.AppendLine("  </PropertyGroup>");
				}
				else if (Unreal.IsEngineInstalled() && Combination.ProjectTarget != null && Combination.ProjectTarget.TargetRules != null &&
					(Combination.Platform == null || !Combination.ProjectTarget.SupportedPlatforms.Contains(Combination.Platform.Value)))
				{
					string TargetName = Combination.ProjectTarget.TargetFilePath.GetFileNameWithoutAnyExtensions();
					string ValidPlatforms = String.Join(", ", Combination.ProjectTarget.SupportedPlatforms.Select(x => x.ToString()));

					VCProjectFileContent.AppendLine("  <PropertyGroup {0}>", ConditionString);
					VCProjectFileContent.AppendLine("    <NMakeBuildCommandLine>@echo {0} is not a supported platform for {1}. Valid platforms are {2}.</NMakeBuildCommandLine>", Combination.Platform!, TargetName, ValidPlatforms);
					VCProjectFileContent.AppendLine("    <NMakeReBuildCommandLine>@echo {0} is not a supported platform for {1}. Valid platforms are {2}.</NMakeReBuildCommandLine>", Combination.Platform!, TargetName, ValidPlatforms);
					VCProjectFileContent.AppendLine("    <NMakeCleanCommandLine>@echo {0} is not a supported platform for {1}. Valid platforms are {2}.</NMakeCleanCommandLine>", Combination.Platform!, TargetName, ValidPlatforms);
					VCProjectFileContent.AppendLine("    <NMakeOutput/>");
					if (ProjectFileGenerator.bVisualStudioLinux)
					{
						VCProjectFileContent.AppendLine("    <BuildCommandLine>$(NMakeBuildCommandLine)</BuildCommandLine>");
						VCProjectFileContent.AppendLine("    <ReBuildCommandLine>$(NMakeReBuildCommandLine)</ReBuildCommandLine>");
						VCProjectFileContent.AppendLine("    <CleanCommandLine>$(NMakeCleanCommandLine)</CleanCommandLine>");
						VCProjectFileContent.AppendLine("    <LocalBuildOutputs />");
						VCProjectFileContent.AppendLine("    <SourcesToCopyRemotelyOverride />");
					}
					VCProjectFileContent.AppendLine("  </PropertyGroup>");
				}
				else
				{
					UnrealTargetPlatform Platform = Combination.Platform!.Value;
					TargetRules TargetRulesObject;
					try
					{
						if (Combination.Architecture != null)
						{
							TargetRulesObject = Combination.ProjectTarget!.CreateRulesDelegateWithArch(Platform, Configuration, (UnrealArch)Combination.Architecture);
						}
						else
						{
							TargetRulesObject = Combination.ProjectTarget!.CreateRulesDelegate(Platform, Configuration);
						}
					}
					catch (BuildException)
					{
						TargetRulesObject = Combination.ProjectTarget!.TargetRules!;
					}
					FileReference TargetFilePath = Combination.ProjectTarget.TargetFilePath;
					string UBTPlatformName = Platform.ToString();
					string UBTConfigurationName = Configuration.ToString();
					VSSettings VSSettings = new(Platform, Configuration, ProjectFileFormat, null);
					if (Combination.Architecture != null)
					{
						VSSettings.Architecture = Combination.Architecture;
					}

					// Setup output path
					UEBuildPlatform BuildPlatform = UEBuildPlatform.GetBuildPlatform(Platform);

					// Figure out if this is a monolithic build
					bool bShouldCompileMonolithic = BuildPlatform.ShouldCompileMonolithicBinary(Platform) | (TargetRulesObject.LinkType == TargetLinkType.Monolithic);

					// Get the .uproject directory
					DirectoryReference? UProjectDirectory = DirectoryReference.FromFile(Combination.ProjectTarget.UnrealProjectFilePath);

					// Get the output directory
					DirectoryReference RootOutputDirectory;
					if (UProjectDirectory != null && (bShouldCompileMonolithic || TargetRulesObject.BuildEnvironment == TargetBuildEnvironment.Unique) && TargetRulesObject.File!.IsUnderDirectory(UProjectDirectory))
					{
						RootOutputDirectory = UEBuildTarget.GetOutputDirectoryForExecutable(UProjectDirectory, TargetRulesObject.File!);
					}
					else
					{
						RootOutputDirectory = UEBuildTarget.GetOutputDirectoryForExecutable(Unreal.EngineDirectory, TargetRulesObject.File!);
					}

					if (TargetRulesObject.OutputFile != null)
					{
						NMakePath = FileReference.Combine(RootOutputDirectory, TargetRulesObject.OutputFile);
					}
					else
					{
						// Get the output directory
						DirectoryReference OutputDirectory = DirectoryReference.Combine(RootOutputDirectory, "Binaries", UBTPlatformName);

						if (!String.IsNullOrEmpty(TargetRulesObject.ExeBinariesSubFolder))
						{
							OutputDirectory = DirectoryReference.Combine(OutputDirectory, TargetRulesObject.ExeBinariesSubFolder);
						}

						// Get the executable name (minus any platform or config suffixes)
						string BaseExeName = TargetRulesObject.Name;
						if (!bShouldCompileMonolithic && TargetRulesObject.Type != TargetType.Program && TargetRulesObject.BuildEnvironment != TargetBuildEnvironment.Unique)
						{
							BaseExeName = "Unreal" + TargetRulesObject.Type.ToString();
						}

						// Make the output file path
						NMakePath = FileReference.Combine(OutputDirectory, BaseExeName);
						if (Configuration != TargetRulesObject.UndecoratedConfiguration)
						{
							NMakePath += TargetRulesObject.DecoratedSeparator + UBTPlatformName + TargetRulesObject.DecoratedSeparator + UBTConfigurationName;
						}
						if (UnrealArchitectureConfig.ForPlatform(Platform).RequiresArchitectureFilenames(TargetRulesObject.Architectures))
						{
							NMakePath += TargetRulesObject.Architecture.ToString();
						}
						else if (Combination.Architecture != null) // support the case where the project/platform combination explicitly sets an architecture(e.g. Win64)
						{
							UnrealArch Architecture = (UnrealArch)Combination.Architecture;
							if (UnrealArchitectureConfig.ForPlatform(Platform).RequiresArchitectureFilenames(new UnrealArchitectures(Architecture)))
							{
								NMakePath += Architecture.ToString();
							}
						}
						NMakePath += BuildPlatform.GetBinaryExtension(UEBuildBinaryType.Executable);
					}

					VCProjectFileContent.AppendLine("  <PropertyGroup {0}>", ConditionString);

					if (ProjGenerator != null)
					{
						StringBuilder PathsStringBuilder = new StringBuilder();
						ProjGenerator.GetVisualStudioPathsEntries(VSSettings, TargetRulesObject.Type, TargetFilePath, ProjectFilePath, NMakePath, PathsStringBuilder);
						VCProjectFileContent.Append(PathsStringBuilder);
					}

					// This is the standard UE based project NMake build line:
					//	..\..\Build\BatchFiles\Build.bat <TARGETNAME> <PLATFORM> <CONFIGURATION>
					//	ie ..\..\Build\BatchFiles\Build.bat BlankProgram Win64 Debug

					BuildCommandBuilder Builder = CreateArgumentsBuilder(Combination, UProjectPath, ProjGenerator);
					string BuildArguments = Builder.GetBuildArguments();

					// NMake Build command line
					if (TargetRulesObject.IsTestTarget)
					{
						VCProjectFileContent.AppendLine("    <NMakeBuildCommandLine>$(BuildBatchScript) {0} -Mode=Test</NMakeBuildCommandLine>", BuildArguments);
						VCProjectFileContent.AppendLine("    <NMakeReBuildCommandLine>$(BuildBatchScript) {0} -Mode=Test -RebuildTests</NMakeReBuildCommandLine>", BuildArguments);
						VCProjectFileContent.AppendLine("    <NMakeCleanCommandLine>$(BuildBatchScript) {0} -Mode=Test -CleanTests</NMakeCleanCommandLine>", BuildArguments);
					}
					else
					{
						VCProjectFileContent.AppendLine("    <NMakeBuildCommandLine>$(BuildBatchScript) {0}</NMakeBuildCommandLine>", BuildArguments);
						VCProjectFileContent.AppendLine("    <NMakeReBuildCommandLine>$(RebuildBatchScript) {0}</NMakeReBuildCommandLine>", BuildArguments);
						VCProjectFileContent.AppendLine("    <NMakeCleanCommandLine>$(CleanBatchScript) {0}</NMakeCleanCommandLine>", BuildArguments);
					}
					if (TargetRulesObject.bBuildConsoleAppOnly)
					{
						VCProjectFileContent.AppendLine("    <NMakeOutput>{0}</NMakeOutput>", NormalizeProjectPath(UEBuildBinary.GetAdditionalConsoleAppPath(new FileReference(NMakePath.FullName))));
					} 
					else 
					{
						VCProjectFileContent.AppendLine("    <NMakeOutput>{0}</NMakeOutput>", NormalizeProjectPath(NMakePath));
					}
					if (ProjectFileGenerator.bVisualStudioLinux)
					{
						VCProjectFileContent.AppendLine("    <BuildCommandLine>$(NMakeBuildCommandLine)</BuildCommandLine>");
						VCProjectFileContent.AppendLine("    <ReBuildCommandLine>$(NMakeReBuildCommandLine)</ReBuildCommandLine>");
						VCProjectFileContent.AppendLine("    <CleanCommandLine>$(NMakeCleanCommandLine)</CleanCommandLine>");
						if (TargetRulesObject.Platform.IsInGroup(UnrealPlatformGroup.Linux))
						{
							VCProjectFileContent.AppendLine("    <LocalBuildOutputs>{0};{1};{2}</LocalBuildOutputs>",
								NormalizeProjectPath(NMakePath), NormalizeProjectPath(NMakePath.ChangeExtension(".debug")), NormalizeProjectPath(NMakePath.ChangeExtension(".sym")));
							VCProjectFileContent.AppendLine("    <PreLaunchCommand>chmod +x $(RemoteDeployDir)/{0}</PreLaunchCommand>", NMakePath.GetFileName());
							VCProjectFileContent.AppendLine("    <RemoteDebuggerCommand>$(RemoteDeployDir)/{0}</RemoteDebuggerCommand>", NMakePath.GetFileName());
						}
					}

					if (TargetRulesObject.Type == TargetType.Game || TargetRulesObject.Type == TargetType.Client || TargetRulesObject.Type == TargetType.Server)
					{
						// Allow platforms to add any special properties they require
						PlatformProjectGenerators.GenerateGamePlatformSpecificProperties(Platform, Configuration, TargetRulesObject.Type, VCProjectFileContent, RootOutputDirectory, TargetFilePath);
					}
					else if (TargetRulesObject.Type == TargetType.Editor && TargetRulesObject.ProjectFile != null && TargetRulesObject.LinkType == TargetLinkType.Modular && TargetRulesObject.BuildEnvironment == TargetBuildEnvironment.Unique)
					{
						VCProjectFileContent.AppendLine("    <LocalDebuggerCommandArguments>{0} -Project=\"{1}\"</LocalDebuggerCommandArguments>", TargetRulesObject.Name, NormalizeProjectPath(TargetRulesObject.ProjectFile));
					}

					VCProjectFileContent.AppendLine("  </PropertyGroup>");

					if (ProjGenerator != null)
					{
						VCProjectFileContent.Append(ProjGenerator.GetVisualStudioLayoutDirSection(VSSettings, ConditionString, Combination.ProjectTarget.TargetRules!.Type, Combination.ProjectTarget.TargetFilePath, ProjectFilePath, NMakePath));
					}

					string BuildArgumentsForFileCompiles = Builder.GetBuildArguments(bAllowExtraTargets: false);
					VCProjectFileContent.AppendLine("  <ItemDefinitionGroup {0}>", ConditionString);
					VCProjectFileContent.AppendLine("    <NMakeCompile>");
					VCProjectFileContent.AppendLine($"      <NMakeCompileFileCommandLine>$(BuildBatchScript) {BuildArgumentsForFileCompiles} -WorkingDir=$(MSBuildProjectDirectory) -Files=$(SelectedFiles)</NMakeCompileFileCommandLine>");
					VCProjectFileContent.AppendLine("    </NMakeCompile>");
					if (TargetRulesObject.bIsBuildingConsoleApplication)
					{
						// Let Visual Studio keep the console window open when debugging stops. Ignored by the build.
						VCProjectFileContent.AppendLine("    <Link>");
						VCProjectFileContent.AppendLine("      <SubSystem>Console</SubSystem>");
						VCProjectFileContent.AppendLine("    </Link>");
					}
					if (ProjectFileGenerator.bVisualStudioLinux && TargetRulesObject.Platform.IsInGroup(UnrealPlatformGroup.Linux))
					{
						VCProjectFileContent.AppendLine("    <PostBuildEvent>");
						VCProjectFileContent.AppendLine("      <AdditionalSourcesToCopyMapping>{0}:=$(RemoteDeployDir)/{1};{2}:=$(RemoteDeployDir)/{3};{4}:=$(RemoteDeployDir)/{5}</AdditionalSourcesToCopyMapping>",
							NormalizeProjectPath(NMakePath), NMakePath.GetFileName(),
							NormalizeProjectPath(NMakePath.ChangeExtension(".debug")), NMakePath.ChangeExtension(".debug").GetFileName(),
							NormalizeProjectPath(NMakePath.ChangeExtension(".sym")), NMakePath.ChangeExtension(".sym").GetFileName());
						VCProjectFileContent.AppendLine("    </PostBuildEvent>");
					}
					VCProjectFileContent.AppendLine("  </ItemDefinitionGroup>");
				}

				if (VCUserFileContent != null && VCUserFileSettings != null && Combination.ProjectTarget != null)
				{
					TargetRules TargetRulesObject = Combination.ProjectTarget.TargetRules!;

					if (ProjGenerator != null)
					{
						string? ForeignUProjectPath = (IsForeignProject && !String.IsNullOrEmpty(UProjectPath)) ? UProjectPath : null;
						VCUserFileContent.Append(ProjGenerator.GetVisualStudioUserFileStrings(VCUserFileSettings, new(Combination.Platform!.Value, Configuration, ProjectFileFormat, Combination.Architecture), ConditionString, TargetRulesObject, Combination.ProjectTarget.TargetFilePath, ProjectFilePath, NMakePath, ProjectName, UProjectPath, ForeignUProjectPath));
					}
				}
			}
		}
	}
}
