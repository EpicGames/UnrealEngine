// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Runtime.Versioning;
using System.Security.Cryptography;
using System.Text;
using System.Xml;
using System.Xml.Linq;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;
using UnrealBuildTool.ProjectFiles.VisualStudio;

namespace UnrealBuildTool
{
	enum VCProjectFileFormat
	{
		Default,          // Default to the best installed version, but allow SDKs to override
		VisualStudio2022,
		VisualStudio2026,
	}

	class VCProjectFileSettings
	{
		/// <summary>
		/// The version of Visual Studio to generate project files for.
		/// </summary>
		[XmlConfigFile(Category = "VCProjectFileGenerator", Name = "Version")]
		public VCProjectFileFormat ProjectFileFormat = VCProjectFileFormat.Default;

		/// <summary>
		/// Puts the most common include paths in the IncludePath property in the MSBuild project. This significantly reduces Visual Studio
		/// memory usage (measured 1.1GB -> 500mb), but seems to be causing issues with Visual Assist. Value here specifies maximum length
		/// of the include path list in KB.
		/// </summary>
		[XmlConfigFile(Category = "VCProjectFileGenerator")]
		public int MaxSharedIncludePaths = 24 * 1024;

		/// <summary>
		/// Semi-colon separated list of paths that should not be added to the projects include paths. Useful for omitting third-party headers
		/// (e.g ThirdParty/WebRTC) from intellisense suggestions and reducing memory footprints.
		/// </summary>
		[XmlConfigFile(Category = "VCProjectFileGenerator")]
		public string ExcludedIncludePaths = "";

		/// <summary>
		/// Semi-colon separated list of paths that should not be added to the projects. Useful for omitting third-party files
		/// (e.g ThirdParty/WebRTC) from intellisense suggestions and reducing memory footprints.
		/// </summary>
		[XmlConfigFile(Category = "VCProjectFileGenerator")]
		public string ExcludedFilePaths = "";

		/// <summary>
		/// Whether to write a solution option (suo) file for the sln.
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bWriteSolutionOptionFile = true;

		/// <summary>
		/// Whether to write a .vsconfig file next to the sln to suggest components to install.
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bVsConfigFile = true;

		/// <summary>
		/// Forces UBT to be built in debug configuration, regardless of the solution configuration
		/// </summary>
		[XmlConfigFile(Category = "VCProjectFileGenerator")]
		public bool bBuildUBTInDebug = false;

		/// <summary>
		/// Whether to add the -FastPDB option to build command lines by default.
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bAddFastPDBToProjects = false;

		/// <summary>
		/// Whether to generate per-file intellisense data.
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bUsePerFileIntellisense = true;

		/// <summary>
		/// Whether to treat headers as ClCompile. May cause intellisense issues but allows the VS Compile command to build headers.
		/// </summary>
		[XmlConfigFile(Category = "VCProjectFileGenerator")]
		public bool bHeadersAsClCompile = false;

		/// <summary>
		/// Whether to include a dependency on ShaderCompileWorker when generating project files for the editor.
		/// </summary>
		[XmlConfigFile(Category = "BuildConfiguration")]
		public bool bEditorDependsOnShaderCompileWorker = true;

		/// <summary>
		/// Whether to include a dependency on LiveCodingConsole when building targets that support live coding.
		/// </summary>
		[XmlConfigFile(Category = "VCProjectFileGenerator")]
		public bool bBuildLiveCodingConsole = false;

		/// <summary>
		/// Whether to generate a project file for each individual target, and not include e.g. Editor/Client/Server in the Configuration.
		/// </summary>
		[XmlConfigFile(Category = "VCProjectFileGenerator")]
		public bool bMakeProjectPerTarget = false;
	}

	/// <summary>
	/// Visual C++ project file generator implementation
	/// </summary>
	class VCProjectFileGenerator : ProjectFileGenerator
	{
		/// <summary>
		/// The settings object
		/// </summary>
		protected VCProjectFileSettings Settings = new VCProjectFileSettings();

		/// <summary>
		/// Set to true to enable a project for each target, and do not put the target type into the configuration
		/// </summary>
		protected override bool bMakeProjectPerTarget => Settings.bMakeProjectPerTarget;

		/// <summary>
		/// Override for the build tool to use in generated projects. If the compiler version is specified on the command line, we use the same argument on the 
		/// command line for generated projects.
		/// </summary>
		readonly string? BuildToolOverride;

		/// <summary>
		/// Default constructor
		/// </summary>
		/// <param name="InOnlyGameProject">The single project to generate project files for, or null</param>
		/// <param name="CSharpProjCache">A cache for C# projects to prevent needing to parse projects multiple times</param>
		/// <param name="InProjectFileFormat">Override the project file format to use</param>
		/// <param name="InArguments">Additional command line arguments</param>
		public VCProjectFileGenerator(
			FileReference? InOnlyGameProject,
			CSharpProjectCache CSharpProjCache,
			VCProjectFileFormat InProjectFileFormat,
			CommandLineArguments InArguments)
			: base(InOnlyGameProject, CSharpProjCache)
		{
			XmlConfig.ApplyTo(Settings);

			if (InProjectFileFormat != VCProjectFileFormat.Default)
			{
				Settings.ProjectFileFormat = InProjectFileFormat;
			}

			if (InArguments.HasOption("-2022"))
			{
				BuildToolOverride = "-2022";
			}

			if (InArguments.HasOption("-2026"))
			{
				BuildToolOverride = "-2026";
			}

			// Allow generating the solution even if the only installed toolchain is banned.			
			MicrosoftPlatformSDK.IgnoreToolchainErrors = true;
		}

		public override string[] GetTargetArguments(string[] Arguments)
		{
			return Arguments.Where(s => String.Equals(s, BuildToolOverride, StringComparison.OrdinalIgnoreCase)).ToArray();
		}

		/// File extension for project files we'll be generating (e.g. ".vcxproj")
		public override string ProjectFileExtension => ".vcxproj";

		/// <summary>
		/// </summary>
		public override void CleanProjectFiles(DirectoryReference InPrimaryProjectDirectory, string InPrimaryProjectName, DirectoryReference InIntermediateProjectFilesDirectory, ILogger Logger)
		{
			FileReference PrimaryProjectFile = FileReference.Combine(InPrimaryProjectDirectory, InPrimaryProjectName);
			FileReference PrimaryProjDeleteFilename = PrimaryProjectFile + ".sln";
			if (FileReference.Exists(PrimaryProjDeleteFilename))
			{
				FileReference.Delete(PrimaryProjDeleteFilename);
			}
			PrimaryProjDeleteFilename = PrimaryProjectFile + ".sdf";
			if (FileReference.Exists(PrimaryProjDeleteFilename))
			{
				FileReference.Delete(PrimaryProjDeleteFilename);
			}
			PrimaryProjDeleteFilename = PrimaryProjectFile + ".suo";
			if (FileReference.Exists(PrimaryProjDeleteFilename))
			{
				FileReference.Delete(PrimaryProjDeleteFilename);
			}
			PrimaryProjDeleteFilename = PrimaryProjectFile + ".v11.suo";
			if (FileReference.Exists(PrimaryProjDeleteFilename))
			{
				FileReference.Delete(PrimaryProjDeleteFilename);
			}
			PrimaryProjDeleteFilename = PrimaryProjectFile + ".v12.suo";
			if (FileReference.Exists(PrimaryProjDeleteFilename))
			{
				FileReference.Delete(PrimaryProjDeleteFilename);
			}
			PrimaryProjDeleteFilename = FileReference.Combine(InPrimaryProjectDirectory, ".vsconfig");
			if (FileReference.Exists(PrimaryProjDeleteFilename))
			{
				FileReference.Delete(PrimaryProjDeleteFilename);
			}

			// Delete the project files folder
			if (DirectoryReference.Exists(InIntermediateProjectFilesDirectory))
			{
				try
				{
					DirectoryReference.Delete(InIntermediateProjectFilesDirectory, true);
				}
				catch (Exception Ex)
				{
					Logger.LogInformation("Error while trying to clean project files path {InIntermediateProjectFilesDirectory}. Ignored.", InIntermediateProjectFilesDirectory);
					Logger.LogInformation("\t{Message}", Ex.Message);
				}
			}
		}

		/// <summary>
		/// Allocates a generator-specific project file object
		/// </summary>
		/// <param name="InitFilePath">Path to the project file</param>
		/// <param name="BaseDir">The base directory for files within this project</param>
		/// <returns>The newly allocated project file object</returns>
		protected override ProjectFile AllocateProjectFile(FileReference InitFilePath, DirectoryReference BaseDir)
		{
			return new VCProjectFile(InitFilePath, BaseDir, Settings.ProjectFileFormat, bUsePrecompiled, bMakeProjectPerTarget, BuildToolOverride, Settings, false);
		}

		/// "4.0", "12.0", or "14.0", etc...
		public static string GetProjectFileToolVersionString(VCProjectFileFormat ProjectFileFormat)
		{
			return ProjectFileFormat switch
			{
				VCProjectFileFormat.VisualStudio2022 => "17.0",
				VCProjectFileFormat.VisualStudio2026 => "18.0",
				_ => String.Empty,
			};
		}

		/// for instance: <PlatformToolset>v110</PlatformToolset>
		public static string GetProjectFilePlatformToolsetVersionString(VCProjectFileFormat ProjectFileFormat)
		{
			return ProjectFileFormat switch
			{
				VCProjectFileFormat.VisualStudio2022 => "v143",
				VCProjectFileFormat.VisualStudio2026 => "v145",
				_ => String.Empty,
			};
		}

		public static WindowsCompiler GetCompilerForIntellisense(VCProjectFileFormat ProjectFileFormat)
		{
			return ProjectFileFormat switch
			{
				VCProjectFileFormat.VisualStudio2022 => WindowsCompiler.VisualStudio2022,
				VCProjectFileFormat.VisualStudio2026 => WindowsCompiler.VisualStudio2026,
				_ => WindowsCompiler.VisualStudio2022,
			};
		}

		public static void AppendPlatformToolsetProperty(StringBuilder VCProjectFileContent, VCProjectFileFormat ProjectFileFormat)
		{
			string PlatformToolsetVersionString = GetProjectFilePlatformToolsetVersionString(ProjectFileFormat);
			VCProjectFileContent.AppendLine("    <PlatformToolset>{0}</PlatformToolset>", PlatformToolsetVersionString);
		}

		// parses project ini for Android to get architecture(s) enabled
		public static UnrealArchitectures GetAndroidProjectArchitectures(FileReference? ProjectFile)
		{
			List<string> ActiveArches = new();

			// look in ini settings for what platforms to compile for
			ConfigHierarchy Ini = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, DirectoryReference.FromFile(ProjectFile), UnrealTargetPlatform.Android);
			bool bBuild;

			if (Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bBuildForArm64", out bBuild) && bBuild)
			{
				ActiveArches.Add("arm64");
			}
			if (Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bBuildForx8664", out bBuild) && bBuild)
			{
				ActiveArches.Add("x64");
			}

			// we expect one to be specified
			if (ActiveArches.Count == 0)
			{
				ActiveArches.Add("arm64");
			}

			return new UnrealArchitectures(ActiveArches);
		}

		/// <summary>
		/// Returns a list of architectures to generate unique VS platforms for.
		/// </summary>
		public static UnrealArchitectures? GetPlatformArchitecturesToGenerate(UEBuildPlatform BuildPlatform, ProjectTarget InProjectTarget)
		{
			if (BuildPlatform.ArchitectureConfig.Mode == UnrealArchitectureMode.SingleTargetLinkSeparately)
			{
				// this should only be Android at the moment
				if (BuildPlatform.Platform == UnrealTargetPlatform.Android)
				{
					return InProjectTarget.UnrealProjectFilePath == null ? BuildPlatform.ArchitectureConfig.AllSupportedArchitectures
						: GetAndroidProjectArchitectures(InProjectTarget.UnrealProjectFilePath);
				}
				return BuildPlatform.ArchitectureConfig.AllSupportedArchitectures;
			}
			return (BuildPlatform.ArchitectureConfig.Mode == UnrealArchitectureMode.OneTargetPerArchitecture) ?
				BuildPlatform.ArchitectureConfig.AllSupportedArchitectures : null;
		}

		/// <inheritdoc/>
		protected override void ConfigureProjectFileGeneration(string[] Arguments, ref bool IncludeAllPlatforms, ILogger Logger)
		{
			// Call parent implementation first
			base.ConfigureProjectFileGeneration(Arguments, ref IncludeAllPlatforms, Logger);
		}

		/// <summary>
		/// Selects which platforms and build configurations we want in the project file
		/// </summary>
		/// <param name="IncludeAllPlatforms">True if we should include ALL platforms that are supported on this machine.  Otherwise, only desktop platforms will be included.</param>
		/// <param name="Logger"></param>
		/// <param name="SupportedPlatformNames">Output string for supported platforms, returned as comma-separated values.</param>
		protected override void SetupSupportedPlatformsAndConfigurations(bool IncludeAllPlatforms, ILogger Logger, out string SupportedPlatformNames)
		{
			// Call parent implementation to figure out the actual platforms
			base.SetupSupportedPlatformsAndConfigurations(IncludeAllPlatforms, Logger, out SupportedPlatformNames);

			// If we have a non-default setting for visual studio, check the compiler exists. If not, revert to the default.
			if (Settings.ProjectFileFormat == VCProjectFileFormat.VisualStudio2022)
			{
				if (!WindowsPlatform.HasCompiler(WindowsCompiler.VisualStudio2022, UnrealArch.X64, Logger))
				{
					Logger.LogWarning("Visual Studio C++ 2022 installation not found - ignoring preferred project file format.");
					Settings.ProjectFileFormat = VCProjectFileFormat.Default;
				}
			}
			else if (Settings.ProjectFileFormat == VCProjectFileFormat.VisualStudio2026)
			{
				if (!WindowsPlatform.HasCompiler(WindowsCompiler.VisualStudio2026, UnrealArch.X64, Logger))
				{
					Logger.LogWarning("Visual Studio C++ 2026 installation not found - ignoring preferred project file format.");
					Settings.ProjectFileFormat = VCProjectFileFormat.Default;
				}
			}

			// Certain platforms override the project file format because their debugger add-ins may not yet support the latest
			// version of Visual Studio.  This is their chance to override that.
			// ...but only if the user didn't override this via the command-line.
			if (Settings.ProjectFileFormat == VCProjectFileFormat.Default)
			{
				// Enumerate all the valid installations. This list is already sorted by preference.
				IEnumerable<VisualStudioInstallation> Installations = MicrosoftPlatformSDK.FindVisualStudioInstallations(Logger)
					.Where(x => WindowsPlatform.HasCompiler(x.Compiler, UnrealArch.X64, Logger));

				// Get the corresponding project file format
				VCProjectFileFormat Format = VCProjectFileFormat.Default;
				foreach (VisualStudioInstallation Installation in Installations)
				{
					if (Installation.Compiler == WindowsCompiler.VisualStudio2022)
					{
						Format = VCProjectFileFormat.VisualStudio2022;
						break;
					}
					else if (Installation.Compiler == WindowsCompiler.VisualStudio2026)
					{
						Format = VCProjectFileFormat.VisualStudio2026;
						break;
					}
				}
				Settings.ProjectFileFormat = Format;

				bool DowngradeAvailable = Installations.Any(x => x.Compiler == WindowsCompiler.VisualStudio2022);

				// Allow the SDKs to override
				foreach (UnrealTargetPlatform SupportedPlatform in SupportedPlatforms)
				{
					UEBuildPlatform? BuildPlatform;
					if (UEBuildPlatform.TryGetBuildPlatform(SupportedPlatform, out BuildPlatform))
					{
						// Don't worry about platforms that we're missing SDKs for
						if (BuildPlatform.HasRequiredSDKsInstalled() == SDKStatus.Valid)
						{
							VCProjectFileFormat ProposedFormat = BuildPlatform.GetRequiredVisualStudioVersion();

							if (ProposedFormat != VCProjectFileFormat.Default)
							{
								// Reduce the Visual Studio version to the max supported by each platform we plan to include.
								if (Settings.ProjectFileFormat == VCProjectFileFormat.Default || ProposedFormat < Settings.ProjectFileFormat)
								{
									Logger.LogInformation("Available {SupportedPlatform} SDK does not support Visual Studio 2026.", SupportedPlatform);
									Version Version = BuildPlatform.GetVersionRequiredForVisualStudio(VCProjectFileFormat.VisualStudio2026);
									if (Version > new Version())
									{
										Logger.LogInformation("Please update {SupportedPlatform} SDK to {Version} if Visual Studio 2026 support is desired.", SupportedPlatform, Version);
									}
									if (!DowngradeAvailable)
									{
										Logger.LogInformation("Generated solution cannot be downgraded as no prior Visual Studio version is installed. Please install the prior version of Visual Studio if {SupportedPlatform} SDK support is required.", SupportedPlatform);
									}
									else
									{
										Logger.LogInformation("Downgrading generated solution to {ProposedFormat}.", ProposedFormat);
										Logger.LogInformation("To force {ProjectFileFormat} solutions to always be generated add the following to BuildConfiguration.xml:", Settings.ProjectFileFormat);
										Logger.LogInformation("  <VCProjectFileGenerator>\r\n    <Version>{ProposedFormat}</Version>\r\n  </VCProjectFileGenerator>", Settings.ProjectFileFormat);
										Settings.ProjectFileFormat = ProposedFormat;
									}
									Logger.LogInformation("");
								}
							}
						}
					}
				}
			}

			if (bIncludeDotNetPrograms && Settings.ProjectFileFormat == VCProjectFileFormat.VisualStudio2022)
			{
#if NET10_0_OR_GREATER
				Logger.LogInformation("Visual Studio 2022 does not support .NET 10.0 C# projects, these projects will not be added to the generated solution.");
				Logger.LogInformation("Please generate the Visual Studio 2026 solution if .NET 10.0 C# project support is required.");
				bIncludeDotNetPrograms = false;
#else
				if (System.Environment.Version.Major >= 10)
				{
					Logger.LogInformation("Visual Studio 2022 does not support .NET 10.0 C# projects, these projects will not be added to the generated solution as the current dotnet runtime version is '{Version}'. Compiling with an older dotnet SDK from Visual Studio can cause errors.", System.Runtime.InteropServices.RuntimeInformation.FrameworkDescription);
					Logger.LogInformation("Please generate the Visual Studio 2026 solution if .NET 10.0 C# project support is required.");
					bIncludeDotNetPrograms = false;
				}
#endif
			}
		}

		protected override void GenerateAutomationProjectFiles(
			List<FileReference> AllGameProjects,
			PlatformProjectGeneratorCollection PlatformProjectGenerators,
			ILogger Logger)
		{
			if (Settings.ProjectFileFormat == VCProjectFileFormat.VisualStudio2022)
			{
				Logger.LogInformation(
					"Visual Studio 2022 does not support .NET 10.0 C# projects, and the current dotnet runtime version is '{Version}'. Compiling with an older dotnet SDK from Visual Studio can cause errors.",
					System.Runtime.InteropServices.RuntimeInformation.FrameworkDescription);

				// Some users can't use VS2026 yet (certain platforms are VS2022 only) so we should accomodate for them by generating the Automation solution for 2026 if available.
				if (WindowsPlatform.HasCompiler(WindowsCompiler.VisualStudio2026, UnrealArch.X64, Logger))
				{
					Logger.LogInformation("The {SolutionName} solution will be generated for Visual Studio 2026.", $"Automation_{PrimaryProjectName}");
				}
				else
				{
					Logger.LogInformation("The {SolutionName} solution will not be generated.", $"Automation_{PrimaryProjectName}");
					return;
				}
			}

			// We expect this to be called as part of a call that also writes a C++ solution too, so we have to make sure to put
			// this property back to its original value when done.
			PrimaryProjectFolder OriginalRootFolder = RootFolder;
			List<ProjectFile> OriginalAutomationProjectFiles = [.. AutomationProjectFiles];
			List<ProjectFile> OriginalOtherProjectFiles = [.. OtherProjectFiles];

			try
			{
				RootFolder = new(this, "<Root>");
				AutomationProjectFiles.Clear();
				OtherProjectFiles.Clear();

				AddAutomationProjects(AllGameProjects, Logger);

				using (Timeline.ScopeEvent("Writing automation project files", Logger))
				{
					// All projects must be C# here - if they're not, this is a bug and should be fixed.
					IEnumerable<VCSharpProjectFile> CSharpProjects = AllProjectFiles.Cast<VCSharpProjectFile>();
					List<string> SolutionConfigurations =
					[.. CSharpProjects
						.SelectMany(x => x.Configurations)
						.Distinct(StringComparer.OrdinalIgnoreCase)
						// VS will order the solution configurations alphabetically
						.OrderBy(x => x, StringComparer.OrdinalIgnoreCase)
					];
					List<VisualStudioSolutionProjectParams> SolutionCombinations =
					[..
						SolutionConfigurations.Select(x => new VisualStudioSolutionProjectParams(x, "Any CPU"))
					];
					string SolutionFileName = $"Automation_{PrimaryProjectName}.sln";
					FileReference SlnFilePath = FileReference.Combine(PrimaryProjectPath, SolutionFileName);
					FileReference SlnxFilePath = SlnFilePath.ChangeExtension(".slnx");

					AutomationVisualStudioSolutionBuilder Builder = new(SlnFilePath.Directory, RootFolder, SolutionCombinations, Logger);
					string slnContents = Builder.BuildSolution(Settings.ProjectFileFormat, forSlnx: false);
					string slnxContents = Builder.BuildSolution(Settings.ProjectFileFormat, forSlnx: true);

					Logger.LogInformation("Writing '{SolutionPath}'...", SlnFilePath);
					_ = WriteFileIfChanged(SlnFilePath.FullName, slnContents, Logger);
					Logger.LogInformation("Writing '{SolutionPath}'...", SlnxFilePath);
					_ =WriteFileIfChanged(SlnxFilePath.FullName, slnxContents, Logger);
				}
			}
			finally
			{
				RootFolder = OriginalRootFolder;
				AutomationProjectFiles.Clear();
				AutomationProjectFiles.AddRange(OriginalAutomationProjectFiles);
				OtherProjectFiles.Clear();
				OtherProjectFiles.AddRange(OriginalOtherProjectFiles);
				AllProjectFilesModified = true;
			}
		}

		/// <summary>
		/// Composes a string to use for the Visual Studio solution configuration, given a build configuration and target rules configuration name
		/// </summary>
		/// <param name="Configuration">The build configuration</param>
		/// <param name="TargetType">The type of target being built</param>
		/// <param name="bMakeProjectPerTarget">True if we are making one project per target type, instead of rolling them into the configs</param>
		/// <returns>The generated solution configuration name</returns>
		static string MakeSolutionConfigurationName(UnrealTargetConfiguration Configuration, TargetType TargetType, bool bMakeProjectPerTarget)
		{
			string SolutionConfigName = Configuration.ToString();

			if (!bMakeProjectPerTarget)
			{
				// Don't bother postfixing "Game" or "Program" -- that will be the default when using "Debug", "Development", etc.
				// Also don't postfix "RocketGame" when we're building Rocket game projects.  That's the only type of game there is in that case!
				if (TargetType != TargetType.Game && TargetType != TargetType.Program)
				{
					SolutionConfigName += " " + TargetType.ToString();
				}
			}

			return SolutionConfigName;
		}

		static Guid MakeMd5Guid(byte[] Input)
		{
#pragma warning disable CA5351 // Not used for cryptography, would be disruptive to users to change hash algo
			byte[] Hash = MD5.HashData(Input);
#pragma warning restore CA5351
			Hash[6] = (byte)(0x30 | (Hash[6] & 0x0f)); // 0b0011'xxxx Version 3 UUID (MD5)
			Hash[8] = (byte)(0x80 | (Hash[8] & 0x3f)); // 0b10xx'xxxx RFC 4122 UUID
			Array.Reverse(Hash, 0, 4);
			Array.Reverse(Hash, 4, 2);
			Array.Reverse(Hash, 6, 2);
			return new Guid(Hash);
		}

		public static Guid MakeMd5Guid(Guid Namespace, string Text)
		{
			byte[] Input = new byte[16 + Encoding.UTF8.GetByteCount(Text)];

			Namespace.TryWriteBytes(Input.AsSpan(0, 16));
			Array.Reverse(Input, 0, 4);
			Array.Reverse(Input, 4, 2);
			Array.Reverse(Input, 6, 2);

			Encoding.UTF8.GetBytes(Text, 0, Text.Length, Input, 16);
			return MakeMd5Guid(Input);
		}

		public static Guid MakeMd5Guid(string Text)
		{
			byte[] Input = Encoding.UTF8.GetBytes(Text);

			return MakeMd5Guid(Input);
		}

		private void WriteCommonPropsFile(ILogger Logger)
		{
			StringBuilder VCCommonTargetFileContent = new StringBuilder();
			VCCommonTargetFileContent.AppendLine("<?xml version=\"1.0\" encoding=\"utf-8\"?>");
			VCCommonTargetFileContent.AppendLine("<Project xmlns=\"http://schemas.microsoft.com/developer/msbuild/2003\">");

			// Project globals (project GUID, project type, SCC bindings, etc)
			{
				string ToolVersionString = GetProjectFileToolVersionString(Settings.ProjectFileFormat);
				VCCommonTargetFileContent.AppendLine("  <PropertyGroup Label=\"Globals\">");
				VCCommonTargetFileContent.AppendLine("    <Keyword>MakeFileProj</Keyword>");
				AppendPlatformToolsetProperty(VCCommonTargetFileContent, Settings.ProjectFileFormat);
				VCCommonTargetFileContent.AppendLine("    <MinimumVisualStudioVersion>{0}</MinimumVisualStudioVersion>", ToolVersionString);
				VCCommonTargetFileContent.AppendLine("    <VCProjectVersion>{0}</VCProjectVersion>", ToolVersionString);
				VCCommonTargetFileContent.AppendLine("    <NMakeUseOemCodePage>true</NMakeUseOemCodePage>"); // Fixes mojibake with non-Latin character sets (UE-102825)
				VCCommonTargetFileContent.AppendLine("    <TargetRuntime>Native</TargetRuntime>");
				VCCommonTargetFileContent.AppendLine("  </PropertyGroup>");
			}

			// Write the default configuration info
			VCCommonTargetFileContent.AppendLine("  <PropertyGroup Label=\"Configuration\">");
			VCCommonTargetFileContent.AppendLine($"    <ConfigurationType>{PlatformProjectGenerator.DefaultPlatformConfigurationType}</ConfigurationType>");
			AppendPlatformToolsetProperty(VCCommonTargetFileContent, Settings.ProjectFileFormat);
			VCCommonTargetFileContent.AppendLine("  </PropertyGroup>");

			VCCommonTargetFileContent.AppendLine("  <Import Project=\"$(VCTargetsPath)\\Microsoft.Cpp.Default.props\" />");
			VCCommonTargetFileContent.AppendLine("  <Import Project=\"$(VCTargetsPath)\\Microsoft.Cpp.props\" />");

			// Write the common and invalid configuration values
			{
				const string InvalidMessage = "echo The selected platform/configuration is not valid for this target.";

				string ProjectRelativeUnusedDirectory = ProjectFile.NormalizeProjectPath(DirectoryReference.Combine(Unreal.EngineDirectory, "Intermediate", "Build", "Unused"));

				VCCommonTargetFileContent.AppendLine("  <PropertyGroup>");

				DirectoryReference BatchFilesDirectory = DirectoryReference.Combine(Unreal.EngineDirectory, "Build", "BatchFiles");
				VCCommonTargetFileContent.AppendLine("    <BuildBatchScript>{0}</BuildBatchScript>", ProjectFile.EscapePath(ProjectFile.NormalizeProjectPath(FileReference.Combine(BatchFilesDirectory, "Build.bat"))));
				VCCommonTargetFileContent.AppendLine("    <RebuildBatchScript>{0}</RebuildBatchScript>", ProjectFile.EscapePath(ProjectFile.NormalizeProjectPath(FileReference.Combine(BatchFilesDirectory, "Rebuild.bat"))));
				VCCommonTargetFileContent.AppendLine("    <CleanBatchScript>{0}</CleanBatchScript>", ProjectFile.EscapePath(ProjectFile.NormalizeProjectPath(FileReference.Combine(BatchFilesDirectory, "Clean.bat"))));
				VCCommonTargetFileContent.AppendLine("    <NMakeBuildCommandLine>{0}</NMakeBuildCommandLine>", InvalidMessage);
				VCCommonTargetFileContent.AppendLine("    <NMakeReBuildCommandLine>{0}</NMakeReBuildCommandLine>", InvalidMessage);
				VCCommonTargetFileContent.AppendLine("    <NMakeCleanCommandLine>{0}</NMakeCleanCommandLine>", InvalidMessage);
				VCCommonTargetFileContent.AppendLine("    <NMakeOutput>Invalid Output</NMakeOutput>", InvalidMessage);
				VCCommonTargetFileContent.AppendLine("    <OutDir>{0}{1}</OutDir>", ProjectRelativeUnusedDirectory, Path.DirectorySeparatorChar);
				VCCommonTargetFileContent.AppendLine("    <IntDir>{0}{1}</IntDir>", ProjectRelativeUnusedDirectory, Path.DirectorySeparatorChar);
				// NOTE: We are intentionally overriding defaults for these paths with empty strings.  We never want Visual Studio's
				//       defaults for these fields to be propagated, since they are version-sensitive paths that may not reflect
				//       the environment that UBT is building in.  We'll set these environment variables ourselves!
				// NOTE: We don't touch 'ExecutablePath' because that would result in Visual Studio clobbering the system "Path"
				//       environment variable
				VCCommonTargetFileContent.AppendLine("    <IncludePath />");
				VCCommonTargetFileContent.AppendLine("    <ReferencePath />");
				VCCommonTargetFileContent.AppendLine("    <LibraryPath />");
				VCCommonTargetFileContent.AppendLine("    <LibraryWPath />");
				VCCommonTargetFileContent.AppendLine("    <SourcePath />");
				VCCommonTargetFileContent.AppendLine("    <ExcludePath />");

				// Add all the default system include paths
				if (OperatingSystem.IsWindows())
				{
					if (SupportedPlatforms.Contains(UnrealTargetPlatform.Win64))
					{
						VCCommonTargetFileContent.AppendLine("    <DefaultSystemIncludePaths>{0}</DefaultSystemIncludePaths>", VCToolChain.GetVCIncludePaths(UnrealTargetPlatform.Win64, GetCompilerForIntellisense(Settings.ProjectFileFormat), WindowsCompiler.Default, null, null, Logger));
					}
				}
				else
				{
					Logger.LogInformation("Unable to compute VC include paths on non-Windows host");
					VCCommonTargetFileContent.AppendLine("    <DefaultSystemIncludePaths />");
				}

				VCCommonTargetFileContent.AppendLine("  </PropertyGroup>");

			}

			// Write default import group
			VCCommonTargetFileContent.AppendLine("  <ImportGroup Label=\"PropertySheets\">");
			VCCommonTargetFileContent.AppendLine("    <Import Project=\"$(UserRootDir)\\Microsoft.Cpp.$(Platform).user.props\" Condition=\"exists('$(UserRootDir)\\Microsoft.Cpp.$(Platform).user.props')\" Label=\"LocalAppDataPlatform\" />");
			VCCommonTargetFileContent.AppendLine("  </ImportGroup>");

			VCCommonTargetFileContent.AppendLine("</Project>");

			Utils.WriteFileIfChanged(FileReference.Combine(IntermediateProjectFilesPath, "UECommon.props"), VCCommonTargetFileContent.ToString(), Logger);
		}

		/// <summary>
		/// Writes the project files to disk
		/// </summary>
		/// <returns>True if successful</returns>
		protected override bool WriteProjectFiles(PlatformProjectGeneratorCollection PlatformProjectGenerators, ILogger Logger)
		{
			WriteCommonPropsFile(Logger);

			if (!base.WriteProjectFiles(PlatformProjectGenerators, Logger))
			{
				return false;
			}

			// Write AutomationReferences file
			// Write in in net core expected format
			if (AutomationProjectFiles.Any())
			{
				XNamespace NS = XNamespace.Get("http://schemas.microsoft.com/developer/msbuild/2003");

				DirectoryReference AutomationToolDir = DirectoryReference.Combine(Unreal.EngineSourceDirectory, "Programs", "AutomationTool");
				DirectoryReference AutomationToolBinariesDir = DirectoryReference.Combine(Unreal.EngineDirectory, "Binaries", "DotNET", "AutomationTool");
				XDocument AutomationToolDocument = new XDocument(
					new XElement(NS + "Project",
						new XAttribute("ToolsVersion", VCProjectFileGenerator.GetProjectFileToolVersionString(Settings.ProjectFileFormat)),
						new XAttribute("DefaultTargets", "Build"),
						new XElement(NS + "ItemGroup",
							from AutomationProject in AutomationProjectFiles
							select new XElement(NS + "ProjectReference",
								new XAttribute("Include", AutomationProject.ProjectFilePath.MakeRelativeTo(AutomationToolDir)),
								new XElement(NS + "Private", "false")
							)
						),
						// Delete the private copied dlls in case they were ever next to the .exe - that is a bad place for them
						new XElement(NS + "Target",
							new XAttribute("Name", "CleanUpStaleDlls"),
							new XAttribute("AfterTargets", "Build"),
							AutomationProjectFiles.SelectMany(AutomationProject =>
							{
								string BaseFilename = FileReference.Combine(AutomationToolBinariesDir, AutomationProject.ProjectFilePath.GetFileNameWithoutExtension()).FullName;
								return new List<XElement>() {
										new XElement(NS + "Delete", new XAttribute("Files", BaseFilename + ".dll")),
										new XElement(NS + "Delete", new XAttribute("Files", BaseFilename + ".dll.config")),
										new XElement(NS + "Delete", new XAttribute("Files", BaseFilename + ".pdb"))
									};
							}
							)
						)
					)
				);

				StringBuilder Output = new StringBuilder();
				Output.AppendLine("<?xml version=\"1.0\" encoding=\"utf-8\"?>");

				XmlWriterSettings XmlSettings = new XmlWriterSettings();
				XmlSettings.Encoding = new UTF8Encoding(false);
				XmlSettings.Indent = true;
				XmlSettings.OmitXmlDeclaration = true;

				using (XmlWriter Writer = XmlWriter.Create(Output, XmlSettings))
				{
					AutomationToolDocument.Save(Writer);
				}
				
				// We cannot just use the IntermediateProjectFilesPath, as it can be changed to the project scope under "-Game" solution generation; AutomationTool.csproj always comes from 
				// Engine/Source/Programs
				DirectoryReference engineSourceIntermediateProjectFilesPath = DirectoryReference.Combine(Unreal.EngineDirectory, "Intermediate", "ProjectFiles");
				Utils.WriteFileIfChanged(FileReference.Combine(engineSourceIntermediateProjectFilesPath, "AutomationTool.csproj.References"), Output.ToString(), Logger);
			}

			return true;
		}

		protected override bool WritePrimaryProjectFile(ProjectFile? UBTProject, PlatformProjectGeneratorCollection PlatformProjectGenerators, ILogger Logger)
		{
			using ITimelineEvent _ = Timeline.ScopeEvent("Writing solution file", Logger);

			bool bSuccess = true;

			string SolutionFileName = PrimaryProjectName + ".sln";

			// Clone the root folder as we want to add visualizers to it for VS specifically.
			PrimaryProjectFolder RootFolderClone = new(RootFolder);

			// Get the path to the visualizers file. Try to make it relative to the solution directory, but fall back to a full path if it's a foreign project.
			FileReference VisualizersFile = FileReference.Combine(Unreal.EngineDirectory, "Extras", "VisualStudioDebugging", "Unreal.natvis");
			FileReference VisualizersStepFile = FileReference.Combine(Unreal.EngineDirectory, "Extras", "VisualStudioDebugging", "Unreal.natstepfilter");
			FileReference VisualizersJustMyCodeFile = FileReference.Combine(Unreal.EngineDirectory, "Extras", "VisualStudioDebugging", "Unreal.natjmc");

			// Add the visualizers at the solution level
			PrimaryProjectFolder visualizers = RootFolderClone.AddSubFolder("Visualizers");
			visualizers.Files.AddRange([
				VisualizersFile.MakeRelativeTo(PrimaryProjectPath),
					VisualizersStepFile.MakeRelativeTo(PrimaryProjectPath),
					VisualizersJustMyCodeFile.MakeRelativeTo(PrimaryProjectPath),
				]);

			// Solution configuration platforms. This is just a list of all of the platforms and configurations that
			// appear in Visual Studio's build configuration selector.
			List<VisualStudioPrimarySolutionProjectParams> ProjectParamsList;
			HashSet<UnrealTargetPlatform> PlatformsValidForProjects;
				CollectSolutionConfigurations(SupportedConfigurations, SupportedPlatforms, AllProjectFiles, bMakeProjectPerTarget,
					out PlatformsValidForProjects, out ProjectParamsList);
			bool DebugPredicate(MSBuildProjectFile Project) => Settings.bBuildUBTInDebug && Project == UBTProject;

			// Save the solution file
			FileReference SlnFilePath = FileReference.Combine(PrimaryProjectPath, SolutionFileName);
			FileReference SlnxFilePath = SlnFilePath.ChangeExtension(".slnx");
			PrimaryVisualStudioSolutionBuilder Builder = new(
				SlnFilePath.Directory,
				RootFolderClone,
				ProjectParamsList,
				PlatformProjectGenerators,
				[.. PlatformsValidForProjects],
				DebugPredicate,
				Logger
			);

			string slnContents = Builder.BuildSolution(Settings.ProjectFileFormat, forSlnx: false);
			string slnxContents = Builder.BuildSolution(Settings.ProjectFileFormat, forSlnx: true);

			Logger.LogInformation("Writing '{SolutionPath}'...", SlnFilePath);
			bSuccess &= WriteFileIfChanged(SlnFilePath.FullName, slnContents, Logger);
			Logger.LogInformation("Writing '{SolutionPath}'...", SlnxFilePath);
			bSuccess &= WriteFileIfChanged(SlnxFilePath.FullName, slnxContents, Logger);

			// Save a solution config file which selects the development editor configuration by default.
			// .suo file writable only on Windows, requires ole32
			if (bSuccess && Settings.bWriteSolutionOptionFile && OperatingSystem.IsWindows())
			{
				WriteSolutionOptionFile(SolutionFileName, ProjectParamsList);
			}

			if (bSuccess && Settings.bVsConfigFile && OperatingSystem.IsWindows())
			{
				bSuccess = WriteVsConfigFile(Logger);
			}

			return bSuccess;
		}

		[SupportedOSPlatform("windows")]
		private void WriteSolutionOptionFile(string SolutionFileName, List<VisualStudioPrimarySolutionProjectParams> SolutionVcxProjectParamsList)
		{
			// Figure out the filename for the SUO file. VS will automatically import the options from earlier versions if necessary.
			FileReference SolutionOptionsFileName = Settings.ProjectFileFormat switch
			{
				VCProjectFileFormat.VisualStudio2022 => FileReference.Combine(PrimaryProjectPath, ".vs", Path.GetFileNameWithoutExtension(SolutionFileName), "v17", ".suo"),
				VCProjectFileFormat.VisualStudio2026 => FileReference.Combine(PrimaryProjectPath, ".vs", Path.GetFileNameWithoutExtension(SolutionFileName), "v18", ".suo"),
				_ => throw new BuildException("Unsupported Visual Studio version"),
			};

			// Check it doesn't exist before overwriting it. Since these files store the user's preferences, it'd be bad form to overwrite them.
			if (!FileReference.Exists(SolutionOptionsFileName))
			{
				DirectoryReference.CreateDirectory(SolutionOptionsFileName.Directory);

				VCSolutionOptions Options = new VCSolutionOptions(Settings.ProjectFileFormat);

				// Set the default configuration and startup project
				VisualStudioPrimarySolutionProjectParams? DefaultConfig = SolutionVcxProjectParamsList.Find(x =>
					x.ProjectConfiguration == UnrealTargetConfiguration.Development &&
					x.ProjectPlatform == UnrealTargetPlatform.Win64 &&
					(x.ProjectArchitecture == null || x.ProjectArchitecture == UnrealArch.X64) &&
					(bMakeProjectPerTarget || x.ProjectTargetType == TargetType.Editor));
				if (DefaultConfig != null)
				{
					List<VCBinarySetting> Settings = new List<VCBinarySetting>();
					Settings.Add(new VCBinarySetting("ActiveCfg", DefaultConfig.SolutionConfigurationAndPlatform));
					if (DefaultProject != null)
					{
						Settings.Add(new VCBinarySetting("StartupProject", ((MSBuildProjectFile)DefaultProject).ProjectGUID.ToString("B")));
					}
					Options.SetConfiguration(Settings);
				}

				// Mark all the projects as closed by default, apart from the startup project
				VCSolutionExplorerState ExplorerState = new VCSolutionExplorerState();
				Options.SetExplorerState(ExplorerState);

				// Write the file
				if (Options.Sections.Count > 0)
				{
					Options.Write(SolutionOptionsFileName.FullName);
				}
			}
		}

		private bool WriteVsConfigFile(ILogger Logger)
		{
			StringBuilder VsConfigFileContent = new StringBuilder();

			VsConfigFileContent.AppendLine("{");
			VsConfigFileContent.AppendLine("  \"version\": \"1.0\",");
			VsConfigFileContent.AppendLine("  \"components\": [");
			IEnumerable<string> Components = MicrosoftPlatformSDK.GetVisualStudioSuggestedComponents(Settings.ProjectFileFormat);
			string ComponentsString = String.Join($",{Environment.NewLine}    ", Components.Select(x => $"\"{x}\""));
			VsConfigFileContent.AppendLine($"    {ComponentsString}");
			VsConfigFileContent.AppendLine("  ]");
			VsConfigFileContent.AppendLine("}");

			FileReference VsConfigFileName = FileReference.Combine(PrimaryProjectPath, ".vsconfig");
			return WriteFileIfChanged(VsConfigFileName.FullName, VsConfigFileContent.ToString(), Logger);
		}

		public static void CollectSolutionConfigurations(List<UnrealTargetConfiguration> AllConfigurations,
			List<UnrealTargetPlatform> AllPlatforms, List<ProjectFile> AllProjectFiles, bool bMakeProjectPerTarget,
			out HashSet<UnrealTargetPlatform> OutValidPlatforms, out List<VisualStudioPrimarySolutionProjectParams> OutSolutionVcxProjectParamsList)
		{
			OutValidPlatforms = new HashSet<UnrealTargetPlatform>();
			OutSolutionVcxProjectParamsList = new List<VisualStudioPrimarySolutionProjectParams>();
			Dictionary<string, Tuple<UnrealTargetConfiguration, Tuple<ProjectTarget, TargetType>>> SolutionConfigurationsValidForProjects = new();

			foreach (UnrealTargetConfiguration CurConfiguration in AllConfigurations)
			{
				if (InstalledPlatformInfo.IsValidConfiguration(CurConfiguration, EProjectType.Code))
				{
					foreach (UnrealTargetPlatform CurPlatform in AllPlatforms)
					{
						if (InstalledPlatformInfo.IsValidPlatform(CurPlatform, EProjectType.Code))
						{
							foreach (ProjectFile CurProject in AllProjectFiles)
							{
								if (!CurProject.IsStubProject)
								{
									// Figure out the set of valid target configuration names
									foreach (ProjectTarget ProjectTarget in CurProject.ProjectTargets.OfType<ProjectTarget>())
									{
										if (VCProjectFile.IsValidProjectPlatformAndConfiguration(ProjectTarget, CurPlatform, CurConfiguration))
										{
											OutValidPlatforms.Add(CurPlatform);

											// Default to a target configuration name of "Game", since that will collapse down to an empty string
											TargetType TargetType = TargetType.Game;
											if (ProjectTarget.TargetRules != null)
											{
												if (!ProjectTarget.TargetRules.IsTestTarget)
												{
													TargetType = ProjectTarget.TargetRules.Type;
												}
											}

											string SolutionConfigName =
												MakeSolutionConfigurationName(CurConfiguration, TargetType, bMakeProjectPerTarget);
											SolutionConfigurationsValidForProjects[SolutionConfigName] =
												new Tuple<UnrealTargetConfiguration, Tuple<ProjectTarget, TargetType>>(CurConfiguration, new Tuple<ProjectTarget, TargetType>(ProjectTarget, TargetType));
										}
									}
								}
							}
						}
					}
				}
			}

			foreach (UnrealTargetPlatform CurPlatform in OutValidPlatforms)
			{
				UEBuildPlatform? BuildPlatform;
				if (UEBuildPlatform.TryGetBuildPlatform(CurPlatform, out BuildPlatform))
				{
					foreach (KeyValuePair<string, Tuple<UnrealTargetConfiguration, Tuple<ProjectTarget, TargetType>>> SolutionConfigKeyValue in
						SolutionConfigurationsValidForProjects)
					{
						ProjectTarget ProjectTarget = SolutionConfigKeyValue.Value.Item2.Item1;

						void AddSolutionConfig(UnrealArch? Arch, List<VisualStudioPrimarySolutionProjectParams> OutSolutionConfigs, bool ForceArchSuffix = false)
						{
							// e.g.  "Development|Win64 = Development|Win64"
							string SolutionConfigName = SolutionConfigKeyValue.Key;
							UnrealTargetConfiguration Configuration = SolutionConfigKeyValue.Value.Item1;
							TargetType TargetType = SolutionConfigKeyValue.Value.Item2.Item2;

							string SolutionPlatformName = CurPlatform.ToString();
							// We use RequiresArchitectureFilenames to determine whether the architecture suffix should be added.
							// This is used to tell us what the "default" architecture is.
							if (Arch != null && (ForceArchSuffix || BuildPlatform.ArchitectureConfig.RequiresArchitectureFilenames(new UnrealArchitectures(Arch.Value))))
							{
								SolutionPlatformName += $"-{Arch}";
							}

							string SolutionConfigAndPlatformPair = SolutionConfigName + "|" + SolutionPlatformName;
							OutSolutionConfigs.Add(
								new VisualStudioPrimarySolutionProjectParams(SolutionConfigName, SolutionPlatformName, Configuration, CurPlatform, TargetType, Arch));
						}

						UnrealArchitectures? Architectures = GetPlatformArchitecturesToGenerate(BuildPlatform, ProjectTarget);
						if (Architectures == null)
						{
							AddSolutionConfig(null, OutSolutionVcxProjectParamsList);
						}
						else
						{
							if (BuildPlatform.ArchitectureConfig.Mode == UnrealArchitectureMode.SingleTargetLinkSeparately && BuildPlatform.Platform == UnrealTargetPlatform.Android)
							{
								AddSolutionConfig(null, OutSolutionVcxProjectParamsList);

								foreach (UnrealArch Arch in BuildPlatform.ArchitectureConfig.AllSupportedArchitectures.Architectures)
								{
									AddSolutionConfig(Arch, OutSolutionVcxProjectParamsList, true);
								}
							}
							else
							{
								foreach (UnrealArch Arch in Architectures.Architectures)
								{
									AddSolutionConfig(Arch, OutSolutionVcxProjectParamsList);
								}
							}
						}
					}
				}
			}

			// Sort the list of solution platform strings alphabetically (Visual Studio prefers it)
			OutSolutionVcxProjectParamsList.Sort(
				new Comparison<VisualStudioPrimarySolutionProjectParams>(
					(x, y) =>
					{
						return String.Compare(x.SolutionConfigurationAndPlatform, y.SolutionConfigurationAndPlatform,
							StringComparison.OrdinalIgnoreCase);
					}
				)
			);
		}

		protected override void WriteDebugSolutionFiles(PlatformProjectGeneratorCollection PlatformProjectGenerators, DirectoryReference IntermediateProjectFilesPath, ILogger Logger)
		{
			//build and collect UnrealVS configuration
			StringBuilder UnrealVSContent = new StringBuilder();
			foreach (UnrealTargetPlatform SupportedPlatform in SupportedPlatforms)
			{
				PlatformProjectGenerator? ProjGenerator = PlatformProjectGenerators.GetPlatformProjectGenerator(SupportedPlatform, true);
				ProjGenerator?.GetUnrealVSConfigurationEntries(UnrealVSContent);
			}
			if (UnrealVSContent.Length > 0)
			{
				UnrealVSContent.Insert(0, "<UnrealVS>" + ProjectFileGenerator.NewLine);
				UnrealVSContent.Append("</UnrealVS>" + ProjectFileGenerator.NewLine);

				string ConfigFilePath = FileReference.Combine(IntermediateProjectFilesPath, "UnrealVS.xml").FullName;
				/* bool bSuccess = */
				ProjectFileGenerator.WriteFileIfChanged(ConfigFilePath, UnrealVSContent.ToString(), Logger);
			}
		}
	}
}
