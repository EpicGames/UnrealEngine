// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text.RegularExpressions;
using System.Xml;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;
using UnrealBuildTool.ProjectFiles.VisualStudio;

namespace UnrealBuildTool;

/// <summary>
/// Represents an existing C# project file, created by using MSBuild to parse the file from disk.
/// </summary>
internal class VCSharpProjectFile : MSBuildProjectFile
{
	/// <summary>
	/// This is the GUID that Visual Studio uses to identify a C# project file in the solution
	/// </summary>
	public override string ProjectTypeGUID => "{FAE04EC0-301F-11D3-BF4B-00C04F79EFBC}";

	/// <summary>
	/// Configurations that this project supports
	/// </summary>
	public IReadOnlySet<string> Configurations => _configurations;

	/// <summary>
	/// Constructs a new project file object
	/// </summary>
	/// <param name="InitFilePath">The path to the project file on disk</param>
	/// <param name="Logger">Logger for output</param>
	public VCSharpProjectFile(FileReference InitFilePath, ILogger Logger)
		: base(InitFilePath, InitFilePath.Directory, true)
	{
		try
		{
			XmlDocument Document = new XmlDocument();
			Document.Load(InitFilePath.FullName);

			// Check the root element is the right type
			if (Document.DocumentElement?.Name != "Project")
			{
				throw new BuildException("Unexpected root element '{0}' in project file", Document.DocumentElement?.Name);
			}

			// Parse all the configurations and platforms
			// Parse the basic structure of the document, updating properties and recursing into other referenced projects as we go
			if (!IsDotnetCoreOrDotnet5Plus())
			{
				foreach (XmlElement Element in Document.DocumentElement.ChildNodes.OfType<XmlElement>())
				{
					if (Element.Name == "PropertyGroup")
					{
						string Condition = Element.GetAttribute("Condition");
						if (!String.IsNullOrEmpty(Condition))
						{
							Match Match = Regex.Match(Condition, "^\\s*'\\$\\(Configuration\\)\\|\\$\\(Platform\\)'\\s*==\\s*'(.+)\\|(.+)'\\s*$");
							if (Match.Success && Match.Groups.Count == 3)
							{
								_configurations.Add(Match.Groups[1].Value);
								_platforms.Add(Match.Groups[2].Value);
							}
							else
							{
								Logger.LogWarning("Unable to parse configuration/platform from condition '{InitFilePath}': {Condition}", InitFilePath, Condition);
							}
						}
					}
				}
			}
			else
			{
				foreach (string c in GetProjectProperty("Configurations").Split(';'))
				{
					_configurations.Add(c);
				}
				bool ConfigurationsFound = _configurations.Any();

				if (!ConfigurationsFound)
				{
					foreach (XmlElement PropertyGroup in Document.DocumentElement.ChildNodes.OfType<XmlElement>()
						.Where(element => element.Name == "PropertyGroup"))
					{
						XmlNodeList ConfigNodeList = PropertyGroup.GetElementsByTagName("Configurations");
						// if this property group does not set configurations we do not care about it
						if (ConfigNodeList.Count == 0)
						{
							continue;
						}

						if (PropertyGroup.HasAttribute("Condition"))
						{
							string Condition = PropertyGroup.GetAttribute("Condition");
							Logger.LogWarning("Unable to parse configuration from property group with condition '{InitFilePath}': {Condition}. UBT Requires you to set the configuration without conditionals.", InitFilePath, Condition);
							continue;
						}
						string[]? ParsedConfigurations = ConfigNodeList[0]?.FirstChild?.Value?.Split(';');
						if (ParsedConfigurations != null)
						{
							foreach (string c in ParsedConfigurations)
							{
								_configurations.Add(c);
							}
						}

						// platforms change meaning quite a bit in .net core but typically you do not specify this and its derived from the build instead
						// for most intents it is just Any CPU from .net framework
						_platforms.Add("AnyCPU");

						ConfigurationsFound = true;
						break;
					}
				}

				// dotnet does not require you to specify configurations or platforms, if you do not debug and release are the defaults
				if (!ConfigurationsFound)
				{
					_configurations.Add("Debug");
					_configurations.Add("Release");
					_platforms.Add("AnyCPU");
				}
			}
		}
		catch (Exception Ex)
		{
			Logger.LogWarning("Unable to parse {Path}: {Ex}", InitFilePath, Ex.ToString());
		}
	}

	/// <summary>
	/// Partially clone a VCSharpProjectFile, sharing platforms, configurations and cached project info, but with separate base class properties.
	/// </summary>
	/// <param name="other">The instance to partially clone</param>
	public VCSharpProjectFile(VCSharpProjectFile other)
		: base(other.ProjectFilePath, other.BaseDir, true)
	{
		_platforms = other._platforms;
		_configurations = other._configurations;
		_cachedProjectInfo = other._cachedProjectInfo;
	}

	/// <summary>
	/// Extract information from the csproj file based on the supplied configuration
	/// </summary>
	public IDotnetProjectInfo GetProjectInfo(UnrealTargetConfiguration InConfiguration)
	{
		if (_cachedProjectInfo.TryGetValue(InConfiguration, out IDotnetProjectInfo? value))
		{
			return value;
		}

		Dictionary<string, string> Properties = [];
		Properties.Add("Platform", "AnyCPU");
		Properties.Add("Configuration", InConfiguration.ToString());
		Properties.Add("EngineDir", Unreal.EngineDirectory.FullName);
		Properties.Add("EnginePath", Unreal.EngineDirectory.FullName);

		DotnetProjectInfo Info = DotnetProjectInfo.Read(ProjectFilePath, Properties);
		_cachedProjectInfo.Add(InConfiguration, Info);
		return Info;
	}

	/// <summary>
	/// Determine if this project is a .NET Core or .NET 5+ project
	/// </summary>
	public bool IsDotnetCoreOrDotnet5Plus()
	{
		IDotnetProjectInfo Info = GetProjectInfo(UnrealTargetConfiguration.Debug);
		return Info.IsDotnetCoreOrDotnet5Plus();
	}

	/// <summary>
	/// Gets a property from the project
	/// </summary>
	public string GetProjectProperty(string property)
	{
		IDotnetProjectInfo Info = GetProjectInfo(UnrealTargetConfiguration.Debug)!;
		Info.Properties.TryGetValue(property, out string? value);
		return value ?? String.Empty;
	}

	/// <inheritdoc/>
	public override MSBuildProjectContext? GetMatchingProjectContext(
		VisualStudioPrimarySolutionProjectParams SolutionCombination,
		PlatformProjectGeneratorCollection PlatformProjectGenerators,
		ILogger Logger)
	{
		bool bPreferBuildByDefault = SolutionCombination.ProjectTargetType is TargetType.Game or TargetType.Editor;
		return GetMatchingProjectContextForCSharp(bPreferBuildByDefault, SolutionCombination.SolutionConfiguration);
	}

	public MSBuildProjectContext GetMatchingProjectContextForCSharp(
		bool bPreferBuildByDefault,
		string SolutionConfiguration)
	{
		// Find the matching platform name
		string ProjectPlatformName;
		if (_platforms.Contains("x64"))
		{
			ProjectPlatformName = "x64";
		}
		else
		{
			ProjectPlatformName = "Any CPU";
		}

		// Find the matching configuration
		string ProjectConfigurationName;
		if (_configurations.Contains(SolutionConfiguration))
		{
			ProjectConfigurationName = SolutionConfiguration;
		}
		else if (_configurations.Contains("Development"))
		{
			ProjectConfigurationName = "Development";
		}
		else
		{
			ProjectConfigurationName = "Release";
		}

		// Figure out whether to build it by default
		bool bBuildByDefault = ShouldBuildByDefaultForSolutionTargets || bPreferBuildByDefault;

		// Create the context
		return new MSBuildProjectContext(ProjectConfigurationName, ProjectPlatformName) { bBuildByDefault = bBuildByDefault };
	}

	/// <summary>
	/// Basic csproj file support. Generates C# library project with one build config.
	/// </summary>
	/// <param name="InPlatforms">Not used.</param>
	/// <param name="InConfigurations">Not Used.</param>
	/// <param name="PlatformProjectGenerators">Set of platform project generators</param>
	/// <param name="Logger"></param>
	/// <returns>true if the opration was successful, false otherwise</returns>
	public override bool WriteProjectFile(List<UnrealTargetPlatform> InPlatforms, List<UnrealTargetConfiguration> InConfigurations, PlatformProjectGeneratorCollection PlatformProjectGenerators, ILogger Logger)
	{
		throw new BuildException("Support for writing C# projects from UnrealBuildTool has been removed.");
	}

	/// <summary>
	/// Platforms that this project supports
	/// </summary>
	private readonly HashSet<string> _platforms = [];

	/// <summary>
	/// Configurations that this project supports
	/// </summary>
	private readonly HashSet<string> _configurations = [];

	/// Cache of parsed info about this project
	protected readonly Dictionary<UnrealTargetConfiguration, IDotnetProjectInfo> _cachedProjectInfo = [];
}
