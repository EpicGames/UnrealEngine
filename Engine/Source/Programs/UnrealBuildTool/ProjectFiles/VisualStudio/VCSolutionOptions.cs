// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using System.Linq;
using System.Runtime.InteropServices;
using System.Runtime.Versioning;
using System.Text;
using FILETIME = System.Runtime.InteropServices.ComTypes.FILETIME;
using STATSTG = System.Runtime.InteropServices.ComTypes.STATSTG;

namespace UnrealBuildTool
{
	class VCBinarySetting
	{
		public enum ValueType
		{
			Bool = 3,
			String = 8,
		}

		public string Name;
		public ValueType Type;
		public bool BoolValue;
		public string? StringValue;

		public VCBinarySetting(string Name)
		{
			this.Name = Name;
		}

		public VCBinarySetting(string InName, bool InValue)
		{
			Name = InName;
			Type = ValueType.Bool;
			BoolValue = InValue;
		}

		public VCBinarySetting(string InName, string InValue)
		{
			Name = InName;
			Type = ValueType.String;
			StringValue = InValue;
		}

		public static VCBinarySetting Read(BinaryReader Reader)
		{
			// Read the key
			string Name = new string(Reader.ReadChars(Reader.ReadInt32())).TrimEnd('\0');

			VCBinarySetting Setting = new VCBinarySetting(Name);

			// Read an equals sign
			if (Reader.ReadChar() != '=')
			{
				throw new InvalidDataException("Expected equals symbol");
			}

			// Read the value
			Setting.Type = (ValueType)Reader.ReadUInt16();
			if (Setting.Type == ValueType.Bool)
			{
				Setting.BoolValue = (Reader.ReadUInt32() != 0);
			}
			else if (Setting.Type == ValueType.String)
			{
				Setting.StringValue = new string(Reader.ReadChars(Reader.ReadInt32()));
			}
			else
			{
				throw new InvalidDataException("Unknown value type");
			}

			// Read a semicolon
			if (Reader.ReadChar() != ';')
			{
				throw new InvalidDataException("Expected semicolon");
			}

			return Setting;
		}

		public void Write(BinaryWriter Writer)
		{
			// Write the name
			Writer.Write(Name.Length + 1);
			Writer.Write(Name.ToCharArray());
			Writer.Write('\0');

			// Write an equals sign
			Writer.Write('=');

			// Write the value type
			Writer.Write((ushort)Type);
			if (Type == ValueType.Bool)
			{
				Writer.Write(BoolValue ? (uint)0xffff0000 : 0);
			}
			else if (Type == ValueType.String)
			{
				Writer.Write(StringValue!.Length);
				Writer.Write(StringValue.ToCharArray());
			}
			else
			{
				throw new InvalidDataException("Unknown value type");
			}

			// Write a semicolon
			Writer.Write(';');
		}

		public override string ToString()
		{
			switch (Type)
			{
				case ValueType.Bool:
					return String.Format("{0} = {1}", Name, BoolValue);
				case ValueType.String:
					return String.Format("{0} = {1}", Name, StringValue);
			}
			return String.Format("{0} = ???", Name);
		}
	}

	[SupportedOSPlatform("windows")]
	class VCOleContainer
	{
		[Flags]
		enum STGC : int
		{
			Default = 0,
			Overwrite = 1,
			OnlyIfCurrent = 2,
			DangerouslyCommitMerelyToDiskCache = 4,
			Consolidate = 8,
		}

#pragma warning disable CA1069 // Enums should not have duplicate values - must match https://learn.microsoft.com/en-us/windows/win32/stg/stgm-constants
		[Flags]
		public enum STGM : int
		{
			Direct = 0x00000000,
			Transacted = 0x00010000,
			Simple = 0x08000000,
			Read = 0x00000000,
			Write = 0x00000001,
			ReadWrite = 0x00000002,
			ShareDenyNone = 0x00000040,
			ShareDenyRead = 0x00000030,
			ShareDenyWrite = 0x00000020,
			ShareExclusive = 0x00000010,
			Priority = 0x00040000,
			DeleteOnRelease = 0x04000000,
			NoScratch = 0x00100000,
			Create = 0x00001000,
			Convert = 0x00020000,
			FailIfThere = 0x00000000,
			NoSnapshot = 0x00200000,
			DirectSWMR = 0x00400000,
		}
#pragma warning restore CA1069

		[ComImport, Guid("0000000c-0000-0000-C000-000000000046"), InterfaceType(ComInterfaceType.InterfaceIsIUnknown)]
		interface IOleStream
		{
			void Read([Out, MarshalAs(UnmanagedType.LPArray, SizeParamIndex = 1)] byte[] pv, uint cb, out uint pcbRead);
			void Write([MarshalAs(UnmanagedType.LPArray, SizeParamIndex = 1)] byte[] pv, uint cb, out uint pcbWritten);
			void Seek(long dlibMove, uint dwOrigin, out long plibNewPosition);
			void SetSize(long libNewSize);
			void CopyTo(IOleStream pstm, long cb, out long pcbRead, out long pcbWritten);
			void Commit(STGC grfCommitFlags);
			void Revert();
			void LockRegion(long libOffset, long cb, uint dwLockType);
			void UnlockRegion(long libOffset, long cb, uint dwLockType);
			void Stat(out STATSTG pstatstg, uint grfStatFlag);
			void Clone(out IOleStream ppstm);
		}

		[ComImport, Guid("0000000b-0000-0000-C000-000000000046"), InterfaceType(ComInterfaceType.InterfaceIsIUnknown)]
		interface IOleStorage
		{
			void CreateStream(string pwcsName, STGM grfMode, uint reserved1, uint reserved2, out IOleStream ppstm);
			void OpenStream(string pwcsName, IntPtr reserved1, STGM grfMode, uint reserved2, out IOleStream ppstm);
			void CreateStorage(string pwcsName, STGM grfMode, uint reserved1, uint reserved2, out IOleStorage ppstg);
			void OpenStorage(string pwcsName, IOleStorage pstgPriority, uint grfMode, IntPtr snbExclude, uint reserved, out IOleStorage ppstg);
			void CopyTo(uint ciidExclude, Guid rgiidExclude, IntPtr snbExclude, IOleStorage pstgDest);
			void MoveElementTo(string pwcsName, IOleStorage pstgDest, string pwcsNewName, uint grfFlags);
			void Commit(STGC grfCommitFlags);
			void Revert();
			void EnumElements(uint reserved1, IntPtr reserved2, uint reserved3, out IOleEnumSTATSTG ppenum);
			void DestroyElement(string pwcsName);
			void RenameElement(string pwcsOldName, string pwcsNewName);
			void SetElementTimes(string pwcsName, FILETIME pctime, FILETIME patime, FILETIME pmtime);
			void SetClass(Guid clsid);
			void SetStateBits(uint grfStateBits, uint grfMask);
			void Stat(out STATSTG pstatstg, uint grfStatFlag);
		}

		[ComImport, Guid("0000000d-0000-0000-C000-000000000046"), InterfaceType(ComInterfaceType.InterfaceIsIUnknown)]
		interface IOleEnumSTATSTG
		{
			void Next(uint celt, [Out, MarshalAs(UnmanagedType.LPArray, SizeParamIndex = 0)] STATSTG[] rgelt, out uint pceltFetched);
			void RemoteNext(uint celt, [Out, MarshalAs(UnmanagedType.LPArray)] STATSTG[] rgelt, out uint pceltFetched);
			void Skip(out uint celt);
			void Reset();
			void Clone(out IOleEnumSTATSTG ppenum);
		}

		[DllImport("ole32.dll")]
		static extern int StgCreateDocfile([MarshalAs(UnmanagedType.LPWStr)] string pwcsName, STGM grfMode, uint reserved, out IOleStorage ppstgOpen);

		[DllImport("ole32.dll")]
		static extern int StgOpenStorage([MarshalAs(UnmanagedType.LPWStr)] string pwcsName, IOleStorage? pstgPriority, STGM grfMode, IntPtr snbExclude, uint reserved, out IOleStorage ppstgOpen);

		public List<KeyValuePair<string, byte[]>> Sections = new List<KeyValuePair<string, byte[]>>();

		public VCOleContainer()
		{
		}

		public VCOleContainer(string InputFileName)
		{
			int HResult = StgOpenStorage(InputFileName, null, STGM.Direct | STGM.Read | STGM.ShareExclusive, IntPtr.Zero, 0, out IOleStorage? Storage);
			if (HResult != 0)
			{
				throw new COMException("Call to StgOpenStorage failed", HResult);
			}

			try
			{
				IOleEnumSTATSTG? Enumerator = null;
				Storage.EnumElements(0, IntPtr.Zero, 0, out Enumerator);
				try
				{
					uint Fetched;
					STATSTG[] Stats = new STATSTG[200];
					Enumerator.Next((uint)Stats.Length, Stats, out Fetched);

					for (uint Idx = 0; Idx < Fetched; Idx++)
					{
						IOleStream OleStream;
						Storage.OpenStream(Stats[Idx].pwcsName, IntPtr.Zero, STGM.Read | STGM.ShareExclusive, 0, out OleStream);
						try
						{
							uint SizeRead;
							byte[] Buffer = new byte[Stats[Idx].cbSize];
							OleStream.Read(Buffer, (uint)Stats[Idx].cbSize, out SizeRead);
							Sections.Add(new KeyValuePair<string, byte[]>(Stats[Idx].pwcsName, Buffer));
						}
						finally
						{
							Marshal.ReleaseComObject(OleStream);
						}
					}
				}
				finally
				{
					Marshal.ReleaseComObject(Enumerator);
				}
			}
			finally
			{
				Marshal.ReleaseComObject(Storage);
			}
		}

		public void Write(string OutputFileName)
		{
			int HResult = StgCreateDocfile(OutputFileName, STGM.Direct | STGM.Create | STGM.Write | STGM.ShareExclusive, 0, out IOleStorage? OleStorage);
			if (HResult != 0)
			{
				throw new COMException("Call to StgCreateDocfile failed", HResult);
			}

			try
			{
				foreach (KeyValuePair<string, byte[]> Section in Sections)
				{
					IOleStream? OleStream = null;
					OleStorage.CreateStream(Section.Key, STGM.Write | STGM.ShareExclusive, 0, 0, out OleStream);
					try
					{
						uint Written;
						OleStream.Write(Section.Value, (uint)Section.Value.Length, out Written);
						OleStream.Commit(STGC.Overwrite);
					}
					finally
					{
						Marshal.ReleaseComObject(OleStream);
					}
				}
			}
			finally
			{
				Marshal.ReleaseComObject(OleStorage);
			}
		}

		int FindIndex(string Name)
		{
			return Sections.FindIndex(x => x.Key == Name);
		}

		public void SetSection(string Name, byte[] Data)
		{
			int Idx = FindIndex(Name);
			if (Idx == -1)
			{
				Sections.Add(new KeyValuePair<string, byte[]>(Name, Data));
			}
			else
			{
				Sections[Idx] = new KeyValuePair<string, byte[]>(Name, Data);
			}
		}

		public byte[] GetSection(string Name)
		{
			byte[]? Data;
			if (!TryGetSection(Name, out Data))
			{
				throw new KeyNotFoundException();
			}
			return Data;
		}

		public bool TryGetSection(string Name, [NotNullWhen(true)] out byte[]? Data)
		{
			int Idx = FindIndex(Name);
			if (Idx == -1)
			{
				Data = null;
				return false;
			}
			else
			{
				Data = Sections[Idx].Value;
				return true;
			}
		}
	}

	class VCSolutionExplorerState
	{
		const string EpilogueGuid = "00000000-0000-0000-0000-000000000000";

		public List<Tuple<string, string[]>> OpenProjects = new List<Tuple<string, string[]>>();
		public List<Tuple<string, string[]>> SelectedProjects = new List<Tuple<string, string[]>>();

		public VCSolutionExplorerState()
		{
		}

		public void Read(Stream InputStream, VCProjectFileFormat Format)
		{
			BinaryReader Reader = new BinaryReader(InputStream, Encoding.Unicode);
			ReadOpenProjects(Reader, Format);
			ReadSelectedProjects(Reader);
			ReadEpilogue(Reader);
		}

		public void Write(Stream OutputStream, VCProjectFileFormat Format)
		{
			BinaryWriter Writer = new BinaryWriter(OutputStream, Encoding.Unicode);
			WriteOpenProjects(Writer, Format);
			WriteSelectedProjects(Writer);
			WriteEpilogue(Writer);
		}

		void ReadOpenProjects(BinaryReader Reader, VCProjectFileFormat Format)
		{
			long OpenFoldersEnd = Reader.BaseStream.Position + Reader.ReadInt32();

			if (Format >= VCProjectFileFormat.VisualStudio2022)
			{
				int Header1 = Reader.ReadInt32();
				/*int Header2 =*/
				Reader.ReadInt32();
				/*int Header3 =*/
				Reader.ReadByte();
				if (Header1 != 15)
				{
					throw new BuildException("Unexpected data in open projects section");
				}
			}
			else
			{
				throw new BuildException("Unexpected VCProjectFileFormat {0}", Format);
			}

			int NumProjects = Reader.ReadInt16();
			for (int ProjectIdx = 0; ProjectIdx < NumProjects; ProjectIdx++)
			{
				string ProjectName = ReadString(Reader);

				string[] Folders = new string[Reader.ReadInt16()];
				for (int FolderIdx = 0; FolderIdx < Folders.Length; FolderIdx++)
				{
					Folders[FolderIdx] = ReadString(Reader);
				}

				OpenProjects.Add(new Tuple<string, string[]>(ProjectName, Folders));
			}

			Debug.Assert(Reader.BaseStream.Position == OpenFoldersEnd);
		}

		void WriteOpenProjects(BinaryWriter Writer, VCProjectFileFormat Format)
		{
			if (Format >= VCProjectFileFormat.VisualStudio2022)
			{
				Writer.Write(4 + (4 + 4 + 1) + 2 + OpenProjects.Sum(x => GetStringSize(x.Item1) + 2 + x.Item2.Sum(y => GetStringSize(y))));
				Writer.Write(15);
				Writer.Write(65536);
				Writer.Write((byte)0);
			}
			else
			{
				throw new BuildException("Unexpected VCProjectFileFormat {0}", Format);
			}
			Writer.Write((short)OpenProjects.Count);
			foreach (Tuple<string, string[]> OpenProject in OpenProjects)
			{
				WriteString(Writer, OpenProject.Item1);
				Writer.Write((short)OpenProject.Item2.Length);
				foreach (string OpenFolder in OpenProject.Item2)
				{
					WriteString(Writer, OpenFolder);
				}
			}
		}

		void ReadSelectedProjects(BinaryReader Reader)
		{
			long SelectedProjectsEnd = Reader.BaseStream.Position + Reader.ReadInt32();
			if (Reader.ReadInt32() != 1)
			{
				throw new Exception("Unexpected data in selected projects section");
			}

			int NumProjects = Reader.ReadInt32();
			for (int ProjectIdx = 0; ProjectIdx < NumProjects; ProjectIdx++)
			{
				string ProjectName = ReadString(Reader);

				string[] Items = new string[Reader.ReadInt32()];
				for (int ItemIdx = 0; ItemIdx < Items.Length; ItemIdx++)
				{
					Items[ItemIdx] = ReadString(Reader);
				}

				SelectedProjects.Add(new Tuple<string, string[]>(ProjectName, Items));
			}

			Debug.Assert(Reader.BaseStream.Position == SelectedProjectsEnd);
		}

		void WriteSelectedProjects(BinaryWriter Writer)
		{
			Writer.Write(4 + 4 + 4 + SelectedProjects.Sum(x => GetStringSize(x.Item1) + 4 + x.Item2.Sum(y => GetStringSize(y))));
			Writer.Write(1);
			Writer.Write(SelectedProjects.Count);
			foreach (Tuple<string, string[]> SelectedProject in SelectedProjects)
			{
				WriteString(Writer, SelectedProject.Item1);
				Writer.Write(SelectedProject.Item2.Length);
				foreach (string SelectedItem in SelectedProject.Item2)
				{
					WriteString(Writer, SelectedItem);
				}
			}
		}

		private static void ReadEpilogue(BinaryReader Reader)
		{
			long EpilogueEnd = Reader.BaseStream.Position + Reader.ReadInt32();
			if (Reader.ReadInt32() != 1 || ReadString(Reader) != EpilogueGuid || Reader.ReadInt32() != 0)
			{
				throw new Exception("Unexpected data in epilogue");
			}
			Debug.Assert(Reader.BaseStream.Position == EpilogueEnd);
		}

		private static void WriteEpilogue(BinaryWriter Writer)
		{
			Writer.Write(4 + 4 + GetStringSize(EpilogueGuid) + 4);
			Writer.Write(1);
			WriteString(Writer, EpilogueGuid);
			Writer.Write(0);
		}

		private static string ReadString(BinaryReader Reader)
		{
			// Read the number of bytes
			int NumBytes = Reader.ReadByte();
			for (int Shift = 7; (NumBytes & (1 << Shift)) != 0; Shift += 7)
			{
				NumBytes &= (1 << Shift) - 1;
				NumBytes |= Reader.ReadByte() << Shift;
			}

			// Read the characters
			return new string(Reader.ReadChars(NumBytes / 2));
		}

		private static void WriteString(BinaryWriter Writer, string Text)
		{
			// Write the number of bytes in the string, encoded as a sequence of 7-bit values with the top bit set on all but the last
			int NumBytes = Text.Length * 2;
			while (NumBytes >= 128)
			{
				Writer.Write((byte)((NumBytes & 127) | 128));
				NumBytes >>= 7;
			}
			Writer.Write((byte)NumBytes);

			// Write the characters
			Writer.Write(Text.ToCharArray());
		}

		private static int GetStringSize(string Text)
		{
			int Size = 1 + (Text.Length * 2);
			for (int Length = Text.Length; Length > 127; Length >>= 7)
			{
				Size++;
			}
			return Size;
		}
	}

	[SupportedOSPlatform("windows")]
	class VCSolutionOptions : VCOleContainer
	{
		readonly VCProjectFileFormat Format;

		public VCSolutionOptions(VCProjectFileFormat Format)
		{
			this.Format = Format;
		}

		public VCSolutionOptions(string FileName, VCProjectFileFormat Format)
			: base(FileName)
		{
			this.Format = Format;
		}

		public IEnumerable<VCBinarySetting> GetConfiguration()
		{
			byte[]? Data;
			if (TryGetSection("SolutionConfiguration", out Data))
			{
				using (MemoryStream InputStream = new MemoryStream(Data, false))
				{
					BinaryReader Reader = new BinaryReader(InputStream, Encoding.Unicode);
					while (InputStream.Position < InputStream.Length)
					{
						yield return VCBinarySetting.Read(Reader);
					}
				}
			}
		}

		public void SetConfiguration(IEnumerable<VCBinarySetting> Settings)
		{
			using (MemoryStream OutputStream = new MemoryStream())
			{
				BinaryWriter Writer = new BinaryWriter(OutputStream, Encoding.Unicode);
				foreach (VCBinarySetting Setting in Settings)
				{
					Setting.Write(Writer);
				}
				SetSection("SolutionConfiguration", OutputStream.ToArray());
			}
		}

		public VCSolutionExplorerState? GetExplorerState()
		{
			byte[]? Data;
			if (TryGetSection("ProjExplorerState", out Data))
			{
				VCSolutionExplorerState State = new VCSolutionExplorerState();
				State.Read(new MemoryStream(Data, false), Format);
				return State;
			}
			return null;
		}

		public void SetExplorerState(VCSolutionExplorerState State)
		{
			using (MemoryStream OutputStream = new MemoryStream())
			{
				State.Write(OutputStream, Format);
				SetSection("ProjExplorerState", OutputStream.ToArray());
			}
		}
	}
}
