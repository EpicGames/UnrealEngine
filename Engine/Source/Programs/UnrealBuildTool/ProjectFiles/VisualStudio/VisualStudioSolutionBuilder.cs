// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Collections.Immutable;
using System.IO;
using System.Linq;
using System.Text;
using System.Threading;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using Microsoft.VisualStudio.SolutionPersistence.Model;
using Microsoft.VisualStudio.SolutionPersistence.Serializer;

namespace UnrealBuildTool.ProjectFiles.VisualStudio;

internal abstract class VisualStudioSolutionBuilder<TProjectParams> where TProjectParams : VisualStudioSolutionProjectParams
{
	public VisualStudioSolutionBuilder(
		DirectoryReference solutionDirectory,
		PrimaryProjectFolder rootFolder,
		IReadOnlyList<TProjectParams> solutionCombinations,
		ILogger logger)
	{
		_solutionDirectory = solutionDirectory;
		_rootFolder = rootFolder;
		_solutionCombinations = [.. solutionCombinations];
		_solutionConfigurations = [.. solutionCombinations.Select(x => x.SolutionConfiguration).Distinct()];
		_solutionPlatforms = [.. solutionCombinations.Select(x => x.SolutionPlatform).Distinct()];
		_logger = logger;
	}

	public string BuildSolution(VCProjectFileFormat format, bool forSlnx)
	{
		SolutionModel solution = new();

		if (!forSlnx)
		{
#pragma warning disable CS0618 // Write HideSolutionNode for legacy compatibility.
			solution.VisualStudioProperties.HideSolutionNode = false;
#pragma warning restore CS0618
			solution.VisualStudioProperties.MinimumVersion = new(10, 0, 40219, 1);

			switch (format)
			{
				case VCProjectFileFormat.VisualStudio2022:
					solution.VisualStudioProperties.OpenWith = "Visual Studio Version 17";
					solution.VisualStudioProperties.Version = new(17, 0, 31314, 256);
					break;
				case VCProjectFileFormat.VisualStudio2026:
					solution.VisualStudioProperties.OpenWith = "Visual Studio Version 18";
					solution.VisualStudioProperties.Version = new(18, 0, 11205, 157);
					break;
				default:
					throw new BuildException("Unexpected ProjectFileFormat '{0}'", format);
			}
		}

		foreach (string platform in _solutionPlatforms)
		{
			solution.AddPlatform(platform.ToString());
		}

		foreach (string configuration in _solutionConfigurations)
		{
			solution.AddBuildType(configuration);
		}

		AddUnrealVS(solution);

		Queue<(PrimaryProjectFolder folder, string path)> folders = new();

		AddProjectsAndItems(solution, null, _rootFolder, !forSlnx);
		foreach (PrimaryProjectFolder folder in _rootFolder.SubFolders)
		{
			folders.Enqueue((folder, $"/{folder.FolderName}/"));
		}

		while (folders.Count > 0)
		{
			(PrimaryProjectFolder folder, string path) = folders.Dequeue();
			SolutionFolderModel solutionFolder = solution.AddFolder(path);
			if (!forSlnx)
			{
				// Legacy compatibility: Set the ID of the generated folder.
				if (path == "/Visualizers/")
				{
					solutionFolder.Id = Guid.Parse("1CCEC849-CC72-4C59-8C36-2F7C38706D4C");
				}
				else
				{
					solutionFolder.Id = VCProjectFileGenerator.MakeMd5Guid($"UE5{path[..^1]}");
				}
			}

			AddProjectsAndItems(solution, solutionFolder, folder, !forSlnx);

			foreach (PrimaryProjectFolder childFolder in folder.SubFolders)
			{
				folders.Enqueue((childFolder, $"{path}{childFolder.FolderName}/"));
			}
		}

		solution.DistillProjectConfigurations();

		using MemoryStream memoryStream = new();
		UTF8Encoding encoding = new UTF8Encoding(encoderShouldEmitUTF8Identifier: false);
		if (forSlnx)
		{
			SolutionSerializers.SlnXml.SaveAsync(memoryStream, solution, CancellationToken.None).Wait();
		}
		else
		{
			SolutionSerializers.SlnFileV12.SaveAsync(memoryStream, solution, CancellationToken.None).Wait();
		}

		return encoding.GetString(memoryStream.ToArray());
	}

	private void AddProjectsAndItems(SolutionModel solution, SolutionFolderModel? solutionFolder, PrimaryProjectFolder folder, bool setID)
	{
		foreach (ProjectFile projectFile in folder.ChildProjects)
		{
			// Cast must be valid here, there is a bug further up if not.
			MSBuildProjectFile msBuildProjectFile = (MSBuildProjectFile)projectFile;
			string relativeProjectPath = projectFile.ProjectFilePath.MakeRelativeTo(_solutionDirectory);
			SolutionProjectModel project = solution.AddProject(relativeProjectPath, folder: solutionFolder);
			// To prevent the solution from being marked dirty, always set the ID if it's a C++ project.
			if (projectFile is VCProjectFile || setID)
			{
				project.Id = msBuildProjectFile.ProjectGUID;
			}

			foreach (TProjectParams combination in _solutionCombinations)
			{
				string solutionConfiguration = combination.SolutionConfiguration;
				string solutionPlatform = combination.SolutionPlatform;
				MSBuildProjectContext? projectContext = GetContextForProject(msBuildProjectFile, combination);

				if (projectContext is null)
				{
					continue;
				}

				AdjustProjectContext(projectContext, msBuildProjectFile);

				// Only set these if they're not the default values.
				if (!String.Equals(projectContext.ConfigurationName, solutionConfiguration, StringComparison.Ordinal))
				{
					project.AddProjectConfigurationRule(new(BuildDimension.BuildType, solutionConfiguration, solutionPlatform, projectContext.ConfigurationName));
				}
				if (!String.Equals(projectContext.PlatformName, solutionPlatform, StringComparison.Ordinal))
				{
					project.AddProjectConfigurationRule(new(BuildDimension.Platform, solutionConfiguration, solutionPlatform, projectContext.PlatformName));
				}
				if (!projectContext.bBuildByDefault)
				{
					project.AddProjectConfigurationRule(new(BuildDimension.Build, solutionConfiguration, solutionPlatform, "false"));
				}
				if (projectContext.bDeployByDefault)
				{
					project.AddProjectConfigurationRule(new(BuildDimension.Deploy, solutionConfiguration, solutionPlatform, "true"));
				}
			}
		}

		if (solutionFolder is not null)
		{
			foreach (string item in folder.Files)
			{
				solutionFolder.AddFile(item);
			}
		}
	}

	protected virtual void AddUnrealVS(SolutionModel solution)
	{
	}

	protected virtual void AdjustProjectContext(MSBuildProjectContext projectContext, MSBuildProjectFile projectFile)
	{
	}

	protected abstract MSBuildProjectContext? GetContextForProject(MSBuildProjectFile projectFile, TProjectParams projectParams);

	private readonly DirectoryReference _solutionDirectory;
	private readonly PrimaryProjectFolder _rootFolder;
	private readonly ImmutableArray<TProjectParams> _solutionCombinations;
	private readonly ImmutableArray<string> _solutionConfigurations;
	private readonly ImmutableArray<string> _solutionPlatforms;
	protected readonly ILogger _logger;
}

internal sealed class PrimaryVisualStudioSolutionBuilder : VisualStudioSolutionBuilder<VisualStudioPrimarySolutionProjectParams>
{
	public PrimaryVisualStudioSolutionBuilder(
		DirectoryReference solutionDirectory,
		PrimaryProjectFolder rootFolder,
		IReadOnlyList<VisualStudioPrimarySolutionProjectParams> solutionProjectParams,
		PlatformProjectGeneratorCollection platformProjectGeneratorCollection,
		IReadOnlyList<UnrealTargetPlatform> unrealVSPlatforms,
		Predicate<MSBuildProjectFile> debugPredicate,
		ILogger logger
	) : base(solutionDirectory, rootFolder, solutionProjectParams, logger)
	{
		_platformProjectGeneratorCollection = platformProjectGeneratorCollection;
		_debugPredicate = debugPredicate;
		_unrealVSPlatformsString = String.Join(";", unrealVSPlatforms.Select(platform => platform.ToString()));
	}

	protected override void AddUnrealVS(SolutionModel solution)
	{
		const string UnrealVSGuid = "ddbf523f-7eb6-4887-bd51-85a714ff87eb";
		SolutionPropertyBag unrealVSProperties = solution.AddProperties(UnrealVSGuid);
		unrealVSProperties.Add("AvailablePlatforms", _unrealVSPlatformsString);
	}

	protected override void AdjustProjectContext(MSBuildProjectContext projectContext, MSBuildProjectFile projectFile)
	{
		if (_debugPredicate(projectFile))
		{
			projectContext.ConfigurationName = "Debug";
		}
	}

	protected override MSBuildProjectContext? GetContextForProject(MSBuildProjectFile projectFile, VisualStudioPrimarySolutionProjectParams projectParams)
	{
		return projectFile.GetMatchingProjectContext(projectParams, _platformProjectGeneratorCollection, _logger);
	}

	private readonly PlatformProjectGeneratorCollection _platformProjectGeneratorCollection;
	private readonly Predicate<MSBuildProjectFile> _debugPredicate;
	private readonly string _unrealVSPlatformsString;

}

internal sealed class AutomationVisualStudioSolutionBuilder(
	DirectoryReference solutionDirectory,
	PrimaryProjectFolder rootFolder,
	IReadOnlyList<VisualStudioSolutionProjectParams> solutionProjectParams,
	ILogger logger
) : VisualStudioSolutionBuilder<VisualStudioSolutionProjectParams>(solutionDirectory, rootFolder, solutionProjectParams, logger)
{
	protected override MSBuildProjectContext? GetContextForProject(MSBuildProjectFile projectFile, VisualStudioSolutionProjectParams projectParams)
	{
		if (projectFile is not VCSharpProjectFile cSharpProjectFile)
		{
			throw new ArgumentException($"{nameof(AutomationVisualStudioSolutionBuilder)} only supports C# projects.");
		}

		return cSharpProjectFile.GetMatchingProjectContextForCSharp(bPreferBuildByDefault: true, projectParams.SolutionConfiguration);
	}
}
