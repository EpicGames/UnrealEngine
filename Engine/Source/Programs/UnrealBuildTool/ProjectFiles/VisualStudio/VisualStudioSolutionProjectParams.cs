// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace UnrealBuildTool.ProjectFiles.VisualStudio;

/// <summary>
/// Parameters to configure a project inside a Visual Studio solution.
/// </summary>
internal class VisualStudioSolutionProjectParams(string solutionConfiguration, string solutionPlatform)
{
	public string SolutionConfiguration { get; } = solutionConfiguration;

	public string SolutionPlatform { get; } = solutionPlatform;

	public string SolutionConfigurationAndPlatform => $"{SolutionConfiguration}|{SolutionPlatform}";
}

/// <summary>
/// Parameters to configure a project inside the primary Visual Studio solution for this Unreal Engine project.
/// </summary>
internal sealed class VisualStudioPrimarySolutionProjectParams(
	string solutionConfiguration,
	string solutionPlatform,
	UnrealTargetConfiguration projectConfiguration,
	UnrealTargetPlatform projectPlatform,
	TargetType projectTargetType,
	UnrealArch? projectArchitecture) :
	VisualStudioSolutionProjectParams(solutionConfiguration, solutionPlatform)
{
	public override string ToString() => $"{SolutionConfiguration}|{SolutionPlatform}={ProjectConfiguration} {ProjectPlatform} {ProjectTargetType}{(ProjectArchitecture != null ? " " + ProjectArchitecture : String.Empty)}";

	public UnrealTargetConfiguration ProjectConfiguration { get; } = projectConfiguration;

	public UnrealTargetPlatform ProjectPlatform { get; } = projectPlatform;

	public TargetType ProjectTargetType { get; } = projectTargetType;

	public UnrealArch? ProjectArchitecture { get; } = projectArchitecture;
}
