// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Text;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	class VSCodeProject : ProjectFile
	{
		public VSCodeProject(FileReference InitFilePath, DirectoryReference BaseDir)
			: base(InitFilePath, BaseDir)
		{
		}

		public override bool WriteProjectFile(List<UnrealTargetPlatform> InPlatforms, List<UnrealTargetConfiguration> InConfigurations, PlatformProjectGeneratorCollection PlatformProjectGenerators, ILogger Logger)
		{
			return true;
		}
	}

	class VSCodeProjectFileGenerator : ProjectFileGenerator
	{
		private UnrealTargetPlatform HostPlatform = BuildHostPlatform.Current.Platform;
		private bool bForeignProject;
		private readonly DirectoryReference ProjectRoot;
		private readonly string FrameworkExecutableExtension = OperatingSystem.IsWindows() ? ".exe" : "";
		private readonly string FrameworkLibraryExtension = ".dll";

		private readonly ConcurrentBag<BuildTarget> BuildTargets = new();

#pragma warning disable IDE0044 // Make field readonly - these private static fields are set by XML configuration.
		/// <summary>
		/// Includes all files in the generated workspace.
		/// </summary>
		[XmlConfigFile(Name = "IncludeAllFiles")]
		private bool IncludeAllFiles = false;

		/// <summary>
		/// Whether VS Code project generation should include debug configurations to allow attaching to already running processes
		/// </summary>
		[XmlConfigFile(Name = "AddDebugAttachConfig")]
		private bool bAddDebugAttachConfig = false;

		/// <summary>
		/// Whether VS Code project generation should include debug configurations to allow core dump debugging
		/// </summary>
		[XmlConfigFile(Name = "AddDebugCoreConfig")]
		private bool bAddDebugCoreConfig = false;

		/// <summary>
		/// Do not create compile commands json files with compiler arguments for each file; works better with VS Code extension using
		/// UBT server mode.
		/// </summary>
		[XmlConfigFile(Name = "NoCompileCommands")]
		[CommandLine("-NoCompileCommands")]
		private bool bNoCompileCommands = false;

		/// <summary>
		/// Create a workspace file for use with VS Code extension that communicates directly with UBT. 
		/// </summary>
		[XmlConfigFile(Name = "UseVSCodeExtension")]
		[CommandLine("-UseVSCodeExtension")]
		private bool bUseVSCodeExtension = false;

		/// <summary>
		/// Debugger type to be used with Linux.  Defaults to cppdbg.
		/// </summary>
		[XmlConfigFile(Name = "LinuxDebuggerType")]
		private string LinuxDebuggerType = "cppdbg";
#pragma warning restore IDE0044

		private enum EPathType
		{
			Absolute,
			Relative,
		}

		private enum EQuoteType
		{
			Single, // can be ignored on platforms that don't need it (windows atm)
			Double,
		}

		private string CommonMakePathString(FileSystemReference InRef, EPathType InPathType, DirectoryReference? InRelativeRoot)
		{
			if (InRelativeRoot == null)
			{
				InRelativeRoot = ProjectRoot;
			}

			string Processed = InRef.ToString();

			switch (InPathType)
			{
				case EPathType.Relative:
					{
						if (InRef.IsUnderDirectory(InRelativeRoot))
						{
							Processed = InRef.MakeRelativeTo(InRelativeRoot).ToString();
						}

						break;
					}

				default:
					{
						break;
					}
			}

			if (HostPlatform == UnrealTargetPlatform.Win64)
			{
				Processed = Processed.Replace('/', '\\');
			}
			else
			{
				Processed = Processed.Replace('\\', '/');
			}

			return Processed;
		}

		private string MakeQuotedPathString(FileSystemReference InRef, EPathType InPathType, DirectoryReference? InRelativeRoot = null, EQuoteType InQuoteType = EQuoteType.Double)
		{
			string Processed = CommonMakePathString(InRef, InPathType, InRelativeRoot);

			if (Processed.Contains(' ', StringComparison.Ordinal))
			{
				if (HostPlatform == UnrealTargetPlatform.Win64 && InQuoteType == EQuoteType.Double)
				{
					Processed = "\"" + Processed + "\"";
				}
				else
				{
					Processed = "'" + Processed + "'";
				}
			}

			return Processed;
		}

		private string MakeUnquotedPathString(FileSystemReference InRef, EPathType InPathType, DirectoryReference? InRelativeRoot = null)
		{
			return CommonMakePathString(InRef, InPathType, InRelativeRoot);
		}

		private string MakePathString(FileSystemReference InRef, bool bInAbsolute = false, bool bForceSkipQuotes = false)
		{
			if (bForceSkipQuotes)
			{
				return MakeUnquotedPathString(InRef, bInAbsolute ? EPathType.Absolute : EPathType.Relative, ProjectRoot);
			}
			else
			{
				return MakeQuotedPathString(InRef, bInAbsolute ? EPathType.Absolute : EPathType.Relative, ProjectRoot);
			}
		}

		public VSCodeProjectFileGenerator(FileReference? InOnlyGameProject, CSharpProjectCache CSharpProjCache)
			: base(InOnlyGameProject, CSharpProjCache)
		{
			ProjectRoot = Unreal.RootDirectory;
		}

		// TODO: Do we really need this instead of just using System.Text.Json?
		class JsonFile
		{
			public JsonFile()
			{
			}

			public void BeginRootObject()
			{
				BeginObject();
			}

			public void EndRootObject()
			{
				EndObject();
				if (TabString.Length > 0)
				{
					throw new Exception("Called EndRootObject before all objects and arrays have been closed");
				}
			}

			public void BeginObject(string? Name = null)
			{
				string Prefix = Name == null ? "" : Quoted(JsonWriter.EscapeString(Name)) + ": ";
				Lines.Add(TabString + Prefix + "{");
				TabString += "\t";
			}

			public void EndObject()
			{
				Lines[^1] = Lines[^1].TrimEnd(',');
				TabString = TabString.Remove(TabString.Length - 1);
				Lines.Add(TabString + "},");
			}

			public void BeginArray(string? Name = null)
			{
				string Prefix = Name == null ? "" : Quoted(JsonWriter.EscapeString(Name)) + ": ";
				Lines.Add(TabString + Prefix + "[");
				TabString += "\t";
			}

			public void EndArray()
			{
				Lines[^1] = Lines[^1].TrimEnd(',');
				TabString = TabString.Remove(TabString.Length - 1);
				Lines.Add(TabString + "],");
			}

			public void AddField(string Name, bool Value)
			{
				Lines.Add(TabString + Quoted(JsonWriter.EscapeString(Name)) + ": " + (Value ? "true" : "false") + ",");
			}

			public void AddField(string Name, string Value)
			{
				Lines.Add(TabString + Quoted(JsonWriter.EscapeString(Name)) + ": " + Quoted(JsonWriter.EscapeString(Value)) + ",");
			}

			public void AddUnnamedField(string Value)
			{
				Lines.Add(TabString + Quoted(JsonWriter.EscapeString(Value)) + ",");
			}

			public void Write(FileReference File)
			{
				Lines[^1] = Lines[^1].TrimEnd(',');
				FileReference.WriteAllLines(File, Lines.ToArray());
			}

			private static string Quoted(string Value)
			{
				return "\"" + Value + "\"";
			}

			private readonly List<string> Lines = new List<string>();
			private string TabString = "";
		}

		public override string ProjectFileExtension => ".vscode";

		public override void CleanProjectFiles(DirectoryReference InPrimaryProjectDirectory, string InPrimaryProjectName, DirectoryReference InIntermediateProjectFilesPath, ILogger Logger)
		{
		}

		public override bool ShouldGenerateIntelliSenseData()
		{
			return !bNoCompileCommands && !bUseVSCodeExtension;
		}

		protected override ProjectFile AllocateProjectFile(FileReference InitFilePath, DirectoryReference BaseDir)
		{
			return new VSCodeProject(InitFilePath, BaseDir);
		}

		protected override bool WritePrimaryProjectFile(ProjectFile? UBTProject, PlatformProjectGeneratorCollection PlatformProjectGenerators, ILogger Logger)
		{
			DirectoryReference VSCodeDir = DirectoryReference.Combine(PrimaryProjectPath, ".vscode");
			DirectoryReference.CreateDirectory(VSCodeDir);

			bForeignProject = !VSCodeDir.IsUnderDirectory(ProjectRoot);

			List<ProjectFile> Projects;

			if (bForeignProject)
			{
				Projects = new List<ProjectFile>();
				foreach (ProjectFile Project in AllProjectFiles)
				{
					if (GameProjectName == Project.ProjectFilePath.GetFileNameWithoutAnyExtensions())
					{
						Projects.Add(Project);
						break;
					}
				}
			}
			else
			{
				Projects = new List<ProjectFile>(AllProjectFiles);
			}
			Projects.Sort((A, B) => { return String.Compare(A.ProjectFilePath.GetFileName(), B.ProjectFilePath.GetFileName(), StringComparison.Ordinal); });

			ProjectData ProjectData = GatherProjectData(Projects);

			WriteWorkspaceIgnoreFile(Projects);
			WriteCppPropertiesFile(VSCodeDir, ProjectData);
			WriteWorkspaceFile(ProjectData);

			if (bForeignProject && bIncludeEngineSource)
			{
				// for installed builds we need to write the cpp properties file under the installed engine as well for intellisense to work
				DirectoryReference VsCodeDirectory = DirectoryReference.Combine(Unreal.RootDirectory, ".vscode");
				WriteCppPropertiesFile(VsCodeDirectory, ProjectData);
			}

			return true;
		}
		private class BuildTarget
		{
			public readonly string Name;
			public readonly TargetType Type;
			public readonly UnrealTargetPlatform Platform;
			public readonly UnrealTargetConfiguration Configuration;
			public readonly CppStandardVersion CppStandard;
			public readonly FileReference CompilerPath;
			public readonly DirectoryReference? SysRootPath;
			public readonly Dictionary<DirectoryReference, string> ModuleCommandLines;

			public BuildTarget(string InName, TargetType InType, UnrealTargetPlatform InPlatform, UnrealTargetConfiguration InConfiguration, CppStandardVersion InCppStandard, FileReference InCompilerPath, DirectoryReference? InSysRootPath, Dictionary<DirectoryReference, string> InModulesCommandLines)
			{
				Name = InName;
				Type = InType;
				Platform = InPlatform;
				Configuration = InConfiguration;
				CppStandard = InCppStandard;
				CompilerPath = InCompilerPath;
				SysRootPath = InSysRootPath;
				ModuleCommandLines = InModulesCommandLines;
			}

			public override string ToString()
			{
				return Name.ToString() + " " + Type.ToString();
			}
		}

		protected override void AddTargetForIntellisense(UEBuildTarget Target, IEnumerable<(UEBuildModuleCPP, CppCompileEnvironment)> Modules, ILogger Logger)
		{
			base.AddTargetForIntellisense(Target, Modules, Logger);

			if (!Modules.Any())
			{
				return;
			}

			bool UsingClang = true;
			FileReference CompilerPath;
			DirectoryReference? SysRootPath = null;
			if (OperatingSystem.IsWindows())
			{
				VCEnvironment Environment = VCEnvironment.Create(WindowsPlatform.GetDefaultCompiler(null, Target.Rules.WindowsPlatform.Architecture, Logger, true), WindowsCompiler.Default, Target.Platform, Target.Rules.WindowsPlatform.Architecture, null, null, Target.Rules.WindowsPlatform.WindowsSdkVersion, null, Target.Rules.WindowsPlatform.bUseCPPWinRT, Target.Rules.WindowsPlatform.bAllowClangLinker, Target.Rules.WindowsPlatform.bAllowRadLinker, Logger);
				CompilerPath = Environment.CompilerPath;
				UsingClang = false;
			}
			else if (OperatingSystem.IsLinux())
			{
				string? ClangPath = LinuxCommon.WhichClang(Logger);
				if (ClangPath == null)
				{
					return;
				}
				CompilerPath = new(ClangPath);
				string? InternalSDKPath = UEBuildPlatform.GetSDK(UnrealTargetPlatform.Linux)?.GetInternalSDKPath();
				if (!String.IsNullOrEmpty(InternalSDKPath))
				{
					SysRootPath = DirectoryReference.FromString(InternalSDKPath);
				}
			}
			else if (OperatingSystem.IsMacOS())
			{
				MacToolChainSettings Settings = new MacToolChainSettings(false, Logger);
				CompilerPath = FileReference.Combine(MacToolChainSettings.ToolchainDir, "clang++");
				SysRootPath = Settings.GetSDKPath(UnrealArch.Host.Value);
			}
			else
			{
				return;
			}

			// we do not need to keep track of which binary the invocation belongs to, only which target, as such we join all binaries into a single set
			Dictionary<DirectoryReference, string> ModuleDirectoryToCompileCommand = new Dictionary<DirectoryReference, string>();

			CppStandardVersion CppStandard = CppStandardVersion.EngineDefault;
			foreach ((UEBuildModuleCPP Module, CppCompileEnvironment ModuleCompileEnvironment) in Modules)
			{
				List<FileReference> ForceIncludePaths = new List<FileReference>(ModuleCompileEnvironment.ForceIncludeFiles.Select(x => x.Location));
				if (ModuleCompileEnvironment.PrecompiledHeaderIncludeFilename != null)
				{
					ForceIncludePaths.Add(ModuleCompileEnvironment.PrecompiledHeaderIncludeFilename);
				}

				StringBuilder CommandBuilder = new StringBuilder();

				foreach (FileReference ForceIncludeFile in ForceIncludePaths)
				{
					CommandBuilder.AppendFormat("{0} \"{1}\" {2}", UsingClang ? "-include" : "/FI", ForceIncludeFile.FullName, Environment.NewLine);
				}
				foreach (string Definition in ModuleCompileEnvironment.Definitions)
				{
					CommandBuilder.AppendFormat("{0} \"{1}\" {2}", UsingClang ? "-D" : "/D", Definition, Environment.NewLine);
				}
				foreach (DirectoryReference IncludePath in ModuleCompileEnvironment.UserIncludePaths)
				{
					CommandBuilder.AppendFormat("{0} \"{1}\" {2}", UsingClang ? "-I" : "/I", IncludePath, Environment.NewLine);
				}
				foreach (DirectoryReference IncludePath in ModuleCompileEnvironment.SystemIncludePaths)
				{
					CommandBuilder.AppendFormat("{0} \"{1}\" {2}", UsingClang ? "-I" : "/I", IncludePath, Environment.NewLine);
				}

				ModuleDirectoryToCompileCommand.TryAdd(Module.ModuleDirectory, CommandBuilder.ToString());
				if (Module.Rules.CppStandard != null && Module.Rules.CppStandard > CppStandard)
				{
					CppStandard = (CppStandardVersion)Module.Rules.CppStandard;
				}
			}

			BuildTargets.Add(new BuildTarget(Target.TargetName, Target.TargetType, Target.Platform, Target.Configuration, CppStandard, CompilerPath, SysRootPath, ModuleDirectoryToCompileCommand));
		}

		private class ProjectData
		{
			public enum EOutputType
			{
				Library,
				Exe,

				WinExe, // some projects have this so we need to read it, but it will be converted across to Exe so no code should handle it!
			}

			public class BuildProduct
			{
				public FileReference OutputFile { get; set; }
				public FileReference? UProjectFile { get; set; }
				public UnrealTargetConfiguration Config { get; set; }
				public UnrealTargetPlatform Platform { get; set; }
				public EOutputType OutputType { get; set; }

				public IDotnetProjectInfo? CSharpInfo { get; set; }

				public override string ToString()
				{
					return Platform.ToString() + " " + Config.ToString();
				}

				public BuildProduct(FileReference OutputFile)
				{
					this.OutputFile = OutputFile;
				}
			}

			public class Target
			{
				public string Name;
				public TargetType Type;
				// For native projects only
				public TargetRules? TargetRules;
				public List<BuildProduct> BuildProducts = new List<BuildProduct>();

				public Target(Project InParentProject, TargetRules InTargetRules)
				{
					Name = InTargetRules.Name;
					Type = InTargetRules.Type;
					TargetRules = InTargetRules;
					InParentProject.Targets.Add(this);
				}
				public Target(Project InParentProject, string InName, TargetType InType)
				{
					Name = InName;
					Type = InType;
					InParentProject.Targets.Add(this);
				}

				public override string ToString()
				{
					return Name.ToString() + " " + Type.ToString();
				}
			}

			public class Project
			{
				public string Name;
				public ProjectFile SourceProject;
				public List<Target> Targets = new List<Target>();

				public override string ToString()
				{
					return Name;
				}

				public Project(string Name, ProjectFile SourceProject)
				{
					this.Name = Name;
					this.SourceProject = SourceProject;
				}
			}

			public List<Project> NativeProjects = new List<Project>();
			public List<Project> CSharpProjects = new List<Project>();
			public List<Project> AllProjects = new List<Project>();
		}

		private ProjectData GatherProjectData(List<ProjectFile> InProjects)
		{
			ProjectData ProjectData = new ProjectData();

			foreach (ProjectFile Project in InProjects)
			{
				// Create new project record
				ProjectData.Project NewProject = new ProjectData.Project(Project.ProjectFilePath.GetFileNameWithoutExtension(), Project);

				ProjectData.AllProjects.Add(NewProject);

				// Add into the correct easy-access list
				if (Project is VSCodeProject)
				{
					foreach (ProjectTarget Target in Project.ProjectTargets.OfType<ProjectTarget>())
					{
						UnrealTargetConfiguration[] Configs = (UnrealTargetConfiguration[])Enum.GetValues(typeof(UnrealTargetConfiguration));
						List<UnrealTargetPlatform> Platforms = new List<UnrealTargetPlatform>(Target.TargetRules!.GetSupportedPlatforms());

						ProjectData.Target NewTarget = new ProjectData.Target(NewProject, Target.TargetRules);

						if (HostPlatform != UnrealTargetPlatform.Win64)
						{
							Platforms.Remove(UnrealTargetPlatform.Win64);
						}

						foreach (UnrealTargetPlatform Platform in Platforms)
						{
							UEBuildPlatform.TryGetBuildPlatform(Platform, out UEBuildPlatform? BuildPlatform);
							if (SupportedPlatforms.Contains(Platform) && (BuildPlatform != null) && (BuildPlatform.HasRequiredSDKsInstalled() == SDKStatus.Valid))
							{
								foreach (UnrealTargetConfiguration Config in Configs)
								{
									if (MSBuildProjectFile.IsValidProjectPlatformAndConfiguration(Target, Platform, Config))
									{
										NewTarget.BuildProducts.Add(new ProjectData.BuildProduct(GetExecutableFilename(Project, Target, Platform, Config))
										{
											Platform = Platform,
											Config = Config,
											UProjectFile = Target.UnrealProjectFilePath,
											OutputType = ProjectData.EOutputType.Exe,
											CSharpInfo = null
										});
									}
								}
							}
						}
					}

					ProjectData.NativeProjects.Add(NewProject);
				}
				else
				{
					VCSharpProjectFile VCSharpProject = (VCSharpProjectFile)Project;

					string ProjectName = Project.ProjectFilePath.GetFileNameWithoutExtension();

					ProjectData.Target Target = new ProjectData.Target(NewProject, ProjectName, TargetType.Program);

					UnrealTargetConfiguration[] Configs = { UnrealTargetConfiguration.Debug, UnrealTargetConfiguration.Development };

					foreach (UnrealTargetConfiguration Config in Configs)
					{
						IDotnetProjectInfo? Info = VCSharpProject.GetProjectInfo(Config)!;

						{
							ProjectData.EOutputType OutputType;
							string? OutputTypeName;
							if (Info.Properties.TryGetValue("OutputType", out OutputTypeName))
							{
								OutputType = (ProjectData.EOutputType)Enum.Parse(typeof(ProjectData.EOutputType), OutputTypeName);
							}
							else
							{
								OutputType = ProjectData.EOutputType.Library;
							}

							if (OutputType == ProjectData.EOutputType.WinExe)
							{
								OutputType = ProjectData.EOutputType.Exe;
							}

							DirectoryReference OutputPath = Info.GetOutputDir(VCSharpProject.ProjectFilePath.Directory);
							FileReference? OutputFile = null;
							HashSet<FileReference> ProjectBuildProducts = new HashSet<FileReference>();
							Info.FindCompiledBuildProducts(OutputPath, ProjectBuildProducts);
							foreach (FileReference ProjectBuildProduct in ProjectBuildProducts)
							{
								if ((OutputType == ProjectData.EOutputType.Exe && ProjectBuildProduct.GetExtension() == FrameworkExecutableExtension) ||
									(OutputType == ProjectData.EOutputType.Library && ProjectBuildProduct.GetExtension() == FrameworkLibraryExtension))
								{
									OutputFile = ProjectBuildProduct;
									break;
								}
							}

							if (OutputFile != null)
							{
								Target.BuildProducts.Add(new ProjectData.BuildProduct(OutputFile)
								{
									Platform = HostPlatform,
									Config = Config,
									OutputType = OutputType,
									CSharpInfo = Info
								});
							}
						}
					}

					ProjectData.CSharpProjects.Add(NewProject);
				}
			}

			return ProjectData;
		}

		private void WriteCppPropertiesFile(DirectoryReference OutputDirectory, ProjectData Projects)
		{
			DirectoryReference.CreateDirectory(OutputDirectory);

			JsonFile OutFile = new JsonFile();

			OutFile.BeginRootObject();
			{
				OutFile.BeginArray("configurations");
				{
					HashSet<FileReference> AllSourceFiles = new HashSet<FileReference>();
					Dictionary<DirectoryReference, string> AllModuleCommandLines = new Dictionary<DirectoryReference, string>();
					FileReference? CompilerPath = null;
					DirectoryReference? SysRootPath = null;
					CppStandardVersion CppStandard = CppStandardVersion.Default;

					foreach (ProjectData.Project Project in Projects.AllProjects)
					{
						AllSourceFiles.UnionWith(Project.SourceProject.SourceFiles.Select(x => x.Reference));

						foreach (ProjectData.Target ProjectTarget in Project.Targets)
						{
							BuildTarget? BuildTarget = BuildTargets.FirstOrDefault(Target => Target.Name == ProjectTarget.Name);

							// we do not generate intellisense for every target, as that just causes a lot of redundancy, as such we will not find a mapping for a lot of the targets
							if (BuildTarget == null)
							{
								continue;
							}

							string Name = String.Format("{0} {1} {2} {3} ({4})", ProjectTarget.Name, ProjectTarget.Type, BuildTarget.Platform, BuildTarget.Configuration, Project.Name);
							WriteConfiguration(Name, Project.Name, Project.SourceProject.SourceFiles.Select(x => x.Reference), BuildTarget.CppStandard, BuildTarget.CompilerPath, BuildTarget.SysRootPath, BuildTarget.ModuleCommandLines, OutFile, OutputDirectory);

							CompilerPath = BuildTarget.CompilerPath;

							foreach (KeyValuePair<DirectoryReference, string> Pair in BuildTarget.ModuleCommandLines)
							{
								if (!AllModuleCommandLines.ContainsKey(Pair.Key))
								{
									AllModuleCommandLines[Pair.Key] = Pair.Value;
								}
							}

							if (BuildTarget.CppStandard > CppStandard)
							{
								CppStandard = BuildTarget.CppStandard;
							}
							if (BuildTarget.SysRootPath != null)
							{
								SysRootPath = BuildTarget.SysRootPath;
							}
						}
					}

					string DefaultConfigName;
					if (HostPlatform == UnrealTargetPlatform.Linux)
					{
						DefaultConfigName = "Linux";
					}
					else if (HostPlatform == UnrealTargetPlatform.Mac)
					{
						DefaultConfigName = "Mac";
					}
					else
					{
						DefaultConfigName = "Win32";
					}

					WriteConfiguration(DefaultConfigName, "Default", AllSourceFiles, CppStandard, CompilerPath, SysRootPath, AllModuleCommandLines, OutFile, OutputDirectory);
				}
				OutFile.EndArray();
			}
			OutFile.EndRootObject();

			OutFile.Write(FileReference.Combine(OutputDirectory, "c_cpp_properties.json"));
		}

		private void WriteConfiguration(string Name, string ProjectName, IEnumerable<FileReference> SourceFiles, CppStandardVersion CppStandard, FileReference? CompilerPath, DirectoryReference? SysRootPath, Dictionary<DirectoryReference, string> ModuleCommandLines, JsonFile OutFile, DirectoryReference OutputDirectory)
		{
			OutFile.BeginObject();

			OutFile.AddField("name", Name);
			if (CompilerPath != null)
			{
				OutFile.AddField("compilerPath", CompilerPath.FullName);
			}

			if (HostPlatform == UnrealTargetPlatform.Mac)
			{
				string SysRoot = SysRootPath != null ? SysRootPath.FullName : "/Applications/Xcode.app/Contents/Developer/Platforms/MacOSX.platform/Developer/SDKs/MacOSX.sdk";
				OutFile.BeginArray("compilerArgs");
				{
					OutFile.AddUnnamedField("-isysroot");
					OutFile.AddUnnamedField(SysRoot);
				}
				OutFile.EndArray();

				OutFile.BeginArray("macFrameworkPath");
				{
					OutFile.AddUnnamedField(SysRoot + "/System/Library/Frameworks");
				}
				OutFile.EndArray();
			}
			else if (SysRootPath != null)
			{
				OutFile.BeginArray("compilerArgs");
				{
					OutFile.AddUnnamedField("-isysroot");
					OutFile.AddUnnamedField(SysRootPath.FullName);
				}
				OutFile.EndArray();
			}

			OutFile.AddField("cStandard", "c17");

			switch (CppStandard)
			{
				default:
				case CppStandardVersion.Cpp20:
					OutFile.AddField("cppStandard", "c++20");
					break;
				case CppStandardVersion.Cpp23:
					OutFile.AddField("cppStandard", "c++23");
					break;
				case CppStandardVersion.Latest:
					OutFile.AddField("cppStandard", "c++26");
					break;
			}

			string CompilerPrefix = HostPlatform == UnrealTargetPlatform.Win64 ? "msvc" : "clang";
			string ArchSuffix = UnrealArch.Host.Value.bIsX64 ? "x64" : "arm64";
			// make msvc-x64, clang-arm64, etc
			OutFile.AddField("intelliSenseMode", $"{CompilerPrefix}-{ArchSuffix}");

			if (bUseVSCodeExtension)
			{
				OutFile.AddField("configurationProvider", "epic.ue");
			}

			if (ShouldGenerateIntelliSenseData() && CompilerPath != null)
			{
				FileReference CompileCommands = FileReference.Combine(OutputDirectory, String.Format("compileCommands_{0}.json", ProjectName));
				WriteCompileCommands(CompileCommands, SourceFiles, CompilerPath, ModuleCommandLines);
				OutFile.AddField("compileCommands", MakePathString(CompileCommands, bInAbsolute: true, bForceSkipQuotes: true));
			}

			OutFile.EndObject();
		}

		private void WriteNativeTaskDeployAndroid(JsonFile OutFile, ProjectData.Target Target, ProjectData.BuildProduct BuildProduct)
		{
			if (BuildProduct.UProjectFile == null)
			{
				return;
			}

			string[] ConfigTypes = new string[] { "Cook+Deploy", "Cook", "Deploy" };

			foreach (string ConfigType in ConfigTypes)
			{
				OutFile.BeginObject();
				{
					string TaskName = String.Format("{0} {1} {2} {3}", Target.Name, BuildProduct.Platform.ToString(), BuildProduct.Config, ConfigType);
					OutFile.AddField("label", TaskName);
					OutFile.AddField("group", "build");

					if (HostPlatform == UnrealTargetPlatform.Win64)
					{
						OutFile.AddField("command", MakePathString(FileReference.Combine(ProjectRoot, "Engine", "Build", "BatchFiles", "RunUAT.bat")));
					}
					else
					{
						OutFile.AddField("command", MakePathString(FileReference.Combine(ProjectRoot, "Engine", "Build", "BatchFiles", "RunUAT.sh")));
					}

					OutFile.BeginArray("args");
					{
						OutFile.AddUnnamedField("BuildCookRun");
						OutFile.AddUnnamedField("-ScriptsForProject=" + BuildProduct.UProjectFile.ToNormalizedPath());
						OutFile.AddUnnamedField("-Project=" + BuildProduct.UProjectFile.ToNormalizedPath());
						OutFile.AddUnnamedField("-noP4");
						OutFile.AddUnnamedField(String.Format("-ClientConfig={0}", BuildProduct.Config.ToString()));
						OutFile.AddUnnamedField(String.Format("-ServerConfig={0}", BuildProduct.Config.ToString()));
						OutFile.AddUnnamedField("-NoCompileEditor");
						OutFile.AddUnnamedField("-utf8output");
						OutFile.AddUnnamedField(String.Format("-Platform={0}", BuildProduct.Platform.ToString()));
						OutFile.AddUnnamedField(String.Format("-TargetPlatform={0}", BuildProduct.Platform.ToString()));
						OutFile.AddUnnamedField("-Compressed");
						OutFile.AddUnnamedField("-IterativeCooking");
						OutFile.AddUnnamedField("-IterativeDeploy");
						switch (ConfigType)
						{
							case "Cook+Deploy":
								{
									OutFile.AddUnnamedField("-Cook");
									OutFile.AddUnnamedField("-Stage");
									OutFile.AddUnnamedField("-Deploy");
									break;
								}
							case "Cook":
								{
									OutFile.AddUnnamedField("-Cook");
									break;
								}
							case "Deploy":
								{
									OutFile.AddUnnamedField("-DeploySoToDevice");
									OutFile.AddUnnamedField("-SkipCook");
									OutFile.AddUnnamedField("-Stage");
									OutFile.AddUnnamedField("-Deploy");
									break;
								}
						}
					}
					OutFile.EndArray();
					OutFile.BeginArray("dependsOn");
					{
						switch (ConfigType)
						{
							case "Cook+Deploy":
							case "Cook":
								{
									OutFile.AddUnnamedField(String.Format("{0}Editor {1} Development Build", Target.Name, HostPlatform.ToString()));
									OutFile.AddUnnamedField(String.Format("{0} {1} {2} Build", Target.Name, BuildProduct.Platform.ToString(), BuildProduct.Config));
									break;
								}
							default:
								{
									OutFile.AddUnnamedField(String.Format("{0} {1} {2} Build", Target.Name, BuildProduct.Platform.ToString(), BuildProduct.Config));
									break;
								}
						}
					}
					OutFile.EndArray();

					OutFile.AddField("type", "shell");

					OutFile.BeginObject("options");
					{
						OutFile.AddField("cwd", MakeUnquotedPathString(ProjectRoot, EPathType.Absolute));
					}
					OutFile.EndObject();
				}
				OutFile.EndObject();
			}
		}

		private void WriteCompileCommands(FileReference CompileCommandsFile, IEnumerable<FileReference> SourceFiles,
			FileReference CompilerPath, Dictionary<DirectoryReference, string> ModuleCommandLines)
		{
			// this creates a compileCommands.json
			// see VsCode Docs - https://code.visualstudio.com/docs/cpp/c-cpp-properties-schema-reference (compileCommands attribute)
			// and the clang format description https://clang.llvm.org/docs/JSONCompilationDatabase.html

			using (JsonWriter Writer = new JsonWriter(CompileCommandsFile))
			{
				Writer.WriteArrayStart();

				DirectoryReference ResponseFileDir = DirectoryReference.Combine(CompileCommandsFile.Directory, CompileCommandsFile.GetFileNameWithoutExtension());
				DirectoryReference.CreateDirectory(ResponseFileDir);

				Dictionary<DirectoryReference, FileReference?> DirectoryToResponseFile = new Dictionary<DirectoryReference, FileReference?>();
				foreach (KeyValuePair<DirectoryReference, string> Pair in ModuleCommandLines)
				{
					FileReference ResponseFile = FileReference.Combine(ResponseFileDir, String.Format("{0}.{1}.rsp", Pair.Key.GetDirectoryName(), DirectoryToResponseFile.Count));
					FileReference.WriteAllText(ResponseFile, Pair.Value);
					DirectoryToResponseFile.Add(Pair.Key, ResponseFile);
				}

				IEnumerable<FileReference> SourceFilesForClangCompilationDatabase = SourceFiles
					.Where(x => x.HasExtension(".cpp") || x.HasExtension(".cc") || x.HasExtension(".cxx") || x.HasExtension(".c"))
					.OrderBy(x => x.FullName);
				foreach (FileReference File in SourceFilesForClangCompilationDatabase)
				{
					DirectoryReference Directory = File.Directory;

					FileReference? ResponseFile = null;
					if (!DirectoryToResponseFile.TryGetValue(Directory, out ResponseFile))
					{
						for (DirectoryReference? ParentDir = Directory; ParentDir != null && ParentDir != Unreal.RootDirectory; ParentDir = ParentDir.ParentDirectory)
						{
							if (DirectoryToResponseFile.TryGetValue(ParentDir, out ResponseFile))
							{
								break;
							}
						}
						DirectoryToResponseFile[Directory] = ResponseFile;
					}

					if (ResponseFile == null)
					{
						// no compiler command associated with the file, will happen for any file that is not a C++ file and is not an error
						continue;
					}

					Writer.WriteObjectStart();
					Writer.WriteValue("file", MakePathString(File, bInAbsolute: true, bForceSkipQuotes: true));
					Writer.WriteArrayStart("arguments");
					Writer.WriteValue(MakePathString(CompilerPath, bInAbsolute: true, bForceSkipQuotes: true));
					Writer.WriteValue($"@{MakePathString(ResponseFile, bInAbsolute: true, bForceSkipQuotes: true)}");
					Writer.WriteArrayEnd();
					Writer.WriteValue("directory", Unreal.EngineSourceDirectory.ToString());
					Writer.WriteObjectEnd();
				}
				Writer.WriteArrayEnd();
			}
		}

		private void WriteNativeTask(ProjectData.Project InProject, JsonFile OutFile)
		{
			string[] Commands = { "Build", "Rebuild", "Clean" };

			foreach (ProjectData.Target Target in InProject.Targets)
			{
				foreach (ProjectData.BuildProduct BuildProduct in Target.BuildProducts)
				{
					foreach (string BaseCommand in Commands)
					{
						string Command = BaseCommand == "Rebuild" ? "Build" : BaseCommand;
						string TaskName = String.Format("{0} {1} {2} {3}", Target.Name, BuildProduct.Platform.ToString(), BuildProduct.Config, BaseCommand);
						string CleanTaskName = String.Format("{0} {1} {2} {3}", Target.Name, BuildProduct.Platform.ToString(), BuildProduct.Config, "Clean");

						OutFile.BeginObject();
						{
							OutFile.AddField("label", TaskName);
							OutFile.AddField("group", "build");

							string? CleanParam = Command == "Clean" ? "-clean" : null;

							if (HostPlatform == UnrealTargetPlatform.Win64)
							{
								OutFile.AddField("command", MakePathString(FileReference.Combine(ProjectRoot, "Engine", "Build", "BatchFiles", Command + ".bat")));
								CleanParam = null;
							}
							else
							{
								OutFile.AddField("command", MakePathString(FileReference.Combine(ProjectRoot, "Engine", "Build", "BatchFiles", HostPlatform.ToString(), "Build.sh")));

								if (Command == "Clean")
								{
									CleanParam = "-clean";
								}
							}

							OutFile.BeginArray("args");
							{
								OutFile.AddUnnamedField(Target.Name);
								OutFile.AddUnnamedField(BuildProduct.Platform.ToString());
								OutFile.AddUnnamedField(BuildProduct.Config.ToString());
								if (bForeignProject)
								{
									OutFile.AddUnnamedField(MakeUnquotedPathString(BuildProduct.UProjectFile!, EPathType.Relative, null));
								}
								OutFile.AddUnnamedField("-waitmutex");

								if (!String.IsNullOrEmpty(CleanParam))
								{
									OutFile.AddUnnamedField(CleanParam);
								}
							}
							OutFile.EndArray();
							OutFile.AddField("problemMatcher", "$msCompile");
							if (!bForeignProject || BaseCommand == "Rebuild")
							{
								OutFile.BeginArray("dependsOn");
								{
									if (!bForeignProject)
									{
										if (Command == "Build" && Target.Type == TargetType.Editor)
										{
											OutFile.AddUnnamedField("ShaderCompileWorker " + HostPlatform.ToString() + " Development Build");
										}
										else
										{
											OutFile.AddUnnamedField("UnrealBuildTool " + HostPlatform.ToString() + " Development Build");
										}
									}

									if (BaseCommand == "Rebuild")
									{
										OutFile.AddUnnamedField(CleanTaskName);
									}
								}
								OutFile.EndArray();
							}

							OutFile.AddField("type", "shell");

							OutFile.BeginObject("options");
							{
								OutFile.AddField("cwd", MakeUnquotedPathString(ProjectRoot, EPathType.Absolute));
							}
							OutFile.EndObject();
						}
						OutFile.EndObject();

						if (BuildProduct.Platform == UnrealTargetPlatform.Android && BaseCommand.Equals("Build", StringComparison.Ordinal))
						{
							WriteNativeTaskDeployAndroid(OutFile, Target, BuildProduct);
						}
					}
				}
			}
		}

		private void WriteCSharpTask(ProjectData.Project InProject, JsonFile OutFile)
		{
			string[] Commands = { "Build", "Clean" };

			foreach (ProjectData.Target Target in InProject.Targets)
			{
				foreach (ProjectData.BuildProduct BuildProduct in Target.BuildProducts)
				{
					foreach (string Command in Commands)
					{
						string TaskName = String.Format("{0} {1} {2} {3}", Target.Name, BuildProduct.Platform, BuildProduct.Config, Command);

						OutFile.BeginObject();
						{
							OutFile.AddField("label", TaskName);
							OutFile.AddField("group", "build");
							OutFile.AddField("command", MakePathString(Unreal.DotnetPath));
							OutFile.BeginArray("args");
							{
#pragma warning disable CA1308 // Normalize strings to uppercase - preserve existing behaviour as it may be case-sensitive
								OutFile.AddUnnamedField(Command.ToLowerInvariant());
#pragma warning restore CA1308

								OutFile.AddUnnamedField("--configuration");
								OutFile.AddUnnamedField(BuildProduct.Config.ToString());
								OutFile.AddUnnamedField(MakeUnquotedPathString(BuildProduct.CSharpInfo!.ProjectPath, EPathType.Absolute));
							}
							OutFile.EndArray();

							OutFile.AddField("problemMatcher", "$msCompile");
							OutFile.AddField("type", "shell");

							OutFile.BeginObject("options");
							{
								OutFile.AddField("cwd", MakeUnquotedPathString(ProjectRoot, EPathType.Absolute));
								OutFile.BeginObject("env");
								{
									OutFile.AddField("PATH", $"{Unreal.DotnetDirectory}{Path.PathSeparator}${{env:PATH}}");
									OutFile.AddField("DOTNET_MULTILEVEL_LOOKUP", "0");
									OutFile.AddField("DOTNET_ROLL_FORWARD", "LatestMajor");
								}
								OutFile.EndObject();
							}
							OutFile.EndObject();
						}
						OutFile.EndObject();
					}
				}
			}

			if (InProject.Name.Equals("AutomationTool", StringComparison.OrdinalIgnoreCase))
			{
				WriteAutomationScriptTask(InProject, OutFile);
			}
		}

		private void WriteAutomationScriptTask(ProjectData.Project InProject, JsonFile OutFile)
		{
			string[] Commands = { "Build" };

			foreach (ProjectData.Target Target in InProject.Targets)
			{
				foreach (ProjectData.BuildProduct BuildProduct in Target.BuildProducts.Where(x => x.OutputType == ProjectData.EOutputType.Exe))
				{
					foreach (string Command in Commands)
					{
						string TaskName = String.Format("{0}+Scripts {1} {2} {3}", Target.Name, BuildProduct.Platform, BuildProduct.Config, Command);
						string DependsOnTaskName = String.Format("{0} {1} {2} {3}", Target.Name, BuildProduct.Platform, BuildProduct.Config, Command);

						OutFile.BeginObject();
						{
							OutFile.AddField("label", TaskName);
							OutFile.AddField("group", "build");
							OutFile.AddField("command", MakePathString(BuildProduct.OutputFile));
							OutFile.BeginArray("args");
							{
								OutFile.AddUnnamedField("-list");
							}
							OutFile.EndArray();
							OutFile.BeginArray("dependsOn");
							{
								OutFile.AddUnnamedField(DependsOnTaskName);
							}
							OutFile.EndArray();

							OutFile.AddField("problemMatcher", "$msCompile");
							OutFile.AddField("type", "shell");

							OutFile.BeginObject("options");
							{
								OutFile.AddField("cwd", MakeUnquotedPathString(ProjectRoot, EPathType.Absolute));
								OutFile.BeginObject("env");
								{
									OutFile.AddField("PATH", $"{Unreal.DotnetDirectory}{Path.PathSeparator}${{env:PATH}}");
									OutFile.AddField("DOTNET_MULTILEVEL_LOOKUP", "0");
									OutFile.AddField("DOTNET_ROLL_FORWARD", "LatestMajor");
								}
								OutFile.EndObject();
							}
							OutFile.EndObject();
						}
						OutFile.EndObject();
					}
				}
			}
		}

		private void WriteTasks(JsonFile OutFile, ProjectData ProjectData)
		{
			OutFile.AddField("version", "2.0.0");

			OutFile.BeginArray("tasks");
			{
				if (!bUseVSCodeExtension)
				{
					foreach (ProjectData.Project NativeProject in ProjectData.NativeProjects)
					{
						WriteNativeTask(NativeProject, OutFile);
					}
				}

				foreach (ProjectData.Project CSharpProject in ProjectData.CSharpProjects)
				{
					WriteCSharpTask(CSharpProject, OutFile);
				}

				OutFile.EndArray();
			}
		}

		private static FileReference GetExecutableFilename(ProjectFile Project, ProjectTarget Target, UnrealTargetPlatform Platform, UnrealTargetConfiguration Configuration)
		{
			TargetRules? TargetRulesObject = Target.TargetRules;
			FileReference TargetFilePath = Target.TargetFilePath;
			string TargetName = TargetFilePath == null ? Project.ProjectFilePath.GetFileNameWithoutExtension() : TargetFilePath.GetFileNameWithoutAnyExtensions();
			string UBTPlatformName = Platform.ToString();

			// Setup output path
			UEBuildPlatform BuildPlatform = UEBuildPlatform.GetBuildPlatform(Platform);

			// Figure out if this is a monolithic build
			bool bShouldCompileMonolithic = BuildPlatform.ShouldCompileMonolithicBinary(Platform);

			if (TargetRulesObject != null)
			{
				try
				{
					bShouldCompileMonolithic |= (Target.CreateRulesDelegate(Platform, Configuration).LinkType == TargetLinkType.Monolithic);
				}
				catch (BuildException)
				{
				}
			}

			TargetType TargetRulesType = Target.TargetRules == null ? TargetType.Program : Target.TargetRules.Type;

			// Get the output directory
			DirectoryReference RootDirectory = Unreal.EngineDirectory;
			if (TargetRulesType != TargetType.Program && (bShouldCompileMonolithic || TargetRulesObject!.BuildEnvironment == TargetBuildEnvironment.Unique))
			{
				if (Target.UnrealProjectFilePath != null)
				{
					RootDirectory = Target.UnrealProjectFilePath.Directory;
				}
			}

			if (TargetRulesType == TargetType.Program)
			{
				if (Target.UnrealProjectFilePath != null)
				{
					RootDirectory = Target.UnrealProjectFilePath.Directory;
				}
			}

			// Get the output directory
			DirectoryReference OutputDirectory = DirectoryReference.Combine(RootDirectory, "Binaries", UBTPlatformName);

			// Get the executable name (minus any platform or config suffixes)
			string BinaryName;
			if (Target.TargetRules!.BuildEnvironment == TargetBuildEnvironment.Shared && TargetRulesType != TargetType.Program)
			{
				BinaryName = UEBuildTarget.GetAppNameForTargetType(TargetRulesType);
			}
			else
			{
				BinaryName = TargetName;
			}

			// Make the output file path
			string BinaryFileName = UEBuildTarget.MakeBinaryFileName(BinaryName, TargetRulesObject!.DecoratedSeparator, Platform, Configuration, TargetRulesObject!.Architectures, TargetRulesObject.UndecoratedConfiguration, UEBuildBinaryType.Executable);
			string ExecutableFilename = FileReference.Combine(OutputDirectory, BinaryFileName).FullName;

			// Include the path to the actual executable for a Mac app bundle
			if (Platform == UnrealTargetPlatform.Mac && !Target.TargetRules.bIsBuildingConsoleApplication)
			{
				ExecutableFilename += ".app/Contents/MacOS/" + Path.GetFileName(ExecutableFilename);
			}

			return new FileReference(ExecutableFilename);
		}

		private void WriteNativeLaunchConfigAndroidOculus(JsonFile OutFile, ProjectData.Target Target, ProjectData.BuildProduct BuildProduct)
		{
			ConfigHierarchy Ini = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, DirectoryReference.FromFile(BuildProduct.UProjectFile), BuildProduct.Platform);

			// Check if packaging for Meta Quest
			bool BoolResult = Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bPackageForMetaQuest", out bool bPackageForMetaQuest);
			if (!BoolResult || !bPackageForMetaQuest)
			{
				return;
			}

			// Get package name
			string PackageName;
			Ini.GetString("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "PackageName", out PackageName);
			if (PackageName.Contains("[PROJECT]", StringComparison.Ordinal))
			{
				// project name must start with a letter
				if (!Char.IsLetter(Target.Name[0]))
				{
					Trace.TraceWarning("Package name segments must all start with a letter. Please replace [PROJECT] with a valid name");
				}

				string ProjectName = Target.Name;
				// hyphens not allowed so change them to underscores in project name
				if (ProjectName.Contains('-', StringComparison.Ordinal))
				{
					Trace.TraceWarning("Project name contained hyphens, converted to underscore");
					ProjectName = ProjectName.Replace("-", "_", StringComparison.Ordinal);
				}

				// check for special characters
				for (int Index = 0; Index < ProjectName.Length; Index++)
				{
					char c = ProjectName[Index];
					if (c != '.' && c != '_' && !Char.IsLetterOrDigit(c))
					{
						Trace.TraceWarning("Project name contains illegal characters (only letters, numbers, and underscore allowed); please replace [PROJECT] with a valid name");
						ProjectName = ProjectName.Replace(c, '_');
					}
				}

				PackageName = PackageName.Replace("[PROJECT]", ProjectName, StringComparison.Ordinal);
			}

			// Get store version
			int StoreVersionArm64;
			Ini.GetInt32("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "StoreVersion", out int StoreVersion);
			Ini.GetInt32("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "StoreVersionOffsetArm64", out int StoreVersionOffsetArm64);
			StoreVersionArm64 = StoreVersion + StoreVersionOffsetArm64;

			DirectoryReference SymbolPathArm64 = DirectoryReference.Combine(
				BuildProduct.OutputFile.Directory,
				Target.Name + "_Symbols_v" + StoreVersionArm64.ToString(),
				Target.Name + "-arm64");

			string LaunchTaskName = String.Format("{0} {1} {2} Deploy", Target.Name, BuildProduct.Platform, BuildProduct.Config);

			List<string> ConfigTypes = new List<string>();
			ConfigTypes.Add("Launch");
			if (BuildProduct.Config == UnrealTargetConfiguration.Development)
			{
				ConfigTypes.Add("Attach");
			}

			foreach (string ConfigType in ConfigTypes)
			{
				OutFile.BeginObject();
				{
					OutFile.AddField("name", Target.Name + " Oculus (" + BuildProduct.Config.ToString() + ") " + ConfigType);
#pragma warning disable CA1308 // Normalize strings to uppercase - should maintain compatibility with the output file schema
					OutFile.AddField("request", ConfigType.ToLowerInvariant());
#pragma warning restore CA1308
					if (ConfigType == "Launch")
					{
						OutFile.AddField("preLaunchTask", LaunchTaskName);
					}
					OutFile.AddField("type", "fb-lldb");

					OutFile.BeginObject("android");
					{
						OutFile.BeginObject("application");
						{
							OutFile.AddField("package", PackageName);
							OutFile.AddField("activity", "com.epicgames.unreal.GameActivity");
						}
						OutFile.EndObject();

						OutFile.BeginObject("lldbConfig");
						{
							OutFile.BeginArray("librarySearchPaths");
							OutFile.AddUnnamedField("\\\"" + SymbolPathArm64.ToNormalizedPath() + "\\\"");
							OutFile.EndArray();

							OutFile.BeginArray("lldbPreTargetCreateCommands");
							FileReference DataFormatters = FileReference.Combine(ProjectRoot, "Engine", "Extras", "LLDBDataFormatters", "UEDataFormatters_2ByteChars.py");
							OutFile.AddUnnamedField("command script import \\\"" + DataFormatters.FullName.Replace("\\", "/", StringComparison.Ordinal) + "\\\"");
							OutFile.EndArray();

							OutFile.BeginArray("lldbPostTargetCreateCommands");
							//on Oculus devices, we use SIGILL for input redirection, so the debugger shouldn't catch it.
							OutFile.AddUnnamedField("process handle --pass true --stop false --notify true SIGILL");
							OutFile.EndArray();
						}
						OutFile.EndObject();
					}
					OutFile.EndObject();
				}
				OutFile.EndObject();
			}
		}

		private void WriteNativeLaunchConfig(ProjectData.Project InProject, JsonFile OutFile, ProjectData.Target Target, ProjectData.BuildProduct BuildProduct)
		{
			bool bIsLinux = BuildProduct.Platform == UnrealTargetPlatform.Linux;
			List<string> Types = new List<string>();
			Types.Add("Launch");

			if (bAddDebugAttachConfig && bIsLinux)
			{
				Types.Add("Attach");
			}

			if (bAddDebugCoreConfig && bIsLinux)
			{
				Types.Add("Debug Core");
			}

			string LaunchTaskName = String.Format("{0} {1} {2} Build", Target.Name, BuildProduct.Platform, BuildProduct.Config);

			foreach (string Type in Types)
			{
				OutFile.BeginObject();
				{
					OutFile.AddField("name", Type + " " + Target.Name + " (" + BuildProduct.Config.ToString() + ")");
					OutFile.AddField("request", (Type == "Attach") ? "attach" : "launch");
					OutFile.AddField("program", MakeUnquotedPathString(BuildProduct.OutputFile, EPathType.Absolute));
					switch (Type)
					{
						case "Launch":
							OutFile.AddField("preLaunchTask", LaunchTaskName);
							break;
						case "Debug Core":
							OutFile.AddField("coreDumpPath", "${input:coreFileName}");
							break;
						case "Attach":
							OutFile.AddField("processId", "${command:pickProcess}");
							break;
					}

					if (Type != "Attach")
					{
						OutFile.BeginArray("args");
						{
							if (Target.Type == TargetRules.TargetType.Editor)
							{
								if (InProject.Name != "UE5")
								{
									if (bForeignProject)
									{
										OutFile.AddUnnamedField(MakePathString(BuildProduct.UProjectFile!, false, true));
									}
									else
									{
										OutFile.AddUnnamedField(InProject.Name);
									}
								}
							}
						}
						OutFile.EndArray();
					}

					/*
									DirectoryReference CWD = BuildProduct.OutputFile.Directory;
									while (HostPlatform == UnrealTargetPlatform.Mac && CWD != null && CWD.ToString().Contains(".app"))
									{
										CWD = CWD.ParentDirectory;
									}
									if (CWD != null)
									{
										OutFile.AddField("cwd", MakePathString(CWD, true, true));
									}
					*/
					OutFile.AddField("cwd", MakeUnquotedPathString(ProjectRoot, EPathType.Absolute));

					if (HostPlatform == UnrealTargetPlatform.Win64)
					{
						OutFile.AddField("stopAtEntry", false);
						OutFile.AddField("console", "integratedTerminal");

						OutFile.AddField("type", "cppvsdbg");
						OutFile.AddField("visualizerFile", MakeUnquotedPathString(FileReference.Combine(ProjectRoot, "Engine", "Extras", "VisualStudioDebugging", "Unreal.natvis"), EPathType.Absolute));
					}
					else if (HostPlatform == UnrealTargetPlatform.Linux)
					{
						OutFile.AddField("type", LinuxDebuggerType);
						OutFile.AddField("visualizerFile", MakeUnquotedPathString(FileReference.Combine(ProjectRoot, "Engine", "Extras", "VisualStudioDebugging", "Unreal.natvis"), EPathType.Absolute));
						OutFile.AddField("showDisplayString", true);
					}
					else
					{
						OutFile.AddField("type", "lldb");
					}

					if (Target.TargetRules is not null)
					{ 
						Dictionary<DirectoryReference, DirectoryReference> SourceFileMap = GenerateSourceMap(Target.TargetRules, BuildProduct.UProjectFile);
						if (SourceFileMap.Count != 0)
						{
							// https://github.com/llvm/vscode-lldb/blob/3c2eb665730175c3dacf624d760c22e8837f1d7b/README.md?plain=1#L236
							// https://code.visualstudio.com/docs/cpp/launch-json-reference#_sourcefilemap
							// The documentation disagrees on whether this should be `sourceMap` or `sourceFileMap`.
							// From testing, lldb uses `sourceMap`, gdb uses 'sourceFileMap', so write both.
							WriteSourceMap(OutFile, SourceFileMap, "sourceFileMap");
							WriteSourceMap(OutFile, SourceFileMap, "sourceMap");
						}
					}
				}
				OutFile.EndObject();
			}
		}

		private void WriteSourceMap(JsonFile OutFile, Dictionary<DirectoryReference, DirectoryReference> SourceFileMap, string FieldName)
		{
			OutFile.BeginObject(FieldName);

			foreach (KeyValuePair<DirectoryReference, DirectoryReference> entry in SourceFileMap)
			{
				OutFile.AddField(
					MakeUnquotedPathString(entry.Key, EPathType.Absolute),
					MakeUnquotedPathString(entry.Value, EPathType.Absolute));
			}

			OutFile.EndObject();
		}

		private void WriteNativeLaunchConfig(ProjectData.Project InProject, JsonFile OutFile)
		{
			foreach (ProjectData.Target Target in InProject.Targets)
			{
				foreach (ProjectData.BuildProduct BuildProduct in Target.BuildProducts)
				{
					if (BuildProduct.Platform == HostPlatform)
					{
						WriteNativeLaunchConfig(InProject, OutFile, Target, BuildProduct);
					}
					else if (BuildProduct.Platform == UnrealTargetPlatform.Android)
					{
						WriteNativeLaunchConfigAndroidOculus(OutFile, Target, BuildProduct);
					}
				}
			}
		}

		private void WriteSingleCSharpLaunchConfig(JsonFile OutFile, string InTaskName, string InBuildTaskName, FileReference InExecutable, string[]? InArgs)
		{
			OutFile.BeginObject();
			{
				OutFile.AddField("name", InTaskName);
				OutFile.AddField("type", "coreclr");
				OutFile.AddField("request", "launch");

				if (!String.IsNullOrEmpty(InBuildTaskName))
				{
					OutFile.AddField("preLaunchTask", InBuildTaskName);
				}

				DirectoryReference CWD = ProjectRoot;

				OutFile.AddField("program", MakeUnquotedPathString(InExecutable, EPathType.Absolute));
				OutFile.BeginArray("args");
				{
					if (InArgs != null)
					{
						foreach (string Arg in InArgs)
						{
							OutFile.AddUnnamedField(Arg);
						}
					}
				}
				OutFile.EndArray();
				OutFile.BeginObject("env");
				{
					OutFile.AddField("PATH", $"{Unreal.DotnetDirectory}{Path.PathSeparator}${{env:PATH}}");
					OutFile.AddField("DOTNET_ROOT", Unreal.DotnetDirectory.FullName);
					OutFile.AddField("DOTNET_HOST_PATH", Unreal.DotnetPath.FullName);
					OutFile.AddField("DOTNET_MULTILEVEL_LOOKUP", "0");
					OutFile.AddField("DOTNET_ROLL_FORWARD", "LatestMajor");
				}
				OutFile.EndObject();
				if (HostPlatform == UnrealTargetPlatform.Win64)
				{
					OutFile.AddField("console", "integratedTerminal");
				}
				else
				{
					OutFile.AddField("console", "internalConsole");
					OutFile.AddField("internalConsoleOptions", "openOnSessionStart");
				}

				OutFile.AddField("stopAtEntry", false);

				OutFile.AddField("cwd", MakeUnquotedPathString(CWD, EPathType.Absolute));
			}
			OutFile.EndObject();
		}

		private void WriteCSharpLaunchConfig(ProjectData.Project InProject, JsonFile OutFile)
		{
			foreach (ProjectData.Target Target in InProject.Targets)
			{
				foreach (ProjectData.BuildProduct BuildProduct in Target.BuildProducts)
				{
					if (BuildProduct.OutputType == ProjectData.EOutputType.Exe)
					{
						string TaskName = String.Format("{0} ({1})", Target.Name, BuildProduct.Config);
						string BuildTaskName = String.Format("{0} {1} {2} Build", Target.Name, HostPlatform, BuildProduct.Config);
						string[]? Args = null;
						if (Target.Name.Equals("AutomationTool", StringComparison.OrdinalIgnoreCase))
						{
							BuildTaskName = String.Format("{0}+Scripts {1} {2} Build", Target.Name, HostPlatform, BuildProduct.Config);
							Args = new string[] {"-NoCompile"};
						}

						WriteSingleCSharpLaunchConfig(OutFile, TaskName, BuildTaskName, BuildProduct.OutputFile, Args);
					}
				}
			}
		}

		private void WriteLaunch(JsonFile OutFile, ProjectData ProjectData)
		{
			OutFile.AddField("version", "0.2.0");
			if (bAddDebugCoreConfig)
			{
				OutFile.BeginArray("inputs");
				OutFile.BeginObject();
				OutFile.AddField("id", "coreFileName");
				OutFile.AddField("type", "command");
				OutFile.AddField("command", "filePicker.pick");
				OutFile.BeginObject("args");
				OutFile.AddField("masks", "core*");
				OutFile.BeginObject("display");
				OutFile.AddField("type", "fileRelativePath");
				OutFile.AddField("detail", "filePath");
				OutFile.EndObject();
				OutFile.AddField("output", "filePath");
				OutFile.EndObject();
				OutFile.EndObject();
				OutFile.EndArray();
			}

			OutFile.BeginArray("configurations");
			{
				if (!bUseVSCodeExtension)
				{
					foreach (ProjectData.Project Project in ProjectData.NativeProjects)
					{
						WriteNativeLaunchConfig(Project, OutFile);
					}
				}

				foreach (ProjectData.Project Project in ProjectData.CSharpProjects)
				{
					WriteCSharpLaunchConfig(Project, OutFile);
				}
			}

			// Add in a special task for regenerating project files
			string PreLaunchTask = "";
			List<string> Args = new List<string>();
			Args.Add("-projectfiles");
			Args.Add("-vscode");

			if (bGeneratingGameProjectFiles)
			{
				Args.Add("-project=" + MakeUnquotedPathString(OnlyGameProject!, EPathType.Absolute));
				Args.Add("-game");
			}
			if (bIncludeEngineSource)
			{
				Args.Add("-engine");
			}

			if (bIncludeDotNetPrograms)
			{
				Args.Add("-dotnet");
				PreLaunchTask = "UnrealBuildTool " + HostPlatform.ToString() + " Development Build";
			}

			FileReference RunUbtPath;
			string ScriptExt = HostPlatform == UnrealTargetPlatform.Win64 ? ".bat" : ".sh";
			RunUbtPath = FileReference.Combine(ProjectRoot, "Engine", "Build", "BatchFiles", $"RunUBT{ScriptExt}");

			WriteSingleCSharpLaunchConfig(
				OutFile,
				"Generate Project Files",
				PreLaunchTask,
				RunUbtPath,
				Args.ToArray()
			);

			OutFile.EndArray();
		}

		private void WriteWorkspaceIgnoreFile(List<ProjectFile> Projects)
		{
			List<string> PathsToExclude = new List<string>();

			foreach (ProjectFile Project in Projects)
			{
				bool bFoundTarget = false;
				foreach (ProjectTarget Target in Project.ProjectTargets.OfType<ProjectTarget>())
				{
					if (Target.TargetFilePath != null)
					{
						DirectoryReference ProjDir = Target.TargetFilePath.Directory.GetDirectoryName() == "Source" ? Target.TargetFilePath.Directory.ParentDirectory! : Target.TargetFilePath.Directory;
						GetExcludePathsCPP(ProjDir, PathsToExclude);

						DirectoryReference PluginRootDir = DirectoryReference.Combine(ProjDir, "Plugins");
						WriteWorkspaceIgnoreFileForPlugins(PluginRootDir, PathsToExclude);

						bFoundTarget = true;
					}
				}

				if (!bFoundTarget)
				{
					GetExcludePathsCSharp(Project.ProjectFilePath.Directory.ToString(), PathsToExclude);
				}
			}

			StringBuilder OutFile = new StringBuilder();
			if (!IncludeAllFiles)
			{
				// TODO: Adding ignore patterns to .ignore hides files from Open File Dialog but it does not hide them in the File Explorer
				// but using files.exclude with our full set of excludes breaks vscode for larger code bases so a verbose file explorer
				// seems like less of an issue and thus we are not adding these to files.exclude.
				// see https://github.com/microsoft/vscode/issues/109380 for discussions with vscode team
				DirectoryReference WorkspaceRoot = bForeignProject ? Projects[0].BaseDir : Unreal.RootDirectory;
				string WorkspaceRootPath = WorkspaceRoot.ToString().Replace('\\', '/') + "/";

				if (!bForeignProject)
				{
					OutFile.AppendLine(".vscode");
				}

				foreach (string PathToExclude in PathsToExclude)
				{
					OutFile.AppendLine(PathToExclude.Replace('\\', '/').Replace(WorkspaceRootPath, "/", StringComparison.Ordinal));
				}
			}
			FileReference.WriteAllText(FileReference.Combine(PrimaryProjectPath, ".ignore"), OutFile.ToString());
		}

		private static void WriteWorkspaceIgnoreFileForPlugins(DirectoryReference PluginBaseDir, List<string> PathsToExclude)
		{
			if (DirectoryReference.Exists(PluginBaseDir))
			{
				foreach (DirectoryReference SubDir in DirectoryReference.EnumerateDirectories(PluginBaseDir, "*", SearchOption.TopDirectoryOnly))
				{
					string[] UPluginFiles = Directory.GetFiles(SubDir.ToString(), "*.uplugin");
					if (UPluginFiles.Length == 1)
					{
						DirectoryReference PluginDir = SubDir;
						GetExcludePathsCPP(PluginDir, PathsToExclude);
					}
					else
					{
						WriteWorkspaceIgnoreFileForPlugins(SubDir, PathsToExclude);
					}
				}
			}
		}

		private void WriteWorkspaceFile(ProjectData ProjectData)
		{
			JsonFile WorkspaceFile = new JsonFile();

			WorkspaceFile.BeginRootObject();
			{
				WorkspaceFile.BeginArray("folders");
				{
					// Add the directory in which which the code-workspace file exists.
					// This is also known as ${workspaceRoot}
					WorkspaceFile.BeginObject();
					{
						string ProjectName = bForeignProject ? GameProjectName! : "UE5";
						WorkspaceFile.AddField("name", ProjectName);
						WorkspaceFile.AddField("path", ".");
					}
					WorkspaceFile.EndObject();

					// If this project is outside the engine folder, add the root engine directory
					if (bIncludeEngineSource && bForeignProject)
					{
						WorkspaceFile.BeginObject();
						{
							WorkspaceFile.AddField("name", "UE5");
							WorkspaceFile.AddField("path", MakeUnquotedPathString(Unreal.RootDirectory, EPathType.Absolute));
						}
						WorkspaceFile.EndObject();
					}
				}
				WorkspaceFile.EndArray();
			}

			WorkspaceFile.BeginObject("settings");
			{
				// disable autodetect for javascript/typescript files to workaround slowdown in vscode as a result of parsing all files
				WorkspaceFile.AddField("js/ts.tsc.autoDetect", "off");
				// disable npm script autodetect to avoid lag populating tasks list 
				WorkspaceFile.AddField("npm.autoDetect", "off");
				// set environment variables so correct dotnet is used
				WorkspaceFile.BeginObject($"terminal.integrated.env.{(OperatingSystem.IsWindows() ? "windows" : OperatingSystem.IsMacOS() ? "osx" : "linux")}");
				{
					WorkspaceFile.AddField("PATH", $"{Unreal.DotnetDirectory}{Path.PathSeparator}${{env:PATH}}");
					WorkspaceFile.AddField("DOTNET_ROOT", Unreal.DotnetDirectory.FullName);
					WorkspaceFile.AddField("DOTNET_HOST_PATH", Unreal.DotnetPath.FullName);
					WorkspaceFile.AddField("DOTNET_MULTILEVEL_LOOKUP", "0");
					WorkspaceFile.AddField("DOTNET_ROLL_FORWARD", "LatestMajor");
				}
				WorkspaceFile.EndObject();

				if (bUseVSCodeExtension)
				{
					if (HostPlatform == UnrealTargetPlatform.Win64)
					{
						WorkspaceFile.AddField("UE.UBTScriptPath", MakePathString(FileReference.Combine(ProjectRoot, "Engine", "Build", "BatchFiles", "RunUBT.bat"), true));
					}
					else
					{
						WorkspaceFile.AddField("UE.UBTScriptPath", MakePathString(FileReference.Combine(ProjectRoot, "Engine", "Build", "BatchFiles", HostPlatform.ToString(), "RunUBT.sh"), true));
					}

					// Exclude some large directories/filetypes by default
					WorkspaceFile.BeginObject("files.exclude");
					WorkspaceFile.AddField("**/Intermediate/", true);
					WorkspaceFile.AddField("**/Binaries/", true);
					WorkspaceFile.AddField("Engine/DerivedDataCache", true);
					WorkspaceFile.AddField("**/*.uasset", true);
					WorkspaceFile.AddField("**/*.umap", true);
					WorkspaceFile.AddField("**/*.uexp", true);
					WorkspaceFile.AddField("**/*.upayload", true);
					WorkspaceFile.AddField("**/*.ubulk", true);
					WorkspaceFile.AddField("**/*.m.ubulk", true);
					WorkspaceFile.AddField("**/*.uptnl", true);
					WorkspaceFile.EndObject();
					WorkspaceFile.BeginObject("search.exclude");
					WorkspaceFile.EndObject();
				}
			}
			WorkspaceFile.EndObject();

			WorkspaceFile.BeginObject("extensions");
			{
				// extensions is a set of recommended extensions that a user should install.
				// Adding this section aids discovery of extensions which are helpful to have installed for Unreal development.
				WorkspaceFile.BeginArray("recommendations");
				{
					// Add when/if published to marketplace. 
					// WorkspaceFile.AddUnnamedField("epic.vscode-ue");
					WorkspaceFile.AddUnnamedField("ms-vscode.cpptools");
					WorkspaceFile.AddUnnamedField("ms-dotnettools.csharp");

					if (bUseVSCodeExtension)
					{
						WorkspaceFile.AddUnnamedField("epic.vscode-ue");
					}
				}
				WorkspaceFile.EndArray();
			}
			WorkspaceFile.EndObject();

			WorkspaceFile.BeginObject("tasks");
			WriteTasks(WorkspaceFile, ProjectData);
			WorkspaceFile.EndObject();

			WorkspaceFile.BeginObject("launch");
			WriteLaunch(WorkspaceFile, ProjectData);
			WorkspaceFile.EndObject();

			WorkspaceFile.EndRootObject();

			string? WorkspaceName = bForeignProject ? GameProjectName : PrimaryProjectName;
			WorkspaceFile.Write(FileReference.Combine(PrimaryProjectPath, WorkspaceName + ".code-workspace"));
		}

		private static void GetExcludePathsCPP(DirectoryReference BaseDir, List<string> PathsToExclude)
		{
			string[] DirAllowList = { "Binaries", "Build", "Config", "Plugins", "Source", "Private", "Public", "Internal", "Classes", "Resources", "Restricted" };
			foreach (DirectoryReference SubDir in DirectoryReference.EnumerateDirectories(BaseDir, "*", SearchOption.TopDirectoryOnly))
			{
				if (Array.Find(DirAllowList, Dir => Dir == SubDir.GetDirectoryName()) == null)
				{
					string NewSubDir = SubDir.ToString();
					if (!PathsToExclude.Contains(NewSubDir))
					{
						PathsToExclude.Add(NewSubDir);
					}
				}
			}
		}

		private static void GetExcludePathsCSharp(string BaseDir, List<string> PathsToExclude)
		{
			string[] DenyList =
			{
				"obj",
				"bin"
			};

			foreach (string DenyListDir in DenyList)
			{
				string ExcludePath = Path.Combine(BaseDir, DenyListDir);
				if (!PathsToExclude.Contains(ExcludePath))
				{
					PathsToExclude.Add(ExcludePath);
				}
			}
		}
	}
}
