// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;

namespace UnrealBuildTool
{
	internal class AvailableConfiguration
	{
		public HashSet<string> Configurations { get; set; } = [];

		public HashSet<string> Platforms { get; set; } = [];

		public string TargetPath { get; set; } = String.Empty;

		public string ProjectPath { get; set; } = String.Empty;

		public string TargetType { get; set; } = String.Empty;
	}
}
