// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;

namespace UnrealBuildTool
{
	internal class AvailableTargetsConfigurations
	{
		public Dictionary<string, AvailableConfiguration> Targets { get; set; } = new();
	}
}
