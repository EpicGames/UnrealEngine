// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using System.Text.Json;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;
using static UnrealBuildTool.VCProjectFile;

namespace UnrealBuildTool
{
	internal class VSWorkspaceProjectFile : ProjectFile
	{
		private readonly HashSet<TargetType> TargetTypes;
		private readonly CommandLineArguments Arguments;

		/// <summary>
		/// Collection of output files for this project
		/// </summary>
		public List<ExportedTargetInfo> ExportedTargetProjects { get; set; } = new();

		public DirectoryReference RootPath { get; }

		public VSWorkspaceProjectFile(FileReference InProjectFilePath, DirectoryReference BaseDir,
			DirectoryReference RootPath, HashSet<TargetType> TargetTypes, CommandLineArguments Arguments)
			: base(InProjectFilePath, BaseDir)
		{
			this.RootPath = RootPath;
			this.TargetTypes = TargetTypes;
			this.Arguments = Arguments;
		}

		/// <summary>
		/// Collect available configurations for the project storing it at <c>availableConfigurations</c> dictionary.
		/// </summary>
		/// <remarks>
		/// Keys for the dictionary are available project names.
		/// </remarks>
		/// <param name="availableConfigurations"></param>
		/// <param name="InPlatforms"></param>
		/// <param name="InConfigurations"></param>
		public void CollectAvailableConfigurations(Dictionary<string, AvailableConfiguration> availableConfigurations,
			List<UnrealTargetPlatform> InPlatforms,
			List<UnrealTargetConfiguration> InConfigurations)
		{
			ProcessProjectFiles(
				InPlatforms,
				InConfigurations,
				// always provide null in here as we want to collect all available data.
				null,
				(UnrealTargetConfiguration Configuration, UnrealTargetPlatform Platform, ProjectTarget ProjectTarget) =>
				{
					if (!availableConfigurations.TryGetValue(ProjectTarget.Name, out AvailableConfiguration? availableConfiguration))
					{
						availableConfiguration = new AvailableConfiguration();
						availableConfigurations.Add(ProjectTarget.Name, availableConfiguration);
					}
					availableConfiguration.TargetType = ProjectTarget.TargetRules?.Type.ToString() ?? String.Empty;
					availableConfiguration.Configurations.Add(Configuration.ToString());
					availableConfiguration.Platforms.Add(Platform.ToString());
					availableConfiguration.TargetPath = ProjectTarget.TargetFilePath.FullName;
					availableConfiguration.ProjectPath = ProjectTarget.UnrealProjectFilePath?.FullName ?? String.Empty;

					return Task.CompletedTask;
				});
		}

		/// <summary>
		/// Write project file info in JSON file.
		/// For every combination of <c>UnrealTargetPlatform</c>, <c>UnrealTargetConfiguration</c> and <c>TargetType</c>
		/// will be generated separate JSON file.
		/// </summary>
		/// <remarks>
		/// * <c>TargetType.Editor</c> will be generated for current platform only and will ignore <c>UnrealTargetConfiguration.Test</c> and <c>UnrealTargetConfiguration.Shipping</c> configurations
		/// * <c>TargetType.Program</c>  will be generated for current platform only and <c>UnrealTargetConfiguration.Development</c> configuration only
		/// </remarks>
		/// <param name="InPlatforms"></param>
		/// <param name="InConfigurations"></param>
		/// <param name="PlatformProjectGenerators"></param>
		/// <param name="Minimize"></param>
		/// <param name="Logger"></param>
		/// <param name="ProjectGenerationDescriptionData"></param>
		/// <param name="VCFormat"></param>
		/// <returns></returns>
		public bool WriteProjectFile(List<UnrealTargetPlatform> InPlatforms,
			List<UnrealTargetConfiguration> InConfigurations,
			PlatformProjectGeneratorCollection PlatformProjectGenerators,
			JsonWriterStyle Minimize,
			ILogger Logger,
			Dictionary<string, AvailableConfiguration>? ProjectGenerationDescriptionData,
			VCProjectFileFormat VCFormat)
		{
			DirectoryReference ProjectRootFolder = RootPath;

			ProcessProjectFiles(
				InPlatforms,
				InConfigurations,
				ProjectGenerationDescriptionData,
				(UnrealTargetConfiguration Configuration, UnrealTargetPlatform Platform, ProjectTarget ProjectTarget) => Task.Run(async () =>
				{
					bool bBuildByDefault = ShouldBuildByDefaultForSolutionTargets && ProjectTarget.SupportedPlatforms.Contains(Platform);

					UnrealArchitectures ProjectArchitectures = UEBuildPlatform
						.GetBuildPlatform(Platform)
						.ArchitectureConfig.ActiveArchitectures(ProjectTarget.UnrealProjectFilePath, ProjectTarget.Name);

					TargetDescriptor TargetDesc = new(ProjectTarget.UnrealProjectFilePath, ProjectTarget.Name,
						Platform, Configuration, ProjectArchitectures, Arguments);
					TargetDesc.IntermediateEnvironment = UnrealIntermediateEnvironment.GenerateProjectFiles;

					try
					{
						foreach (UnrealArch Architecture in TargetDesc.Architectures.Architectures)
						{
							UEBuildTarget BuildTarget = UEBuildTarget.Create(TargetDesc, false, false, false, UnrealIntermediateEnvironment.GenerateProjectFiles, Logger);
							BuildTarget.PreBuildSetup(Logger);

							ExportedTargetInfo TargetInfo = ExportTarget(BuildTarget, Architecture, bBuildByDefault, PlatformProjectGenerators, Logger, VCFormat);

							StringBuilder sb = new();
							sb.Append(ProjectTarget.TargetFilePath.GetFileNameWithoutAnyExtensions());
							sb.Append('_');
							sb.Append(TargetInfo.Configuration);
							sb.Append('_');
							sb.Append(TargetInfo.Platform);

							if (!String.IsNullOrEmpty(TargetInfo.Architecture))
							{
								sb.Append('_');
								sb.Append(TargetInfo.Architecture);
							}

							sb.Append(".json");

							FileReference OutputFile = FileReference.Combine(ProjectRootFolder, sb.ToString());

							DirectoryReference.CreateDirectory(OutputFile.Directory);
							using FileStream Stream = new(OutputFile.FullName, FileMode.Create, FileAccess.Write);
							await JsonSerializer.SerializeAsync(Stream, TargetInfo, options: new JsonSerializerOptions()
							{
								PropertyNamingPolicy = JsonNamingPolicy.CamelCase,
								WriteIndented = Minimize == JsonWriterStyle.Readable,
							});

							lock (ExportedTargetProjects)
							{
								ExportedTargetProjects.Add(TargetInfo);
							}
						}
					}
					catch (Exception Ex)
					{
						Logger.LogWarning(
							Ex,
							"Failed to generate include data for Target:{TargetName}, Platform: {Platform}, Configuration: {Configuration}",
							TargetDesc.Name,
							Platform.ToString(),
							Configuration.ToString());
					}
				}));

			return true;
		}

		/// <summary>
		/// Filter configurations based on what is supported by the project.
		/// </summary>
		/// <param name="InPlatforms"></param>
		/// <param name="InConfigurations"></param>
		/// <param name="ProjectGenerationDescriptionData"></param>
		/// <param name="Handler"></param>
		private void ProcessProjectFiles(List<UnrealTargetPlatform> InPlatforms,
			List<UnrealTargetConfiguration> InConfigurations,
			Dictionary<string, AvailableConfiguration>? ProjectGenerationDescriptionData,
			Func<UnrealTargetConfiguration, UnrealTargetPlatform, ProjectTarget, Task> Handler)
		{
			List<Task> Tasks = new List<Task>();
			foreach (UnrealTargetConfiguration Configuration in InConfigurations)
			{
				foreach (UnrealTargetPlatform Platform in InPlatforms)
				{
					foreach (ProjectTarget ProjectTarget in ProjectTargets.OfType<ProjectTarget>())
					{
						if (TargetTypes.Any() && !TargetTypes.Contains(ProjectTarget.TargetRules!.Type))
						{
							continue;
						}

						// Skip Programs for all configs except for current platform + Development & Debug configurations
						if (ProjectTarget.TargetRules!.Type == TargetType.Program &&
							(BuildHostPlatform.Current.Platform != Platform ||
							 !(Configuration == UnrealTargetConfiguration.Development || Configuration == UnrealTargetConfiguration.Debug)))
						{
							continue;
						}

						// Skip Editor for all platforms except for current platform
						if (ProjectTarget.TargetRules.Type == TargetType.Editor &&
							(BuildHostPlatform.Current.Platform != Platform ||
							(Configuration == UnrealTargetConfiguration.Test || Configuration == UnrealTargetConfiguration.Shipping)))
						{
							continue;
						}

						// Skip invalid platforms for the current TargetType
						if (!ProjectTarget.SupportedPlatforms.Contains(Platform))
						{
							continue;
						}

						if (ShouldSkipConfiguration(ProjectGenerationDescriptionData, Configuration, Platform, ProjectTarget))
						{
							continue;
						}

						Tasks.Add(Handler(Configuration, Platform, ProjectTarget));
					}
				}

				Task.WaitAll(Tasks.ToArray());
			}
		}

		/// <summary>
		/// Write a Target to a JSON writer. Is array is empty, don't write anything
		/// </summary>
		/// <param name="Target"></param>
		/// <param name="Architecture"></param>
		/// <param name="bBuildByDefault"></param>
		/// <param name="PlatformProjectGenerators"></param>
		/// <param name="Logger">Logger for output</param>
		/// <param name="VCFormat">Project file format.</param>
		private ExportedTargetInfo ExportTarget(
			UEBuildTarget Target, UnrealArch? Architecture, bool bBuildByDefault, PlatformProjectGeneratorCollection PlatformProjectGenerators, ILogger Logger, VCProjectFileFormat VCFormat)
		{
			ExportedTargetInfo TargetInfo = new()
			{
				TargetName = Target.TargetName,
				TargetPath = Target.TargetRulesFile.FullName,
				ProjectPath = Target.ProjectFile?.FullName ?? String.Empty,
				TargetType = Target.TargetType.ToString(),
				Platform = Target.Platform.ToString(),
				VCPlatform = GetVCPlatform(Target, Architecture, PlatformProjectGenerators, VCFormat),
				Configuration = Target.Configuration.ToString(),
				VCConfiguration = GetVCConfiguration(Target, Architecture, PlatformProjectGenerators, VCFormat),
				Architecture = GetArchitecture(Target, Architecture, PlatformProjectGenerators, VCFormat),
				BuildInfo = ExportBuildInfo(Target, Architecture, PlatformProjectGenerators, bBuildByDefault, Logger)
			};

			UEToolChain TargetToolChain = Target.CreateToolchain(Target.Platform, Logger);
			CppCompileEnvironment GlobalCompileEnvironment = Target.CreateCompileEnvironmentForProjectFiles(Logger);

			HashSet<string> ModuleNames = new();
			foreach (UEBuildBinary Binary in Target.Binaries)
			{
				CppCompileEnvironment BinaryCompileEnvironment = Binary.CreateBinaryCompileEnvironment(GlobalCompileEnvironment);
				IEnumerable<UEBuildModuleCPP> CandidateModules = Binary.Modules.Where(x => x is UEBuildModuleCPP).Cast<UEBuildModuleCPP>();

				foreach (UEBuildModuleCPP ModuleCpp in CandidateModules)
				{
					if (!ModuleNames.Add(ModuleCpp.Name))
					{
						continue;
					}

					CppCompileEnvironment ModuleCompileEnvironment = ModuleCpp.CreateCompileEnvironmentForIntellisense(Target.Rules, BinaryCompileEnvironment, Logger);
					TargetInfo.ModuleToCompileSettings.Add(ModuleCpp.Name, ExportModule(ModuleCpp, TargetToolChain, ModuleCompileEnvironment));

					foreach (DirectoryItem ModuleDirectory in ModuleCpp.ModuleDirectories)
					{
						TargetInfo.DirToModule.TryAdd(ModuleDirectory.FullName, ModuleCpp.Name);
					}

					if (ModuleCpp.GeneratedCodeDirectory != null)
					{
						TargetInfo.DirToModule.TryAdd(ModuleCpp.GeneratedCodeDirectory.FullName, ModuleCpp.Name);
					}
				}
			}

			return TargetInfo;
		}

		private static bool ShouldSkipConfiguration(Dictionary<string, AvailableConfiguration>? ProjectGenerationDescriptionData,
			UnrealTargetConfiguration Configuration,
			UnrealTargetPlatform Platform,
			ProjectTarget ProjectTarget)
		{
			// No restrictions were provided, we should not skip anything.
			if (ProjectGenerationDescriptionData == null)
			{
				return false;
			}

			// Configs were provided, but we could not find it in the dictionary, so we should not skip it.
			if (!ProjectGenerationDescriptionData.TryGetValue(ProjectTarget.Name, out AvailableConfiguration? availableConfiguration))
			{
				return true;
			}

			return !availableConfiguration.Configurations.Contains(Configuration.ToString()) || !availableConfiguration.Platforms.Contains(Platform.ToString());
		}

		/// <summary>
		/// Write a Module to a JSON writer. If array is empty, don't write anything
		/// </summary>
		/// <param name="TargetToolChain"></param>
		/// <param name="ModuleCompileEnvironment"></param>
		/// <param name="Module"></param>
		private static ExportedModuleInfo ExportModule(UEBuildModuleCPP Module, UEToolChain TargetToolChain, CppCompileEnvironment ModuleCompileEnvironment)
		{
			ExportedModuleInfo Result = new()
			{
				Name = Module.Name,
				Directory = Module.ModuleDirectory.FullName,
				Rules = Module.RulesFile.FullName,
				GeneratedCodeDirectory = Module.GeneratedCodeDirectory != null ? Module.GeneratedCodeDirectory.FullName : String.Empty,
				Standard = ModuleCompileEnvironment.CppStandard.ToString(),
			};

			Result.IncludePaths.AddRange(Module.PublicIncludePaths.Select(x => x.FullName));
			Result.IncludePaths.AddRange(Module.PublicSystemIncludePaths.Select(x => x.FullName));
			Result.IncludePaths.AddRange(Module.InternalIncludePaths.Select(x => x.FullName));
			Result.IncludePaths.AddRange(Module.LegacyPublicIncludePaths.Select(x => x.FullName));
			Result.IncludePaths.AddRange(Module.LegacyParentIncludePaths.Select(x => x.FullName));
			Result.IncludePaths.AddRange(Module.PrivateIncludePaths.Select(x => x.FullName));

			Result.IncludePaths.AddRange(ModuleCompileEnvironment.UserIncludePaths.Select(x => x.FullName));

			Result.IncludePaths.AddRange(Module.PublicSystemLibraryPaths.Select(x => x.FullName));
			Result.IncludePaths.AddRange(Module.PublicSystemLibraries.Concat(Module.PublicLibraries.Select(x => x.FullName)));

			if (TargetToolChain is VCToolChain TargetVCToolChain)
			{
				Result.IncludePaths.AddRange(TargetVCToolChain.GetVCIncludePaths().Select(x => x.ToString()));
			}

			Result.Defines.AddRange(Module.PublicDefinitions);
			Result.Defines.AddRange(Module.Rules.PrivateDefinitions);
			Result.Defines.AddRange(Module.Rules.bTreatAsEngineModule ? Array.Empty<string>() : Module.Rules.Target.ProjectDefinitions);
			Result.Defines.AddRange(Module.GetEmptyApiMacros());

			List<FileItem> ForcedIncludes = ModuleCompileEnvironment.ForceIncludeFiles.ToList();
			if (ModuleCompileEnvironment.PrecompiledHeaderAction == PrecompiledHeaderAction.Include)
			{
				FileItem IncludeHeader = FileItem.GetItemByFileReference(ModuleCompileEnvironment.PrecompiledHeaderIncludeFilename!);
				ForcedIncludes.Insert(0, IncludeHeader);

				Result.PrecompiledHeader = IncludeHeader.FullName;
			}

			Result.ForcedIncludes.AddRange(ForcedIncludes.Select(x => x.FullName));

			Result.CompilerPath = TargetToolChain.GetCppCompilerPath()?.ToString();
			Result.CompilerArgs.AddRange(TargetToolChain.GetGlobalCommandLineArgs(ModuleCompileEnvironment));
			Result.CompilerAdditionalArgs.Add("c", TargetToolChain.GetCCommandLineArgs(ModuleCompileEnvironment).ToList());
			Result.CompilerAdditionalArgs.Add("cpp", TargetToolChain.GetCPPCommandLineArgs(ModuleCompileEnvironment).ToList());

			return Result;
		}

		private TargetBuildInfo? ExportBuildInfo(UEBuildTarget Target, UnrealArch? Architecture, PlatformProjectGeneratorCollection PlatformProjectGenerators, bool bBuildByDefault, ILogger Logger)
		{
			if (!BuildHostPlatform.Current.Platform.IsInGroup(UnrealPlatformGroup.Windows))
			{
				Logger.LogWarning("Unsupported platform for Build Information: {Platform}", BuildHostPlatform.Current.Platform.ToString());
				return null;
			}

			if (IsStubProject)
			{
				return null;
			}

			ProjectTarget ProjectTarget = ProjectTargets.OfType<ProjectTarget>().Single(It => Target.TargetRulesFile == It.TargetFilePath);
			UnrealTargetPlatform Platform = Target.Platform;
			UnrealTargetConfiguration Configuration = Target.Configuration;

			string UProjectPath = "";
			if (IsForeignProject)
			{
				UProjectPath = String.Format("\"{0}\"", ProjectTarget.UnrealProjectFilePath!.FullName);
			}

			PlatformProjectGenerator? ProjGenerator = PlatformProjectGenerators.GetPlatformProjectGenerator(Platform, true);
			VCProjectFile.BuildCommandBuilder BuildCommandBuilder = new(
				new PlatformProjectGenerator.VSSettings(Platform, Configuration, VCProjectFileFormat.Default, Architecture), ProjectTarget, UProjectPath)
			{
				ProjectGenerator = ProjGenerator,
				bIsForeignProject = IsForeignProject
			};

			string BuildArguments = BuildCommandBuilder.GetBuildArguments();

			return new TargetBuildInfo()
			{
				BuildCmd = $"\"{BuildCommandBuilder.BuildScript.FullName}\" {BuildArguments}",
				RebuildCmd = $"\"{BuildCommandBuilder.RebuildScript.FullName}\" {BuildArguments}",
				CleanCmd = $"\"{BuildCommandBuilder.CleanScript.FullName}\" {BuildArguments}",
				PrimaryOutput = Target.Binaries[0].OutputFilePath.FullName,
				BuildByDefault = bBuildByDefault,
			};
		}

		private static string GetVCPlatform(UEBuildTarget Target, UnrealArch? Architecture, PlatformProjectGeneratorCollection Generators, VCProjectFileFormat VCFormat)
		{
			PlatformProjectGenerator? Generator = Generators.GetPlatformProjectGenerator(Target.Platform, true);
			if (Generator != null)
			{
				PlatformProjectGenerator.VSSettings VSSettings = new(Target.Platform, Target.Configuration, VCFormat, Architecture);
				if (Generator.HasVisualStudioSupport(VSSettings))
				{
					return Generator.GetVisualStudioPlatformName(VSSettings);
				}

				return DefaultPlatformName;
			}

			return String.Empty;
		}

		private static string GetVCConfiguration(UEBuildTarget Target, UnrealArch? Architecture, PlatformProjectGeneratorCollection Generators, VCProjectFileFormat VCFormat)
		{
			PlatformProjectGenerator? Generator = Generators.GetPlatformProjectGenerator(Target.Platform, true);
			if (Generator != null)
			{
				PlatformProjectGenerator.VSSettings VSSettings = new(Target.Platform, Target.Configuration, VCFormat, Architecture);
				if (Generator.HasVisualStudioSupport(VSSettings))
				{
					if (!Generator.RequiresDistinctVisualStudioConfigurationName(VSSettings))
					{
						return Target.Configuration.ToString();
					}
				}

				return String.Format("{0}{1}_{2}", Target.Platform.ToString(), Architecture != null ? "_" + Architecture.ToString() : String.Empty, Target.Configuration.ToString());
			}

			return String.Empty;
		}

		private static string GetArchitecture(UEBuildTarget Target, UnrealArch? Architecture, PlatformProjectGeneratorCollection Generators, VCProjectFileFormat VCFormat)
		{
			PlatformProjectGenerator? Generator = Generators.GetPlatformProjectGenerator(Target.Platform, true);
			if (Generator != null)
			{
				PlatformProjectGenerator.VSSettings VSSettings = new(Target.Platform, Target.Configuration, VCFormat, Architecture);
				if (Generator.HasVisualStudioSupport(VSSettings))
				{
					if (!Generator.RequiresDistinctVisualStudioConfigurationName(VSSettings))
					{
						return String.Empty;
					}
				}

				if (Architecture != null)
				{
					string? result = Architecture.ToString();
					return result != null ? result : String.Empty;
				}
			}

			return String.Empty;
		}

		internal class ExportedTargetInfo
		{
			public string TargetName { get; set; } = String.Empty;
			public string TargetPath { get; set; } = String.Empty;
			public string ProjectPath { get; set; } = String.Empty;
			public string TargetType { get; set; } = String.Empty;
			public string Platform { get; set; } = String.Empty;
			public string VCPlatform { get; set; } = String.Empty;
			public string Configuration { get; set; } = String.Empty;
			public string VCConfiguration { get; set; } = String.Empty;
			public string Architecture { get; set; } = String.Empty;
			public TargetBuildInfo? BuildInfo { get; set; }
			public Dictionary<string, ExportedModuleInfo> ModuleToCompileSettings { get; set; } = new();
			public Dictionary<string, string> DirToModule { get; set; } = new();
		}

		internal class ExportedModuleInfo
		{
			public string Name { get; set; } = String.Empty;
			public string Directory { get; set; } = String.Empty;
			public string Rules { get; set; } = String.Empty;
			public string GeneratedCodeDirectory { get; set; } = String.Empty;
			public List<string> IncludePaths { get; set; } = new();
			public List<string> Defines { get; set; } = new();
			public string? Standard { get; set; }
			public string? PrecompiledHeader { get; set; }
			public List<string> ForcedIncludes { get; set; } = new();
			public string? CompilerPath { get; set; }
			public List<string> CompilerArgs { get; set; } = new();
			public Dictionary<string, List<string>> CompilerAdditionalArgs { get; set; } = new();
			public string? WindowsSdkVersion { get; set; }
		}

		internal class TargetBuildInfo
		{
			public string BuildCmd { get; set; } = String.Empty;
			public string RebuildCmd { get; set; } = String.Empty;
			public string CleanCmd { get; set; } = String.Empty;
			public string PrimaryOutput { get; set; } = String.Empty;
			public bool BuildByDefault { get; internal set; }
		}
	}
}
