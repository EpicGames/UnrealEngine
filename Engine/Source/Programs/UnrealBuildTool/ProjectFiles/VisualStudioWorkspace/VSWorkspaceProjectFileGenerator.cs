// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using System.Text.Json;
using System.Text.RegularExpressions;
using System.Xml;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	class VSWorkspaceProjectFileGenerator : ProjectFileGenerator
	{
		public override string ProjectFileExtension => ".json";

		// These properties are used by Visual Studio to determine where to read the project files.
		// So they must remain constant.
		private const string VSUnrealWorkspaceFileName = ".vs-unreal-workspace";
		private const string ProjectFilesFolder = "VisualStudio";
		private const string WorkspaceGeneratorVersion = "1.0.0";

		private readonly CommandLineArguments Arguments;

		/// <summary>
		/// List of deprecated platforms.
		/// Don't generate project model for these platforms unless they are specified in "Platforms" console arguments.
		/// </summary>
		/// <returns></returns>
		private readonly HashSet<UnrealTargetPlatform> DeprecatedPlatforms = new();

#pragma warning disable IDE0044 // Make field readonly - these private static fields are set by command-line parsing.
		/// <summary>
		/// Platforms to generate project files for
		/// </summary>
		[CommandLine("-Platforms=", ListSeparator = '+')]
		HashSet<UnrealTargetPlatform> Platforms = new();

		/// <summary>
		/// Target types to generate project files for
		/// </summary>
		[CommandLine("-TargetTypes=", ListSeparator = '+')]
		HashSet<TargetType> TargetTypes = new();

		/// <summary>
		/// Target configurations to generate project files for
		/// </summary>
		[CommandLine("-TargetConfigurations=", ListSeparator = '+')]
		HashSet<UnrealTargetConfiguration> TargetConfigurations = new();

		/// <summary>
		/// Projects to generate project files for
		/// </summary>
		[CommandLine("-ProjectNames=", ListSeparator = '+')]
		HashSet<string> ProjectNames = new();

		/// <summary>
		/// Should format JSON files in human readable form, or use packed one without indents
		/// </summary>
		[CommandLine("-Minimize", Value = "Compact")]
		private JsonWriterStyle Minimize = JsonWriterStyle.Readable;

		[CommandLine("-Query=", Description = "Outputs all the available configurations, platforms and targets to a file provided by the user.")]
		private string Query = String.Empty;

		[CommandLine("-TargetDescriptionFile=", Description = "")]
		private string TargetDescriptionFile = String.Empty;

		[CommandLine("-WorkspaceGeneratorVersion", Description = "Show version information and exit.")]
		private bool ShowVersionInfo = false;

		[CommandLine("-GenerateVsConfigOnly", Description = "Generate .vsconfig file with dependencies and exit")]
		private bool GenerateVsConfigOnly = false;
#pragma warning restore IDE0044

		public VSWorkspaceProjectFileGenerator(
			FileReference? InOnlyGameProject,
			CSharpProjectCache CSharpProjCache,
			CommandLineArguments InArguments)
			: base(InOnlyGameProject, CSharpProjCache)
		{
			Arguments = InArguments;
			Arguments.ApplyTo(this);
		}

		public override bool ShouldGenerateIntelliSenseData() => true;

		public override void CleanProjectFiles(DirectoryReference InPrimaryProjectDirectory, string InPrimaryProjectName,
			DirectoryReference InIntermediateProjectFilesDirectory, ILogger Logger)
		{
			DirectoryReference.Delete(InPrimaryProjectDirectory);
		}

		protected override void ConfigureProjectFileGeneration(string[] Arguments, ref bool IncludeAllPlatforms, ILogger Logger)
		{
			base.ConfigureProjectFileGeneration(Arguments, ref IncludeAllPlatforms, Logger);
		}

		protected override ProjectFile AllocateProjectFile(FileReference InitFilePath, DirectoryReference BaseDir)
		{
			VSWorkspaceProjectFile projectFile = new(InitFilePath, BaseDir, RootPath: InitFilePath.Directory,
				Arguments: Arguments, TargetTypes: TargetTypes);
			return projectFile;
		}

		protected override bool WriteProjectFiles(PlatformProjectGeneratorCollection PlatformProjectGenerators, ILogger Logger)
		{
			Dictionary<string, AvailableConfiguration>? ProjectGenerationDescriptionData = ReadProjectGenerateDescriptionFilePath()?.Targets;
			using ProgressWriter Progress = new("Writing project files...", true, Logger);
			List<ProjectFile> ProjectsToGenerate = new(GeneratedProjectFiles);
			if (ProjectNames.Any())
			{
				ProjectsToGenerate = ProjectsToGenerate.Where(Project =>
					ProjectNames.Contains(Project.ProjectFilePath.GetFileNameWithoutAnyExtensions())).ToList();
			}

			int TotalProjectFileCount = ProjectsToGenerate.Count;

			HashSet<UnrealTargetPlatform> PlatformsToGenerate = new(SupportedPlatforms);
			if (Platforms.Any())
			{
				PlatformsToGenerate.IntersectWith(Platforms);
			}

			List<UnrealTargetPlatform> FilteredPlatforms = PlatformsToGenerate.Where(Platform =>
			{
				// Skip deprecated unless explicitly specified in the command line.
				return (!DeprecatedPlatforms.Contains(Platform) || Platforms.Contains(Platform)) && UEBuildPlatform.IsPlatformAvailable(Platform);
			}).ToList();

			HashSet<UnrealTargetConfiguration> ConfigurationsToGenerate = new(SupportedConfigurations);
			if (TargetConfigurations.Any())
			{
				ConfigurationsToGenerate.IntersectWith(TargetConfigurations);
			}

			List<UnrealTargetConfiguration> Configurations = ConfigurationsToGenerate.ToList();
			VCProjectFileFormat VCFormat = GetVCProjectFileFormat(Logger);
			bool wroteVCProjectFiles = false;

			for (int ProjectFileIndex = 0; ProjectFileIndex < ProjectsToGenerate.Count; ++ProjectFileIndex)
			{
				if (ProjectsToGenerate[ProjectFileIndex] is not VSWorkspaceProjectFile CurrentProject)
				{
					return false;
				}

				if (!CurrentProject.WriteProjectFile(FilteredPlatforms, Configurations, PlatformProjectGenerators, Minimize, Logger, ProjectGenerationDescriptionData, VCFormat))
				{
					return false;
				}

				foreach (ProjectTarget Target in CurrentProject.ProjectTargets.OfType<ProjectTarget>())
				{
					if (!TargetTypes.Any() || (Target.TargetRules != null && TargetTypes.Contains(Target.TargetRules.Type)))
					{
						// Ignore errors.
						WriteVCProjectFileForTarget(CurrentProject.RootPath, Target, VCFormat, FilteredPlatforms, Configurations, PlatformProjectGenerators, ProjectGenerationDescriptionData, Logger);
						wroteVCProjectFiles = true;
					}
				}

				Progress.Write(ProjectFileIndex + 1, TotalProjectFileCount);
			}

			if (wroteVCProjectFiles)
			{
				WriteVCProjectCommonPropsFile(DirectoryReference.Combine(IntermediateProjectFilesPath, ProjectFilesFolder), VCFormat, Logger);
			}

			Progress.Write(TotalProjectFileCount, TotalProjectFileCount);

			return true;
		}

		private bool WriteAvailableConfigurations()
		{
			Dictionary<string, AvailableConfiguration> AvailableConfigurations = new Dictionary<string, AvailableConfiguration>();
			HashSet<UnrealTargetPlatform> PlatformsToGenerate = new(SupportedPlatforms);
			List<UnrealTargetPlatform> FilteredPlatforms = PlatformsToGenerate.Where(Platform =>
			{
				// Skip deprecated unless explicitly specified in the command line.
				return (!DeprecatedPlatforms.Contains(Platform) || Platforms.Contains(Platform)) && UEBuildPlatform.IsPlatformAvailable(Platform);
			}).ToList();

			for (int ProjectFileIndex = 0; ProjectFileIndex < GeneratedProjectFiles.Count; ++ProjectFileIndex)
			{
				if (GeneratedProjectFiles[ProjectFileIndex] is not VSWorkspaceProjectFile CurrentProject)
				{
					return false;
				}

				CurrentProject.CollectAvailableConfigurations(AvailableConfigurations, FilteredPlatforms, new HashSet<UnrealTargetConfiguration>(SupportedConfigurations).ToList());
			}

			FileReference OutputFile = FileReference.Combine(IntermediateProjectFilesPath, ProjectFilesFolder, Query);
			DirectoryReference.CreateDirectory(OutputFile.Directory);
			using FileStream Stream = new(OutputFile.FullName, FileMode.Create, FileAccess.Write);

			var Result = new
			{
				Targets = AvailableConfigurations,
			};
			JsonSerializer.Serialize(Stream, Result, options: new JsonSerializerOptions()
			{
				PropertyNamingPolicy = JsonNamingPolicy.CamelCase,
				WriteIndented = Minimize == JsonWriterStyle.Readable,
			});

			return true;
		}

		public override bool GenerateProjectFiles(PlatformProjectGeneratorCollection PlatformProjectGenerators, string[] Arguments, ILogger Logger)
		{
			if (ShowVersionInfo)
			{
				Logger.LogInformation("Workspace Generator Version: {WorkspaceGeneratorVersion}", WorkspaceGeneratorVersion);
				return true;
			}

			WriteVsConfigFile(Logger, OnlyGameProject!.Directory);
			if (GenerateVsConfigOnly)
			{
				return true;
			}

			bool IncludeAllPlatforms = true;
			ConfigureProjectFileGeneration(Arguments, ref IncludeAllPlatforms, Logger);

			if (bGeneratingGameProjectFiles || Unreal.IsEngineInstalled())
			{
				PrimaryProjectPath = OnlyGameProject!.Directory;
				PrimaryProjectName = OnlyGameProject.GetFileNameWithoutExtension();

				IntermediateProjectFilesPath =
					DirectoryReference.Combine(PrimaryProjectPath, "Intermediate", "ProjectFiles");
			}

			SetupSupportedPlatformsAndConfigurations(IncludeAllPlatforms: true, Logger, out string SupportedPlatformNames);
			Logger.LogDebug("Supported platforms: {Platforms}", SupportedPlatformNames);

			List<FileReference> AllGames = FindGameProjects(Logger);

			{
				// Find all of the target files.
				List<FileReference> AllTargetFiles = DiscoverTargets(
					AllGames,
					Logger,
					OnlyGameProject,
					SupportedPlatforms,
					bIncludeEngineSource: bIncludeEngineSource,
					bIncludeTempTargets: bIncludeTempTargets);

				// If there are multiple targets of a given type for a project, use the order to determine which one gets a suffix.
				AllTargetFiles = AllTargetFiles.OrderBy(x => x.FullName, StringComparer.OrdinalIgnoreCase).ToList();

				List<ProjectFile> EngineProjects = new();
				List<ProjectFile> GameProjects = new();
				List<ProjectFile> ModProjects = new();
				Dictionary<FileReference, ProjectFile> ProgramProjects = new();
				Dictionary<RulesAssembly, DirectoryReference> RulesAssemblies = new();
				Dictionary<ProjectFile, FileReference> ProjectFileToUProjectFile = new();

				AddProjectsForAllTargets(
					PlatformProjectGenerators,
					AllGames,
					AllTargetFiles,
					Arguments,
					EngineProjects,
					GameProjects,
					ProjectFileToUProjectFile,
					ProgramProjects,
					RulesAssemblies,
					Logger);

				AddAllGameProjects(GameProjects);
			}

			if (!String.IsNullOrEmpty(Query))
			{
				return WriteAvailableConfigurations();
			}

			WriteProjectFiles(PlatformProjectGenerators, Logger);
			WritePrimaryProjectFile(UBTProject, PlatformProjectGenerators, Logger);

			return true;
		}

		protected override bool WritePrimaryProjectFile(ProjectFile? UBTProject,
			PlatformProjectGeneratorCollection PlatformProjectGenerators,
			ILogger Logger)
		{
			try
			{
				FileReference PrimaryProjectFile = FileReference.Combine(
					IntermediateProjectFilesPath, ProjectFilesFolder, VSUnrealWorkspaceFileName);

				DirectoryReference.CreateDirectory(PrimaryProjectFile.Directory);

				// Collect all the resulting project files and aggregate the target-level data
				var AggregatedProjectInfo = GeneratedProjectFiles
					.Where(Project => Project is VSWorkspaceProjectFile)
					.OfType<VSWorkspaceProjectFile>()
					.SelectMany(Project => Project.ExportedTargetProjects)
					.GroupBy(TargetProject => TargetProject.TargetName)
					.Select(g => (g.Key, Target: new
					{
						TargetType = g.Select(i => i.TargetType).Distinct().Single(),
						TargetPath = g.Select(i => i.TargetPath).Distinct().Single(),
						ProjectPath = g.Select(i => i.ProjectPath).Distinct().Single(),
						Configurations = g.Select(i => i.Configuration).Distinct().ToList(),
						Platforms = g.Select(i => i.Platform).Distinct().ToList(),
						Architectures = g.Select(i => i.Architecture).Where(i => !String.IsNullOrEmpty(i)).Distinct().ToList(),
						ConfigurationsAndPlatforms = g.Select(i => String.Format("{0}-{1}{2}", i.Configuration, i.Platform, !String.IsNullOrEmpty(i.Architecture) ? "-" + i.Architecture : String.Empty)).Distinct().ToList(),
					}));

				// The inner Targets object is needed for schema compatibility with the Query Mode API.
				var Result = new
				{
					Targets = AggregatedProjectInfo.ToDictionary(item => item.Key, item => item.Target)
				};

				using FileStream Stream = new(PrimaryProjectFile.FullName, FileMode.Create, FileAccess.Write);
				JsonSerializer.Serialize(Stream, Result, new JsonSerializerOptions
				{
					PropertyNamingPolicy = JsonNamingPolicy.CamelCase,
					WriteIndented = Minimize == JsonWriterStyle.Readable,
				});
			}
			catch (Exception Ex)
			{
				Logger.LogWarning(Ex, "Failed to write root project file");
				return false;
			}

			return true;
		}

		/// <inheritdoc />
		protected override FileReference GetProjectLocation(string BaseName)
		{
			return FileReference.Combine(IntermediateProjectFilesPath, ProjectFilesFolder, BaseName + ProjectFileExtension);
		}

		private AvailableTargetsConfigurations? ReadProjectGenerateDescriptionFilePath()
		{
			if (String.IsNullOrEmpty(TargetDescriptionFile))
			{
				return null;
			}

			using FileStream Stream = new(TargetDescriptionFile, FileMode.Open, FileAccess.Read);
			return JsonSerializer.Deserialize<AvailableTargetsConfigurations>(Stream, new JsonSerializerOptions
			{
				PropertyNamingPolicy = JsonNamingPolicy.CamelCase,
			});
		}

		private static readonly Dictionary<string, string> NodesToUpdate = new Dictionary<string, string>
		{
			{ "NMakeBuildCommandLine", "$(_UProjectNMakeBuildCommandLine)" },
			{ "NMakeCleanCommandLine", "$(_UProjectNMakeCleanCommandLine)" },
			{ "NMakeCompileFileCommandLine", "$(_UProjectNMakeCompileFileCommandLine)" },
			{ "NMakeOutput", "$(_UProjectNMakeOutput)" },
			{ "NMakeReBuildCommandLine", "$(_UProjectNMakeReBuildCommandLine)" }
		};

		private static readonly HashSet<string> NodesAllowList = new HashSet<string>
		{
			"CleanDependsOn",
			"Configuration",
			"CppCleanDependsOn",
			"Import",
			"ImportGroup",
			"ItemDefinitionGroup",
			"ItemGroup",
			"NMakeBuildCommandLine",
			"NMakeCleanCommandLine",
			"NMakeCompile",
			"NMakeCompileFileCommandLine",
			"NMakeOutput",
			"NMakeReBuildCommandLine",
			"Platform",
			"Project",
			"ProjectConfiguration",
			"ProjectGuid",
			"PropertyGroup",
			"RootNamespace",
			"xml"
		};

		private static readonly HashSet<string> NodesBlockedList = new HashSet<string>
		{
			"AdditionalOptions",
			"ClCompile",
			"ClInclude",
			"IncludePath",
			"Manifest",
			"NMakeAssemblySearchPath",
			"NMakeForcedIncludes",
			"NMakeForcedUsingAssemblies",
			"NMakePreprocessorDefinitions",
			"None",
			"ProjectForcedIncludeFiles",
			"PropertyPageSchema",
			"ResourceCompile",
			"SourcePath"
		};

		private static readonly List<Regex> BlockedNodesListRegEx = new List<Regex>()
		{
			new Regex("ClCompile_AdditionalIncludeDirectories*"),
			new Regex("ClCompile_AdditionalOptions*"),
			new Regex("ClCompile_ForcedIncludeFiles*"),
			new Regex("ProjectAdditionalIncludeDirectories*"),
			new Regex("ProjectForcedIncludeFiles*")
		};

		private static VCProjectFileFormat GetVCProjectFileFormat(ILogger Logger)
		{
			// Enumerate all the valid installations. This list is already sorted by preference.
			List<VisualStudioInstallation> Installations = MicrosoftPlatformSDK.FindVisualStudioInstallations(Logger).Where(x => WindowsPlatform.HasCompiler(x.Compiler, UnrealArch.X64, Logger)).ToList();

			// Get the corresponding project file format
			VCProjectFileFormat Format = VCProjectFileFormat.Default;
			foreach (VisualStudioInstallation Installation in Installations)
			{
				if (Installation.Compiler == WindowsCompiler.VisualStudio2022)
				{
					Format = VCProjectFileFormat.VisualStudio2022;
					break;
				}
				else if (Installation.Compiler == WindowsCompiler.VisualStudio2026)
				{
					Format = VCProjectFileFormat.VisualStudio2026;
					break;
				}
			}

			return Format;
		}

		private static bool IncludePlatform(AvailableConfiguration? AvailableConfiguration, UnrealTargetPlatform Platform, IList<UnrealTargetPlatform> FilteredPlatforms)
		{
			if (AvailableConfiguration == null || AvailableConfiguration.Platforms.Contains(Platform.ToString()))
			{
				FilteredPlatforms.Add(Platform);
				return true;
			}

			return false;
		}

		private static bool IncludeConfiguration(AvailableConfiguration? AvailableConfiguration, UnrealTargetConfiguration Configuration, IList<UnrealTargetConfiguration> FilteredConfigurations)
		{
			if (AvailableConfiguration == null || AvailableConfiguration.Configurations.Contains(Configuration.ToString()))
			{
				FilteredConfigurations.Add(Configuration);
				return true;
			}

			return false;
		}

		private static bool WriteVCProjectFileForTarget(
			DirectoryReference RootPath,
			ProjectTarget Target,
			VCProjectFileFormat Format,
			List<UnrealTargetPlatform> Platforms,
			List<UnrealTargetConfiguration> Configurations,
			PlatformProjectGeneratorCollection PlatformProjectGenerators,
			Dictionary<string, AvailableConfiguration>? ProjectGenerationDescriptionData,
			ILogger Logger)
		{
			AvailableConfiguration? AvailableConfiguration = null;
			List<UnrealTargetPlatform> FilteredPlatforms = Platforms;
			List<UnrealTargetConfiguration> FilteredConfigurations = Configurations;
			FileReference Output = FileReference.Combine(RootPath, $"{Target.TargetFilePath.GetFileNameWithoutAnyExtensions()}.proj");

			try
			{
				if (ProjectGenerationDescriptionData != null)
				{
					if (!ProjectGenerationDescriptionData.TryGetValue(Target.Name, out AvailableConfiguration))
					{
						AvailableConfiguration = new AvailableConfiguration();
					}
				}

				if (Target.TargetRules != null)
				{
					if (Target.TargetRules.Type == TargetType.Program)
					{
						return true;
					}

					if (Target.TargetRules.Type == TargetType.Editor)
					{
						FilteredPlatforms = new List<UnrealTargetPlatform>();
						IncludePlatform(AvailableConfiguration, BuildHostPlatform.Current.Platform, FilteredPlatforms);

						FilteredConfigurations = new List<UnrealTargetConfiguration>();
						IncludeConfiguration(AvailableConfiguration, UnrealTargetConfiguration.Debug, FilteredConfigurations);
						IncludeConfiguration(AvailableConfiguration, UnrealTargetConfiguration.DebugGame, FilteredConfigurations);
						IncludeConfiguration(AvailableConfiguration, UnrealTargetConfiguration.Development, FilteredConfigurations);
					}
				}

				if (AvailableConfiguration != null && FilteredPlatforms == Platforms && FilteredConfigurations == Configurations)
				{
					FilteredPlatforms = new List<UnrealTargetPlatform>();
					FilteredConfigurations = new List<UnrealTargetConfiguration>();

					foreach (UnrealTargetPlatform Platform in Platforms)
					{
						IncludePlatform(AvailableConfiguration, Platform, FilteredPlatforms);
					}

					foreach (UnrealTargetConfiguration Configuration in Configurations)
					{
						IncludeConfiguration(AvailableConfiguration, Configuration, FilteredConfigurations);
					}
				}

				if (FilteredPlatforms.Count == 0 || FilteredConfigurations.Count == 0)
				{
					return true;
				}

				VCProjectFileSettings Settings = new();
				Settings.MaxSharedIncludePaths = 0;

				VCProjectFile VCProject = new(Output, RootPath, Format, false, true, null, Settings, true);
				VCProject.ProjectTargets.Add(Target);
				if (VCProject.WriteProjectFile(FilteredPlatforms, FilteredConfigurations, PlatformProjectGenerators, Logger))
				{
					XmlDocument XmlDoc = new XmlDocument();
					XmlDoc.Load(Output.FullName);

					List<XmlNode> RemovedChildNodes = new List<XmlNode>();

					Queue<XmlNode> Nodes = new Queue<XmlNode>();
					Nodes.Enqueue(XmlDoc);

					while (Nodes.Count != 0)
					{
						XmlNode Node = Nodes.Dequeue();
						using (XmlNodeList NodeList = Node.ChildNodes)
						{
							if (NodeList != null)
							{
								foreach (XmlNode ChildNode in NodeList)
								{
									if (ChildNode != null && ChildNode.NodeType != XmlNodeType.Text)
									{
										if (NodesAllowList.Contains(ChildNode.Name))
										{
											Nodes.Enqueue(ChildNode);
											continue;
										}
										else if (NodesBlockedList.Contains(ChildNode.Name))
										{
											RemovedChildNodes.Add(ChildNode);
											continue;
										}
										else
										{
											foreach (Regex RegEx in BlockedNodesListRegEx)
											{
												if (RegEx.Match(ChildNode.Name).Success)
												{
													RemovedChildNodes.Add(ChildNode);
													continue;
												}
											}

											Nodes.Enqueue(ChildNode);
											continue;
										}
									}
									else
									{
										if (NodesToUpdate.TryGetValue(Node.Name, out string? NewNodeValue) && !String.IsNullOrEmpty(NewNodeValue))
										{
											Node.InnerText = NewNodeValue;
										}
									}
								}
							}
						}

						foreach (XmlNode ChildNode in RemovedChildNodes)
						{
							Node.RemoveChild(ChildNode);
						}

						if (Node.ChildNodes.Count == 0 && Node.Attributes?.Count == 0)
						{
							Node.ParentNode?.RemoveChild(Node);
						}

						RemovedChildNodes.Clear();
					}

					XmlDoc.Save(Output.FullName);
				}
				else
				{
					throw new BuildException("Failed to create workspace project file {0}", Output.FullName);
				}
			}
			catch (Exception Ex)
			{
				Logger.LogWarning(Ex, "Failed to create workspace project files. Disabling Visual Studio workspace MSBuild integration for target {TargetName}.", Target.Name);
				if (File.Exists(Output.FullName))
				{
					File.Delete(Output.FullName);
				}

				return false;
			}

			return true;
		}

		private void WriteVCProjectCommonPropsFile(DirectoryReference RootPath, VCProjectFileFormat Format, ILogger Logger)
		{
			FileReference Output = FileReference.Combine(RootPath, "UECommon.props");

			try
			{
				StringBuilder VCCommonTargetFileContent = new StringBuilder();
				VCCommonTargetFileContent.AppendLine("<?xml version=\"1.0\" encoding=\"utf-8\"?>");
				VCCommonTargetFileContent.AppendLine("<Project xmlns=\"http://schemas.microsoft.com/developer/msbuild/2003\">");

				// Project globals (project GUID, project type, SCC bindings, etc)
				{
					string ToolVersionString = VCProjectFileGenerator.GetProjectFileToolVersionString(Format);
					VCCommonTargetFileContent.AppendLine("  <PropertyGroup Label=\"Globals\">");
					VCCommonTargetFileContent.AppendLine("    <Keyword>MakeFileProj</Keyword>");
					VCProjectFileGenerator.AppendPlatformToolsetProperty(VCCommonTargetFileContent, Format);
					VCCommonTargetFileContent.AppendLine("    <MinimumVisualStudioVersion>{0}</MinimumVisualStudioVersion>", ToolVersionString);
					VCCommonTargetFileContent.AppendLine("    <VCProjectVersion>{0}</VCProjectVersion>", ToolVersionString);
					VCCommonTargetFileContent.AppendLine("    <NMakeUseOemCodePage>true</NMakeUseOemCodePage>"); // Fixes mojibake with non-Latin character sets (UE-102825)
					VCCommonTargetFileContent.AppendLine("    <TargetRuntime>Native</TargetRuntime>");
					VCCommonTargetFileContent.AppendLine("  </PropertyGroup>");
				}

				// Write the default configuration info
				VCCommonTargetFileContent.AppendLine("  <PropertyGroup Label=\"Configuration\">");
				VCCommonTargetFileContent.AppendLine($"    <ConfigurationType>{PlatformProjectGenerator.DefaultPlatformConfigurationType}</ConfigurationType>");
				VCProjectFileGenerator.AppendPlatformToolsetProperty(VCCommonTargetFileContent, Format);
				VCCommonTargetFileContent.AppendLine("  </PropertyGroup>");

				VCCommonTargetFileContent.AppendLine("  <Import Project=\"$(VCTargetsPath)\\Microsoft.Cpp.Default.props\" />");
				VCCommonTargetFileContent.AppendLine("  <Import Project=\"$(VCTargetsPath)\\Microsoft.Cpp.props\" />");

				// Write the common and invalid configuration values
				{
					const string InvalidMessage = "echo The selected platform/configuration is not valid for this target.";

					string ProjectRelativeUnusedDirectory = ProjectFile.NormalizeProjectPath(DirectoryReference.Combine(Unreal.EngineDirectory, "Intermediate", "Build", "Unused"));

					VCCommonTargetFileContent.AppendLine("  <PropertyGroup>");

					DirectoryReference BatchFilesDirectory = DirectoryReference.Combine(Unreal.EngineDirectory, "Build", "BatchFiles");
					VCCommonTargetFileContent.AppendLine("    <BuildBatchScript>{0}</BuildBatchScript>", ProjectFile.EscapePath(ProjectFile.NormalizeProjectPath(FileReference.Combine(BatchFilesDirectory, "Build.bat"))));
					VCCommonTargetFileContent.AppendLine("    <RebuildBatchScript>{0}</RebuildBatchScript>", ProjectFile.EscapePath(ProjectFile.NormalizeProjectPath(FileReference.Combine(BatchFilesDirectory, "Rebuild.bat"))));
					VCCommonTargetFileContent.AppendLine("    <CleanBatchScript>{0}</CleanBatchScript>", ProjectFile.EscapePath(ProjectFile.NormalizeProjectPath(FileReference.Combine(BatchFilesDirectory, "Clean.bat"))));
					VCCommonTargetFileContent.AppendLine("    <NMakeBuildCommandLine>{0}</NMakeBuildCommandLine>", InvalidMessage);
					VCCommonTargetFileContent.AppendLine("    <NMakeReBuildCommandLine>{0}</NMakeReBuildCommandLine>", InvalidMessage);
					VCCommonTargetFileContent.AppendLine("    <NMakeCleanCommandLine>{0}</NMakeCleanCommandLine>", InvalidMessage);
					VCCommonTargetFileContent.AppendLine("    <NMakeOutput>Invalid Output</NMakeOutput>", InvalidMessage);
					VCCommonTargetFileContent.AppendLine("    <OutDir>{0}{1}</OutDir>", ProjectRelativeUnusedDirectory, Path.DirectorySeparatorChar);
					VCCommonTargetFileContent.AppendLine("    <IntDir>{0}{1}</IntDir>", ProjectRelativeUnusedDirectory, Path.DirectorySeparatorChar);
					// NOTE: We are intentionally overriding defaults for these paths with empty strings.  We never want Visual Studio's
					//       defaults for these fields to be propagated, since they are version-sensitive paths that may not reflect
					//       the environment that UBT is building in.  We'll set these environment variables ourselves!
					// NOTE: We don't touch 'ExecutablePath' because that would result in Visual Studio clobbering the system "Path"
					//       environment variable
					VCCommonTargetFileContent.AppendLine("    <IncludePath />");
					VCCommonTargetFileContent.AppendLine("    <ReferencePath />");
					VCCommonTargetFileContent.AppendLine("    <LibraryPath />");
					VCCommonTargetFileContent.AppendLine("    <LibraryWPath />");
					VCCommonTargetFileContent.AppendLine("    <SourcePath />");
					VCCommonTargetFileContent.AppendLine("    <ExcludePath />");

					// Add all the default system include paths
					if (OperatingSystem.IsWindows())
					{
						if (SupportedPlatforms.Contains(UnrealTargetPlatform.Win64))
						{
							VCCommonTargetFileContent.AppendLine("    <DefaultSystemIncludePaths>{0}</DefaultSystemIncludePaths>", VCToolChain.GetVCIncludePaths(UnrealTargetPlatform.Win64, VCProjectFileGenerator.GetCompilerForIntellisense(Format), WindowsCompiler.Default, null, null, Logger));
						}
					}
					else
					{
						VCCommonTargetFileContent.AppendLine("    <DefaultSystemIncludePaths />");
					}

					VCCommonTargetFileContent.AppendLine("  </PropertyGroup>");

				}

				// Write default import group
				VCCommonTargetFileContent.AppendLine("  <ImportGroup Label=\"PropertySheets\">");
				VCCommonTargetFileContent.AppendLine("    <Import Project=\"$(UserRootDir)\\Microsoft.Cpp.$(Platform).user.props\" Condition=\"exists('$(UserRootDir)\\Microsoft.Cpp.$(Platform).user.props')\" Label=\"LocalAppDataPlatform\" />");
				VCCommonTargetFileContent.AppendLine("  </ImportGroup>");

				VCCommonTargetFileContent.AppendLine("</Project>");

				Utils.WriteFileIfChanged(Output, VCCommonTargetFileContent.ToString(), Logger);
			}
			catch (Exception Ex)
			{
				Logger.LogWarning(Ex, "Failed to create workspace project file {ProjectFilePath}.", Output.FullName);
			}
		}

		private static bool WriteVsConfigFile(ILogger Logger, DirectoryReference primaryProjectPath)
		{
			StringBuilder VsConfigFileContent = new StringBuilder();

			VsConfigFileContent.AppendLine("{");
			VsConfigFileContent.AppendLine("  \"version\": \"1.0\",");
			VsConfigFileContent.AppendLine("  \"components\": [");
			IEnumerable<string> Components = MicrosoftPlatformSDK.GetVisualStudioSuggestedComponents(VCProjectFileFormat.VisualStudio2022);
			string ComponentsString = String.Join($",{Environment.NewLine}    ", Components.Select(x => $"\"{x}\""));
			VsConfigFileContent.AppendLine($"    {ComponentsString}");
			VsConfigFileContent.AppendLine("  ]");
			VsConfigFileContent.AppendLine("}");

			FileReference VsConfigFileName = FileReference.Combine(primaryProjectPath, ".vsconfig");
			return WriteFileIfChanged(VsConfigFileName.FullName, VsConfigFileContent.ToString(), Logger);
		}
	}
}
