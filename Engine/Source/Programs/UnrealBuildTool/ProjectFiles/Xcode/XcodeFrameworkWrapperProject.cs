// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text.RegularExpressions;
using EpicGames.Core;
using UnrealBuildBase;

namespace UnrealBuildTool.ProjectFiles.Xcode
{
	/// <summary>
	/// Generates an Xcode project that acts as a native wrapper for an Unreal Project built as a framework.
	/// </summary>
	class XcodeFrameworkWrapperProject
	{
		private const string PROJECT_FILE_SEARCH_EXPRESSION = "*.pbxproj";
		private const string TEMPLATE_NAME = "FrameworkWrapper";
		private static readonly string FRAMEWORK_WRAPPER_TEMPLATE_DIRECTORY = Path.Combine(Unreal.EngineDirectory.ToNormalizedPath(), "Build", "IOS", "Resources", TEMPLATE_NAME);
		private const string TEMPLATE_PROJECT_NAME = "PROJECT_NAME";
		private const string COMMANDLINE_FILENAME = "uecommandline.txt";

		/// <summary>
		/// Recursively copies all of the files and directories that are inside <paramref name="SourceDirectory"/> into <paramref name="DestinationDirectory"/>.
		/// </summary>
		/// <param name="SourceDirectory">The directory whose contents should be copied.</param>
		/// <param name="DestinationDirectory">The directory into which the files should be copied.</param>
		private static void CopyAll(string SourceDirectory, string DestinationDirectory)
		{

			IEnumerable<string> Directories = Directory.EnumerateDirectories(SourceDirectory, "*", System.IO.SearchOption.AllDirectories);

			// Create all the directories
			foreach (string DirSrc in Directories)
			{
				string DirDst = DirSrc.Replace(SourceDirectory, DestinationDirectory, StringComparison.Ordinal);
				Directory.CreateDirectory(DirDst);
			}

			IEnumerable<string> Files = Directory.EnumerateFiles(SourceDirectory, "*", System.IO.SearchOption.AllDirectories);

			// Copy all the files
			foreach (string FileSrc in Files)
			{
				string FileDst = FileSrc.Replace(SourceDirectory, DestinationDirectory, StringComparison.Ordinal);
				if (!File.Exists(FileDst))
				{
					File.Copy(FileSrc, FileDst);
				}
			}
		}

		/// <summary>
		/// An enumeration specifying the type of a filesystem entry, either directory, file, or something else.
		/// </summary>
		private enum EntryType { None, Directory, File }

		/// <summary>
		/// Gets the type of filesystem entry pointed to by <paramref name="Path"/>.
		/// </summary>
		/// <returns>The type of filesystem entry pointed to by <paramref name="Path"/>.</returns>
		/// <param name="Path">The path to a filesystem entry.</param>
		private static EntryType GetEntryType(string Path)
		{
			if (Directory.Exists(Path))
			{
				return EntryType.Directory;
			}
			else if (File.Exists(Path))
			{
				return EntryType.File;
			}
			else
			{
				return EntryType.None;
			}
		}

		/// <summary>
		/// Recursively renames all files and directories that contain <paramref name="OldValue"/> in their name by replacing
		/// <paramref name="OldValue"/> with <paramref name="NewValue"/>.
		/// </summary>
		/// <param name="RootDirectory">Root directory.</param>
		/// <param name="OldValue">Old value.</param>
		/// <param name="NewValue">New value.</param>
		private static void RenameFilesAndDirectories(string RootDirectory, string OldValue, string NewValue)
		{
			IEnumerable<string> Entries = Directory.EnumerateFileSystemEntries(RootDirectory, "*", SearchOption.TopDirectoryOnly);
			foreach (string Entry in Entries)
			{
				string NewEntryName = Path.GetFileName(Entry).Replace(OldValue, NewValue, StringComparison.Ordinal);
				string ParentDirectory = Path.GetDirectoryName(Entry)!;

				string EntryDestination = Path.Combine(ParentDirectory, NewEntryName);

				switch (GetEntryType(Entry))
				{
					case EntryType.Directory:
						if (Entry != EntryDestination)
						{
							Directory.Move(Entry, EntryDestination);
						}
						RenameFilesAndDirectories(EntryDestination, OldValue, NewValue);
						break;
					case EntryType.File:
						if (Entry != EntryDestination)
						{
							File.Move(Entry, EntryDestination);
						}
						break;
					default:
						break;
				}
			}
		}

		/// <summary>
		/// Opens each file in <paramref name="RootDirectory"/> and replaces all occurrences of <paramref name="OldValue"/>
		/// with <paramref name="NewValue"/>.
		/// </summary>
		/// <param name="RootDirectory">The directory in which all files should be subject to replacements.</param>
		/// <param name="SearchPattern">Only replace text in files that match this pattern. Default is all files.</param>
		/// <param name="OldValue">The value that should be replaced in all files.</param>
		/// <param name="NewValue">The replacement value.</param>
		private static void ReplaceTextInFiles(string RootDirectory, string OldValue, string NewValue, string SearchPattern = "*")
		{
			IEnumerable<string> Files = Directory.EnumerateFiles(RootDirectory, SearchPattern, SearchOption.AllDirectories);
			foreach (string SrcFile in Files)
			{
				string FileContents = File.ReadAllText(SrcFile);
				FileContents = FileContents.Replace(OldValue, NewValue, StringComparison.Ordinal);
				File.WriteAllText(SrcFile, FileContents);
			}
		}

		/// <summary>
		/// Modifies the Xcode project file to change a few build settings.
		/// </summary>
		/// <param name="RootDirectory">The root directory of the template project that was created.</param>
		/// <param name="FrameworkName">The name of the framework that this project is wrapping.</param>
		/// <param name="BundleId">The Bundle ID to give to the wrapper project.</param>
		/// <param name="SrcFrameworkPath">The path to the directory containing the framework to be wrapped.</param>
		/// <param name="EnginePath">The path to the root Unreal Engine directory.</param>
		/// <param name="CookedDataPath">The path to the 'cookeddata' folder that accompanies the framework.</param>
		/// <param name="ProvisionName"></param>
		/// <param name="TeamUUID"></param>
		private static void SetProjectFileSettings(string RootDirectory, string FrameworkName, string BundleId, string SrcFrameworkPath, string EnginePath, string CookedDataPath, string? ProvisionName, string? TeamUUID)
		{
			List<string> ProjectFiles = Directory.EnumerateFiles(RootDirectory, PROJECT_FILE_SEARCH_EXPRESSION, SearchOption.AllDirectories).ToList();

			if (ProjectFiles.Count != 1)
			{
				throw new BuildException(String.Format("Should only find 1 Xcode project file in the resources, but {0} were found.", ProjectFiles.Count));
			}
			else
			{
				string ProjectContents = File.ReadAllText(ProjectFiles[0]);

				Dictionary<string, string?> Settings = new Dictionary<string, string?>()
				{
					["FRAMEWORK_NAME"] = FrameworkName,
					["SRC_FRAMEWORK_PATH"] = SrcFrameworkPath,
					["ENGINE_PATH"] = EnginePath,
					["SRC_COOKEDDATA"] = CookedDataPath,
					["PRODUCT_BUNDLE_IDENTIFIER"] = BundleId,
					["PROVISIONING_PROFILE_SPECIFIER"] = ProvisionName,
					["DEVELOPMENT_TEAM"] = TeamUUID,
				};

				foreach (KeyValuePair<string, string?> Setting in Settings)
				{
					ProjectContents = ChangeProjectSetting(ProjectContents, Setting.Key, Setting.Value);
				}

				File.WriteAllText(ProjectFiles[0], ProjectContents);
			}
		}

		/// <summary>
		/// Removes the readonly attribute from all files in a directory file while retaining all other attributes, thus making them writeable.
		/// </summary>
		/// <param name="RootDirectory">The path to the directory that will be make writeable.</param>
		private static void MakeAllFilesWriteable(string RootDirectory)
		{
			IEnumerable<string> FileNames = Directory.EnumerateFiles(RootDirectory, "*", SearchOption.AllDirectories);
			foreach (string FileName in FileNames)
			{
				File.SetAttributes(FileName, File.GetAttributes(FileName) & ~FileAttributes.ReadOnly);
			}
		}

		/// <summary>
		/// Changes the value of a setting in a project file. 
		/// </summary>
		/// <returns>The project file contents with the setting replaced.</returns>
		/// <param name="ProjectContents">The contents of a settings file.</param>
		/// <param name="SettingName">The name of the setting to change.</param>
		/// <param name="SettingValue">The new value for the setting.</param>
		private static string ChangeProjectSetting(string ProjectContents, string SettingName, string? SettingValue)
		{
			string SettingNameRegexString = String.Format("(\\s+{0}\\s=\\s)\"?(.+)\"?;", SettingName);

			string SettingValueReplaceString = String.Format("$1\"{0}\";", SettingValue);

			Regex SettingNameRegex = new Regex(SettingNameRegexString);
			return SettingNameRegex.Replace(ProjectContents, SettingValueReplaceString);
		}

		/// <summary>
		/// There are some autogenerated directories that could have accidentally made it into the template.
		/// This method tries to delete those directories as an extra precaution.
		/// </summary>
		/// <param name="RootDirectory">The directory which should be recursively searched for unwanted directories.</param>
		private static void DeleteUnwantedDirectories(string RootDirectory)
		{
			HashSet<string> UnwantedDirectories = new HashSet<string>()
			{
				"Build",
				"xcuserdata"
			};

			IEnumerable<string> Directories = Directory.EnumerateDirectories(RootDirectory, "*", SearchOption.AllDirectories);
			foreach (string Dir in Directories)
			{
				string DirectoryName = Path.GetFileName(Dir);
				if (UnwantedDirectories.Contains(DirectoryName) && Directory.Exists(Dir))
				{
					Directory.Delete(Dir, true);
				}
			}
		}

		/// <summary>
		/// Generates an Xcode project that acts as a native wrapper around an Unreal Project built as a framework.
		///
		/// Wrapper projects are generated by copying a template xcode project from the Build Resources directory,
		/// deleting any user-specific or build files, renaming files and folders to match the framework, setting specific
		/// settings in the project to accommodate the framework, and replacing text in all the files to match the framework.
		/// </summary>
		/// <param name="OutputDirectory">The directory in which to place the framework. The framework will be placed in 'outputDirectory/frameworkName/'.</param>
		/// <param name="ProjectName">The name of the project. If blueprint-only, use the actual name of the project, not just UnrealGame.</param>
		/// <param name="FrameworkName">The name of the framework that this project is wrapping.</param>
		/// <param name="BundleId">The Bundle ID to give to the wrapper project.</param>
		/// <param name="SrcFrameworkPath">The path to the directory containing the framework to be wrapped.</param>
		/// <param name="EnginePath">The path to the root Unreal Engine directory.</param>
		/// <param name="CookedDataPath">The path to the 'cookeddata' folder that accompanies the framework.</param>
		/// <param name="ProvisionName"></param>
		/// <param name="TeamUUID"></param>
		public static void GenerateXcodeFrameworkWrapper(string OutputDirectory, string ProjectName, string FrameworkName, string BundleId, string SrcFrameworkPath, string EnginePath, string CookedDataPath, string? ProvisionName, string? TeamUUID)
		{
			string OutputDir = Path.Combine(OutputDirectory, FrameworkName);

			CopyAll(FRAMEWORK_WRAPPER_TEMPLATE_DIRECTORY, OutputDir);

			DeleteUnwantedDirectories(OutputDir);

			MakeAllFilesWriteable(OutputDir);

			RenameFilesAndDirectories(OutputDir, TEMPLATE_NAME, FrameworkName);

			SetProjectFileSettings(OutputDir, FrameworkName, BundleId, SrcFrameworkPath, EnginePath, CookedDataPath, ProvisionName, TeamUUID);

			ReplaceTextInFiles(OutputDir, TEMPLATE_NAME, FrameworkName);

			ReplaceTextInFiles(OutputDir, TEMPLATE_PROJECT_NAME, ProjectName, COMMANDLINE_FILENAME);

		}
	}

	class XcodeFrameworkWrapperUtils
	{
		private static ConfigHierarchy GetIni(DirectoryReference ProjectDirectory)
		{
			return ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, ProjectDirectory, UnrealTargetPlatform.IOS);
			//return ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, DirectoryReference.FromFile(ProjectFile), UnrealTargetPlatform.IOS);
		}

		public static string GetBundleID(DirectoryReference ProjectDirectory, FileReference? ProjectFile)
		{
			ConfigHierarchy Ini = GetIni(ProjectDirectory);
			string BundleId;
			Ini.GetString("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "BundleIdentifier", out BundleId);

			string ProjectName = (ProjectFile != null) ? ProjectFile.GetFileNameWithoutAnyExtensions() : "UnrealGame";
			BundleId = BundleId.Replace("[PROJECT_NAME]", ProjectName, StringComparison.Ordinal).Replace("_", "", StringComparison.Ordinal);
			return BundleId;
		}

		public static string GetBundleName(DirectoryReference ProjectDirectory, FileReference? ProjectFile)
		{
			ConfigHierarchy Ini = GetIni(ProjectDirectory);
			string BundleName;
			Ini.GetString("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "BundleName", out BundleName);

			string ProjectName = (ProjectFile != null) ? ProjectFile.GetFileNameWithoutAnyExtensions() : "UnrealGame";
			BundleName = BundleName.Replace("[PROJECT_NAME]", ProjectName, StringComparison.Ordinal).Replace("_", "", StringComparison.Ordinal);
			return BundleName;
		}

		public static bool GetBuildAsFramework(DirectoryReference ProjectDirectory)
		{
			ConfigHierarchy Ini = GetIni(ProjectDirectory);
			bool bBuildAsFramework;
			Ini.GetBool("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bBuildAsFramework", out bBuildAsFramework);
			return bBuildAsFramework;
		}

		public static bool GetGenerateFrameworkWrapperProject(DirectoryReference ProjectDirectory)
		{
			ConfigHierarchy Ini = GetIni(ProjectDirectory);
			bool bGenerateFrameworkWrapperProject;
			Ini.GetBool("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bGenerateFrameworkWrapperProject", out bGenerateFrameworkWrapperProject);
			return bGenerateFrameworkWrapperProject;
		}
	}
}