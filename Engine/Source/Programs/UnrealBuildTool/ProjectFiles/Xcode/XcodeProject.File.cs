// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using EpicGames.Core;
using UnrealBuildBase;

namespace UnrealBuildTool.XcodeProjectXcconfig
{
	/// <summary>
	/// Info needed to make a file a member of specific group
	/// </summary>
	class XcodeSourceFile : ProjectFile.SourceFile
	{
		/// <summary>
		/// Constructor
		/// </summary>
		public XcodeSourceFile(FileReference InitFilePath, DirectoryReference? InitRelativeBaseFolder, string? FileRefGuid = null)
			: base(InitFilePath, InitRelativeBaseFolder)
		{
			FileGuid = XcodeProjectFileGenerator.MakeXcodeGuid();
			this.FileRefGuid = FileRefGuid ?? XcodeProjectFileGenerator.MakeXcodeGuid();
		}

		/// <summary>
		/// File Guid for use in Xcode project
		/// </summary>
		public string FileGuid
		{
			get;
			private set;
		}

		public void ReplaceGuids(string NewFileGuid, string NewFileRefGuid)
		{
			FileGuid = NewFileGuid;
			FileRefGuid = NewFileRefGuid;
		}

		/// <summary>
		/// File reference Guid for use in Xcode project
		/// </summary>
		public string FileRefGuid
		{
			get;
			private set;
		}
	}

	/// <summary>
	/// Represents a group of files shown in Xcode's project navigator as a folder
	/// </summary>
	internal class XcodeFileGroup
	{
		public XcodeFileGroup(string InName, string InPath, bool InIsReference)
		{
			GroupName = InName;
			GroupPath = InPath;
			GroupGuid = XcodeProjectFileGenerator.MakeXcodeGuid();
			bIsReference = InIsReference;
		}

		public XcodeFileGroup(XcodeFileGroup Other)
		{
			GroupName = Other.GroupName;
			GroupPath = Other.GroupPath;
			GroupGuid = Other.GroupGuid;
			bIsReference = Other.bIsReference;
			Files.AddRange(Other.Files);
			foreach (string Key in Other.Children.Keys)
			{
				Children.Add(Key, new XcodeFileGroup(Other.Children[Key]));
			}
		}

		public string GroupGuid;
		public string GroupName;
		public string GroupPath;
		public Dictionary<string, XcodeFileGroup> Children = new Dictionary<string, XcodeFileGroup>();
		public List<XcodeSourceFile> Files = new List<XcodeSourceFile>();
		public bool bIsReference;
	}

	class XcconfigFile
	{
		public string Name;
		public string Guid;
		public FileReference FileRef;
		internal StringBuilder Text;

		public XcconfigFile(DirectoryReference XcodeProjectDirectory, UnrealTargetPlatform? Platform, string ConfigName)
		{
			Name = ConfigName;
			FileRef = FileReference.Combine(XcodeProjectDirectory, "Xcconfigs" + (Platform == null ? "" : Platform.ToString()), $"{ConfigName}.xcconfig");
			Guid = XcodeProjectFileGenerator.MakeXcodeGuid();
			Text = new StringBuilder();
		}

		public void AppendLine(string Line)
		{
			Text.Append(Line);
			Text.Append(ProjectFileGenerator.NewLine);
		}

		public void Write()
		{
			// write the file to disk
			DirectoryReference.CreateDirectory(FileRef.Directory);
			FileReference.WriteAllTextIfDifferent(FileRef, Text.ToString());
		}
	}

	class XcodeFileCollection
	{
		static readonly string[] ResourceExtensions =
		{
			".xcassets",
			".storyboardc",
			".storyboard",
			".xcframework",
			".framework",
			".bundle",
			".lproj",
			".png",
			".xcprivacy",
			".plist",
			".bin",
			".wav",
			".aiff",
			".caff",
			// no extensions will be folders, like "cookeddata" that we want to copy as a resource
			"",
		};

		static readonly string[] SourceCodeExtensions =
		{
			".c",
			".cc",
			".cpp",
			".cxx",
			".m",
			".mm",
		};

		static readonly Dictionary<string, string> FileTypeMap = new()
		{
			{ ".c", "sourcecode.c.objc" },
			{ ".m", "sourcecode.c.objc" },
			{ ".cc", "sourcecode.cpp.objcpp" },
			{ ".cpp", "sourcecode.cpp.objcpp" },
			{ ".cxx", "sourcecode.cpp.objcpp" },
			{ ".mm", "sourcecode.cpp.objcpp" },
			{ ".h", "sourcecode.c.h" },
			{ ".hpp", "sourcecode.cpp.h" },
			{ ".inl", "sourcecode.c.h" },
			{ ".pch", "sourcecode.c.h" },
			{ ".framework", "wrapper.framework" },
			{ ".plist", "text.plist.xml" },
			{ ".png", "image.png" },
			{ ".icns", "image.icns" },
			{ ".xcassets", "folder.assetcatalog" },
			{ ".storyboardc", "wrapper.storyboardc" },
			{ ".storyboard", "file.storyboard" },
			{ ".xcframework", "wrapper.xcframework" },
			{ ".bundle", "\"wrapper.plug-in\"" },
			{ ".ush", "sourcecode.metal" },
			{ ".usf", "sourcecode.metal" }
		};

		const string FileTypeDefault = "file.txt";

		class ManualFileReference
		{
			public string Guid = XcodeProjectFileGenerator.MakeXcodeGuid();
			public string RelativePath = "";
			public string FileKey = "";
			public string FileType = "";
			public string SourceTree = "";
			public string GroupName = "";
		}

		// we need relative paths, so this is the directory that paths are relative to
		private readonly DirectoryReference ProjectDirectory;

		// the location of the uproject file, or the engine directory for non-uproject projects
		public DirectoryReference RootDirectory;

		public string MainGroupGuid = "";
		//public string ProductRefGroupGuid = XcodeProjectFileGenerator.MakeXcodeGuid();

		internal Dictionary<string, XcodeFileGroup> Groups = new();
		internal Dictionary<XcodeSourceFile, Dictionary<UnrealTargetPlatform, FileReference>> BuildableFilesToResponseFile = new();
		internal List<Tuple<XcodeSourceFile, string>> BuildableResourceFiles = new();
		internal List<XcodeSourceFile> AllFiles = new();

		private readonly List<ManualFileReference> ManualFiles = new();
		private readonly Dictionary<string, string> GuidsForGroups = new();

		public XcodeFileCollection(XcodeProjectFile ProjectFile)
		{
			ProjectDirectory = ProjectFile.ProjectFilePath.Directory;
			RootDirectory = Unreal.EngineDirectory;
			SetUProjectLocation(ProjectFile.UnrealData.UProjectFileLocation);
		}

		public XcodeFileCollection(XcodeFileCollection Other)
		{
			ProjectDirectory = Other.ProjectDirectory;
			RootDirectory = Other.RootDirectory;
			MainGroupGuid = Other.MainGroupGuid;
			AllFiles.AddRange(Other.AllFiles);
			BuildableResourceFiles.AddRange(Other.BuildableResourceFiles);
			ManualFiles.AddRange(Other.ManualFiles);

			BuildableFilesToResponseFile = new(Other.BuildableFilesToResponseFile);
			GuidsForGroups = new(Other.GuidsForGroups);

			foreach (string Key in Other.Groups.Keys)
			{
				Groups.Add(Key, new XcodeFileGroup(Other.Groups[Key]));
			}
		}

		public void SetUProjectLocation(FileReference? UProjectLocation)
		{
			RootDirectory = UProjectLocation == null || (UProjectLocation.IsUnderDirectory(Unreal.EngineDirectory) && UProjectLocation.ContainsName("Programs", Unreal.EngineDirectory.FullName.Length))
				? (Unreal.EngineDirectory)
				: UProjectLocation!.Directory;
		}

		public void ProcessFile(XcodeSourceFile File, bool bIsForBuild, bool bIsFolder, string? GroupName = null, string ExtraResourceSettings = "", Dictionary<DirectoryReference, int>? SourceToBuildFileMap = null)
		{
			// remember all buildable files
			if (bIsForBuild)
			{
				string FileExtension = File.Reference.GetExtension();

				if (IsSourceCode(FileExtension))
				{
					DirectoryReference? SourceDir = File.Reference.Directory;
					int BuildFileOffset = 0;
					if (SourceToBuildFileMap != null)
					{
						if (!SourceToBuildFileMap.TryGetValue(SourceDir, out BuildFileOffset))
						{
							while (SourceDir != null && !DirectoryReference.EnumerateFiles(SourceDir, "*.Build.cs").Any())
							{
								SourceDir = SourceDir.ParentDirectory!;
							}

							// if we didn't find a 
							if (SourceDir != null)
							{
								BuildFileOffset = SourceDir.FullName.Length;
							}
							else
							{
								// mark that we don't want to index this file, if we didn;t find a Build.cs file
								BuildFileOffset = -1;
							}
							SourceToBuildFileMap[File.Reference.Directory] = BuildFileOffset;
						}
					}

					if (BuildFileOffset != -1)
					{
						// first find the build.cs file 
						// look if it contains any of the excluded names, and is so, don't include it
						bool isExtension = File.Reference.FullName.Contains("Extensions", StringComparison.Ordinal);
						if (isExtension || !ExcludedFolders.Any(x => File.Reference.ContainsName(x, BuildFileOffset)))
						{
							// will fill in the response file later
							BuildableFilesToResponseFile[File] = new();
						}
					}
				}
				else if (IsResourceFile(FileExtension))
				{
					BuildableResourceFiles.Add(Tuple.Create(File, ExtraResourceSettings));
				}
			}

			if (GroupName == null || GroupName.Length > 0)
			{
				// group the files by path
				XcodeFileGroup? Group = GroupName == null ? FindGroupByAbsolutePath(File.Reference.Directory) : null;
				if (Group != null)
				{
					AllFiles.Add(File);
					Group.Files.Add(File);
				}
				else
				{
					GroupName ??= (File.Reference.IsUnderDirectory(Unreal.EngineDirectory) ? "EngineReferences" : "ExternalReferences");
					if (bIsFolder)
					{
						AddFolderReference(File.FileRefGuid, File.Reference.MakeRelativeTo(ProjectDirectory), GroupName);
					}
					else
					{
						AddFileReference(File.FileRefGuid, File.Reference.FullName, GetFileKey(File.Reference.GetExtension()), GetFileType(File.Reference.GetExtension()), "<absolute>", GroupName);
					}
				}
			}
		}

		/// <summary>
		/// The project needs to point to the product group, so we expose the Guid here
		/// </summary>
		/// <returns></returns>
		public string GetProductGroupGuid()
		{
			return GuidsForGroups["Products"];
		}

		public void AddFileReference(string Guid, string RelativePath, string FileKey, string FileType, string SourceTree/*="SOURCE_ROOT"*/, string GroupName)
		{
			ManualFiles.Add(new ManualFileReference()
			{
				Guid = Guid,
				RelativePath = RelativePath,
				FileKey = FileKey,
				FileType = FileType,
				SourceTree = SourceTree.StartsWith('<') ? $"\"{SourceTree}\"" : SourceTree,
				GroupName = GroupName,
			});

			// make sure each unique group has a guid
			if (!GuidsForGroups.ContainsKey(GroupName))
			{
				GuidsForGroups[GroupName] = XcodeProjectFileGenerator.MakeXcodeGuid();
			}
		}

		public void AddFolderReference(string Guid, string RelativePath, string GroupName)
		{
			ManualFiles.Add(new ManualFileReference()
			{
				Guid = Guid,
				RelativePath = RelativePath,
				FileKey = "lastKnownFileType",
				FileType = "folder",
				SourceTree = "\"<group>\"",
				GroupName = GroupName,
			});

			// make sure each unique group has a guid
			if (!GuidsForGroups.ContainsKey(GroupName))
			{
				GuidsForGroups[GroupName] = XcodeProjectFileGenerator.MakeXcodeGuid();
			}
		}

		public void Write(StringBuilder Content, UnrealTargetPlatform? Platform)
		{
			Content.WriteLine("/* Begin PBXBuildFile section */");
			if (Platform == null)
			{
				// Shared platform project should only index for Mac
				Platform = UnrealTargetPlatform.Mac;
			}
			foreach (KeyValuePair<XcodeSourceFile, Dictionary<UnrealTargetPlatform, FileReference>> Pair in BuildableFilesToResponseFile)
			{
				string CompileFlags = "";
				if (Pair.Value.Count > 0)
				{
					if (Pair.Value.ContainsKey(Platform.Value))
					{
						CompileFlags = $" settings = {{COMPILER_FLAGS = \"@{Pair.Value[Platform.Value].FullName}\"; }};";
					}
					else if (Pair.Value.ContainsKey(UnrealTargetPlatform.Mac))
					{
						// We don't have the correct response file for this platform, fall back to use the Mac version
						CompileFlags = $" settings = {{COMPILER_FLAGS = \"@{Pair.Value[UnrealTargetPlatform.Mac].FullName}\"; }};";
					}
				}
				
				Content.WriteLine($"\t\t{Pair.Key.FileGuid} = {{isa = PBXBuildFile; fileRef = {Pair.Key.FileRefGuid};{CompileFlags} }}; /* {Pair.Key.Reference.GetFileName()} */");
			}
			foreach (Tuple<XcodeSourceFile, string> Resource in BuildableResourceFiles)
			{
				XcodeSourceFile File = Resource.Item1;
				string ExtraSettings = Resource.Item2;
				Content.WriteLine($"\t\t{File.FileGuid} = {{isa = PBXBuildFile; fileRef = {File.FileRefGuid}; {ExtraSettings}}}; /* {File.Reference.GetFileName()} */");
			}
			Content.WriteLine("/* End PBXBuildFile section */");
			Content.WriteLine();

			Content.WriteLine("/* Begin PBXFileReference section */");
			foreach (XcodeSourceFile File in AllFiles)
			{
				string FileName = File.Reference.GetFileName();
				string FileExtension = Path.GetExtension(FileName);
				string FileType = GetFileType(FileExtension);
				string FileKey = GetFileKey(FileExtension);
				string RelativePath = Utils.CleanDirectorySeparators(File.Reference.MakeRelativeTo(ProjectDirectory), '/');
				string SourceTree = "SOURCE_ROOT";

				if (FileName.Contains("UserNotifications.framework", StringComparison.Ordinal))
				{
					SourceTree = "DEVELOPER_DIR";
					RelativePath = File.Reference.FullName;
				}
				Content.WriteLine($"\t\t{File.FileRefGuid} = {{isa = PBXFileReference; {FileKey} = {FileType}; name = \"{FileName}\"; path = \"{RelativePath}\"; sourceTree = {SourceTree}; }};");
			}
			foreach (ManualFileReference File in ManualFiles)
			{
				string FileName = Path.GetFileName(File.RelativePath);
				if (FileName.Contains("UserNotifications.framework", StringComparison.Ordinal))
				{
					File.SourceTree = "PLATFORM_DEVELOPER_SDK_DIR";
					File.RelativePath = "iPhoneOS.sdk/System/Library/Frameworks/UserNotifications.framework";
				}
				Content.WriteLine($"\t\t{File.Guid} = {{isa = PBXFileReference; {File.FileKey} = {File.FileType}; name = \"{FileName}\"; path = \"{File.RelativePath}\"; sourceTree = {File.SourceTree}; }};");
			}
			Content.WriteLine("/* End PBXFileReference section */");
			Content.WriteLine();

			WriteGroups(Content);
		}

		private void WriteGroups(StringBuilder Content)
		{
			XcodeFileGroup? RootGroup = FindRootFileGroup(Groups);
			if (RootGroup == null)
			{
				return;
			}

			string XcconfigsRefGroupGuid = XcodeProjectFileGenerator.MakeXcodeGuid();

			Content.WriteLine("/* Begin PBXGroup section */");

			// write main/root group and it's children (adding the manual groups to the main group, which is the true param here0
			WriteGroup(Content, RootGroup, bIsRootGroup: true);

			// write some manual groups
			foreach (KeyValuePair<string, string> Pair in GuidsForGroups)
			{
				Content.WriteLine($"\t\t{Pair.Value} /* {Pair.Key} */ = {{");
				Content.WriteLine("\t\t\tisa = PBXGroup;");
				Content.WriteLine("\t\t\tchildren = (");
				foreach (ManualFileReference Ref in ManualFiles.Where(x => x.GroupName == Pair.Key))
				{
					Content.WriteLine($"\t\t\t\t{Ref.Guid} /* {Path.GetFileName(Ref.RelativePath)} */,");
				}
				Content.WriteLine("\t\t\t);");
				Content.WriteLine($"\t\t\tname = {Pair.Key};");
				Content.WriteLine("\t\t\tsourceTree = \"<group>\";");
				Content.WriteLine("\t\t};");

			}

			Content.WriteLine("/* End PBXGroup section */");
		}

		private void WriteGroup(StringBuilder Content, XcodeFileGroup Group, bool bIsRootGroup)
		{
			if (!Group.bIsReference)
			{
				Content.WriteLine($"\t\t{Group.GroupGuid} = {{");
				Content.WriteLine("\t\t\tisa = PBXGroup;");
				Content.WriteLine("\t\t\tchildren = (");
				foreach (XcodeFileGroup ChildGroup in Group.Children.Values)
				{
					Content.WriteLine($"\t\t\t\t{ChildGroup.GroupGuid} /* {ChildGroup.GroupName} */,");
				}
				foreach (XcodeSourceFile File in Group.Files)
				{
					Content.WriteLine($"\t\t\t\t{File.FileRefGuid} /* {File.Reference.GetFileName()} */,");
				}

				if (bIsRootGroup)
				{
					foreach (KeyValuePair<string, string> Pair in GuidsForGroups)
					{
						Content.WriteLine($"\t\t\t\t{Pair.Value} /* {Pair.Key} */,");
					}
				}
				Content.WriteLine("\t\t\t);");
				if (!bIsRootGroup)
				{
					Content.WriteLine("\t\t\tname = \"" + Group.GroupName + "\";");
					Content.WriteLine("\t\t\tpath = \"" + Group.GroupPath + "\";");
					Content.WriteLine("\t\t\tsourceTree = \"<absolute>\";");
				}
				else
				{
					Content.WriteLine("\t\t\tsourceTree = \"<group>\";");
				}
				Content.WriteLine("\t\t};");

				foreach (XcodeFileGroup ChildGroup in Group.Children.Values)
				{
					WriteGroup(Content, ChildGroup, bIsRootGroup: false);
				}
			}
		}

		public static string GetRootGroupGuid(Dictionary<string, XcodeFileGroup> GroupsDict, FileSystemReference ProjectFile)
		{
			XcodeFileGroup? RootGroup = FindRootFileGroup(GroupsDict);
			if (RootGroup == null)
			{
				throw new BuildException($"The project '{ProjectFile}' had no root group, no files have been added");
			}
			return RootGroup.GroupGuid;
		}

		private static XcodeFileGroup? FindRootFileGroup(Dictionary<string, XcodeFileGroup> GroupsDict)
		{
			foreach (XcodeFileGroup Group in GroupsDict.Values)
			{
				if (Group.Children.Count > 1 || Group.Files.Count > 0)
				{
					return Group;
				}
				else
				{
					XcodeFileGroup? Found = FindRootFileGroup(Group.Children);
					if (Found != null)
					{
						return Found;
					}
				}
			}
			return null;
		}

		/// <summary>
		/// Gets Xcode file type based on its extension
		/// </summary>
		internal static string GetFileType(string Extension)
		{
			string? FileType;
			if (FileTypeMap.TryGetValue(Extension, out FileType))
			{
				return FileType;
			}
			return FileTypeDefault;
		}

		// most types Xcode wants as explicitFileType, but some want lastKnownFileType
		internal static string GetFileKey(string Extension)
		{
			switch (Extension)
			{
				case ".storyboard":
				case ".bundle":
					return "lastKnownFileType";
				default:
					return "explicitFileType";
			}
		}
		/// <summary>
		/// Returns true if Extension is a known extension for files containing source code
		/// </summary>
		internal static bool IsSourceCode(string Extension)
		{
			return SourceCodeExtensions.Contains(Extension);
		}

		/// <summary>
		/// Returns true if Extension is a known extension for files containing resource data
		/// </summary>
		internal static bool IsResourceFile(string Extension)
		{
			return ResourceExtensions.Contains(Extension);
		}

		/// <summary>
		/// Cache the folders excluded by Mac
		/// </summary>
		static readonly string[] ExcludedFolders = PlatformExports.GetExcludedFolderNames(UnrealTargetPlatform.Mac);

		/// <summary>
		/// Returns a project navigator group to which the file should belong based on its path.
		/// Creates a group tree if it doesn't exist yet.
		/// </summary>
		internal XcodeFileGroup? FindGroupByAbsolutePath(DirectoryReference DirectoryPath)
		{
			string AbsolutePath = DirectoryPath.FullName;
			string[] Parts = AbsolutePath.Split(Path.DirectorySeparatorChar);
			string CurrentPath = "/";
			Dictionary<string, XcodeFileGroup> CurrentSubGroups = Groups;

			for (int Index = 1; Index < Parts.Length; ++Index)
			{
				string Part = Parts[Index];

				if (CurrentPath.Length > 1)
				{
					CurrentPath += Path.DirectorySeparatorChar;
				}

				CurrentPath += Part;

				XcodeFileGroup CurrentGroup;
				if (!CurrentSubGroups.ContainsKey(CurrentPath))
				{
					// we only want files under the project/engine dir, we have to make external references to other files, otherwise the grouping gets all broken
					if (!DirectoryPath.IsUnderDirectory(RootDirectory))
					{
						return null;
					}

					CurrentGroup = new XcodeFileGroup(Path.GetFileName(CurrentPath), CurrentPath, GetFileType(Path.GetExtension(CurrentPath)).StartsWith("folder", StringComparison.Ordinal));
					if (Groups.Count == 0)
					{
						MainGroupGuid = CurrentGroup.GroupGuid;
					}
					CurrentSubGroups.Add(CurrentPath, CurrentGroup);
				}
				else
				{
					CurrentGroup = CurrentSubGroups[CurrentPath];
				}

				if (CurrentPath == AbsolutePath)
				{
					return CurrentGroup;
				}

				CurrentSubGroups = CurrentGroup.Children;
			}

			return null;
		}

		/// <summary>
		/// Convert all paths to Apple/Unix format (with forward slashes)
		/// </summary>
		/// <param name="InPath">The path to convert</param>
		/// <returns>The normalized path</returns>
		internal static string ConvertPath(string InPath)
		{
			return InPath.Replace("\\", "/", StringComparison.Ordinal);
		}
	}
}
