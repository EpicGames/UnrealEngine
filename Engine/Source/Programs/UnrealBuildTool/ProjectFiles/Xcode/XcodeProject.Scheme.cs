// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using EpicGames.Core;

namespace UnrealBuildTool.XcodeProjectXcconfig
{
	static class XcodeSchemeFile
	{
		private static FileReference GetUserSchemeManagementFilePath(DirectoryReference ProjectFile, UnrealTargetPlatform? Platform)
		{
			return FileReference.Combine(XcodeUtils.ProjectDirPathForPlatform(ProjectFile, Platform), $"xcuserdata/{Environment.UserName}.xcuserdatad/xcschemes/xcschememanagement.plist");
		}

		private static DirectoryReference GetProjectSchemeDirectory(DirectoryReference ProjectFile, UnrealTargetPlatform? Platform)
		{
			return DirectoryReference.Combine(XcodeUtils.ProjectDirPathForPlatform(ProjectFile, Platform), "xcshareddata/xcschemes");
		}

		private static FileReference GetProjectSchemeFilePathForTarget(DirectoryReference ProjectFile, UnrealTargetPlatform? Platform, string TargetName)
		{
			return FileReference.Combine(GetProjectSchemeDirectory(ProjectFile, Platform), TargetName + ".xcscheme");
		}

		public static void WriteSchemeFile(UnrealData UnrealData, UnrealTargetPlatform? Platform, string ProjectName, List<XcodeRunTarget> RunTargets, string? BuildTargetGuid, string? IndexTargetGuid)
		{
			StringBuilder Content = new StringBuilder();

			string ProductName = UnrealData.ProductName;
			string GameProjectPath = UnrealData.UProjectFileLocation != null ? UnrealData.UProjectFileLocation.FullName : "";

			foreach (XcodeRunTarget RunTarget in RunTargets)
			{
				string TargetName = RunTarget.Name;
				string TargetGuid = RunTarget.Guid;

				bool bHasEditorConfiguration = RunTarget.BuildConfigList!.BuildConfigs.Any(x => x.Info.ProjectTarget!.TargetRules!.Type == TargetType.Editor);
				bool bUseEditorConfiguration = bHasEditorConfiguration && !UnrealData.bMakeProjectPerTarget && !XcodeProjectFileGenerator.bGenerateRunOnlyProject;
				string DefaultConfiguration = bUseEditorConfiguration ? "Development Editor" : "Development";

				FileReference SchemeFilePath = GetProjectSchemeFilePathForTarget(UnrealData.XcodeProjectFileLocation, Platform, TargetName);

				DirectoryReference.CreateDirectory(SchemeFilePath.Directory);

				string? OldCommandLineArguments = null;
				if (FileReference.Exists(SchemeFilePath))
				{
					string OldContents = File.ReadAllText(SchemeFilePath.FullName);
					int OldCommandLineArgumentsStart = OldContents.IndexOf("<CommandLineArguments>", StringComparison.Ordinal) + "<CommandLineArguments>".Length;
					int OldCommandLineArgumentsEnd = OldContents.IndexOf("</CommandLineArguments>", StringComparison.Ordinal);
					if (OldCommandLineArgumentsStart != -1 && OldCommandLineArgumentsEnd != -1)
					{
						OldCommandLineArguments = OldContents.Substring(OldCommandLineArgumentsStart, OldCommandLineArgumentsEnd - OldCommandLineArgumentsStart);
					}
				}

				FileReference lldbInitFile = FileReference.Combine(SchemeFilePath.Directory, "LLDBInit");

				List<string> sourceMappings = [];
				if (UnrealData.RootPaths != null)
				{
					foreach ((uint id, DirectoryReference vfs, DirectoryReference local) in UnrealData.RootPaths)
					{
						sourceMappings.Add($"settings append target.source-map \"{vfs}\" \"{local}\"");
					}

					FileReference.WriteAllLines(lldbInitFile, sourceMappings);
				}

				IEnumerable<string> ScriptLines = new string[]
				{
					"# if a UBTGenerated plist doesn&apos;t exist, create it now for Xcode&apos;s dependency needs",
					"if [ ! -f &quot;${PROJECT_DIR}/${INFOPLIST_FILE}&quot; ]",
					"then",
					"  echo Creating ${PROJECT_DIR}/${INFOPLIST_FILE}...",
					"  touch &quot;${PROJECT_DIR}/${INFOPLIST_FILE}&quot;",
					"fi",
					"",
					"# if we had previously archived the build, then Xcode would have made a soft-link, but it will fail when building normally, so delete a soft-link",
					"if [ -L &quot;${CONFIGURATION_BUILD_DIR}/${PRODUCT_NAME}.app&quot; ]",
					"then",
					"  rm &quot;${CONFIGURATION_BUILD_DIR}/${PRODUCT_NAME}.app&quot;",
					"fi",
					"",
					"# make sure we have a cookeddata directory if needed",
					"if [[ &quot;${PLATFORM_NAME}&quot; == &quot;iphoneos&quot; || &quot;${PLATFORM_NAME}&quot; == &quot;appletvos&quot; ]]",
					"then",
					"  mkdir -p &quot;${CONFIGURATION_BUILD_DIR}/../../../Saved/StagedBuilds/${UE_TARGET_PLATFORM_NAME}/cookeddata&quot;",
					"fi",
					"",
				};

				if (!XcodeProjectFileGenerator.bGenerateRunOnlyProject)
				{
					ScriptLines = ScriptLines.Concat(new string[]
					{
						"# Sometimes Xcode won't redo code sign iOS.app, leads to deployment fails",
						"# Force Xcode to redo code sign during each build",
						"if [ &quot;${UE_PLATFORM_NAME}&quot; != &quot;Mac&quot; ]",
						"then",
						"  rm -f &quot;${UE_BINARIES_DIR}/${PRODUCT_NAME}.app/Info.plist&quot;",
						"fi",
						"",
					});
				}

				// convert special characters to be xml happy
				IEnumerable<string> Lines = ScriptLines.Concat(UnrealData.ExtraPreBuildScriptLines);
				Lines = Lines.Select(x => x.Replace("\"", "&quot;", StringComparison.Ordinal).Replace("&&", "&amp;&amp;", StringComparison.Ordinal));

				Content.WriteLine("<?xml version=\"1.0\" encoding=\"UTF-8\"?>");
				Content.WriteLine("<Scheme");
				Content.WriteLine("   LastUpgradeVersion = \"2000\"");
				Content.WriteLine("   version = \"1.3\">");
				Content.WriteLine("   <BuildAction");
				Content.WriteLine("      parallelizeBuildables = \"YES\"");
				Content.WriteLine("      buildImplicitDependencies = \"YES\">");
				Content.WriteLine("      <PreActions>");
				Content.WriteLine("	        <ExecutionAction");
				Content.WriteLine("            ActionType = \"Xcode.IDEStandardExecutionActionsCore.ExecutionActionType.ShellScriptAction\">");
				Content.WriteLine("            <ActionContent");
				Content.WriteLine("               title = \"Touch UBT generated tiles\"");
				Content.WriteLine("               scriptText = \"" + String.Join("&#10;", Lines) + "\">");
				Content.WriteLine("               <EnvironmentBuildable>");
				Content.WriteLine("                  <BuildableReference");
				Content.WriteLine("                     BuildableIdentifier = \"primary\"");
				Content.WriteLine("                     BlueprintIdentifier = \"" + TargetGuid + "\"");
				Content.WriteLine("                     BuildableName = \"" + ProductName + ".app\"");
				Content.WriteLine("                     BlueprintName = \"" + TargetName + "\"");
				Content.WriteLine("                     ReferencedContainer = \"container:" + ProjectName + ".xcodeproj\">");
				Content.WriteLine("                  </BuildableReference>");
				Content.WriteLine("               </EnvironmentBuildable>");
				Content.WriteLine("            </ActionContent>");
				Content.WriteLine("         </ExecutionAction>");
				Content.WriteLine("      </PreActions>");
				Content.WriteLine("      <BuildActionEntries>");
				Content.WriteLine("         <BuildActionEntry");
				Content.WriteLine("            buildForTesting = \"YES\"");
				Content.WriteLine("            buildForRunning = \"YES\"");
				Content.WriteLine("            buildForProfiling = \"YES\"");
				Content.WriteLine("            buildForArchiving = \"YES\"");
				Content.WriteLine("            buildForAnalyzing = \"YES\">");
				Content.WriteLine("            <BuildableReference");
				Content.WriteLine("               BuildableIdentifier = \"primary\"");
				Content.WriteLine("               BlueprintIdentifier = \"" + TargetGuid + "\"");
				Content.WriteLine("               BuildableName = \"" + ProductName + ".app\"");
				Content.WriteLine("               BlueprintName = \"" + TargetName + "\"");
				Content.WriteLine("               ReferencedContainer = \"container:" + ProjectName + ".xcodeproj\">");
				Content.WriteLine("            </BuildableReference>");
				Content.WriteLine("         </BuildActionEntry>");
				Content.WriteLine("      </BuildActionEntries>");
				Content.WriteLine("   </BuildAction>");
				Content.WriteLine("   <TestAction");
				Content.WriteLine("      buildConfiguration = \"" + DefaultConfiguration + "\"");
				Content.WriteLine("      selectedDebuggerIdentifier = \"Xcode.DebuggerFoundation.Debugger.LLDB\"");
				Content.WriteLine("      selectedLauncherIdentifier = \"Xcode.DebuggerFoundation.Launcher.LLDB\"");
				Content.WriteLine("      shouldUseLaunchSchemeArgsEnv = \"YES\">");
				Content.WriteLine("      <MacroExpansion>");
				Content.WriteLine("         <BuildableReference");
				Content.WriteLine("            BuildableIdentifier = \"primary\"");
				Content.WriteLine("            BlueprintIdentifier = \"" + TargetGuid + "\"");
				Content.WriteLine("            BuildableName = \"" + ProductName + ".app\"");
				Content.WriteLine("            BlueprintName = \"" + TargetName + "\"");
				Content.WriteLine("            ReferencedContainer = \"container:" + ProjectName + ".xcodeproj\">");
				Content.WriteLine("         </BuildableReference>");
				Content.WriteLine("      </MacroExpansion>");
				Content.WriteLine("      <Testables>");
				Content.WriteLine("      </Testables>");
				Content.WriteLine("   </TestAction>");
				Content.WriteLine("   <LaunchAction");
				Content.WriteLine("      buildConfiguration = \"" + DefaultConfiguration + "\"");
				if (sourceMappings.Count > 0)
				{
					Content.WriteLine("      customLLDBInitFile = \"" + lldbInitFile + "\"");
				}
				Content.WriteLine("      selectedDebuggerIdentifier = \"Xcode.DebuggerFoundation.Debugger.LLDB\"");
				Content.WriteLine("      selectedLauncherIdentifier = \"Xcode.DebuggerFoundation.Launcher.LLDB\"");
				Content.WriteLine("      launchStyle = \"0\"");
				Content.WriteLine("      useCustomWorkingDirectory = \"NO\"");
				Content.WriteLine("      ignoresPersistentStateOnLaunch = \"NO\"");
				Content.WriteLine("      debugDocumentVersioning = \"NO\"");
				Content.WriteLine("      debugServiceExtension = \"internal\"");
				Content.WriteLine("      allowLocationSimulation = \"YES\">");
				Content.WriteLine("      <BuildableProductRunnable");
				Content.WriteLine("         runnableDebuggingMode = \"0\">");
				Content.WriteLine("         <BuildableReference");
				Content.WriteLine("            BuildableIdentifier = \"primary\"");
				Content.WriteLine("            BlueprintIdentifier = \"" + TargetGuid + "\"");
				Content.WriteLine("            BuildableName = \"" + ProductName + ".app\"");
				Content.WriteLine("            BlueprintName = \"" + TargetName + "\"");
				Content.WriteLine("            ReferencedContainer = \"container:" + ProjectName + ".xcodeproj\">");
				Content.WriteLine("         </BuildableReference>");
				Content.WriteLine("      </BuildableProductRunnable>");
				Content.WriteLine("      <AdditionalOptions>");
				Content.WriteLine("        <AdditionalOption");
				Content.WriteLine("          key = \"APP_DISTRIBUTOR_ID_OVERRIDE\"");
				Content.WriteLine("          value = \"com.apple.AppStore\"");
				Content.WriteLine("          isEnabled = \"YES\">");
				Content.WriteLine("        </AdditionalOption>");
				Content.WriteLine(".     </AdditionalOptions>");
				if (String.IsNullOrEmpty(OldCommandLineArguments))
				{
					if (bHasEditorConfiguration && !String.IsNullOrEmpty(GameProjectPath))
					{
						Content.WriteLine("      <CommandLineArguments>");
						if (UnrealData.bIsForeignProject)
						{
							Content.WriteLine("         <CommandLineArgument");
							Content.WriteLine("            argument = \"&quot;" + GameProjectPath + "&quot;\"");
							Content.WriteLine("            isEnabled = \"YES\">");
							Content.WriteLine("         </CommandLineArgument>");
						}
						else
						{
							Content.WriteLine("         <CommandLineArgument");
							Content.WriteLine("            argument = \"" + Path.GetFileNameWithoutExtension(GameProjectPath) + "\"");
							Content.WriteLine("            isEnabled = \"YES\">");
							Content.WriteLine("         </CommandLineArgument>");
						}
						Content.WriteLine("      </CommandLineArguments>");
					}
					else if (UnrealData.TargetRules.Type == TargetType.Editor && UnrealData.UProjectFileLocation == null)
					{
						Content.WriteLine("      <CommandLineArguments>");
						Content.WriteLine("         <CommandLineArgument");
						Content.WriteLine("            argument = \"$(UE_CONTENTONLY_EDITOR_STARTUP_PROJECT)\"");
						Content.WriteLine("            isEnabled = \"YES\">");
						Content.WriteLine("         </CommandLineArgument>");
						Content.WriteLine("      </CommandLineArguments>");
					}
					else if (UnrealData.TargetRules.Type == TargetType.Editor && UnrealData.TargetRules.Type != TargetType.Program && UnrealData.UProjectFileLocation == null)
					{
						Content.WriteLine("      <CommandLineArguments>");
						Content.WriteLine("         <CommandLineArgument");
						Content.WriteLine("            argument = \"../../../$(UE_CONTENTONLY_PROJECT_NAME)/$(UE_CONTENTONLY_PROJECT_NAME).uproject\"");
						Content.WriteLine("            isEnabled = \"YES\">");
						Content.WriteLine("         </CommandLineArgument>");
						Content.WriteLine("      </CommandLineArguments>");
					}
				}
				else
				{
					Content.WriteLine("      <CommandLineArguments>" + OldCommandLineArguments + "</CommandLineArguments>");
				}
				Content.WriteLine("   </LaunchAction>");
				Content.WriteLine("   <ProfileAction");
				Content.WriteLine("      buildConfiguration = \"" + DefaultConfiguration + "\"");
				Content.WriteLine("      shouldUseLaunchSchemeArgsEnv = \"YES\"");
				Content.WriteLine("      savedToolIdentifier = \"\"");
				Content.WriteLine("      useCustomWorkingDirectory = \"NO\"");
				Content.WriteLine("      debugDocumentVersioning = \"NO\">");
				Content.WriteLine("      <BuildableProductRunnable");
				Content.WriteLine("         runnableDebuggingMode = \"0\">");
				Content.WriteLine("         <BuildableReference");
				Content.WriteLine("            BuildableIdentifier = \"primary\"");
				Content.WriteLine("            BlueprintIdentifier = \"" + TargetGuid + "\"");
				Content.WriteLine("            BuildableName = \"" + ProductName + ".app\"");
				Content.WriteLine("            BlueprintName = \"" + TargetName + "\"");
				Content.WriteLine("            ReferencedContainer = \"container:" + ProjectName + ".xcodeproj\">");
				Content.WriteLine("         </BuildableReference>");
				Content.WriteLine("      </BuildableProductRunnable>");
				Content.WriteLine("   </ProfileAction>");
				Content.WriteLine("   <AnalyzeAction");
				Content.WriteLine("      buildConfiguration = \"" + DefaultConfiguration + "\">");
				Content.WriteLine("   </AnalyzeAction>");
				Content.WriteLine("   <ArchiveAction");
				Content.WriteLine("      buildConfiguration = \"Shipping\"");
				Content.WriteLine("      revealArchiveInOrganizer = \"YES\">");
				Content.WriteLine("   </ArchiveAction>");
				Content.WriteLine("</Scheme>");

				File.WriteAllText(SchemeFilePath.FullName, Content.ToString(), new UTF8Encoding());
				Content.Clear();
			}

			Content.WriteLine("<?xml version=\"1.0\" encoding=\"UTF-8\"?>");
			Content.WriteLine("<!DOCTYPE plist PUBLIC \"-//Apple//DTD PLIST 1.0//EN\" \"http://www.apple.com/DTDs/PropertyList-1.0.dtd\">");
			Content.WriteLine("<plist version=\"1.0\">");
			Content.WriteLine("<dict>");
			Content.WriteLine("\t<key>SchemeUserState</key>");
			Content.WriteLine("\t<dict>");
			foreach (XcodeRunTarget RunTarget in RunTargets)
			{
				Content.WriteLine("\t\t<key>" + RunTarget.Name + ".xcscheme_^#shared#^_</key>");
				Content.WriteLine("\t\t<dict>");
				Content.WriteLine("\t\t\t<key>orderHint</key>");
				Content.WriteLine("\t\t\t<integer>1</integer>");
				Content.WriteLine("\t\t</dict>");
			}
			Content.WriteLine("\t</dict>");
			Content.WriteLine("\t<key>SuppressBuildableAutocreation</key>");
			Content.WriteLine("\t<dict>");
			foreach (XcodeRunTarget RunTarget in RunTargets)
			{
				Content.WriteLine("\t\t<key>" + RunTarget.Guid + "</key>");
				Content.WriteLine("\t\t<dict>");
				Content.WriteLine("\t\t\t<key>primary</key>");
				Content.WriteLine("\t\t\t<true/>");
				Content.WriteLine("\t\t</dict>");
			}
			if (BuildTargetGuid != null)
			{
				Content.WriteLine("\t\t<key>" + BuildTargetGuid + "</key>");
				Content.WriteLine("\t\t<dict>");
				Content.WriteLine("\t\t\t<key>primary</key>");
				Content.WriteLine("\t\t\t<true/>");
				Content.WriteLine("\t\t</dict>");
			}
			if (IndexTargetGuid != null)
			{
				Content.WriteLine("\t\t<key>" + IndexTargetGuid + "</key>");
				Content.WriteLine("\t\t<dict>");
				Content.WriteLine("\t\t\t<key>primary</key>");
				Content.WriteLine("\t\t\t<true/>");
				Content.WriteLine("\t\t</dict>");
			}
			Content.WriteLine("\t</dict>");
			Content.WriteLine("</dict>");
			Content.WriteLine("</plist>");

			FileReference ManagementFile = GetUserSchemeManagementFilePath(UnrealData.XcodeProjectFileLocation, Platform);
			if (!DirectoryReference.Exists(ManagementFile.Directory))
			{
				DirectoryReference.CreateDirectory(ManagementFile.Directory);
			}

			File.WriteAllText(ManagementFile.FullName, Content.ToString(), new UTF8Encoding());
		}

		public static void CleanSchemeFile(UnrealData UnrealData, UnrealTargetPlatform? Platform)
		{
			// clean this up because we don't want it persisting if we narrow our project list
			DirectoryReference SchemeDir = GetProjectSchemeDirectory(UnrealData.XcodeProjectFileLocation, Platform);

			if (DirectoryReference.Exists(SchemeDir))
			{
				DirectoryReference.Delete(SchemeDir, true);
			}
		}
	}
}
