// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using System.Text.RegularExpressions;
using EpicGames.Core;

namespace UnrealBuildTool.XcodeProjectXcconfig
{
	static class StringBuilderExtensions
	{
		public static void WriteLine(this StringBuilder SB, string Line = "")
		{
			SB.Append(Line);
			SB.Append(ProjectFileGenerator.NewLine);
		}
		public static void WriteLine(this StringBuilder SB, int Indent, string Line = "")
		{
			SB.Append(new string('\t', Indent));
			SB.Append(Line);
			SB.Append(ProjectFileGenerator.NewLine);
		}
	}

	static class XcodeUtils
	{
		// null platform means use the old way, with multiple platforms per project
		private static string Suffix(UnrealTargetPlatform? Platform)
		{
			return Platform == null ? "" : $" ({Platform})";
		}

		public static DirectoryReference ProjectDirPathForPlatform(DirectoryReference ProjectFilePath, UnrealTargetPlatform? Platform)
		{
			FileReference ProjectAsFile = new FileReference(ProjectFilePath.FullName);
			return DirectoryReference.Combine(ProjectAsFile.Directory, $"{ProjectAsFile.GetFileNameWithoutExtension()}{Suffix(Platform)}{ProjectAsFile.GetExtension()}");
		}

		private static IEnumerable<UnrealTargetConfiguration> GetSupportedConfigurations()
		{
			return new UnrealTargetConfiguration[] {
				UnrealTargetConfiguration.Debug,
				UnrealTargetConfiguration.DebugGame,
				UnrealTargetConfiguration.Development,
				UnrealTargetConfiguration.Test,
				UnrealTargetConfiguration.Shipping
			};
		}

		public static bool ShouldIncludeProjectInWorkspace(ProjectFile Proj)
		{
			// since IOS/TVOS don't have the UnrealEditor project as valid, force it so that we get the source code
			// this is likely temporary until we can put source code into UnrealGame
			if (Proj.ProjectFilePath.GetFileNameWithoutAnyExtensions() == "UnrealEditor")
			{
				return true;
			}
			foreach (Project ProjectTarget in Proj.ProjectTargets)
			{
				foreach (UnrealTargetPlatform Platform in XcodeProjectFileGenerator.XcodePlatforms)
				{
					foreach (UnrealTargetConfiguration Config in GetSupportedConfigurations())
					{
						if (MSBuildProjectFile.IsValidProjectPlatformAndConfiguration(ProjectTarget, Platform, Config))
						{
							return true;
						}
					}
				}
			}

			return false;
		}

		// cache for the below function
		static readonly Dictionary<string, UnrealArchitectures> CachedMacProjectArchitectures = new();

		/// <summary>
		/// Returns the Mac architectures that should be configured for the provided target. If the target has a project we'll adhere
		/// to whether it's set as Intel/Universal/Apple unless the type is denied (pretty much just Editor)
		/// 
		/// If the target has no project we'll support allow-listed targets for installed builds and all non-editor architectures 
		/// for source builds. Not all programs are going to compile for Apple Silicon, but being able to build and fail is useful...
		/// </summary>
		/// <param name="TargetName">The target we're generatin forg</param>
		/// <param name="InProjectFile">Path to the project file, or null if the target has no project</param>
		/// <returns></returns>
		public static UnrealArchitectures GetSupportedMacArchitectures(string TargetName, FileReference? InProjectFile)
		{
			// All architectures supported
			UnrealArchitectures AllArchitectures = new(new[] { UnrealArch.X64, UnrealArch.Arm64 });

			// Add a way on the command line of forcing a project file with all architectures (there isn't a good way to let this be
			// set and checked where we can access it).
			bool ForceAllArchitectures = Environment.GetCommandLineArgs().Contains("AllArchitectures", StringComparer.OrdinalIgnoreCase);

			if (ForceAllArchitectures)
			{
				return AllArchitectures;
			}

			UnrealArchitectures Arches;
			lock (CachedMacProjectArchitectures)
			{
				// First time seeing this target?
				if (!CachedMacProjectArchitectures.ContainsKey(TargetName))
				{
					CachedMacProjectArchitectures[TargetName] = UnrealArchitectureConfig.ForPlatform(UnrealTargetPlatform.Mac).ProjectSupportedArchitectures(InProjectFile, TargetName);
				}
				Arches = CachedMacProjectArchitectures[TargetName];
			}

			return Arches;
		}

		public static void FindPlistId(MetadataItem PlistItem, string Key, ref string? BundleId)
		{
			if (PlistItem.File == null || !FileReference.Exists(PlistItem.File) || new FileInfo(PlistItem.File.FullName).Length == 0)
			{
				return;
			}

			string Identifier = Plist($"Print :{Key}", PlistItem.File.FullName);

			// handle error
			if (String.IsNullOrEmpty(Identifier) || Identifier.StartsWith("Print:", StringComparison.Ordinal))
			{
				if (PlistItem.Mode == MetadataMode.UsePremade)
				{
					Log.TraceErrorOnce($"Premade .plist file '{PlistItem.File}' was found, but it did not contain {Key} (Key is missing or value is empty)");
				}
			}
			else
			{
				BundleId = Identifier;
			}
		}

		private static string? ActivePlistFile;

		public static void SetActivePlistFile(string PlistFile)
		{
			ActivePlistFile = PlistFile;
		}

		public static string Plist(string Command, string PlistFile)
		{
			Command = Command.Replace("\"", "\\\"", StringComparison.Ordinal);
			return Utils.RunLocalProcessAndReturnStdOut("/usr/libexec/PlistBuddy", $"-c \"{Command}\" \"{PlistFile}\"");
		}

		public static string Plist(string Command)
		{
			return Plist(Command, ActivePlistFile!);
		}

		public static void PlistSetAdd(string Entry, string Value, string Type = "string")
		{
			string AddOutput = Plist($"Add {Entry} {Type} {Value}");
			// error will be non-empty string
			if (!String.IsNullOrEmpty(AddOutput))
			{
				Plist($"Set {Entry} {Value}");
			}
		}

		public static bool PlistSetUpdate(string Entry, string Value)
		{
			// see if the setting is already there
			string ExistingSetting = Plist($"Print {Entry}");

			// Print errors start with Print
			if (!ExistingSetting.StartsWith("Print:", StringComparison.Ordinal) && ExistingSetting != Value)
			{
				Plist($"Set {Entry} {Value}");

				return true;
			}

			return false;
		}

		public static IEnumerable<string> PlistArray(string Entry)
		{
			return Plist($"Print {Entry}")
				.Replace("Array {", "", StringComparison.Ordinal)
				.Replace("}", "", StringComparison.Ordinal)
				.Trim()
				.ReplaceLineEndings()
				.Split(Environment.NewLine)
				.Select(x => x.Trim());
		}

		public static List<string> PlistObjects()
		{
			List<string> Result = new();

			IEnumerable<string> Lines = Plist("print :objects")
				.ReplaceLineEndings()
				.Split(Environment.NewLine);

			Regex Regex = new Regex("^\\s*(\\S*) = Dict {$");
			foreach (string Line in Lines)
			{
				Match Match = Regex.Match(Line);
				if (Match.Success)
				{
					Result.Add(Match.Groups[1].Value);
				}
			}
			return Result;
		}

		public static string? PlistFixPath(string Entry, string RelativeToProject)
		{
			string ExistingPath = Plist($"Print {Entry}");
			// skip of errors, or it's an absolute path
			if (!ExistingPath.StartsWith("Print:", StringComparison.Ordinal) && !ExistingPath.StartsWith('/'))
			{
				// fixup the path to be relative to new project instead of old
				string FixedPath = Utils.CollapseRelativeDirectories(Path.Combine(RelativeToProject, ExistingPath));
				// and set it back
				Plist($"Set {Entry} {FixedPath}");

				return FixedPath;
			}

			return null;
		}

		public static List<string> GetSupportedOrientations(ConfigHierarchy Ini)
		{
			List<string> Orientations = new();

			bool bSupported;
			if (Ini.TryGetValue("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bSupportsPortraitOrientation", out bSupported) && bSupported)
			{
				Orientations.Add("UIInterfaceOrientationPortrait");
			}
			if (Ini.TryGetValue("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bSupportsUpsideDownOrientation", out bSupported) && bSupported)
			{
				Orientations.Add("UIInterfaceOrientationPortraitUpsideDown");
			}

			Ini.TryGetValue("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "PreferredLandscapeOrientation", out string? PreferredLandscapeOrientation);
			Ini.TryGetValue("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bSupportsLandscapeLeftOrientation", out bool bSupportsLandscapeLeft);
			Ini.TryGetValue("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bSupportsLandscapeRightOrientation", out bool bSupportsLandscapeRight);

			if (bSupportsLandscapeLeft && PreferredLandscapeOrientation == "LandscapeLeft")
			{
				Orientations.Add("UIInterfaceOrientationLandscapeLeft");
			}
			if (bSupportsLandscapeRight)
			{
				Orientations.Add("UIInterfaceOrientationLandscapeRight");
			}
			if (bSupportsLandscapeLeft && PreferredLandscapeOrientation != "LandscapeLeft")
			{
				Orientations.Add("UIInterfaceOrientationLandscapeLeft");
			}

			return Orientations;
		}
	}
}
