// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Text;
using System.Text.Json;
using EpicGames.Core;
using Microsoft.CodeAnalysis;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

/*****

Here's how this works:

  * An XcodeProjectFile (subclass of generic ProjectFile class) is created, along with it - UnrealData and XcodeFileCollection objects are made
  * High level code calls AddModuleForIntelliSense() which this code will use to cache information about the Modules in the project (including build settings, etc)
    * These are used to determine what source files can be indexed together (we use native xcode code compilation for indexing, so we make compiling successful for best index)
    * A few #defines are removed or modified (FOO_API, etc) which would otherwise make every module a separate target
  * High level code then calls WriteProjectFile() which is the meat of all this
  * This code then creates a hierarchy/reference-chain of xcode project nodes (an xcode project is basically a series of Guid/object pairs that reference each other)
  * Then each node writes itself into the project file that is saved to disk

.xcconfig files:
  * We also now use Xcconfig files for all of the build settings, instead of jamming them into the xcode project file itself
  * This makes it easier to see the various settings (you can also see them in the Xcode UI as before, but now with more read-only columns - they must be set via editing the file)
  * The files are in the Xcode file browsing pane, in the an Xcconfigs folder under each project
  * The files are:
    * _Project - settings apply to all targets in the projectapplies to all targets), one for each configuration (Debug, Development Editor, etc) which applies to all targets
    * _Debug/_DevelopmentEditor, etc - applies to all targets when building in that configuration
    * _Run - applies to the native run project, which does no compiling
    * _Index/SubIndex0, etc - applies when Indexing, which uses native Xcode compiling (these will include #defines from UBT, etc)
  * There is currently no _Build xcconfig file since it's a makefile project and has no build setting needs


Known issues:
  * No PBXFrameworksBuildPhase nodes are made
 
Future ideas:
  * I have started working on a Template project that we merge Build/Index targets into, which will allow a licensee to setup codesigning, add extensions, etc. 
  * Always make a final build xcodeproj from UBT for handling the codesigning/extensions/frameworks
  * Allow for non-conflicting #defines to share SubIndex targets, hopefully will greatly reduce the sub targets in UE5

**/

namespace UnrealBuildTool.XcodeProjectXcconfig
{
	class UnrealBuildConfig
	{
		public UnrealBuildConfig(string InDisplayName, string InBuildTarget, string InExeName, ProjectTarget? InProjectTarget, UnrealTargetConfiguration InBuildConfig, DirectoryReference InRootDirectory)
		{
			DisplayName = InDisplayName;
			BuildTarget = InBuildTarget;
			ExeName = InExeName;
			ProjectTarget = InProjectTarget;
			BuildConfig = InBuildConfig;
			RootDirectory = InRootDirectory;

			if (BuildTarget != ProjectTarget?.Name)
			{
				throw new BuildException($"Name exepcted to match - {BuildTarget} != {ProjectTarget?.Name}");
			}
		}

		public string DisplayName;
		public string BuildTarget;
		public string ExeName;
		public ProjectTarget? ProjectTarget;
		public UnrealTargetConfiguration BuildConfig;
		public DirectoryReference RootDirectory;

		public bool bSupportsMac => Supports(UnrealTargetPlatform.Mac);
		public bool bSupportsIOS => Supports(UnrealTargetPlatform.IOS);
		public bool bSupportsTVOS => Supports(UnrealTargetPlatform.TVOS);
		public bool bSupportsVisionOS => Supports(UnrealTargetPlatform.VisionOS);
		public bool Supports(UnrealTargetPlatform? Platform)
		{
			return UnrealData.Supports(Platform) && (ProjectTarget == null || Platform == null || ProjectTarget.SupportedPlatforms.Contains((UnrealTargetPlatform)Platform));
		}
	};

	class UnrealBatchedFiles
	{
		// build settings that cause uniqueness
		public IEnumerable<string>? ForceIncludeFiles = null;
		// @todo can we actually use this effectively with indexing other than fotced include?
		public FileReference? PCHFile = null;
		public bool bEnableRTTI = false;

		// union of settings for all modules
		public HashSet<string> AllDefines = new() { "__INTELLISENSE__", "MONOLITHIC_BUILD=1" };
		public HashSet<DirectoryReference> SystemIncludePaths = new();
		public HashSet<DirectoryReference> UserIncludePaths = new();

		public List<XcodeSourceFile> Files = new();
		public UEBuildModuleCPP Module;

		public FileReference ResponseFile;

		public UnrealBatchedFiles(UnrealData UnrealData, int Index, UEBuildModuleCPP Module)
		{
			this.Module = Module;
			ResponseFile = FileReference.Combine(UnrealData.XcodeProjectFileLocation.ParentDirectory!, "ResponseFiles", Module.Rules.Target.Platform.ToString(), $"{UnrealData.ProductName}{Index}.response");
		}

		public void GenerateResponseFile()
		{
			StringBuilder ResponseFileContents = new();
			ResponseFileContents.Append("-std=c++20 ");
			ResponseFileContents.Append("-isystem");
			ResponseFileContents.AppendJoin(" -isystem", SystemIncludePaths.Select(x => x.FullName.Contains(' ', StringComparison.Ordinal) ? $"\"{x.FullName}\"" : x.FullName));
			ResponseFileContents.Append(" -I");
			ResponseFileContents.AppendJoin(" -I", UserIncludePaths.Select(x => x.FullName.Contains(' ', StringComparison.Ordinal) ? $"\"{x.FullName}\"" : x.FullName));
			if (ForceIncludeFiles != null)
			{
				ResponseFileContents.Append(" -include ");
				ResponseFileContents.AppendJoin(" -include ", ForceIncludeFiles.Select(x => x.Contains(' ', StringComparison.Ordinal) ? $"\"{x}\"" : x));
			}
			ResponseFileContents.Append(" -D");
			ResponseFileContents.AppendJoin(" -D", AllDefines);

			if (PCHFile != null)
			{
				ResponseFileContents.Append($" -include {PCHFile.FullName}");
			}

			ResponseFileContents.Append(bEnableRTTI ? " -frtti" : " -fno-rtti");

			DirectoryReference.CreateDirectory(ResponseFile.Directory);
			FileReference.WriteAllText(ResponseFile, ResponseFileContents.ToString());
		}
	}

	enum MetadataPlatform
	{
		MacEditor,
		Mac,
		IOS,
	}

	enum MetadataMode
	{
		Unset = -1,
		UsePremade,
		UpdateTemplate,
	}

	class MetadataItem
	{
		public MetadataMode Mode = MetadataMode.Unset;
		public FileReference? File = null;
		public string? XcodeProjectRelative = null;

		//public Metadata(MetadataMode Mode, FileReference File, DirectoryReference XcodeProjectFile)
		//{
		//	this.Mode = Mode;
		//	this.File = File;
		//	XcodeProjectRelative = File.MakeRelativeTo(XcodeProjectFile.ParentDirectory!);
		//}

		// Location: Can be key of setting entry in .ini, or the full file path
		// CopyFromFolderIfNotFound: If the file at "Location" does not exist, try to copy the same named file from this folder
		public MetadataItem(DirectoryReference ProductDirectory, DirectoryReference XcodeProject, ConfigHierarchy Ini, string Location, MetadataMode InMode, DirectoryReference? CopyFromFolderIfNotFound)
		{
			// no extension means it's a .ini entry
			if (Path.GetExtension(Location).Length == 0)
			{
				string? FileLocation;
				if (Ini.TryGetValue("/Script/MacTargetPlatform.XcodeProjectSettings", Location, out FileLocation) && FileLocation.Length > 0)
				{
					File = AppleExports.ConvertFilePath(ProductDirectory, FileLocation);
				}
			}
			else
			{
				File = new FileReference(Location);
			}

			if (File != null)
			{
				// Copy from source location if no such file exist
				// Except UBTGenerated, they will be generated during UBT runs
				if (!FileReference.Exists(File) && !File.ContainsName("UBTGenerated", 0) && CopyFromFolderIfNotFound != null)
				{
					FileReference SourceFile = FileReference.Combine(CopyFromFolderIfNotFound, File.GetFileName());
					if (FileReference.Exists(SourceFile))
					{
						DirectoryReference.CreateDirectory(File.Directory);
						try
						{
							FileReference.Copy(SourceFile, File, false);
						}
						catch (System.IO.IOException ex)
						{
							if (ex.Message.Contains("already exists", StringComparison.Ordinal))
							{
								// Some other thread is probably copying the same file, ignore
							}
							else
							{
								throw;
							}
						}
					}
				}
			}

			if (File != null && (File.ContainsName("UBTGenerated", 0) || FileReference.Exists(File)))
			{
				// We found a valid metadata file
				Mode = InMode;
				XcodeProjectRelative = File.MakeRelativeTo(XcodeProject.ParentDirectory!);
			}
			else
			{
				// Either key is missing, or file is missing
				Mode = MetadataMode.Unset;
			}
		}
	}

	class Metadata
	{
		public Dictionary<MetadataPlatform, MetadataItem> PlistFiles = new();
		public Dictionary<MetadataPlatform, MetadataItem> EntitlementsFiles = new();
		public Dictionary<MetadataPlatform, MetadataItem> ShippingEntitlementsFiles = new();
		public Dictionary<MetadataPlatform, MetadataItem> ProjectPrivacyInfoFiles = new();

		public Metadata(DirectoryReference ProductDirectory, DirectoryReference XcodeProject, ConfigHierarchy Ini, bool bSupportsMac, bool bSupportsIOSOrTVOS)
		{
			if (bSupportsMac)
			{
				DirectoryReference ResourceFolder = DirectoryReference.Combine(Unreal.EngineDirectory, "Build/Mac/Resources/");
	
				// All editor use template plist which is just a copy of default info.plist, user should not need to modify or change this
				// TEMP: currently cooked editor (e.g. QAGCookedEditor) uses this hardcoded file too, we should find a way to expose this in Game/Config/DefaultEngine.ini
				PlistFiles[MetadataPlatform.MacEditor] = new MetadataItem(ProductDirectory, XcodeProject, Ini,
					Unreal.EngineDirectory.FullName + "/Intermediate/Build/Mac/Resources/Info-Editor.Template.plist",
					MetadataMode.UpdateTemplate,
					ResourceFolder);

				if (ProductDirectory == Unreal.EngineDirectory)
				{
					// Engine dir is the same as Product dir, meaning no uproject (UnrealGame.app)
					// Don't use ini value in this case, since ini value points at /Game/, use this hardcoded location instead
					PlistFiles[MetadataPlatform.Mac] = new MetadataItem(ProductDirectory, XcodeProject, Ini,
						Unreal.EngineDirectory.FullName + "/Intermediate/Build/Mac/Resources/Info.Template.plist",
						MetadataMode.UpdateTemplate,
						ResourceFolder);
				}
				else
				{
					// E.g. QAGame.app
					// Try Premade first
					PlistFiles[MetadataPlatform.Mac] = new MetadataItem(ProductDirectory, XcodeProject, Ini,
						"PremadeMacPlist",
						MetadataMode.UsePremade,
						ResourceFolder);
					if (PlistFiles[MetadataPlatform.Mac].Mode == MetadataMode.Unset)
					{
						// Premade not found, try Template
						PlistFiles[MetadataPlatform.Mac] = new MetadataItem(ProductDirectory, XcodeProject, Ini,
						"TemplateMacPlist",
						MetadataMode.UpdateTemplate,
						ResourceFolder);
					}

					ProjectPrivacyInfoFiles[MetadataPlatform.Mac] = new MetadataItem(ProductDirectory, XcodeProject, Ini,
						"AdditionalPrivacyInfoMac",
						MetadataMode.UsePremade,
						null);
				}

				EntitlementsFiles[MetadataPlatform.MacEditor] = new MetadataItem(ProductDirectory, XcodeProject, Ini,
					"PremadeMacEditorEntitlements",
					MetadataMode.UsePremade,
					ResourceFolder);
				EntitlementsFiles[MetadataPlatform.Mac] = new MetadataItem(ProductDirectory, XcodeProject, Ini,
					"PremadeMacEntitlements",
					MetadataMode.UsePremade,
					ResourceFolder);
				ShippingEntitlementsFiles[MetadataPlatform.MacEditor] = new MetadataItem(ProductDirectory, XcodeProject, Ini,
					"ShippingSpecificMacEditorEntitlements",
					MetadataMode.UsePremade,
					ResourceFolder);
				ShippingEntitlementsFiles[MetadataPlatform.Mac] = new MetadataItem(ProductDirectory, XcodeProject, Ini,
					"ShippingSpecificMacEntitlements",
					MetadataMode.UsePremade,
					ResourceFolder);
			}
			if (bSupportsIOSOrTVOS)
			{
				DirectoryReference ResourceFolder = DirectoryReference.Combine(Unreal.EngineDirectory, "Build/IOS/Resources/");

				// Try Premade first
				PlistFiles[MetadataPlatform.IOS] = new MetadataItem(ProductDirectory, XcodeProject, Ini,
					"PremadeIOSPlist",
					MetadataMode.UsePremade,
					ResourceFolder);
				if (PlistFiles[MetadataPlatform.IOS].Mode == MetadataMode.Unset)
				{
					// Premade not found, try Template
					PlistFiles[MetadataPlatform.IOS] = new MetadataItem(ProductDirectory, XcodeProject, Ini,
						"TemplateIOSPlist",
						MetadataMode.UpdateTemplate,
						ResourceFolder);
				}

				if (ProductDirectory != Unreal.EngineDirectory)
				{
					ProjectPrivacyInfoFiles[MetadataPlatform.IOS] = new MetadataItem(ProductDirectory, XcodeProject, Ini,
						"AdditionalPrivacyInfoIOS",
						MetadataMode.UsePremade,
						null);
				}

				EntitlementsFiles[MetadataPlatform.IOS] = new MetadataItem(ProductDirectory, XcodeProject, Ini,
					"PremadeIOSEntitlements",
					MetadataMode.UsePremade,
					ResourceFolder);
				ShippingEntitlementsFiles[MetadataPlatform.IOS] = new MetadataItem(ProductDirectory, XcodeProject, Ini,
					"ShippingSpecificIOSEntitlements",
					MetadataMode.UsePremade,
					ResourceFolder);
			}
		}
	}

	class UnrealData
	{
		public bool bIsStubProject;
		public bool bIsForeignProject;
		public bool bMakeProjectPerTarget;
		public bool bIsContentOnlyProject;

		public TargetRules TargetRules => _TargetRules!;
		bool bIsAppBundle;

		public bool bUseAutomaticSigning = false;
		public bool bIsMergingProjects = false;
		public bool bWriteCodeSigningSettings = true;

		public Metadata? Metadata;

		public List<UnrealBuildConfig> AllConfigs = new();

		public Dictionary<UnrealTargetPlatform, List<UnrealBatchedFiles>> BatchedFiles = new();

		public List<string> ExtraPreBuildScriptLines = new();

		public FileReference? UProjectFileLocation = null;
		public DirectoryReference XcodeProjectFileLocation;
		public DirectoryReference ProductDirectory;
		public DirectoryReference? ConfigDirectory = null;

		// settings read from project configs
		public IOSProjectSettings? IOSProjectSettings;
		public IOSProjectSettings? TVOSProjectSettings;
		public IOSProjectSettings? VisionOSProjectSettings;

		// Root paths for creating lldbinit file with source mappings vfs
		public CppRootPaths? RootPaths = null;

		// Name of the product (usually the project name, but UE5.xcodeproj is actually UnrealGame product)
		public string ProductName;

		// Name of the xcode project
		public string XcodeProjectName;

		// Display name, can be overridden from commandline
		public string DisplayName;

		/// <summary>
		///  Used to mark the project for distribution (some platforms require this)
		/// </summary>
		public bool bForDistribution = false;

		/// <summary>
		///  Used to force not using the entitlements
		/// </summary>
		public bool bNoEntitlements = false;

		/// <summary>
		/// Override for bundle identifier
		/// </summary>
		public string BundleIdentifier = "";

		/// <summary>
		/// Override AppName
		/// </summary>
		public string AppName = "";

		/// <summary>
		/// Architectures supported for iOS
		/// </summary>
		public string[] SupportedIOSArchitectures = { "arm64" };

		/// <summary>
		/// UBT logger object
		/// </summary>
		public ILogger? Logger;

		private XcodeProjectFile? ProjectFile;
		private TargetRules? _TargetRules;

		public static bool bSupportsMac => Supports(UnrealTargetPlatform.Mac);
		public static bool bSupportsIOS => Supports(UnrealTargetPlatform.IOS);
		public static bool bSupportsTVOS => Supports(UnrealTargetPlatform.TVOS);
		public static bool bSupportsVisionOS => Supports(UnrealTargetPlatform.VisionOS);
		public static bool Supports(UnrealTargetPlatform? Platform)
		{
			return Platform == null || XcodeProjectFileGenerator.XcodePlatforms.Contains((UnrealTargetPlatform)Platform);
		}

		public bool IsAppBundle(UnrealTargetPlatform Platform)
		{
			if (Platform != UnrealTargetPlatform.Mac)
			{
				// mobile always need app bundles
				return true;
			}
			return bIsAppBundle;
		}

		public UnrealData(FileReference XcodeProjectFileLocation, bool bIsForDistribution, bool bForceNoEntitlements, string BundleID, string AppName, bool bMakeProjectPerTarget)
		{
			// the .xcodeproj is actually a directory
			this.XcodeProjectFileLocation = new DirectoryReference(XcodeProjectFileLocation.FullName);
			// default to engine director, will be fixed in Initialize if needed
			ProductDirectory = Unreal.EngineDirectory;
			XcodeProjectName = ProductName = XcodeProjectFileLocation.GetFileNameWithoutAnyExtensions();
			if (ProductName == "UE5")
			{
				ProductName = "UnrealGame";
			}

			this.bMakeProjectPerTarget = bMakeProjectPerTarget;
			bForDistribution = bIsForDistribution;
			bNoEntitlements = bForceNoEntitlements;
			BundleIdentifier = BundleID;
			DisplayName = String.IsNullOrEmpty(AppName) ? "$(UE_PRODUCT_NAME)" : AppName;
		}

		public void InitializeUProjectFileLocation(XcodeProjectFile ProjectFile)
		{
			if (UProjectFileLocation != null)
			{
				return;
			}

			// find a uproject file (UE5 target won't have one)
			foreach (Project Target in ProjectFile.ProjectTargets)
			{
				if (Target.UnrealProjectFilePath != null)
				{
					UProjectFileLocation = Target.UnrealProjectFilePath;
					break;
				}
			}

			if (ProjectFile.IsContentOnlyProject && XcodeProjectFileGenerator.Current?.OnlyGameProject != null &&
				XcodeProjectFileGenerator.Current.OnlyGameProject.IsUnderDirectory(ProjectFile.BaseDir))
			{
				UProjectFileLocation = XcodeProjectFileGenerator.Current.OnlyGameProject;
			}

			// now that we have a UProject file (or not), update the FileCollection RootDirectory to point to it
			ProjectFile.FileCollection.SetUProjectLocation(UProjectFileLocation);

			return;
		}

		public bool Initialize(XcodeProjectFile ProjectFile, List<UnrealTargetConfiguration> Configurations, ILogger Logger)
		{
			this.ProjectFile = ProjectFile;
			bIsForeignProject = ProjectFile.IsForeignProject;
			bIsStubProject = ProjectFile.IsStubProject;
			bIsContentOnlyProject = ProjectFile.IsContentOnlyProject;
			this.Logger = Logger;

			InitializeUProjectFileLocation(ProjectFile);

			// make sure ProjectDir is something good
			if (UProjectFileLocation != null)
			{
				ProductDirectory = UProjectFileLocation.Directory;
				ConfigDirectory = ProductDirectory;
			}
			else if (ProjectFile.ProjectTargets[0].TargetRules!.Type == TargetType.Program)
			{
				// if a Programs directory under Source has a Resources directory, then use it as the Product directory - if it doesn't have 
				// a Resources dir, then go up outside of Source and then into Programs (where .ini files are, etc)
				bool bSetProductDirectory = true;
				if (DirectoryReference.Exists(DirectoryReference.Combine(ProjectFile.BaseDir, "Resources")))
				{
					ProductDirectory = ProjectFile.BaseDir;
					// don't set the ProductDirectory, only the ConfigDirectory, below
					bSetProductDirectory = false;
				}

				DirectoryReference? ProgramFinder = DirectoryReference.Combine(ProjectFile.BaseDir);
				while (ProgramFinder != null && !String.Equals(ProgramFinder.GetDirectoryName(), "Source", StringComparison.OrdinalIgnoreCase))
				{
					ProgramFinder = ProgramFinder.ParentDirectory;
				}
				// we are now at Source directory, go up one more, then into Programs, and finally the "project" directory
				if (ProgramFinder != null)
				{
					ProgramFinder = DirectoryReference.Combine(ProgramFinder, "../Programs", ProductName);
					// if it exists, we have a ProductDir we can use for plists, icons, etc
					if (DirectoryReference.Exists(ProgramFinder))
					{
						if (bSetProductDirectory)
						{
							ProductDirectory = ProgramFinder;
						}
						ConfigDirectory = ProgramFinder;
					}
				}
			}

			// setup BundleIdentifier from ini file (if there's a specified plist file with one, that will override this)
			if (String.IsNullOrEmpty(BundleIdentifier))
			{
				ConfigHierarchy Ini = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, ConfigDirectory, UnrealTargetPlatform.Mac);
				Ini.GetString($"/Script/MacTargetPlatform.XcodeProjectSettings", "BundleIdentifier", out BundleIdentifier);
			}
			if (String.IsNullOrEmpty(BundleIdentifier))
			{
				BundleIdentifier = "$(UE_SIGNING_PREFIX).$(UE_PRODUCT_NAME_STRIPPED)";
			}

			InitializeMetadata();

			// Figure out all the desired configurations on the unreal side
			AllConfigs = GetSupportedBuildConfigs(XcodeProjectFileGenerator.XcodePlatforms, Configurations);
			// if we can't find any configs, we will fail to create a project
			if (AllConfigs.Count == 0)
			{
				return false;
			}

			// verify all configs share the same TargetRules
			if (!AllConfigs.All(x => x.ProjectTarget!.TargetRules == AllConfigs[0].ProjectTarget!.TargetRules))
			{
				throw new BuildException("All Configs must share a TargetRules. This indicates bMakeProjectPerTarget is returning false");
			}

			_TargetRules = AllConfigs[0].ProjectTarget!.TargetRules;

			// this project makes an app bundle (.app directory instead of a raw executable or dylib) if none of the fings make a non-appbundle
			bIsAppBundle = !AllConfigs.Any(x => x.ProjectTarget!.TargetRules!.bIsBuildingConsoleApplication || x.ProjectTarget.TargetRules.bShouldCompileAsDLL);

			// read config settings
			if (AllConfigs.Any(x => x.bSupportsIOS))
			{
				IOSPlatform IOSPlatform = ((IOSPlatform)UEBuildPlatform.GetBuildPlatform(UnrealTargetPlatform.IOS));
				IOSProjectSettings = IOSPlatform.ReadProjectSettings(UProjectFileLocation);
			}

			if (AllConfigs.Any(x => x.bSupportsTVOS))
			{
				TVOSPlatform TVOSPlatform = ((TVOSPlatform)UEBuildPlatform.GetBuildPlatform(UnrealTargetPlatform.TVOS));
				TVOSProjectSettings = TVOSPlatform.ReadProjectSettings(UProjectFileLocation);
			}

			if (AllConfigs.Any(x => x.bSupportsVisionOS))
			{
				// this may not exist since it's a PlatformExtension and the VisionOS files may not be preset
				UEBuildPlatform? BuildPlatform;
				if (UEBuildPlatform.TryGetBuildPlatform(UnrealTargetPlatform.VisionOS, out BuildPlatform))
				{
					IOSPlatform VisionOSPlatform = (IOSPlatform)BuildPlatform;
					VisionOSProjectSettings = VisionOSPlatform.ReadProjectSettings(UProjectFileLocation);
				}
			}

			if (TargetRules?.bUseVFS == true)
			{
				CppRootPaths RootPaths = new() { bUseVfs = TargetRules.bUseVFS };
				RootPaths[CppRootPathFolder.Root] = Unreal.RootDirectory;
				if (UProjectFileLocation != null)
				{
					RootPaths[CppRootPathFolder.Project] = UProjectFileLocation.Directory;
				}
			}

			return true;
		}

		private void InitializeMetadata()
		{

			// read setings from the configs, now that we have a project
			ConfigHierarchy SharedPlatformIni = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, ConfigDirectory, UnrealTargetPlatform.Mac);
			SharedPlatformIni.TryGetValue("/Script/MacTargetPlatform.XcodeProjectSettings", "bUseAutomaticCodeSigning", out bUseAutomaticSigning);

			Metadata = new Metadata(ProductDirectory, XcodeProjectFileLocation, SharedPlatformIni, bSupportsMac, bSupportsIOS || bSupportsTVOS || bSupportsVisionOS);
		}

		public string? FindFile(List<string> Paths, UnrealTargetPlatform Platform, bool bMakeRelative)
		{
			foreach (string Entry in Paths)
			{
				string FinalPath = Entry.Replace("$(Engine)", Unreal.EngineDirectory.FullName, StringComparison.Ordinal);
				FinalPath = FinalPath.Replace("$(Project)", ProductDirectory.FullName, StringComparison.Ordinal);
				FinalPath = FinalPath.Replace("$(Platform)", Platform.ToString(), StringComparison.Ordinal);

				//				Console.WriteLine($"Looking for {FinalPath}");
				if (File.Exists(FinalPath) || Directory.Exists(FinalPath))
				{
					//					Console.WriteLine($"  Found it!");
					if (bMakeRelative)
					{
						FinalPath = new FileReference(FinalPath).MakeRelativeTo(XcodeProjectFileLocation.ParentDirectory!);
					}
					return FinalPath;
				}
			}
			return null;
		}

		public string ProjectOrEnginePath(string SubPath, bool bMakeRelative, string? AltProjectSubPath = null)
		{
			string? FinalPath = null;
			if (ProductDirectory != Unreal.EngineDirectory)
			{
				string PathToCheck = Path.Combine(ProductDirectory.FullName, SubPath);
				if (File.Exists(PathToCheck) || Directory.Exists(PathToCheck))
				{
					FinalPath = PathToCheck;
				}
				else if (AltProjectSubPath != null)
				{
					// allow for an alternate project sub path, for back compat. We wouldn't use the the alt path in Engine
					// because we would have fixed it up
					PathToCheck = Path.Combine(ProductDirectory.FullName, AltProjectSubPath);
					if (File.Exists(PathToCheck) || Directory.Exists(PathToCheck))
					{
						FinalPath = PathToCheck;
					}
				}
			}

			// if the SubPath (or optional AlProjectsubPath) wasn't found, then fall back to engine location
			FinalPath ??= Path.Combine(Unreal.EngineDirectory.FullName, SubPath);
			if (bMakeRelative)
			{
				FinalPath = new FileReference(FinalPath).MakeRelativeTo(XcodeProjectFileLocation.ParentDirectory!);
			}

			return FinalPath;
		}

		public void AddModuleForIntelliSense(UEBuildModuleCPP Module, CppCompileEnvironment CompileEnvironment)
		{
			// one batched files per module per platform
			if (!BatchedFiles.ContainsKey(Module.Rules.Target.Platform))
			{
				BatchedFiles[Module.Rules.Target.Platform] = new();
			}
			UnrealBatchedFiles FileBatch = new UnrealBatchedFiles(this, BatchedFiles[Module.Rules.Target.Platform].Count + 1, Module);
			BatchedFiles[Module.Rules.Target.Platform].Add(FileBatch);

			if (CompileEnvironment.ForceIncludeFiles.Count == 0)
			{
				// if there are no ForceInclude files, then that means it's a module that forces the includes to come from a generated PCH file
				// and so we will use this for definitions and uniqueness
				if (CompileEnvironment.PrecompiledHeaderIncludeFilename != null)
				{
					FileBatch.PCHFile = FileReference.Combine(XcodeProjectFileLocation.ParentDirectory!, "PCHFiles", CompileEnvironment.PrecompiledHeaderIncludeFilename.GetFileName());
					DirectoryReference.CreateDirectory(FileBatch.PCHFile.Directory);
					FileReference.Copy(CompileEnvironment.PrecompiledHeaderIncludeFilename, FileBatch.PCHFile, true);
				}
			}
			else
			{
				FileBatch.ForceIncludeFiles = CompileEnvironment.ForceIncludeFiles.Select(x => x.FullName);
			}

			FileBatch.bEnableRTTI = CompileEnvironment.bUseRTTI;
			FileBatch.SystemIncludePaths.UnionWith(CompileEnvironment.SystemIncludePaths);
			FileBatch.UserIncludePaths.UnionWith(CompileEnvironment.UserIncludePaths);
		}

		private List<UnrealBuildConfig> GetSupportedBuildConfigs(List<UnrealTargetPlatform> Platforms, List<UnrealTargetConfiguration> Configurations)
		{
			List<UnrealBuildConfig> BuildConfigs = new List<UnrealBuildConfig>();

			//string ProjectName = ProjectFilePath.GetFileNameWithoutExtension();

			foreach (UnrealTargetConfiguration Configuration in Configurations)
			{
				if (InstalledPlatformInfo.IsValidConfiguration(Configuration, EProjectType.Code))
				{
					foreach (UnrealTargetPlatform Platform in Platforms)
					{
						if (InstalledPlatformInfo.IsValidPlatform(Platform, EProjectType.Code) && Platform.IsInGroup(UnrealPlatformGroup.Apple)) // @todo support other platforms
						{
							UEBuildPlatform? BuildPlatform;
							if (UEBuildPlatform.TryGetBuildPlatform(Platform, out BuildPlatform) && (BuildPlatform.HasRequiredSDKsInstalled() == SDKStatus.Valid))
							{
								// Check we have targets (Expected to be no Engine targets when generating for a single .uproject)
								if (ProjectFile!.ProjectTargets.Count == 0 && ProjectFile!.BaseDir != Unreal.EngineDirectory)
								{
									throw new BuildException($"Expecting at least one ProjectTarget to be associated with project '{XcodeProjectFileLocation}' in the TargetProjects list ");
								}

								// Now go through all of the target types for this project
								foreach (ProjectTarget ProjectTarget in ProjectFile.ProjectTargets.OfType<ProjectTarget>())
								{
									if (MSBuildProjectFile.IsValidProjectPlatformAndConfiguration(ProjectTarget, Platform, Configuration))
									{
										// Figure out if this is a monolithic build
										bool bShouldCompileMonolithic = BuildPlatform.ShouldCompileMonolithicBinary(Platform);
										bool bForceRetainTargetNameAsExeName = false;
										try
										{
											TargetRules ProjectTargetRules = ProjectTarget.CreateRulesDelegate(Platform, Configuration);
											bShouldCompileMonolithic |= ProjectTargetRules.LinkType == TargetLinkType.Monolithic;
											bForceRetainTargetNameAsExeName = ProjectTargetRules.MacPlatform.bForceTargetNameAsExeName;
										}
										catch (BuildException)
										{
										}

										string ConfigName = Configuration.ToString();
										if (!bMakeProjectPerTarget)
										{
											if (ProjectTarget.TargetRules!.Type != TargetType.Game && ProjectTarget.TargetRules.Type != TargetType.Program)
											{
												ConfigName += " " + ProjectTarget.TargetRules.Type.ToString();
											}
										}

										if (BuildConfigs.Where(Config => Config.DisplayName == ConfigName).ToList().Count == 0)
										{
											string TargetName = ProjectTarget.TargetFilePath.GetFileNameWithoutAnyExtensions();
											// Get the .uproject directory
											DirectoryReference? UProjectDirectory = DirectoryReference.FromFile(ProjectTarget.UnrealProjectFilePath);

											// Get the output directory
											DirectoryReference RootDirectory;
											if (UProjectDirectory != null &&
												(bShouldCompileMonolithic || ProjectTarget.TargetRules!.BuildEnvironment == TargetBuildEnvironment.Unique) &&
												ProjectTarget.TargetRules!.File!.IsUnderDirectory(UProjectDirectory))
											{
												RootDirectory = UEBuildTarget.GetOutputDirectoryForExecutable(UProjectDirectory, ProjectTarget.TargetRules.File!);
											}
											else
											{
												RootDirectory = UEBuildTarget.GetOutputDirectoryForExecutable(Unreal.EngineDirectory, ProjectTarget.TargetRules!.File!);
											}

											string ExeName = TargetName;
											if (!bShouldCompileMonolithic
												&& ProjectTarget.TargetRules.Type != TargetType.Program
												&& !bForceRetainTargetNameAsExeName)
											{
												// Figure out what the compiled binary will be called so that we can point the IDE to the correct file
												if (ProjectTarget.TargetRules.Type != TargetType.Game)
												{
													// Only if shared - unique retains the Target Name
													if (ProjectTarget.TargetRules.BuildEnvironment == TargetBuildEnvironment.Shared)
													{
														ExeName = "Unreal" + ProjectTarget.TargetRules.Type.ToString();
													}
												}
											}

											BuildConfigs.Add(new UnrealBuildConfig(ConfigName, TargetName, ExeName, ProjectTarget, Configuration, RootDirectory));
										}
									}
								}
							}
						}
					}
				}
			}

			return BuildConfigs;
		}
	}

	abstract class XcodeProjectNode
	{
		// keeps a list of other node this node references, which is used when writing out the whole xcode project file
		public List<XcodeProjectNode> References = new();

		// optional Xcconfig file 
		public XcconfigFile? Xcconfig = null;

		/// <summary>
		/// Abstract function the individual node classes must override to write out the node to the project file
		/// </summary>
		/// <param name="Content"></param>
		public abstract void Write(StringBuilder Content);

		/// <summary>
		/// Walks the references of the given node to find all nodes of the given type. 
		/// </summary>
		/// <typeparam name="T">Parent class of the nodes to return</typeparam>
		/// <param name="Node">Root node to start with</param>
		/// <returns>Set of matching nodes</returns>
		public static IEnumerable<T> GetNodesOfType<T>(XcodeProjectNode Node) where T : XcodeProjectNode
		{
			// gather the nodes without recursion
			LinkedList<XcodeProjectNode> Nodes = new();
			Nodes.AddLast(Node);

			// pull off the front of the "deque" amd add its references to the back, gather
			List<XcodeProjectNode> Return = new();
			while (Nodes.Count > 0)
			{
				XcodeProjectNode Head = Nodes.First();
				Nodes.RemoveFirst();
				Head.References.ForEach(x => Nodes.AddLast(x));

				// remember them all 
				Return.AddRange(Head.References);
			}

			// filter down
			return Return.OfType<T>();
		}

		public void CreateXcconfigFile(XcodeProject Project, UnrealTargetPlatform? Platform, string Name)
		{
			DirectoryReference XcodeProjectDirectory = Project.UnrealData.XcodeProjectFileLocation.ParentDirectory!;
			Xcconfig = new XcconfigFile(XcodeProjectDirectory, Platform, Name);
			Project.FileCollection.AddFileReference(Xcconfig.Guid, Xcconfig.FileRef.MakeRelativeTo(XcodeProjectDirectory), "explicitFileType", "test.xcconfig", "\"<group>\"", "Xcconfigs");
		}

		public virtual void WriteXcconfigFile(ILogger Logger)
		{

		}

		/// <summary>
		/// THhis will walk the node reference tree and call WRite on each node to add all needed nodes to the xcode poject file
		/// </summary>
		/// <param name="Content"></param>
		/// <param name="Node"></param>
		/// <param name="Logger"></param>
		/// <param name="WrittenNodes"></param>
		public static void WriteNodeAndReferences(StringBuilder Content, XcodeProjectNode Node, ILogger Logger, HashSet<XcodeProjectNode>? WrittenNodes = null)
		{
			WrittenNodes ??= new();

			// write the node into the xcode project file
			Node.Write(Content);
			Node.WriteXcconfigFile(Logger);

			foreach (XcodeProjectNode Reference in Node.References)
			{
				if (!WrittenNodes.Contains(Reference))
				{
					WrittenNodes.Add(Reference);
					WriteNodeAndReferences(Content, Reference, Logger, WrittenNodes);
				}
			}
		}
	}

	class XcodeDependency : XcodeProjectNode
	{
		public XcodeTarget Target;
		public string Guid = XcodeProjectFileGenerator.MakeXcodeGuid();
		public string ProxyGuid = XcodeProjectFileGenerator.MakeXcodeGuid();
		public string ProjectGuid;

		public XcodeDependency(XcodeTarget Target, string ProjectGuid)
		{
			this.Target = Target;
			this.ProjectGuid = ProjectGuid;

			References.Add(Target);
		}

		public override void Write(StringBuilder Content)
		{
			Content.WriteLine("/* Begin PBXContainerItemProxy section */");
			Content.WriteLine($"\t\t{ProxyGuid} /* PBXContainerItemProxy */ = {{");
			Content.WriteLine("\t\t\tisa = PBXContainerItemProxy;");
			Content.WriteLine($"\t\t\tcontainerPortal = {ProjectGuid} /* Project object */;");
			Content.WriteLine("\t\t\tproxyType = 1;");
			Content.WriteLine($"\t\t\tremoteGlobalIDString = {Target.Guid};");
			Content.WriteLine($"\t\t\tremoteInfo = \"{Target.Name}\";");
			Content.WriteLine("\t\t};");
			Content.WriteLine("/* End PBXContainerItemProxy section */");
			Content.WriteLine("");

			Content.WriteLine("/* Begin PBXTargetDependency section */");
			Content.WriteLine($"\t\t{Guid} /* PBXTargetDependency */ = {{");
			Content.WriteLine("\t\t\tisa = PBXTargetDependency;");
			Content.WriteLine($"\t\t\ttarget = {Target.Guid} /* {Target.Name} */;");
			Content.WriteLine($"\t\t\ttargetProxy = {ProxyGuid} /* PBXContainerItemProxy */;");
			Content.WriteLine("\t\t};");
			Content.WriteLine("/* End PBXTargetDependency section */");
		}
	}

	abstract class XcodeBuildPhase : XcodeProjectNode
	{
		public string Name;
		public string IsAType;
		public string Guid = XcodeProjectFileGenerator.MakeXcodeGuid();
		protected List<XcodeSourceFile> FileItems = new();
		protected List<string> MiscItems = new();

		public XcodeBuildPhase(string Name, string IsAType)
		{
			this.Name = Name;
			this.IsAType = IsAType;
		}

		public override void Write(StringBuilder Content)
		{
			Content.WriteLine($"/* Begin {IsAType} section */");
			Content.WriteLine(2, $"{Guid} = {{");
			Content.WriteLine(3, $"isa = {IsAType};");
			Content.WriteLine(3, "buildActionMask = 2147483647;");
			Content.WriteLine(3, "files = (");
			foreach (XcodeSourceFile File in FileItems)
			{
				Content.WriteLine(4, $"{File.FileGuid} /* {File.Reference.GetFileName()} in {Name} */,");
			}
			Content.WriteLine(3, ");");

			foreach (string Line in MiscItems)
			{
				Content.WriteLine(3, Line);
			}

			Content.WriteLine(2, "};");
			Content.WriteLine($"/* End {IsAType} section */");
		}
	}

	class XcodeSourcesBuildPhase : XcodeBuildPhase
	{
		public XcodeSourcesBuildPhase()
			: base("Sources", "PBXSourcesBuildPhase")
		{
		}

		public void AddFile(XcodeSourceFile File)
		{
			FileItems.Add(File);
		}
	}

	class XcodeResourcesBuildPhase : XcodeBuildPhase
	{
		private readonly XcodeFileCollection FileCollection;

		public XcodeResourcesBuildPhase(XcodeFileCollection FileCollection)
			: base("Resources", "PBXResourcesBuildPhase")
		{
			this.FileCollection = FileCollection;
		}

		public void AddResource(FileReference Resource)
		{
			XcodeSourceFile ResourceSource = new XcodeSourceFile(Resource, null);
			FileCollection.ProcessFile(ResourceSource, true, false, "Resources");

			FileItems.Add(ResourceSource);
		}

		public void AddFolderResource(DirectoryReference Resource, string GroupName)
		{
			XcodeSourceFile ResourceSource = new XcodeSourceFile(new FileReference(Resource.FullName), null);
			FileCollection.ProcessFile(ResourceSource, true, false, GroupName);

			//Project.FileCollection.AddFolderReference(CookedData.MakeRelativeTo(UnrealData.XcodeProjectFileLocation.ParentDirectory!), "CookedData_Game");
			FileItems.Add(ResourceSource);
		}
	}

	class XcodeFrameworkBuildPhase : XcodeBuildPhase
	{
		private readonly XcodeFileCollection FileCollection;

		public XcodeFrameworkBuildPhase(XcodeFileCollection FileCollection)
			: base("Frameworks", "PBXFrameworksBuildPhase")
		{
			this.FileCollection = FileCollection;
		}

		public void AddFramework(DirectoryReference Framework, string FileRefGuid)
		{
			XcodeSourceFile FrameworkSource = new XcodeSourceFile(new FileReference(Framework.FullName), null, FileRefGuid);
			FileCollection.ProcessFile(FrameworkSource, true, false, "Frameworks", "");
			;
			FileItems.Add(FrameworkSource);
		}
	}

	class XcodeCopyFilesBuildPhase : XcodeBuildPhase
	{
		private readonly XcodeFileCollection FileCollection;

		public XcodeCopyFilesBuildPhase(XcodeFileCollection FileCollection)
			: base("Embed Frameworks", "PBXCopyFilesBuildPhase")
		{
			this.FileCollection = FileCollection;

			MiscItems.Add($"dstPath = \"\";");
			MiscItems.Add($"dstSubfolderSpec = 10;");
			MiscItems.Add($"name = \"{Name}\";");
		}

		public void AddFramework(DirectoryReference Framework, string FileRefGuid)
		{
			XcodeSourceFile FrameworkSource = new XcodeSourceFile(new FileReference(Framework.FullName), null, FileRefGuid);
			FileCollection.ProcessFile(FrameworkSource, true, false, "", "settings = {ATTRIBUTES = (CodeSignOnCopy, RemoveHeadersOnCopy, ); };");
			FileItems.Add(FrameworkSource);
		}
	}

	class XcodeCopyAppExtFilesBuildPhase : XcodeBuildPhase
	{
		private readonly XcodeFileCollection FileCollection;
		private readonly ExtensionConfig ExtConfig;

		public XcodeCopyAppExtFilesBuildPhase(XcodeFileCollection FileCollection, ExtensionConfig ExtConfig)
			: base("Embed Foundation Extensions", "PBXCopyFilesBuildPhase")
		{
			this.FileCollection = FileCollection;
			this.ExtConfig = ExtConfig ?? new ExtensionConfig();

			string ExtType = this.ExtConfig.ExtensionType ?? "";
			if (ExtType == "BackgroundAssets")
			{
				MiscItems.Add($"dstPath = \"${{EXTENSIONS_FOLDER_PATH}}\";");
				MiscItems.Add($"dstSubfolderSpec = 16;");
			}
			else
			{
				MiscItems.Add($"dstPath = \"\";");
				MiscItems.Add($"dstSubfolderSpec = 13;");
			}

			MiscItems.Add($"name = \"{Name}\";");
		}

		public void AddExtension(DirectoryReference Extension, string FileRefGuid)
		{
			XcodeSourceFile ExtensionSource = new XcodeSourceFile(new FileReference(Extension.FullName), null, FileRefGuid);
			FileCollection.ProcessFile(ExtensionSource, true, true, "", "settings = {ATTRIBUTES = (RemoveHeadersOnCopy, ); };");
			FileItems.Add(ExtensionSource);
		}
	}

	class XcodeShellScriptBuildPhase : XcodeBuildPhase
	{
		public XcodeShellScriptBuildPhase(string Name, IEnumerable<string> ScriptLines, IEnumerable<string> Inputs, IEnumerable<string> Outputs, bool bInstallOnly = false)
			: base(Name, "PBXShellScriptBuildPhase")
		{
			MiscItems.Add($"name = \"{Name}\";");

			MiscItems.Add($"inputPaths = (");
			foreach (string Input in Inputs)
			{
				MiscItems.Add($"\t\"{Input}\"");
			}
			MiscItems.Add($");");

			MiscItems.Add($"outputPaths = (");
			foreach (string Output in Outputs)
			{
				MiscItems.Add($"\t\"{Output}\"");
			}
			MiscItems.Add($");");

			//			string Script = String.Join("&#10", ScriptLines);
			string Script = String.Join("\\n", ScriptLines);
			MiscItems.Add($"shellPath = /bin/sh;");
			MiscItems.Add($"shellScript = \"{Script}\";");
			if (bInstallOnly)
			{
				MiscItems.Add("runOnlyForDeploymentPostprocessing = 1;");
			}
		}
	}

	class XcodeBuildConfig : XcodeProjectNode
	{
		public string Guid = XcodeProjectFileGenerator.MakeXcodeGuid();
		public UnrealBuildConfig Info;
		// Because we don't make project-wide .xcconfig files, we need to specify at the project level that all of the platforms are supported,
		// so that the _Build targets will have any platform as a supported platform, otherwise it can only compile for Mac (default Xcode platform)
		private readonly bool bIncludeAllPlatforms;

		public XcodeBuildConfig(UnrealBuildConfig Info, bool bIncludeAllPlatforms)
		{
			this.Info = Info;
			this.bIncludeAllPlatforms = bIncludeAllPlatforms;
		}

		public override void Write(StringBuilder Content)
		{
			Content.WriteLine(2, $"{Guid} /* {Info.DisplayName} */ = {{");
			Content.WriteLine(3, "isa = XCBuildConfiguration;");
			if (Xcconfig != null)
			{
				Content.WriteLine(3, $"baseConfigurationReference = {Xcconfig.Guid} /* {Xcconfig.Name}.xcconfig */;");
			}
			Content.WriteLine(3, "buildSettings = {");
			Content.WriteLine(4, "SWIFT_VERSION = 5.0;");
			if (bIncludeAllPlatforms)
			{
				Content.WriteLine(4, $"SUPPORTED_PLATFORMS = \"macosx iphoneos appletvos xros xrsimulator\";");
				Content.WriteLine(4, $"ONLY_ACTIVE_ARCH = YES;");

				if (Info.bSupportsMac)
				{
					string SupportedMacArchitectures = String.Join(" ", XcodeUtils.GetSupportedMacArchitectures(Info.BuildTarget, Info.ProjectTarget?.UnrealProjectFilePath).Architectures.Select(x => x.AppleName));
					Content.WriteLine(4, $"\"VALID_ARCHS[sdk=macos*]\" = \"{SupportedMacArchitectures}\";");
				}
			}

			Content.WriteLine(3, "};");
			Content.WriteLine(3, $"name = \"{Info.DisplayName}\";");
			Content.WriteLine(2, "};");
		}
	}

	class XcodeBuildConfigList : XcodeProjectNode
	{
		public string Guid = XcodeProjectFileGenerator.MakeXcodeGuid();
		public string TargetName;
		private readonly UnrealData UnrealData;

		public List<XcodeBuildConfig> BuildConfigs = new();

		public bool bSupportsMac => Supports(UnrealTargetPlatform.Mac);
		public bool bSupportsIOS => Supports(UnrealTargetPlatform.IOS);
		public bool bSupportsTVOS => Supports(UnrealTargetPlatform.TVOS);
		public bool bSupportsVisionOS => Supports(UnrealTargetPlatform.VisionOS);
		public bool Supports(UnrealTargetPlatform? Platform)
		{
			return this.Platform == Platform || (this.Platform == null && BuildConfigs.Any(x => x.Info.Supports(Platform)));
		}

		public UnrealTargetPlatform? Platform;
		public XcodeBuildConfigList(UnrealTargetPlatform? Platform, string TargetName, UnrealData UnrealData, bool bIncludeAllPlatformsInConfig)
		{
			this.Platform = Platform;
			this.UnrealData = UnrealData;

			if (UnrealData.AllConfigs.Count == 0)
			{
				throw new BuildException("Created a XcodeBuildConfigList with no BuildConfigs. This likely means a target was created too early");
			}

			this.TargetName = TargetName;

			// create build config objects for each info passed in, and them as references
			IEnumerable<XcodeBuildConfig> Configs = UnrealData.AllConfigs.Select(x => new XcodeBuildConfig(x, bIncludeAllPlatformsInConfig));
			// filter out configs that dont match a platform if we are single-platform mode
			Configs = Configs.Where(x => Platform == null || x.Info.Supports((UnrealTargetPlatform)Platform));
			BuildConfigs = Configs.ToList();
			References.AddRange(BuildConfigs);
		}

		public override void Write(StringBuilder Content)
		{
			// figure out the default configuration to use
			string Default = "Development";
			if (!UnrealData.bMakeProjectPerTarget && BuildConfigs.Any(x => x.Info.DisplayName.Contains(" Editor", StringComparison.Ordinal)))
			{
				Default = "Development Editor";
			}

			Content.WriteLine(2, $"{Guid} /* Build configuration list for target {TargetName} */ = {{");
			Content.WriteLine(3, "isa = XCConfigurationList;");
			Content.WriteLine(3, "buildConfigurations = (");
			foreach (XcodeBuildConfig Config in BuildConfigs)
			{
				Content.WriteLine(4, $"{Config.Guid} /* {Config.Info.DisplayName} */,");
			}
			Content.WriteLine(3, ");");
			Content.WriteLine(3, "defaultConfigurationIsVisible = 0;");
			Content.WriteLine(3, $"defaultConfigurationName = \"{Default}\";");
			Content.WriteLine(2, "};");
		}
	}

	class XcodeTarget : XcodeProjectNode
	{
		public enum Type
		{
			Run_App,
			App_Ext,
			Run_Tool,
			Build,
			Index,
		}

		// Guid for this target
		public string Guid = XcodeProjectFileGenerator.MakeXcodeGuid();
		//		string TargetAppGuid = XcodeProjectFileGenerator.MakeXcodeGuid();

		// com.apple.product-type.application, etc
		readonly string ProductType;

		// xcode target type name
		readonly string TargetTypeName;
		readonly Type TargetType;

		// UnrealEngine_Build, etc
		public string Name;

		// QAGame, QAGameEditor, etc
		protected string UnrealTargetName;

		// list of build configs this target supports (for instance, the Index target only indexes a Development config) 
		public XcodeBuildConfigList? BuildConfigList;

		// dependencies for this target
		public List<XcodeDependency> Dependencies = new List<XcodeDependency>();

		// build phases for this target (source, resource copying, etc)
		public List<XcodeBuildPhase> BuildPhases = new List<XcodeBuildPhase>();

		private readonly FileReference? GameProject;

		public XcodeTarget(Type Type, UnrealData UnrealData, string? OverrideName = null)
		{
			// when we are content only, we do not want to build with the uproject on the commandline, we will be building UnrealGame 
			GameProject = UnrealData.bIsContentOnlyProject ? null : UnrealData.UProjectFileLocation;

			string ConfigName;
			TargetType = Type;
			switch (Type)
			{
				case Type.Run_App:
					ProductType = "com.apple.product-type.application";
					TargetTypeName = "PBXNativeTarget";
					ConfigName = "_Run";
					break;
				case Type.App_Ext:
					ProductType = "com.apple.product-type.app-extension";
					TargetTypeName = "PBXNativeTarget";
					ConfigName = "_Run";
					break;
				case Type.Run_Tool:
					ProductType = "com.apple.product-type.tool";
					TargetTypeName = "PBXNativeTarget";
					ConfigName = "_Run";
					break;
				case Type.Build:
					ProductType = "com.apple.product-type.library.static";
					TargetTypeName = "PBXLegacyTarget";
					ConfigName = "_Build";
					break;
				case Type.Index:
					ProductType = "com.apple.product-type.library.static";
					TargetTypeName = "PBXNativeTarget";
					ConfigName = "_Index";
					break;
				default:
					throw new BuildException($"Unhandled target type {Type}");
			}

			// set up names
			UnrealTargetName = UnrealData.TargetRules.Name;
			Name = OverrideName ?? (UnrealData.XcodeProjectName + ConfigName);
		}

		public void AddDependency(XcodeTarget Target, XcodeProject Project)
		{
			XcodeDependency Dependency = new XcodeDependency(Target, Project.Guid);
			Dependencies.Add(Dependency);
			References.Add(Dependency);
		}

		public override void Write(StringBuilder Content)
		{
			Content.WriteLine($"/* Begin PBXNativeTarget section ({TargetType}) */");

			Content.WriteLine(2, $"{Guid} /* {Name} */ = {{");
			Content.WriteLine(3, $"isa = {TargetTypeName};");

			Content.WriteLine(3, $"buildConfigurationList = {BuildConfigList!.Guid} /* Build configuration list for {TargetTypeName} \"{Name}\" */;");

			if (TargetType == Type.Build)
			{
				// get paths to Unreal bits to be able to run UBT
				string UProjectParam = GameProject == null ? "" : $"{GameProject.FullName.Replace(" ", "\\ ", StringComparison.Ordinal)}";
				string UEDir = XcodeFileCollection.ConvertPath(Path.GetFullPath(Directory.GetCurrentDirectory() + "../../.."));
				string BuildToolPath = UEDir + "/Engine/Build/BatchFiles/Mac/XcodeBuild.sh";

				// insert elements to call UBT when building
				Content.WriteLine(3, $"buildArgumentsString = \"$(ACTION) {UnrealTargetName} $(PLATFORM_NAME) $(CONFIGURATION) \\\"{UProjectParam}\\\"\";");
				Content.WriteLine(3, $"buildToolPath = \"{BuildToolPath}\";");
				Content.WriteLine(3, $"buildWorkingDirectory = \"{UEDir}\";");
			}
			Content.WriteLine(3, "buildPhases = (");
			foreach (XcodeBuildPhase BuildPhase in BuildPhases)
			{
				Content.WriteLine(4, $"{BuildPhase.Guid} /* {BuildPhase.Name} */,");
			}
			Content.WriteLine(3, ");");
			Content.WriteLine(3, "dependencies = (");
			foreach (XcodeDependency Dependency in Dependencies)
			{
				Content.WriteLine(4, $"{Dependency.Guid} /* {Dependency.Target.Name} */,");
			}
			Content.WriteLine(3, ");");
			Content.WriteLine(3, $"name = \"{Name}\";");
			Content.WriteLine(3, "passBuildSettingsInEnvironment = 1;");
			Content.WriteLine(3, $"productType = \"{ProductType}\";");
			WriteExtraTargetProperties(Content);
			Content.WriteLine(2, "};");

			Content.WriteLine($"/* End PBXNativeTarget section ({TargetType})*/");
		}

		/// <summary>
		/// Let subclasses add extra properties into this target section
		/// </summary>
		protected virtual void WriteExtraTargetProperties(StringBuilder Content)
		{
			// nothing by default
		}
		
		private static readonly Dictionary<string, string> PlistFileMap = new(StringComparer.OrdinalIgnoreCase);
		protected static string GetPlistSigningName(string ProvisionSetting, UnrealData UnrealData, ILogger Logger)
		{
			string? SigningName;
			lock (PlistFileMap)
			{
				if (!PlistFileMap.TryGetValue(ProvisionSetting, out SigningName))
				{
					FileReference ProfileFile = AppleExports.ConvertFilePath(UnrealData.UProjectFileLocation?.Directory, ProvisionSetting);

					// get mobile provision UUID, either directly or from the provision
					if (ProfileFile.GetExtension() == ".mobileprovision")
					{
						// extract UUID from .mobileprovision
						string command = $"-c \"egrep -a -A 2 UUID {ProfileFile.FullName.Replace(" ", "\\ ", StringComparison.Ordinal)} | grep string | xargs\"";
						string UUID = Utils.RunLocalProcessAndReturnStdOut("/bin/sh", command);
						// extract from <string>XXX-XXX-XXX</string>
						UUID = UUID.Substring("<string>".Length, UUID.Length - "<string></string>".Length);

						// make sure it's living in user's library
						// (note: i couldn't find a way to install it without opening Xcode, so copying it works well enough)
						FileReference InstalledProfileFile = FileReference.Combine(AppleExports.GetProvisionDirectory(), $"{UUID}.mobileprovision");

						// copy if needed
						if (!FileReference.Exists(InstalledProfileFile))
						{
							DirectoryReference.CreateDirectory(InstalledProfileFile.Directory);
							FileReference.Copy(ProfileFile, InstalledProfileFile);
							Logger.LogInformation("Copying project's provision '{SourceProvision}' to your library: '{TargetProvision}'", ProfileFile, InstalledProfileFile);
						}

						SigningName = UUID;
					}
					else
					{
						SigningName = ProvisionSetting;
					}

					PlistFileMap[ProvisionSetting] = SigningName;
				}
			}

			return SigningName!;
		}
	}

	#region Read-only accessor properties
#pragma warning disable CS1591
	public class ExtensionConfig
	{
		public int FileVersion { get; set; }
		public string? ExtensionType { get; set; }
		public string? Description { get; set; }
		public string[] Products { get; set; } = Array.Empty<string>();
		public string[] Frameworks { get; set; } = Array.Empty<string>();
	}
#pragma warning restore CS1591
	#endregion
	
	class XcodeRunTarget : XcodeTarget
	{
		private readonly string ProductGuid = XcodeProjectFileGenerator.MakeXcodeGuid();
		private readonly UnrealData UnrealData;
		public readonly UnrealTargetPlatform Platform;
		public readonly TargetType TargetType;

		public XcodeRunTarget(XcodeProject Project, string TargetName, TargetType TargetType, UnrealTargetPlatform Platform, XcodeBuildTarget? BuildTarget, XcodeProjectFile ProjectFile, ILogger Logger)
			: base(Project.UnrealData.IsAppBundle(Platform) ? XcodeTarget.Type.Run_App : XcodeTarget.Type.Run_Tool, Project.UnrealData,
				  TargetName + (XcodeProjectFileGenerator.PerPlatformMode == XcodePerPlatformMode.OneWorkspacePerPlatform ? "" : $"_{Platform}"))
		{
			this.TargetType = TargetType;
			this.Platform = Platform;
			UnrealData = Project.UnrealData;

			BuildConfigList = new XcodeBuildConfigList(Platform, Name, Project.UnrealData, bIncludeAllPlatformsInConfig: false);
			References.Add(BuildConfigList);

			// add the Product item to the project to be visible in left pane
			Project.FileCollection.AddFileReference(ProductGuid, UnrealData.ProductName, "explicitFileType", Project.UnrealData.IsAppBundle(Platform) ? "wrapper.application" : "\"compiled.mach-o.executable\"", "BUILT_PRODUCTS_DIR", "Products");

			if (Project.UnrealData.IsAppBundle(Platform))
			{
				XcodeResourcesBuildPhase ResourcesBuildPhase = new XcodeResourcesBuildPhase(Project.FileCollection);
				BuildPhases.Add(ResourcesBuildPhase);
				References.Add(ResourcesBuildPhase);

				ProcessFrameworks(ResourcesBuildPhase, Project, ProjectFile, Logger);
				ProcessScripts(Project, ProjectFile);
				ProcessAssets(ResourcesBuildPhase);
			}

			if (BuildTarget != null)
			{
				AddDependency(BuildTarget, Project);
			}

			// Check if there's any app extensions for iOS 
			if (Platform == UnrealTargetPlatform.IOS)
			{
				if (Project.UnrealData.UProjectFileLocation != null)
				{
					DirectoryReference ProjectDirectory = Project.UnrealData.ProductDirectory;
					string ProjectExtensionsDir = Path.Combine(ProjectDirectory.FullName, "Build", "IOS", "Extensions");
					if (Directory.Exists(ProjectExtensionsDir))
					{
						foreach (DirectoryInfo DI in new System.IO.DirectoryInfo(ProjectExtensionsDir).EnumerateDirectories())
						{
							// Get the .uappex details file for this extension
							ExtensionConfig? ExtConfig;
							string UExtPath = Path.Combine(DI.FullName, DI.Name) + ".uappex";
							if (File.Exists(UExtPath))
							{
								string JsonText = File.ReadAllText(UExtPath);
								JsonSerializerOptions Options = new() { PropertyNameCaseInsensitive = true };
								ExtConfig = JsonSerializer.Deserialize<ExtensionConfig>(JsonText, Options);
								if (ExtConfig == null)
								{
									Logger.LogWarning("Error loading extension '{Name}.uappex' file. Skipping Extension...", DI.Name);
									continue;
								}
							}
							else
							{
								Logger.LogWarning("There is no extension '{Name}.uappex' file. Skipping Extension...", DI.Name);
								continue;
							}
							
							// Check that this appex should be added to this Produst instance. ie: there maybe be multiple programs/products
							// defined in the Project, but only a one or more might need this extension
							if (!ExtConfig.Products.Contains(Project.UnrealData.ProductName))
							{
								continue;
							}

							Logger.LogInformation("Adding App Extension Target '{Name}' to '{ProductName}'", DI.Name, Project.UnrealData.ProductName);
							
							// assume each Extension in here will create a resulting Extension.appex
							string Extension = DI.Name + ".appex";
							string ExtensionGuid = XcodeProjectFileGenerator.MakeXcodeGuid();
							XcodeCopyAppExtFilesBuildPhase EmbedAppExt = new XcodeCopyAppExtFilesBuildPhase(Project.FileCollection, ExtConfig);
							
							// Get the source files
							foreach (FileSystemInfo FSI in DI.EnumerateFileSystemInfos())
							{
								if (FSI.Name.StartsWith('.') || FSI.Name.EndsWith(".uappex", StringComparison.Ordinal))
								{
									continue;
								}

								FileReference FileRef = new FileReference(FSI.FullName);
								string FileGuid = XcodeProjectFileGenerator.MakeXcodeGuid();
								XcodeSourceFile SourceFile = new XcodeSourceFile(FileRef, FileRef.Directory, FileGuid);
								Project.FileCollection.ProcessFile(SourceFile, true, false, "Source");
								// Logger.LogInformation("  AppEx file:'{FileRef}'  FileRef: '{FileGuid}' GUID:'{ExtensionGuid}'", FileRef, FileGuid, ExtensionGuid);
							}
							
							Project.FileCollection.AddFileReference(ExtensionGuid, Extension, "explicitFileType", "wrapper.app-extension", "BUILT_PRODUCTS_DIR", "Products");

							DirectoryReference FinalAppExtDir = DirectoryReference.Combine(ProjectDirectory, DI.Name);
							EmbedAppExt.AddExtension(FinalAppExtDir, ExtensionGuid);
							
							BuildPhases.Add(EmbedAppExt);
							References.Add(EmbedAppExt);
							
							// Also need a (Native)Run Target for this extension
							XcodeAppExtRunTarget RunTarget = new XcodeAppExtRunTarget(Project, ExtConfig, DI.Name, ExtensionGuid, UnrealTargetPlatform.IOS, BuildTarget, ProjectFile, DI, Logger);
							Project.AppExtRunTargets.Add(RunTarget);
							Project.References.Add(RunTarget);
						}
					}
				}
			}

			CreateXcconfigFile(Project, Platform, $"{Name}");
			// create per-config Xcconfig files
			foreach (XcodeBuildConfig Config in BuildConfigList.BuildConfigs)
			{
				Config.CreateXcconfigFile(Project, Platform, $"{Name}_{Config.Info.DisplayName.Replace(" ", "", StringComparison.Ordinal)}");
			}
		}

		/// <summary>
		/// Write some scripts to do some fixup with how UBT links files. There is currently a difference between Mac and IOS/TVOS:
		/// All:
		///   - The staged files gets pulled into the .app for a self-contained app, created at Xcode build time, unless an envvar is set
		///       that tells this script to skip the copy. This allows UAT to potentially skip copying staged files before we stage
		///       ("BuildCookRun -build -cook -stage -package" can skip the copying until the -package step).
		///       Also we don't copy anything if this is the engine, no-project, build, and an 
		/// Mac:
		///   - UBT will link executable next to .app
		///   - We will copy it into .app here, similar to iOS
		///   - However, during Archiving, the .app is created in a intermediate location, so then here we copy from Binaries/Mac to the intermediate location
		///     - Trying to have UBT link directly to the intermeidate location causes various issues, so we copy it like IOS does
		/// IOS/TVOS:
		///   - IOS will link to Binaries/IOS/Foo
		///   - During normal operation, here we copy from Binaries/IOS/Foo to Binaries/IOS/Foo.app/Foo
		///     - Note that IOS and Mac have different internal directory structures (which EXECUTABLE_PATH expresses)
		///   - When Archiving, we copy from Binaries/IOS/Foo to the intermediate location's .app
		/// All:
		///   - At this point, the executable is in the correct spot, and so CONFIGURATION_BUILD_DIR/EXECUTABLE_PATH points to it
		///   - So here we gneerate a dSYM from the executable, copying it to where Xcode wants it (DWARF_DSYM_FOLDER_PATH/DWARF_DSYM_FILE_NAME), and
		///       then we strip the executable in place
		/// </summary>
		/// <param name="Project"></param>
		/// <param name="ProjectFile"></param>
		protected void ProcessScripts(XcodeProject Project, XcodeProjectFile ProjectFile)
		{
			List<string> CopyScript = new();

			// UBT no longer copies the executable into the .app directory in PostBuild, so we do it here
			// EXECUTABLE_NAME is Foo, EXECUTABLE_PATH is Foo.app/Foo
			// NOTE: We read from hardcoded location where UBT writes to, but we write to CONFIGURATION_BUILD_DIR because
			// when Archiving, the .app is somewhere else
			CopyScript.AddRange(new string[]
			{
				"set -eo pipefail",

				"SRC_EXE=\\\"${UE_BINARIES_DIR}/${UE_UBT_BINARY_SUBPATH}\\\"",
				"DEST_EXE=\\\"${CONFIGURATION_BUILD_DIR}/${EXECUTABLE_PATH}\\\"",
				"DEST_EXE_DIR=`dirname \\\"${DEST_EXE}\\\"`",
				"",
				"echo Copying executable and any standalone dylibs into ${DEST_EXE_DIR} but do not overwrite unless src is newer",
				"mkdir -p \\\"${DEST_EXE_DIR}\\\"",
				"rsync -au \\\"${SRC_EXE}\\\" \\\"${DEST_EXE}\\\"",
			});

			IEnumerable<ModuleRules.RuntimeDependency>? Dylibs;
			Tuple<ProjectFile, UnrealTargetPlatform> DylibKey = Tuple.Create((ProjectFile)ProjectFile, Platform);
			if (XcodeProjectFileGenerator.TargetRawDylibs.TryGetValue(DylibKey, out Dylibs))
			{
				foreach (ModuleRules.RuntimeDependency Dylib in Dylibs)
				{
					// make it absolute if it was relative (to Engine/Source)
					string FixedSource = Dylib.SourcePath!;
					if (!FixedSource.StartsWith('/'))
					{
						FixedSource = $"${{UE_ENGINE_DIR}}/Source/{FixedSource}";
					}

					// make it relative to Binaries dir
					string FixedDest = Dylib.Path.Replace("$(BinaryOutputDir)", "", StringComparison.Ordinal);
					CopyScript.Add($"ditto \\\"{FixedSource}\\\" \\\"${{DEST_EXE_DIR}}{FixedDest}\\\"");
				}
			}

			CopyScript.Add("");

			if (Project.UnrealData.TargetRules.Type == TargetType.Editor && UnrealData.TargetRules.LinkType == TargetLinkType.Modular)
			{
				// Editor just need the above script to copy executable into .app

				XcodeShellScriptBuildPhase EditorCopyScriptPhase = new("Copy Executable into .app", CopyScript, Array.Empty<string>(), new string[] { $"/dev/null" });
				BuildPhases.Add(EditorCopyScriptPhase);
				References.Add(EditorCopyScriptPhase);
				return;
			}
			else
			{
				// rsync the Staged build into the .app, unless the UE_SKIP_STAGEDDATA_SYNC var is set to 1
				// editor builds don't need staged content in them

				bool bIsEngineBuild = Project.UnrealData.UProjectFileLocation == null;
				string DefaultStageDir = bIsEngineBuild ? "" : "${UE_PROJECT_DIR}/Saved/StagedBuilds/${UE_TARGET_PLATFORM_NAME}";
				string SyncSourceSubdir = (Platform == UnrealTargetPlatform.Mac) ? "" : "/cookeddata";
				string SyncDestSubdir = (Platform == UnrealTargetPlatform.Mac) ? "/UE" : "/cookeddata";
				string ExecutableKey = $"UE_{Platform.ToString().ToUpperInvariant()}_EXECUTABLE_NAME";

				CopyScript.AddRange(new string[]
				{
					"# Skip syncing if desired",
					"if [[ ${UE_SKIP_STAGEDDATA_SYNC} -eq 1 ]]; then exit 0; fi",
					"",
					"# When building engine projects, like UnrealGame, we don't have data to stage unless something has specified UE_OVERRIDE_STAGE_DIR",
					"if [[ -z ${UE_OVERRIDE_STAGE_DIR} ]]; then ",
					$"STAGED_DIR=\\\"{DefaultStageDir}\\\"",
					"else",
					"  STAGED_DIR=\\\"${UE_OVERRIDE_STAGE_DIR}\\\"",
					"fi",
					"if [[ -z ${STAGED_DIR} ]]; then exit 0; fi",
				});

				// Programs have an optional Staged dir - usually they aren't staged, but allow it to be staged if it was
				if (TargetType != TargetType.Program)
				{
					CopyScript.AddRange(new string[]
					{
						"# Make sure the staged directory exists and has files in it",
						"if [[ ! -e \\\"${STAGED_DIR}\\\" || ! $(ls -A \\\"${STAGED_DIR}\\\") ]]; then ",
						"  echo =========================================================================================",
						"  echo \\\"WARNING: To run, you must have a valid staged build directory. The Staged location is:\\\"",
						"  echo \\\"  ${STAGED_DIR}\\\"",
						"  echo \\\"Use the editor's Platforms menu, or run a command like::\\\"",
						$"  echo \\\"./RunUAT.sh BuildCookRun -platform={Platform} -project=<project> -build -cook -stage -pak\\\"",
						"  echo =========================================================================================",
						"  exit -0 ",
						"fi",
					});
				}
				else
				{
					CopyScript.AddRange(new string[]
					{
						"# Make sure the staged directory exists and has files in it",
						"if [[ ! -e ${STAGED_DIR} ]]; then ",
						"  # Make sure the target doesn't exist (so if we delete the Staged dir, it goes back to unstaged",
						$"  rm -rf \\\"${{CONFIGURATION_BUILD_DIR}}/${{CONTENTS_FOLDER_PATH}}{SyncDestSubdir}\\\"",
						"  exit -0",
						"fi",
					});
					;
				}

				// when we bring stated data into the .app, we have to skip some temp stuff that went into it
				string[] Exclusions =
				{
					"-/Info.plist",
					"-/Manifest_*",
					$"-/*.app", // remove the staged .app from the root dir, it's hard to do by name due to ProjectName in staging, and TargetName, etc here
				};

				// make a string like --exclude=/Info.plist --exclude=/Manifest_* ...
				string ExcludeString = String.Join(" ", Exclusions.Select(x => (x[0] == '+' ? "--include" : "--exclude") + $"=\\\"{x.Substring(1)}\\\""));

				// copy uecommandline.txt and UPL build output for IOS type platforms
				if (Platform != UnrealTargetPlatform.Mac)
				{
					CopyScript.AddRange(new string[]
					{
						"if [[ -e \\\"${STAGED_DIR}/uecommandline.txt\\\" ]]; then",
						"  echo \\\"Syncing ${STAGED_DIR}/uecommandline.txt to ${CONFIGURATION_BUILD_DIR}/${CONTENTS_FOLDER_PATH}\\\"",
						"  cp \\\"${STAGED_DIR}/uecommandline.txt\\\" \\\"${CONFIGURATION_BUILD_DIR}/${CONTENTS_FOLDER_PATH}\\\"",
						"fi",
						"UPL_BUILD_DIR=${UE_PROJECT_DIR}/Intermediate/IOS/UPLBuild/",
						"if [[ -e \\\"${UPL_BUILD_DIR}\\\" ]]; then",
						"cp -r \\\"${UPL_BUILD_DIR}\\\" \\\"${CONFIGURATION_BUILD_DIR}/${CONTENTS_FOLDER_PATH}\\\"",
						"fi"
					});
				}

				CopyScript.AddRange(new string[]
				{
					"",
					$"echo \\\"Syncing ${{STAGED_DIR}}{SyncSourceSubdir} to ${{CONFIGURATION_BUILD_DIR}}/${{CONTENTS_FOLDER_PATH}}{SyncDestSubdir}\\\"",
					$"if [[ -e  \\\"${{STAGED_DIR}}{SyncSourceSubdir}\\\" ]]; then",
					$"rsync -a --delete {ExcludeString} \\\"${{STAGED_DIR}}{SyncSourceSubdir}/\\\" \\\"${{CONFIGURATION_BUILD_DIR}}/${{CONTENTS_FOLDER_PATH}}{SyncDestSubdir}\\\"",
					"else",
					"  echo =========================================================================================",
					"  echo \\\"WARNING: To run, you must have a valid staged sync source directory. The Staged SyncSource location is:\\\"",
					$"  echo \\\"${{STAGED_DIR}}{SyncSourceSubdir}\\\"",
					"  echo \\\"Use the editor's Platforms menu, or run a command like::\\\"",
					$"  echo \\\"./RunUAT.sh BuildCookRun -platform={Platform} -project=<project> -build -cook -stage -pak\\\"",
					"  echo =========================================================================================",
					"  exit -0 ",
					"fi",
				});
			}

			// run this script every time, but xcode will show a warning if there isn't _some_ output
			string ScriptOutput = $"/dev/null";
			XcodeShellScriptBuildPhase CopyScriptPhase = new("Copy Executable and Staged Data into .app", CopyScript, Array.Empty<string>(), new string[] { ScriptOutput });
			BuildPhases.Add(CopyScriptPhase);
			References.Add(CopyScriptPhase);

			// always generate a dsym file when we archive, and by having Xcode do it, it will be put into the archive properly
			// (note bInstallOnly which will make this onle run when archiving)
			List<string> DsymScript = new();

			bool bUsePremadeDSYMInXcArchive;
			// get ini file for the platform
			ConfigHierarchy PlatformIni = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, UnrealData.ConfigDirectory, Platform);
			PlatformIni.TryGetValue("/Script/MacTargetPlatform.XcodeProjectSettings", "bUsePremadeDSYMInXcArchive", out bUsePremadeDSYMInXcArchive);

			if (bUsePremadeDSYMInXcArchive)
			{
				DsymScript.AddRange(new string[]
				{
					"set -e",
					"",
					"# Copy a dsym from next to the executable",
					"SRC_DSYM=\\\"${UE_BINARIES_DIR}/${UE_UBT_BINARY_SUBPATH}.dSYM\\\"",
					"if [[ -e \\\"${SRC_DSYM}\\\" ]]; then",
					"  echo Using pre-existing dSYM at \\\"${SRC_DSYM}\\\"",
					"  ditto \\\"${SRC_DSYM}\\\" \\\"${DWARF_DSYM_FOLDER_PATH}/${DWARF_DSYM_FILE_NAME}\\\"",
					"fi"
				});
			}
			else
			{
				DsymScript.AddRange(new string[]
				{
					"set -e",
					"",
					"# Run the wrapper dsym generator",
					"\\\"${UE_ENGINE_DIR}/Build/BatchFiles/Mac/GenerateUniversalDSYM.sh\\\" \\\"${CONFIGURATION_BUILD_DIR}/${EXECUTABLE_PATH}\\\" \\\"${DWARF_DSYM_FOLDER_PATH}/${DWARF_DSYM_FILE_NAME}\\\"",
				});
			}
			string StripCommand = "strip -S -no_code_signature_warning -D \\\"${CONFIGURATION_BUILD_DIR}/${EXECUTABLE_PATH}\\\"";
			if (Platform == UnrealTargetPlatform.IOS || Platform == UnrealTargetPlatform.TVOS)
			{
				// We only strip local/debug symbols. Globals should be preserved even in binaries,
				// as we expose operators new/delete.
				StripCommand = "strip -x -S -no_code_signature_warning -D \\\"${CONFIGURATION_BUILD_DIR}/${EXECUTABLE_PATH}\\\"";
			}
			DsymScript.AddRange(new string[]
			{
				StripCommand,
				"",
				"# Remove any unused architectures from dylibs in the .app (param1) that don't match the executable (param2). Also error if a dylib is missing arches",
				"\\\"${UE_ENGINE_DIR}/Build/BatchFiles/Mac/ThinApp.sh\\\" \\\"${CONFIGURATION_BUILD_DIR}/${CONTENTS_FOLDER_PATH}\\\" \\\"${CONFIGURATION_BUILD_DIR}/${EXECUTABLE_PATH}\\\"",
			});

			string DsymScriptInput = $"\\\"$(CONFIGURATION_BUILD_DIR)/$(EXECUTABLE_PATH)\\\"";
			string DsymScriptOutput = $"\\\"$(DWARF_DSYM_FOLDER_PATH)/$(DWARF_DSYM_FILE_NAME)\\\"";
			XcodeShellScriptBuildPhase DsymScriptPhase = new("Generate dsym for archive, and strip", DsymScript, new string[] { DsymScriptInput }, new string[] { DsymScriptOutput }, bInstallOnly: true);
			BuildPhases.Add(DsymScriptPhase);
			References.Add(DsymScriptPhase);
		}
		private void ProcessAssets(XcodeResourcesBuildPhase ResourcesBuildPhase)
		{
			List<string> StoryboardPaths = new List<string>()
			{
				"$(Project)/Build/$(Platform)/Resources/Interface/LaunchScreen.storyboardc",
				"$(Project)/Build/$(Platform)/Resources/Interface/LaunchScreen.storyboard",
				"$(Project)/Build/Apple/Resources/Interface/LaunchScreen.storyboardc",
				"$(Project)/Build/Apple/Resources/Interface/LaunchScreen.storyboard",
				"$(Engine)/Build/$(Platform)/Resources/Interface/LaunchScreen.storyboardc",
				"$(Engine)/Build/$(Platform)/Resources/Interface/LaunchScreen.storyboard",
				"$(Engine)/Build/Apple/Resources/Interface/LaunchScreen.storyboardc",
				"$(Engine)/Build/Apple/Resources/Interface/LaunchScreen.storyboard",
				"$(Project)/Build/IOS/Resources/Interface/LaunchScreen.storyboard",
				"$(Engine)/Build/IOS/Resources/Interface/LaunchScreen.storyboardc",
			};

			// look for Assets (in normal place, or an alternate for Programs)
			string? StoryboardPath;
			string AssetsSubPath = $"Build/{Platform}/Resources/Assets.xcassets";
			string AssetsAltSubPath = $"Resources/{Platform}/Assets.xcassets";

			//default to IOS path for other platforms (eg VisionOS)
			if (Platform != UnrealTargetPlatform.TVOS &&
			    Platform != UnrealTargetPlatform.IOS &&
			    Platform != UnrealTargetPlatform.Mac)
			{
				AssetsSubPath = $"Build/IOS/Resources/Assets.xcassets";
			}
			string AssetsPath = UnrealData.ProjectOrEnginePath(AssetsSubPath, false, AssetsAltSubPath);
			ResourcesBuildPhase.AddResource(new FileReference(AssetsPath));
			StoryboardPath = UnrealData.FindFile(StoryboardPaths, Platform, false);

			if (StoryboardPath != null)
			{
				ResourcesBuildPhase.AddResource(new FileReference(StoryboardPath));
			}

			if (Platform == UnrealTargetPlatform.IOS || Platform == UnrealTargetPlatform.TVOS)
			{
				List<string> LaunchImagePaths = new List<string>()
				{
					"$(Project)/Build/$(Platform)/Resources/Graphics/LaunchScreenIOS.png",
					"$(Engine)/Build/$(Platform)/Resources/Graphics/LaunchScreenIOS.png",
				};

				string? LaunchImagePath = UnrealData.FindFile(LaunchImagePaths, UnrealTargetPlatform.IOS, false);
				if (LaunchImagePath != null)
				{
					ResourcesBuildPhase.AddResource(new FileReference(LaunchImagePath));
				}
			}

			string? LocalizationsPlatform = null;
			
			if (Platform == UnrealTargetPlatform.Mac)
			{
				LocalizationsPlatform = "Mac";
				ResourcesBuildPhase.AddFolderResource(DirectoryReference.Combine(Unreal.EngineDirectory, "Build/Mac/Resources/UEMetadata"), "Resources");
				if (UnrealData.Metadata?.ProjectPrivacyInfoFiles.ContainsKey(MetadataPlatform.Mac) == true)
				{
					MetadataItem PrivacyInfo = UnrealData.Metadata.ProjectPrivacyInfoFiles[MetadataPlatform.Mac];
					if (PrivacyInfo.XcodeProjectRelative != null)
					{
						ResourcesBuildPhase.AddResource(FileReference.Combine(UnrealData.XcodeProjectFileLocation.ParentDirectory!, PrivacyInfo.XcodeProjectRelative));
					}
				}
			}
			else if (Platform == UnrealTargetPlatform.IOS || Platform == UnrealTargetPlatform.TVOS || Platform == UnrealTargetPlatform.VisionOS)
			{
				LocalizationsPlatform = "IOS";
				ResourcesBuildPhase.AddFolderResource(DirectoryReference.Combine(Unreal.EngineDirectory, "Build/IOS/Resources/UEMetadata"), "Resources");
				if (UnrealData.Metadata?.ProjectPrivacyInfoFiles.ContainsKey(MetadataPlatform.IOS) == true)
				{
					MetadataItem PrivacyInfo = UnrealData.Metadata.ProjectPrivacyInfoFiles[MetadataPlatform.IOS];
					if (PrivacyInfo.XcodeProjectRelative != null)
					{
						ResourcesBuildPhase.AddResource(FileReference.Combine(UnrealData.XcodeProjectFileLocation.ParentDirectory!, PrivacyInfo.XcodeProjectRelative));
					}
				}
			}

			if (LocalizationsPlatform != null)
			{
				DirectoryReference LocalizationsDir = DirectoryReference.Combine(UnrealData.ProductDirectory, "Build", LocalizationsPlatform, "Resources", "Localizations");
				if (DirectoryReference.Exists(LocalizationsDir))
				{
					IEnumerable<DirectoryReference> Localizations = DirectoryReference.EnumerateDirectories(LocalizationsDir, "*.lproj");
					foreach (DirectoryReference Localization in Localizations)
					{
						ResourcesBuildPhase.AddFolderResource(Localization, "Resources");
					}					
				}
			}

			string? ExtraFolderToCopyToApp;
			ConfigHierarchy PlatformIni = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, UnrealData.ConfigDirectory, Platform);
			PlatformIni.TryGetValue("/Script/MacTargetPlatform.XcodeProjectSettings", "ExtraFolderToCopyToApp", out ExtraFolderToCopyToApp);

			if (ExtraFolderToCopyToApp != null)
			{
				List<string> ResourcesToBundlePaths = new List<string>()
				{
					ExtraFolderToCopyToApp,
				};

				string? ResourcesToBundlePath = UnrealData.FindFile(ResourcesToBundlePaths, Platform, false);
				if (ResourcesToBundlePath != null)
				{
					ResourcesBuildPhase.AddFolderResource(new DirectoryReference(ResourcesToBundlePath), "Resources");
				}
			}
		}
		protected void ProcessFrameworks(XcodeResourcesBuildPhase ResourcesBuildPhase, XcodeProject Project, XcodeProjectFile ProjectFile, ILogger Logger)
		{
			// look up to see if we had cached any Frameworks
			Tuple<ProjectFile, UnrealTargetPlatform> FrameworkKey = Tuple.Create((ProjectFile)ProjectFile, Platform);
			IEnumerable<UEBuildBundleResource>? Bundles;
			if (XcodeProjectFileGenerator.TargetBundles.TryGetValue(FrameworkKey, out Bundles))
			{
				foreach (UEBuildBundleResource Bundle in Bundles)
				{
					ResourcesBuildPhase.AddFolderResource(new DirectoryReference(Bundle.ResourcePath!), "Resources");
				}
			}

			IEnumerable<UEBuildFramework>? Frameworks;
			if (XcodeProjectFileGenerator.TargetFrameworks.TryGetValue(FrameworkKey, out Frameworks))
			{
				XcodeCopyFilesBuildPhase EmbedFrameworks = new XcodeCopyFilesBuildPhase(Project.FileCollection);
				XcodeFrameworkBuildPhase FrameworkPhase = new XcodeFrameworkBuildPhase(Project.FileCollection);

				// filter frameworks that need to installed into the .app (either the framework or a bundle inside a .zip)
				IEnumerable<UEBuildFramework> InstalledFrameworks = Frameworks.Where(x => x.bCopyFramework || !String.IsNullOrEmpty(x.CopyBundledAssets));
				// filter frameworks that need to be unzipped before we compile
				IEnumerable<UEBuildFramework> ZippedFrameworks = Frameworks.Where(x => x.ZipFile != null);

				bool bHasEmbeddedFrameworks = false;
				// only look at frameworks that need anything copied into 
				foreach (UEBuildFramework Framework in InstalledFrameworks)
				{
					DirectoryReference? FinalFrameworkDir = Framework.GetFrameworkDirectory(null, null, Logger);
					if (FinalFrameworkDir == null)
					{
						continue;
					}
					// the framework may come with FrameworkDir being parent of a .framework with name of Framework.Name
					if (!FinalFrameworkDir.HasExtension(".framework") && !FinalFrameworkDir.HasExtension(".xcframework"))
					{
						FinalFrameworkDir = DirectoryReference.Combine(FinalFrameworkDir, Framework.Name + ".framework");
					}

					DirectoryReference BundleRootDir = Framework.GetFrameworkDirectory(null, null, Logger)!;

					if (Framework.ZipFile != null)
					{
						if (Framework.ZipFile.FullName.EndsWith(".embeddedframework.zip", StringComparison.Ordinal))
						{
							// foo.embeddedframework.zip would have foo.framework inside it, which is what we want to install
							FinalFrameworkDir = DirectoryReference.Combine(Framework.ZipOutputDirectory!, Framework.ZipFile.GetFileNameWithoutAnyExtensions() + ".framework");
							BundleRootDir = Framework.ZipOutputDirectory!;
						}
						else
						{
							FinalFrameworkDir = Framework.ZipOutputDirectory!;
						}
					}

					// set up the CopyBundle to be copied
					if (!String.IsNullOrEmpty(Framework.CopyBundledAssets))
					{
						ResourcesBuildPhase.AddFolderResource(DirectoryReference.Combine(BundleRootDir, Framework.CopyBundledAssets), "Resources");
					}

					if (Framework.bCopyFramework)
					{
						string FileRefGuid = XcodeProjectFileGenerator.MakeXcodeGuid();
						EmbedFrameworks.AddFramework(FinalFrameworkDir, FileRefGuid);
						FrameworkPhase.AddFramework(FinalFrameworkDir, FileRefGuid);

						bHasEmbeddedFrameworks = true;
					}
				}

				if (bHasEmbeddedFrameworks)
				{
					BuildPhases.Add(EmbedFrameworks);
					References.Add(EmbedFrameworks);
					BuildPhases.Add(FrameworkPhase);
					References.Add(FrameworkPhase);
				}

				// each zipped framework needs to be unzipped in case C++ code needs to compile/link against it - this will add unzip commands
				// to the PreBuild script that is run before anything else happens - note that the ZipDependToken is shared with UBT, so that
				// if a new framework is unzipped, it will dirty any source files in modules that use this framework
				foreach (UEBuildFramework Framework in ZippedFrameworks)
				{
					string ZipIn = Utils.MakePathSafeToUseWithCommandLine(Framework.ZipFile!.FullName);
					string ZipOut = Utils.MakePathSafeToUseWithCommandLine(Framework.ZipOutputDirectory!.FullName);
					// Zip contains folder with the same name, hence ParentDirectory
					string ZipOutParent = Utils.MakePathSafeToUseWithCommandLine(Framework.ZipOutputDirectory.ParentDirectory!.FullName);
					string ZipDependToken = Utils.MakePathSafeToUseWithCommandLine(Framework.ExtractedTokenFile!.FullName);

					UnrealData.ExtraPreBuildScriptLines.AddRange(new[]
					{
						// delete any output and make sure parent dir exists
						$"if [ {ZipIn} -nt {ZipDependToken} ] ",
						$"then",
						$"  [ -d {ZipOut} ] &amp;&amp; rm -rf {ZipOut}",
						$"  mkdir -p {ZipOutParent}",
						// unzip the framework and maybe extra data
						$"  unzip -q -o {ZipIn} -d {ZipOutParent}",
						$"  touch {ZipDependToken}",
						$"fi",
						$"",
					});
				}
			}
		}

		protected override void WriteExtraTargetProperties(StringBuilder Content)
		{
			Content.WriteLine($"\t\t\tproductReference = {ProductGuid};");
			Content.WriteLine($"\t\t\tproductName = \"{UnrealData.ProductName}\";");
		}

		private string? CopyAndWriteEntitlements(ref MetadataItem? EntitlementsMetadata, string ConfigDisplayName, bool bUseEntitlementsForPrograms, DirectoryReference ProjectOrEngineDir)
		{
			ConfigHierarchy PlatformGameConfig = ConfigCache.ReadHierarchy(ConfigHierarchyType.Game, UnrealData.ConfigDirectory, Platform);
			PlatformGameConfig.GetBool("/Script/AppleTargetPlatform.AppleTargetSettings", "bEnableContactsManager", out bool bEnableContactsManager);

			// programs almost never want entitlements, so handle them specially
			if ((UnrealData.bNoEntitlements) || (UnrealData.TargetRules.Type == TargetType.Program && !bUseEntitlementsForPrograms))
			{
				EntitlementsMetadata = null;
			}
			else if(EntitlementsMetadata != null && EntitlementsMetadata.File != null && bEnableContactsManager && Platform == UnrealTargetPlatform.Mac)
			{
				// if we are on iOS, IOSExports.cs already handles adding the addressbook entitlement if needed for us
				string BuildIntermediateDir = $"{DirectoryReference.Combine(ProjectOrEngineDir, "Intermediate/Mac")}";
				string TargetEntitlements = $"{BuildIntermediateDir}/{ConfigDisplayName}-{Path.GetFileName(EntitlementsMetadata.XcodeProjectRelative)}";

				Directory.CreateDirectory(BuildIntermediateDir);

				File.Copy(EntitlementsMetadata.File.FullName, TargetEntitlements, true);

				FileInfo fileInfo = new FileInfo(TargetEntitlements);
				fileInfo.IsReadOnly = false;

				Process PLUtilProcess = new Process();
				PLUtilProcess.StartInfo.FileName = "plutil";
				PLUtilProcess.StartInfo.Arguments = $"-insert \"com\\.apple\\.security\\.personal-information\\.addressbook\" -bool true \"{TargetEntitlements}\"";

				Utils.RunLocalProcess(PLUtilProcess);

				return TargetEntitlements;
			}

			return EntitlementsMetadata == null ? null : EntitlementsMetadata.XcodeProjectRelative;
		}

		public override void WriteXcconfigFile(ILogger Logger)
		{
			// gather general, all-platform, data we are doing to put into the configs
			UnrealBuildConfig BuildConfig = BuildConfigList!.BuildConfigs[0].Info;
			TargetRules TargetRules = BuildConfig.ProjectTarget!.TargetRules!;
			DirectoryReference ProjectOrEngineDir = UnrealData.UProjectFileLocation?.Directory ?? Unreal.EngineDirectory;

			// point to the shader Engine/Binaries for content only project (sadly, the TargetRules.OutputFile is not filled out)
			bool bUseEngineDirectory = TargetRules.Type == TargetType.Editor || UnrealData.bIsContentOnlyProject;
			if (TargetRules.BuildEnvironment == TargetBuildEnvironment.Unique)
			{
				// unique build environment editors will put their binaries in the project
				bUseEngineDirectory = false;
			}

			DirectoryReference ConfigBuildDir = bUseEngineDirectory ? Unreal.EngineDirectory : ProjectOrEngineDir;
			if (TargetRules.Type == TargetType.Program && TargetRules.File!.IsUnderDirectory(Unreal.EngineDirectory))
			{
				ConfigBuildDir = BuildConfig.RootDirectory;
			}
			string BinariesBaseDir = ConfigBuildDir.GetDirectoryName();

			ConfigBuildDir = DirectoryReference.Combine(ConfigBuildDir, "Binaries", Platform.ToString(), TargetRules.ExeBinariesSubFolder);

			MetadataPlatform MetadataPlatform;
			string TargetPlatformName = Platform.ToString();
			if (TargetRules.Type != TargetType.Game && TargetRules.Type != TargetType.Program)
			{
				TargetPlatformName += TargetRules.Type.ToString();
			}

			// get ini file for the platform
			ConfigHierarchy PlatformIni = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, UnrealData.ConfigDirectory, Platform);

			// settings for all platforms
			bool bAutomaticSigning;
			bool bMacSignToRunLocally;
			bool bUseEntitlementsForPrograms;
			string? SigningTeam;
			string? SigningPrefix;
			string? AppCategory;
			string SupportedPlatforms;
			string SDKRoot;
			string DeploymentTarget;
			string DeploymentTargetKey;
			string? SigningIdentity = null;
			string? ProvisioningProfile = null;
			string? SupportedDevices = null;
			string? MarketingVersion = null;
			string? BundleIdentifier;
			string? ApplicationDisplayName = null;
			List<string> ExtraConfigLines = new();

			// get signing settings
			PlatformIni.TryGetValue("/Script/MacTargetPlatform.XcodeProjectSettings", "bUseAutomaticCodeSigning", out bAutomaticSigning);
			PlatformIni.TryGetValue("/Script/MacTargetPlatform.XcodeProjectSettings", "bUseEntitlementsForPrograms", out bUseEntitlementsForPrograms);
			PlatformIni.TryGetValue("/Script/MacTargetPlatform.XcodeProjectSettings", "bMacSignToRunLocally", out bMacSignToRunLocally);
			PlatformIni.TryGetValue("/Script/MacTargetPlatform.XcodeProjectSettings", "CodeSigningTeam", out SigningTeam);
			PlatformIni.TryGetValue("/Script/MacTargetPlatform.XcodeProjectSettings", "CodeSigningPrefix", out SigningPrefix);
			PlatformIni.TryGetValue("/Script/MacTargetPlatform.XcodeProjectSettings", $"{Platform}ProvisioningProfile", out ProvisioningProfile);
			PlatformIni.TryGetValue("/Script/MacTargetPlatform.XcodeProjectSettings", $"{Platform}SigningIdentity", out SigningIdentity);
			PlatformIni.TryGetValue("/Script/MacTargetPlatform.XcodeProjectSettings", "AppCategory", out AppCategory);
			PlatformIni.TryGetValue("/Script/MacTargetPlatform.XcodeProjectSettings", "ApplicationDisplayName", out ApplicationDisplayName);

			if (Platform == UnrealTargetPlatform.Mac)
			{
				// editor vs game metadata
				bool bIsEditor = UnrealData.TargetRules.Type == TargetType.Editor;
				MetadataPlatform = bIsEditor ? MetadataPlatform.MacEditor : MetadataPlatform.Mac;

				SDKRoot = "macosx";
				SupportedPlatforms = "macosx";
				DeploymentTargetKey = "MACOSX_DEPLOYMENT_TARGET";
				DeploymentTarget = MacToolChainSettings.MinMacDeploymentVersion(UnrealData.TargetRules.Type);
				BundleIdentifier = bIsEditor ? "com.epicgames.UnrealEditor" : UnrealData.BundleIdentifier;

				PlatformIni.TryGetValue($"/Script/MacRuntimeSettings.MacRuntimeSettings", "VersionInfo", out MarketingVersion);
				if (String.IsNullOrEmpty(MarketingVersion))
				{
					MarketingVersion = MacToolChain.LoadEngineDisplayVersion();
				}
			}
			else
			{
				MetadataPlatform = MetadataPlatform.IOS;

				// get IOS (same as TVOS) BundleID, and if there's a specified plist with a bundleID, use it, as Xcode would warn if they don't match
				BundleIdentifier = UnrealData.BundleIdentifier;

				// short version string
				PlatformIni.GetString($"/Script/IOSRuntimeSettings.IOSRuntimeSettings", "VersionInfo", out MarketingVersion);

				if (Platform == UnrealTargetPlatform.IOS)
				{
					SDKRoot = "iphoneos";
					SupportedPlatforms = "iphoneos";

					DeploymentTargetKey = "IPHONEOS_DEPLOYMENT_TARGET";
					SupportedDevices = UnrealData.IOSProjectSettings!.RuntimeDevices;
					DeploymentTarget = UnrealData.IOSProjectSettings.RuntimeVersion;

					// only iphone deals with orientation
					List<string> SupportedOrientations = XcodeUtils.GetSupportedOrientations(PlatformIni);
					ExtraConfigLines.Add($"INFOPLIST_KEY_UISupportedInterfaceOrientations = " + String.Join(" ", SupportedOrientations));

					// iPhone is always Fullscreen, however, an iPad can support SplitView mode (dynamic view resizeing), so check if that's enabled and then add iPad UISupportedInterfaceOrientations
					bool bEnableSplitView = false;
					PlatformIni.GetBool("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bEnableSplitView", out bEnableSplitView);
					if (bEnableSplitView)
					{
						ExtraConfigLines.Add($"INFOPLIST_KEY_UISupportedInterfaceOrientations_iPad = UIInterfaceOrientationLandscapeLeft UIInterfaceOrientationLandscapeRight UIInterfaceOrientationPortrait UIInterfaceOrientationPortraitUpsideDown");
						ExtraConfigLines.Add($"INFOPLIST_KEY_UIRequiresFullScreen_iPad = false");
					}
				}
				else if (Platform == UnrealTargetPlatform.TVOS) // tvos
				{
					SDKRoot = "appletvos";
					SupportedPlatforms = "appletvos"; // appletvsimulator
					DeploymentTargetKey = "TVOS_DEPLOYMENT_TARGET";
					SupportedDevices = UnrealData.TVOSProjectSettings!.RuntimeDevices;
					DeploymentTarget = UnrealData.TVOSProjectSettings.RuntimeVersion;
				}
				else if (Platform == UnrealTargetPlatform.VisionOS)
				{
					SDKRoot = "xros";
					SupportedPlatforms = "xrsimulator xros";
					DeploymentTargetKey = "XROS_DEPLOYMENT_TARGET";
					SupportedDevices = UnrealData.VisionOSProjectSettings!.RuntimeDevices;
					DeploymentTarget = UnrealData.VisionOSProjectSettings.RuntimeVersion;

					ExtraConfigLines.Add($"VALID_ARCHS = arm64");
					ExtraConfigLines.Add($"ARCHS = arm64");

					// if we are doing immersive with SwiftUI we need a plist key
					bool bUseSwiftUIMain;
					AppleExports.GetSwiftIntegrationSettings(UnrealData.UProjectFileLocation, UnrealData.TargetRules.Type, Platform, out bUseSwiftUIMain, out _);
					if (bUseSwiftUIMain)
					{
						ExtraConfigLines.Add($"INFOPLIST_KEY_UIApplicationSceneManifest_Generation = YES");
					}
				}
				else
				{
					throw new BuildException($"Unsupported platform {Platform}");
				}
			}

			// get metadata for the platform set above
			MetadataItem PlistMetadata = UnrealData.Metadata!.PlistFiles[MetadataPlatform];
			// now pull the bundle id's out, as xcode will warn if they don't match (this has to happen after each platform set bundle id above)
			XcodeUtils.FindPlistId(PlistMetadata, "CFBundleIdentifier", ref BundleIdentifier);
			// if the user had a ini setting, but also set it in the template plist, the plist one should win (this is only used if we are updating a template plist)
			XcodeUtils.FindPlistId(PlistMetadata, "LSApplicationCategoryType", ref AppCategory);

			// Seed a default for UE_<PLATFORM>_BUILD_VERSION so CURRENT_PROJECT_VERSION and CFBundleVersion are never empty.
			// xcconfig is last-wins, so when UBT-link has run and produced a Versions.xcconfig the include below overrides this default. 
			Xcconfig!.AppendLine($"UE_{Platform.ToString().ToUpperInvariant()}_BUILD_VERSION = {ReadOnlyBuildVersion.Current.Changelist}.0");

			// include another xcconfig for versions that UBT writes out

			// Blueprint-only projects have Versions.xcconfig in the engine dir, not the project dir because it technically builds UnrealGame.
			// If the project directory has a valid version file (like in code projects), that will be used as the last declared one overrides everything else. 
			// If any of them do not exist, they are skipped. This is why we add both here with "?"
			
			Xcconfig!.AppendLine($"#include? \"{Unreal.EngineDirectory}/Intermediate/Build/Versions.xcconfig\"");

			if (UnrealData.UProjectFileLocation != null)
			{
				Xcconfig!.AppendLine($"#include? \"{UnrealData.UProjectFileLocation.Directory}/Intermediate/Build/Versions.xcconfig\"");
			}
			else
			{
				Xcconfig!.AppendLine($"#include? \"{XcodeProjectFileGenerator.ContentOnlySettingsFile}\"");
			}

			// write out some UE variables that can be used in premade .plist files, etc
			Xcconfig.AppendLine("");
			Xcconfig.AppendLine("// Unreal project-wide variables");
			Xcconfig.AppendLine($"UE_XCODE_BUILD_MODE = Default");
			Xcconfig.AppendLine($"UE_PRODUCT_NAME = {UnrealData.ProductName}");
			Xcconfig.AppendLine($"UE_PRODUCT_NAME_STRIPPED = {UnrealData.ProductName.Replace("_", "", StringComparison.Ordinal).Replace(" ", "", StringComparison.Ordinal)}");
			Xcconfig.AppendLine($"UE_DISPLAY_NAME = {UnrealData.DisplayName}");
			Xcconfig.AppendLine($"UE_SIGNING_PREFIX = {SigningPrefix}");
			Xcconfig.AppendLine($"UE_PLATFORM_NAME = {Platform}");
			Xcconfig.AppendLine($"UE_TARGET_NAME = {UnrealData.TargetRules.OriginalName}");
			Xcconfig.AppendLine($"UE_TARGET_PLATFORM_NAME = {TargetPlatformName}");
			Xcconfig.AppendLine($"UE_ENGINE_DIR = {Unreal.EngineDirectory}");
			Xcconfig.AppendLine($"UE_BINARIES_DIR = {ConfigBuildDir}");
			Xcconfig.AppendLine($"UE_STAGED_BINARIES_DIR_BASE = {BinariesBaseDir}");
			if (UnrealData.UProjectFileLocation != null)
			{
				Xcconfig.AppendLine($"UE_PROJECT_DIR = {UnrealData.UProjectFileLocation.Directory}");
			}

			Xcconfig.AppendLine("");
			Xcconfig.AppendLine("// Constant settings (same for all platforms and targets)");
			Xcconfig.AppendLine("INFOPLIST_OUTPUT_FORMAT = xml");
			Xcconfig.AppendLine("COMBINE_HIDPI_IMAGES = YES");
			Xcconfig.AppendLine("USE_HEADERMAP = NO");
			Xcconfig.AppendLine("ONLY_ACTIVE_ARCH = YES");

			Xcconfig.AppendLine("");
			Xcconfig.AppendLine("// Platform settings");
			Xcconfig.AppendLine($"SUPPORTED_PLATFORMS = {SupportedPlatforms}");
			Xcconfig.AppendLine($"SDKROOT = {SDKRoot}");

			if (UnrealData.TargetRules.IOSPlatform.bEnableAddressSanitizer)
			{
				Xcconfig.AppendLine("ENABLE_ADDRESS_SANITIZER = YES");
			}

			// Xcode creates the Build Dir (where the .app is) by combining {SYMROOT}/{CONFIGURATION}{EFFECTIVE_PLATFORM_NAME}, so we set SYMROOT
			// to the Parent directory of the diectory the binary is in, CONFIGURATION to nothing, and EFFECTIVE_PLATFORM_NAME to the directory
			// the binary is in
			Xcconfig.AppendLine("");
			Xcconfig.AppendLine($"// These settings combined will tell Xcode to write to Binaries/{Platform} (instead of something like Binaries/Development-iphoneos)");
			Xcconfig.AppendLine($"SYMROOT = {ConfigBuildDir.ParentDirectory}");
			Xcconfig.AppendLine($"CONFIGURATION = ");
			Xcconfig.AppendLine($"EFFECTIVE_PLATFORM_NAME = {ConfigBuildDir.GetDirectoryName()}");

			if (ExtraConfigLines.Count > 0)
			{
				Xcconfig.AppendLine("");
				Xcconfig.AppendLine("// Misc settings");
				foreach (string Line in ExtraConfigLines)
				{
					Xcconfig.AppendLine(Line);
				}
			}

			Xcconfig.AppendLine("");
			Xcconfig.AppendLine("// Project settings");
			Xcconfig.AppendLine($"TARGETED_DEVICE_FAMILY = {SupportedDevices}");
			Xcconfig.AppendLine($"PRODUCT_BUNDLE_IDENTIFIER = {BundleIdentifier}");
			Xcconfig.AppendLine($"{DeploymentTargetKey} = {DeploymentTarget}");

			Xcconfig.AppendLine("");
			Xcconfig.AppendLine("// Plist settings");
			Xcconfig.AppendLine($"INFOPLIST_FILE = {PlistMetadata.XcodeProjectRelative}");
			if (PlistMetadata.Mode == MetadataMode.UpdateTemplate)
			{
				bool bAssetCatalogAllIcons = false;
				string? AssetCatalogAlternates = null;
				PlatformIni.TryGetValue("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bAssetCatalogAllIcons", out bAssetCatalogAllIcons);
				PlatformIni.TryGetValue("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "AssetCatalogAlternates", out AssetCatalogAlternates);

				// allow Xcode to generate the final plist file from our input, some INFOPLIST settings and other settings 
				Xcconfig.AppendLine("GENERATE_INFOPLIST_FILE = YES");
				Xcconfig.AppendLine($"ASSETCATALOG_COMPILER_APPICON_NAME = AppIcon");
				string AllIcons = bAssetCatalogAllIcons ? "YES" : "NO";
				Xcconfig.AppendLine($"ASSETCATALOG_COMPILER_INCLUDE_ALL_APPICON_ASSETS = {AllIcons}");
				if (!String.IsNullOrEmpty(AssetCatalogAlternates))
				{
					Xcconfig.AppendLine($"ASSETCATALOG_COMPILER_ALTERNATE_APPICON_NAMES = {AssetCatalogAlternates}");
				}
				Xcconfig.AppendLine($"CURRENT_PROJECT_VERSION = $(UE_{Platform.ToString().ToUpperInvariant()}_BUILD_VERSION)");
				Xcconfig.AppendLine($"MARKETING_VERSION = {MarketingVersion}");
				Xcconfig.AppendLine($"INFOPLIST_KEY_LSApplicationCategoryType = {AppCategory}");
			}

			// always use default codesigning for Editor, which is for running locally
			if (UnrealData.bWriteCodeSigningSettings && UnrealData.TargetRules.Type != TargetType.Editor)
			{
				Xcconfig.AppendLine("");
				Xcconfig.AppendLine("// Code-signing settings");
				Xcconfig.AppendLine("CODE_SIGN_STYLE = " + (bAutomaticSigning ? "Automatic" : "Manual"));
				Xcconfig.AppendLine($"DEVELOPMENT_TEAM = {SigningTeam}");

				// Mac has Sign to Run Locally to deal with
				if (Platform == UnrealTargetPlatform.Mac)
				{
					// when set, always use the - identity, that's how Xcode selects "Sign to Run Locally"
					// if it is not set, and Automatic is on, then it will use it's default of "Apple Development", but we don't specify it in case default changes
					if (bMacSignToRunLocally)
					{
						Xcconfig.AppendLine($"CODE_SIGN_IDENTITY = -");
					}
					else if (!bAutomaticSigning && SigningIdentity != null)
					{
						Xcconfig.AppendLine($"CODE_SIGN_IDENTITY = {SigningIdentity}");
					}
				}
				else
				{
					if (!bAutomaticSigning)
					{
						// only use profile on IOS
						if (!String.IsNullOrEmpty(ProvisioningProfile))
						{
							string SigningName = GetPlistSigningName(ProvisioningProfile, UnrealData, Logger);
							Xcconfig.AppendLine($"PROVISIONING_PROFILE_SPECIFIER = {SigningName}");
						}
						if (SigningIdentity != null)
						{
							Xcconfig.AppendLine($"CODE_SIGN_IDENTITY = {SigningIdentity}");
						}
					}
				}
			}

			Xcconfig.Write();

			// Now for each config write out the specific settings
			foreach (UnrealBuildConfig Config in UnrealData.AllConfigs)
			{
				XcodeBuildConfig? MatchedConfig = BuildConfigList!.BuildConfigs.FirstOrDefault(x => x.Info == Config);
				if (MatchedConfig == null || !Config.Supports(Platform))
				{
					continue;
				}

				// hook up the Buildconfig that matches this info to this xcconfig file
				XcconfigFile ConfigXcconfig = MatchedConfig.Xcconfig!;

				string ExecutableName = AppleExports.MakeBinaryFileName(Config.ExeName, TargetRules.DecoratedSeparator, Platform, Config.BuildConfig, TargetRules.Architectures, TargetRules.UndecoratedConfiguration, null);
				string ExetuableSubPath = FileReference.Combine(ConfigBuildDir, ExecutableName).MakeRelativeTo(ConfigBuildDir);
				string ExecutableKey = $"UE_{Platform.ToString().ToUpperInvariant()}_EXECUTABLE_NAME";

				// we want to make Foo.app, not FooGame.app, since we added on the target type when making targets
				string PerTargetTypeProductName = UnrealData.ProductName;
				if (UnrealData.bIsContentOnlyProject && TargetRules.Type == TargetType.Game)
				{
					PerTargetTypeProductName = UnrealData.UProjectFileLocation!.GetFileNameWithoutAnyExtensions();
				}
				string ProductName = ExecutableName;
				// content only projects don't want UnrealGame, etc as the ProductName
				if (UnrealData.bIsContentOnlyProject && TargetRules.Type != TargetType.Editor)
				{
					ProductName = AppleExports.MakeBinaryFileName(PerTargetTypeProductName, TargetRules.DecoratedSeparator, Platform, Config.BuildConfig, TargetRules.Architectures, TargetRules.UndecoratedConfiguration, null);
				}

				MetadataItem? EntitlementsMetadata = UnrealData.Metadata!.EntitlementsFiles[MetadataPlatform];

				// if we have shipping specified, use it instead
				if (Config.BuildConfig == UnrealTargetConfiguration.Shipping)
				{
					MetadataItem ShippingEntitlementsMetadata = UnrealData.Metadata!.ShippingEntitlementsFiles[MetadataPlatform];
					if (ShippingEntitlementsMetadata.Mode == MetadataMode.UsePremade)
					{
						EntitlementsMetadata = ShippingEntitlementsMetadata;
					}
				}

				string? TargetEntitlementsFilePath = CopyAndWriteEntitlements(ref EntitlementsMetadata, Config.DisplayName, bUseEntitlementsForPrograms, ProjectOrEngineDir);

				ConfigXcconfig.AppendLine("// pull in the shared settings for all configs for this target");
				ConfigXcconfig.AppendLine($"#include \"{Xcconfig.Name}.xcconfig\"");
				ConfigXcconfig.AppendLine("");
				ConfigXcconfig.AppendLine("// Unreal per-config variables");
				ConfigXcconfig.AppendLine($"UE_TARGET_CONFIG = {Config.BuildConfig}");
				ConfigXcconfig.AppendLine($"UE_UBT_BINARY_SUBPATH = {ExetuableSubPath}");
				ConfigXcconfig.AppendLine($"{ExecutableKey} = {ExecutableName}");
				if (EntitlementsMetadata != null && TargetEntitlementsFilePath != null && EntitlementsMetadata.Mode == MetadataMode.UsePremade)
				{
					ConfigXcconfig.AppendLine($"CODE_SIGN_ENTITLEMENTS = {TargetEntitlementsFilePath}");
				}

				// debug settings
				if (Config.BuildConfig == UnrealTargetConfiguration.Debug)
				{
					ConfigXcconfig.AppendLine("ENABLE_TESTABILITY = YES");
				}

				if (Platform == UnrealTargetPlatform.Mac)
				{
					// on Mac, we need to name the .app nicely before pushing to App store, otherwise distributing, so use the ini setting if it's there ("Unreal Match 3"), otherwise use the uproject name (ie "Lyra" instead of "LyraGame")
					ConfigXcconfig.AppendLine("");
					ConfigXcconfig.AppendLine($"// this variable trickery will set the proper name for debugging, building, and archiving,");
					ConfigXcconfig.AppendLine($"// where archiving (the '_install' action type) may need a differnet name so it shows up nicely");
					ConfigXcconfig.AppendLine($"// on end-users machines in Finder, Spotlight, etc. The trailing _ on the next line is correct.");

					ConfigXcconfig.AppendLine($"PRODUCT_NAME_ = {ProductName}");
					ConfigXcconfig.AppendLine($"PRODUCT_NAME_build = $(PRODUCT_NAME_)");
					if (String.IsNullOrEmpty(ApplicationDisplayName))
					{
						ConfigXcconfig.AppendLine($"PRODUCT_NAME_install = {(UnrealData.UProjectFileLocation == null ? ProductName : UnrealData.UProjectFileLocation!.GetFileNameWithoutAnyExtensions())}");
					}
					else
					{
						ConfigXcconfig.AppendLine($"PRODUCT_NAME_install = {ApplicationDisplayName}");
					}

					// this will choose the proper PRODUCT_NAME when archiving vs normal building
					ConfigXcconfig.AppendLine("PRODUCT_NAME = $(PRODUCT_NAME_$(ACTION))");
				}
				else
				{
					ConfigXcconfig.AppendLine($"PRODUCT_NAME = {ProductName}");
				}

				ConfigXcconfig.Write();
			}
		}
	}
	
	class XcodeAppExtRunTarget : XcodeTarget
	{
		public enum ExtType
		{
			BackgroundAssets,
			NotificationService
		}

		private readonly UnrealData UnrealData;
		private readonly UnrealTargetPlatform Platform;
		private readonly string TargetName;
		private readonly string AppGuid;
		private readonly ExtensionConfig ExtConfig;
		
		public XcodeAppExtRunTarget(XcodeProject Project, ExtensionConfig? ExtConfig, string TargetName, string AppGuid, UnrealTargetPlatform Platform, XcodeBuildTarget? BuildTarget, XcodeProjectFile ProjectFile, DirectoryInfo ExtensionPath, ILogger Logger)
			: base(XcodeTarget.Type.App_Ext, Project.UnrealData,
				TargetName + (XcodeProjectFileGenerator.PerPlatformMode == XcodePerPlatformMode.OneWorkspacePerPlatform ? "" : $"_{Platform}"))
		{
			this.Platform = Platform;
			this.AppGuid = AppGuid;
			this.TargetName = TargetName;
			this.ExtConfig = ExtConfig ?? new ExtensionConfig();
			UnrealData = Project.UnrealData;

			BuildConfigList = new XcodeBuildConfigList(Platform, Name, Project.UnrealData, bIncludeAllPlatformsInConfig: false);
			References.Add(BuildConfigList);

			if (Project.UnrealData.IsAppBundle(Platform))
			{
				XcodeFileCollection AppExtFileList = new XcodeFileCollection(ProjectFile);	

				// add all the files to be natively compiled by this target
				XcodeSourcesBuildPhase SourcesBuildPhase = new XcodeSourcesBuildPhase();
				BuildPhases.Add(SourcesBuildPhase);
				References.Add(SourcesBuildPhase);

				foreach (FileSystemInfo FSI in ExtensionPath.EnumerateFileSystemInfos())
				{
					if (FSI.Name.EndsWith(".h", StringComparison.Ordinal) ||
						FSI.Name.StartsWith('.') ||
						FSI.Name.EndsWith(".plist", StringComparison.Ordinal) ||
						FSI.Name.EndsWith(".uappex", StringComparison.Ordinal))
					{
						continue;
					}

					string FileRef = XcodeProjectFileGenerator.MakeXcodeGuid();
					FileReference FR = new FileReference(FSI.FullName);
					XcodeSourceFile XSourceFile = new XcodeSourceFile(FR, FR.Directory, FileRef);
					SourcesBuildPhase.AddFile(XSourceFile);
					if (!Project.FileCollection.AllFiles.Contains(XSourceFile))
					{
						Project.FileCollection.AllFiles.Add(XSourceFile);
						Project.FileCollection.BuildableFilesToResponseFile[XSourceFile] = new();
					}
				}
				
				XcodeResourcesBuildPhase ResourcesBuildPhase = new XcodeResourcesBuildPhase(AppExtFileList);
				BuildPhases.Add(ResourcesBuildPhase);
				References.Add(ResourcesBuildPhase);

				ProcessFrameworks(ResourcesBuildPhase, Project, ProjectFile, Logger);
			}

			if (BuildTarget != null)
			{
				AddDependency(BuildTarget, Project);
			}

			CreateXcconfigFile(Project, Platform, $"{Name}");
			// create per-config Xcconfig files
			foreach (XcodeBuildConfig Config in BuildConfigList.BuildConfigs)
			{
				Config.CreateXcconfigFile(Project, Platform, $"{Name}_{Config.Info.DisplayName.Replace(" ", "", StringComparison.Ordinal)}");
			}
		}

		protected void ProcessFrameworks(XcodeResourcesBuildPhase ResourcesBuildPhase, XcodeProject Project, XcodeProjectFile ProjectFile, ILogger Logger)
		{
			// look up to see if we had cached any Frameworks
			Tuple<ProjectFile, UnrealTargetPlatform> FrameworkKey = Tuple.Create((ProjectFile)ProjectFile, Platform);
			IEnumerable<UEBuildBundleResource>? Bundles;
			if (XcodeProjectFileGenerator.TargetBundles.TryGetValue(FrameworkKey, out Bundles))
			{
				foreach (UEBuildBundleResource Bundle in Bundles)
				{
					ResourcesBuildPhase.AddFolderResource(new DirectoryReference(Bundle.ResourcePath!), "Resources");
				}
			}

			IEnumerable<UEBuildFramework>? Frameworks;
			if (XcodeProjectFileGenerator.TargetFrameworks.TryGetValue(FrameworkKey, out Frameworks))
			{
				// filter frameworks that need to installed into the .app (either the framework or a bundle inside a .zip)
				IEnumerable<UEBuildFramework> InstalledFrameworks = Frameworks.Where(x => x.bCopyFramework || !String.IsNullOrEmpty(x.CopyBundledAssets));

				// only look at frameworks that need anything copied into 
				foreach (UEBuildFramework Framework in InstalledFrameworks)
				{
					DirectoryReference? FinalFrameworkDir = Framework.GetFrameworkDirectory(null, null, Logger);
					if (FinalFrameworkDir == null)
					{
						continue;
					}

					DirectoryReference BundleRootDir = Framework.GetFrameworkDirectory(null, null, Logger)!;

					// set up the CopyBundle to be copied
					if (!String.IsNullOrEmpty(Framework.CopyBundledAssets))
					{
						ResourcesBuildPhase.AddFolderResource(DirectoryReference.Combine(BundleRootDir, Framework.CopyBundledAssets), "Resources");
					}
				}
			}
			
			XcodeFrameworkBuildPhase FrameworkPhase = new XcodeFrameworkBuildPhase(Project.FileCollection);
			foreach (string Framework in ExtConfig.Frameworks)
			{
				DirectoryReference FrameworkRef = new DirectoryReference(Framework + ".framework");
				string FileRefGuid = XcodeProjectFileGenerator.MakeXcodeGuid();

				FrameworkPhase.AddFramework(FrameworkRef, FileRefGuid);
			}

			BuildPhases.Add(FrameworkPhase);
			References.Add(FrameworkPhase);
		}
		
		public override void WriteXcconfigFile(ILogger Logger)
		{
			// gather general, all-platform, data we are doing to put into the configs
			UnrealBuildConfig BuildConfig = BuildConfigList!.BuildConfigs[0].Info;
			TargetRules TargetRules = BuildConfig.ProjectTarget!.TargetRules!;
			DirectoryReference ProjectOrEngineDir = UnrealData.UProjectFileLocation?.Directory ?? Unreal.EngineDirectory;

			// point to the shader Engine/Binaries for content only project (sadly, the TargetRules.OutputFile is not filled out)
			bool bUseEngineDirectory = TargetRules.Type == TargetType.Editor || UnrealData.bIsContentOnlyProject;
			if (TargetRules.BuildEnvironment == TargetBuildEnvironment.Unique)
			{
				// unique build environment editors will put their binaries in the project
				bUseEngineDirectory = false;
			}

			DirectoryReference ConfigBuildDir = bUseEngineDirectory ? Unreal.EngineDirectory : ProjectOrEngineDir;
			if (TargetRules.Type == TargetType.Program && TargetRules.File!.IsUnderDirectory(Unreal.EngineDirectory))
			{
				ConfigBuildDir = BuildConfig.RootDirectory;
			}
			string BinariesBaseDir = ConfigBuildDir.GetDirectoryName();

			ConfigBuildDir = DirectoryReference.Combine(ConfigBuildDir, "Binaries", Platform.ToString(), TargetRules.ExeBinariesSubFolder);

			MetadataPlatform MetadataPlatform;
			string TargetPlatformName = Platform.ToString();
			if (TargetRules.Type != TargetType.Game && TargetRules.Type != TargetType.Program)
			{
				TargetPlatformName += TargetRules.Type.ToString();
			}

			// get ini file for the platform
			ConfigHierarchy PlatformIni = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, UnrealData.ConfigDirectory, Platform);

			// settings for all platforms
			bool bAutomaticSigning;
			bool bMacSignToRunLocally;
			bool bUseEntitlementsForPrograms;
			string? SigningTeam;
			string? SigningPrefix;
			string? AppCategory;
			string SupportedPlatforms;
			string SDKRoot;
			string DeploymentTarget;
			string DeploymentTargetKey;
			string? SigningIdentity = null;
			string? ProvisioningProfile = null;
			string? SupportedDevices = null;
			string? MarketingVersion = null;
			string? BundleIdentifier;
			string? ApplicationDisplayName = null;
			List<string> ExtraConfigLines = new();

			// get signing settings
			PlatformIni.TryGetValue("/Script/MacTargetPlatform.XcodeProjectSettings", "bUseAutomaticCodeSigning", out bAutomaticSigning);
			PlatformIni.TryGetValue("/Script/MacTargetPlatform.XcodeProjectSettings", "bUseEntitlementsForPrograms", out bUseEntitlementsForPrograms);
			PlatformIni.TryGetValue("/Script/MacTargetPlatform.XcodeProjectSettings", "bMacSignToRunLocally", out bMacSignToRunLocally);
			PlatformIni.TryGetValue("/Script/MacTargetPlatform.XcodeProjectSettings", "CodeSigningTeam", out SigningTeam);
			PlatformIni.TryGetValue("/Script/MacTargetPlatform.XcodeProjectSettings", "CodeSigningPrefix", out SigningPrefix);
			PlatformIni.TryGetValue("/Script/MacTargetPlatform.XcodeProjectSettings", $"{Platform}ProvisioningProfile_{TargetName}", out ProvisioningProfile);
			PlatformIni.TryGetValue("/Script/MacTargetPlatform.XcodeProjectSettings", $"{Platform}SigningIdentity", out SigningIdentity);
			PlatformIni.TryGetValue("/Script/MacTargetPlatform.XcodeProjectSettings", "AppCategory", out AppCategory);
			PlatformIni.TryGetValue("/Script/MacTargetPlatform.XcodeProjectSettings", "ApplicationDisplayName", out ApplicationDisplayName);

			// If there's no specific App Extension signing profile, default to the app's profile
			if (String.IsNullOrEmpty(ProvisioningProfile))
			{
				PlatformIni.TryGetValue("/Script/MacTargetPlatform.XcodeProjectSettings", $"{Platform}ProvisioningProfile", out ProvisioningProfile);
			}
			
			if (Platform == UnrealTargetPlatform.Mac)
			{
				// editor vs game metadata
				bool bIsEditor = UnrealData.TargetRules.Type == TargetType.Editor;
				MetadataPlatform = bIsEditor ? MetadataPlatform.MacEditor : MetadataPlatform.Mac;

				SDKRoot = "macosx";
				SupportedPlatforms = "macosx";
				DeploymentTargetKey = "MACOSX_DEPLOYMENT_TARGET";
				DeploymentTarget = MacToolChainSettings.MinMacDeploymentVersion(UnrealData.TargetRules.Type);
				BundleIdentifier = bIsEditor ? "com.epicgames.UnrealEditor" : UnrealData.BundleIdentifier;

				PlatformIni.TryGetValue($"/Script/MacRuntimeSettings.MacRuntimeSettings", "VersionInfo", out MarketingVersion);
				if (String.IsNullOrEmpty(MarketingVersion))
				{
					MarketingVersion = MacToolChain.LoadEngineDisplayVersion();
				}
			}
			else
			{
				MetadataPlatform = MetadataPlatform.IOS;

				// get IOS (same as TVOS) BundleID, and if there's a specified plist with a bundleID, use it, as Xcode would warn if they don't match
				BundleIdentifier = UnrealData.BundleIdentifier;

				// short version string
				PlatformIni.GetString($"/Script/IOSRuntimeSettings.IOSRuntimeSettings", "VersionInfo", out MarketingVersion);

				if (Platform == UnrealTargetPlatform.IOS)
				{
					SDKRoot = "iphoneos";
					SupportedPlatforms = "iphoneos";
					DeploymentTargetKey = "IPHONEOS_DEPLOYMENT_TARGET";
					SupportedDevices = UnrealData.IOSProjectSettings!.RuntimeDevices;
					DeploymentTarget = UnrealData.IOSProjectSettings.RuntimeVersion;
				}
				else if (Platform == UnrealTargetPlatform.TVOS) // tvos
				{
					SDKRoot = "appletvos";
					SupportedPlatforms = "appletvos";
					DeploymentTargetKey = "TVOS_DEPLOYMENT_TARGET";
					SupportedDevices = UnrealData.TVOSProjectSettings!.RuntimeDevices;
					DeploymentTarget = UnrealData.TVOSProjectSettings.RuntimeVersion;
				}
				else if (Platform == UnrealTargetPlatform.VisionOS)
				{
					SDKRoot = "xros";
					SupportedPlatforms = "xrsimulator xros";
					DeploymentTargetKey = "XROS_DEPLOYMENT_TARGET";
					SupportedDevices = UnrealData.VisionOSProjectSettings!.RuntimeDevices;
					DeploymentTarget = UnrealData.VisionOSProjectSettings.RuntimeVersion;

					ExtraConfigLines.Add($"VALID_ARCHS = arm64");
					ExtraConfigLines.Add($"ARCHS = arm64");

					// if we are doing immersive with SwiftUI we need a plist key
					bool bUseSwiftUIMain;
					AppleExports.GetSwiftIntegrationSettings(UnrealData.UProjectFileLocation, UnrealData.TargetRules.Type, Platform, out bUseSwiftUIMain, out _);
					if (bUseSwiftUIMain)
					{
						ExtraConfigLines.Add($"INFOPLIST_KEY_UIApplicationSceneManifest_Generation = YES");
					}
				}
				else
				{
					throw new BuildException($"Unsupported platform {Platform}");
				}
			}

			// get metadata for the platform set above
			MetadataItem PlistMetadata = UnrealData.Metadata!.PlistFiles[MetadataPlatform];
			// now pull the bundle id's out, as xcode will warn if they don't match (this has to happen after each platform set bundle id above)
			XcodeUtils.FindPlistId(PlistMetadata, "CFBundleIdentifier", ref BundleIdentifier);
			// if the user had a ini setting, but also set it in the template plist, the plist one should win (this is only used if we are updating a template plist)
			XcodeUtils.FindPlistId(PlistMetadata, "LSApplicationCategoryType", ref AppCategory);

			// Seed a default for UE_<PLATFORM>_BUILD_VERSION so CURRENT_PROJECT_VERSION and CFBundleVersion are never empty.
			// See the matching comment in the Xcconfig variant above.
			Xcconfig!.AppendLine($"UE_{Platform.ToString().ToUpperInvariant()}_BUILD_VERSION = {ReadOnlyBuildVersion.Current.Changelist}.0");

			// include another xcconfig for versions that UBT writes out
	
			// Blueprint-only projects have Versions.xcconfig in the engine dir, not the project dir
			// If the project directory has a valid version file (like in code projects), that will be used as the last declared one overrides everything else. 
			// If any of them do not exist, they are skipped. This is why we add both here with "?"
			
			Xcconfig!.AppendLine($"#include? \"{Unreal.EngineDirectory}/Intermediate/Build/Versions.xcconfig\"");

			if (UnrealData.UProjectFileLocation != null)
			{
				Xcconfig!.AppendLine($"#include? \"{UnrealData.UProjectFileLocation.Directory}/Intermediate/Build/Versions.xcconfig\"");
			}
			else
			{
				Xcconfig!.AppendLine($"#include? \"{XcodeProjectFileGenerator.ContentOnlySettingsFile}\"");
			}

			// write out some UE variables that can be used in premade .plist files, etc
			Xcconfig.AppendLine("");
			Xcconfig.AppendLine("// Unreal project-wide variables");
			Xcconfig.AppendLine($"UE_XCODE_BUILD_MODE = Default");
			Xcconfig.AppendLine($"UE_PRODUCT_NAME = {Name}");
			Xcconfig.AppendLine($"UE_PRODUCT_NAME_STRIPPED = {Name.Replace("_", "", StringComparison.Ordinal).Replace(" ", "", StringComparison.Ordinal)}");
			Xcconfig.AppendLine($"UE_DISPLAY_NAME = {UnrealData.DisplayName}");
			Xcconfig.AppendLine($"UE_SIGNING_PREFIX = {SigningPrefix}");
			Xcconfig.AppendLine($"UE_PLATFORM_NAME = {Platform}");
			Xcconfig.AppendLine($"UE_TARGET_NAME = {TargetName}");
			// Xcconfig.AppendLine($"UE_TARGET_NAME = {UnrealData.TargetRules.OriginalName}");
			Xcconfig.AppendLine($"UE_TARGET_PLATFORM_NAME = {TargetPlatformName}");
			Xcconfig.AppendLine($"UE_ENGINE_DIR = {Unreal.EngineDirectory}");
			Xcconfig.AppendLine($"UE_BINARIES_DIR = {ConfigBuildDir}");
			Xcconfig.AppendLine($"UE_STAGED_BINARIES_DIR_BASE = {BinariesBaseDir}");
			if (UnrealData.UProjectFileLocation != null)
			{
				Xcconfig.AppendLine($"UE_PROJECT_DIR = {UnrealData.UProjectFileLocation.Directory}");
			}

			Xcconfig.AppendLine("");
			Xcconfig.AppendLine("// Constant settings (same for all platforms and targets)");
			Xcconfig.AppendLine("INFOPLIST_OUTPUT_FORMAT = xml");
			Xcconfig.AppendLine("COMBINE_HIDPI_IMAGES = YES");
			Xcconfig.AppendLine("USE_HEADERMAP = NO");
			Xcconfig.AppendLine("ONLY_ACTIVE_ARCH = YES");
			
			Xcconfig.AppendLine("");
			Xcconfig.AppendLine("// Platform settings");
			Xcconfig.AppendLine($"SUPPORTED_PLATFORMS = {SupportedPlatforms}");
			Xcconfig.AppendLine($"SDKROOT = {SDKRoot}");

			// Xcode creates the Build Dir (where the .app is) by combining {SYMROOT}/{CONFIGURATION}{EFFECTIVE_PLATFORM_NAME}, so we set SYMROOT
			// to the Parent directory of the directory the binary is in, CONFIGURATION to nothing, and EFFECTIVE_PLATFORM_NAME to the directory
			// the binary is in
			Xcconfig.AppendLine("");
			Xcconfig.AppendLine($"// These settings combined will tell Xcode to write to Binaries/{Platform} (instead of something like Binaries/Development-iphoneos)");
			Xcconfig.AppendLine($"SYMROOT = {ConfigBuildDir.ParentDirectory}");
			Xcconfig.AppendLine($"CONFIGURATION = ");
			Xcconfig.AppendLine($"EFFECTIVE_PLATFORM_NAME = {ConfigBuildDir.GetDirectoryName()}");

			if (ExtraConfigLines.Count > 0)
			{
				Xcconfig.AppendLine("");
				Xcconfig.AppendLine("// Misc settings");
				foreach (string Line in ExtraConfigLines)
				{
					// Xcconfig.AppendLine(Line);
				}
			}

			Xcconfig.AppendLine("");
			Xcconfig.AppendLine("// Project settings");
			Xcconfig.AppendLine($"TARGETED_DEVICE_FAMILY = {SupportedDevices}");
			Xcconfig.AppendLine($"PRODUCT_BUNDLE_IDENTIFIER = {BundleIdentifier}.{Name}");
			Xcconfig.AppendLine($"{DeploymentTargetKey} = {DeploymentTarget}");

			Xcconfig.AppendLine("");
			Xcconfig.AppendLine("// Plist settings");
			// Xcconfig.AppendLine($"INFOPLIST_FILE = {PlistMetadata.XcodeProjectRelative}");
			
			DirectoryReference GameDir = UnrealData.UProjectFileLocation?.Directory ?? new DirectoryReference("./");
			Xcconfig.AppendLine($"INFOPLIST_FILE = {GameDir}/Build/IOS/Extensions/{Name}/Info.plist");
			Xcconfig.AppendLine($"INFOPLIST_KEY_CFBundleDisplayName = {Name}");
			if (PlistMetadata.Mode == MetadataMode.UpdateTemplate)
			{
				// allow Xcode to generate the final plist file from our input, some INFOPLIST settings and other settings 
				Xcconfig.AppendLine("GENERATE_INFOPLIST_FILE = YES");
				Xcconfig.AppendLine($"CURRENT_PROJECT_VERSION = $(UE_{Platform.ToString().ToUpperInvariant()}_BUILD_VERSION)");
				Xcconfig.AppendLine($"MARKETING_VERSION = {MarketingVersion}");
			}

			// always use default codesigning for Editor, which is for running locally
			if (UnrealData.bWriteCodeSigningSettings && UnrealData.TargetRules.Type != TargetType.Editor)
			{
				Xcconfig.AppendLine("");
				Xcconfig.AppendLine("// Code-signing settings");
				Xcconfig.AppendLine("CODE_SIGN_STYLE = " + (bAutomaticSigning ? "Automatic" : "Manual"));
				Xcconfig.AppendLine($"DEVELOPMENT_TEAM = {SigningTeam}");

				// Mac has Sign to Run Locally to deal with
				if (Platform == UnrealTargetPlatform.Mac)
				{
					// when set, always use the - identity, that's how Xcode selects "Sign to Run Locally"
					// if it is not set, and Automatic is on, then it will use it's default of "Apple Development", but we don't specify it in case default changes
					if (bMacSignToRunLocally)
					{
						Xcconfig.AppendLine($"CODE_SIGN_IDENTITY = -");
					}
					else if (!bAutomaticSigning && SigningIdentity != null)
					{
						Xcconfig.AppendLine($"CODE_SIGN_IDENTITY = {SigningIdentity}");
					}
				}
				else
				{
					if (!bAutomaticSigning)
					{
						// only use profile on IOS
						if (!String.IsNullOrEmpty(ProvisioningProfile))
						{
							string SigningName = GetPlistSigningName(ProvisioningProfile, UnrealData, Logger);
							Xcconfig.AppendLine($"PROVISIONING_PROFILE_SPECIFIER = {SigningName}");
						}
						if (SigningIdentity != null)
						{
							Xcconfig.AppendLine($"CODE_SIGN_IDENTITY = {SigningIdentity}");
						}
					}
				}
			}

			Xcconfig.Write();

			// Now for each config write out the specific settings
			foreach (UnrealBuildConfig Config in UnrealData.AllConfigs)
			{
				XcodeBuildConfig? MatchedConfig = BuildConfigList!.BuildConfigs.FirstOrDefault(x => x.Info == Config);
				if (MatchedConfig == null || !Config.Supports(Platform))
				{
					continue;
				}

				// hook up the Buildconfig that matches this info to this xcconfig file
				XcconfigFile ConfigXcconfig = MatchedConfig.Xcconfig!;

				string ExecutableName = Name;
				string ExecutableSubPath = FileReference.Combine(ConfigBuildDir, ExecutableName).MakeRelativeTo(ConfigBuildDir);
				string ExecutableKey = $"UE_{Platform.ToString().ToUpperInvariant()}_EXECUTABLE_NAME";

				// we want to make Foo.app, not FooGame.app, since we added on the target type when making targets
				string PerTargetTypeProductName = UnrealData.ProductName;
				if (UnrealData.bIsContentOnlyProject && TargetRules.Type == TargetType.Game)
				{
					PerTargetTypeProductName = UnrealData.UProjectFileLocation!.GetFileNameWithoutAnyExtensions();
				}
				string ProductName = ExecutableName;
				// content only projects don't want UnrealGame, etc as the ProductName
				if (UnrealData.bIsContentOnlyProject && TargetRules.Type != TargetType.Editor)
				{
					ProductName = AppleExports.MakeBinaryFileName(PerTargetTypeProductName, TargetRules.DecoratedSeparator, Platform, Config.BuildConfig, TargetRules.Architectures, TargetRules.UndecoratedConfiguration, null);
				}

				MetadataItem? EntitlementsMetadata = UnrealData.Metadata!.EntitlementsFiles[MetadataPlatform];

				// if we have shipping specified, use it instead
				if (Config.BuildConfig == UnrealTargetConfiguration.Shipping)
				{
					MetadataItem ShippingEntitlementsMetadata = UnrealData.Metadata!.ShippingEntitlementsFiles[MetadataPlatform];
					if (ShippingEntitlementsMetadata.Mode == MetadataMode.UsePremade)
					{
						EntitlementsMetadata = ShippingEntitlementsMetadata;
					}
				}

				// programs almost never want entitlements, so handle them specially
				if ((UnrealData.bNoEntitlements) || (UnrealData.TargetRules.Type == TargetType.Program && !bUseEntitlementsForPrograms))
				{
					EntitlementsMetadata = null;
				}

				ConfigXcconfig.AppendLine("// pull in the shared settings for all configs for this target");
				ConfigXcconfig.AppendLine($"#include \"{Xcconfig.Name}.xcconfig\"");
				ConfigXcconfig.AppendLine("");
				ConfigXcconfig.AppendLine("// Unreal per-config variables");
				ConfigXcconfig.AppendLine($"UE_TARGET_CONFIG = {Config.BuildConfig}");
				ConfigXcconfig.AppendLine($"UE_UBT_BINARY_SUBPATH = {ExecutableSubPath}");
				ConfigXcconfig.AppendLine($"{ExecutableKey} = {ExecutableName}");
				if (EntitlementsMetadata != null && EntitlementsMetadata.Mode == MetadataMode.UsePremade)
				{
					ConfigXcconfig.AppendLine($"CODE_SIGN_ENTITLEMENTS = {EntitlementsMetadata.XcodeProjectRelative}");
				}

				// debug settings
				if (Config.BuildConfig == UnrealTargetConfiguration.Debug)
				{
					ConfigXcconfig.AppendLine("ENABLE_TESTABILITY = YES");
				}

				if (Platform == UnrealTargetPlatform.Mac)
				{
					// on Mac, we need to name the .app nicely before pushing to App store, otherwise distributing, so use the ini setting if it's there ("Unreal Match 3"), otherwise use the uproject name (ie "Lyra" instead of "LyraGame")
					ConfigXcconfig.AppendLine("");
					ConfigXcconfig.AppendLine($"// this variable trickery will set the proper name for debugging, building, and archiving,");
					ConfigXcconfig.AppendLine($"// where archiving (the '_install' action type) may need a differnet name so it shows up nicely");
					ConfigXcconfig.AppendLine($"// on end-users machines in Finder, Spotlight, etc. The trailing _ on the next line is correct.");

					ConfigXcconfig.AppendLine($"PRODUCT_NAME_ = {ProductName}");
					ConfigXcconfig.AppendLine($"PRODUCT_NAME_build = $(PRODUCT_NAME_)");
					if (String.IsNullOrEmpty(ApplicationDisplayName))
					{
						ConfigXcconfig.AppendLine($"PRODUCT_NAME_install = {(UnrealData.UProjectFileLocation == null ? ProductName : UnrealData.UProjectFileLocation!.GetFileNameWithoutAnyExtensions())}");
					}
					else
					{
						ConfigXcconfig.AppendLine($"PRODUCT_NAME_install = {ApplicationDisplayName}");
					}

					// this will choose the proper PRODUCT_NAME when archiving vs normal building
					ConfigXcconfig.AppendLine("PRODUCT_NAME = $(PRODUCT_NAME_$(ACTION))");
				}
				else
				{
					ConfigXcconfig.AppendLine($"PRODUCT_NAME = {ProductName}");
				}

				ConfigXcconfig.Write();
			}
		}
		

		protected override void WriteExtraTargetProperties(StringBuilder Content)
		{
			Content.WriteLine($"\t\t\tproductReference = {AppGuid};");
			Content.WriteLine($"\t\t\tproductName = \"{TargetName}\";");
		}
	}

	class XcodeBuildTarget : XcodeTarget
	{
		public XcodeBuildTarget(XcodeProject Project)
			: base(XcodeTarget.Type.Build, Project.UnrealData)
		{
			BuildConfigList = new XcodeBuildConfigList(Project.Platform, Name, Project.UnrealData, bIncludeAllPlatformsInConfig: false);
			References.Add(BuildConfigList);
		}
	}

	class XcodeIndexTarget : XcodeTarget
	{
		private readonly UnrealData UnrealData;

		// just take the Project since it has everything we need, and is needed when adding target dependencies
		public XcodeIndexTarget(XcodeProject Project, UnrealTargetPlatform Platform)
			: base(XcodeTarget.Type.Index, Project.UnrealData)
		{
			UnrealData = Project.UnrealData;

			BuildConfigList = new XcodeBuildConfigList(Project.Platform, Name, UnrealData, bIncludeAllPlatformsInConfig: false);
			References.Add(BuildConfigList);

			CreateXcconfigFile(Project, Project.Platform, Name);
			// hook up each buildconfig to this Xcconfig
			BuildConfigList.BuildConfigs.ForEach(x => x.Xcconfig = Xcconfig);

			// add all of the files to be natively compiled by this target
			XcodeSourcesBuildPhase SourcesBuildPhase = new XcodeSourcesBuildPhase();
			BuildPhases.Add(SourcesBuildPhase);
			References.Add(SourcesBuildPhase);

			foreach (KeyValuePair<XcodeSourceFile, Dictionary<UnrealTargetPlatform, FileReference>> Pair in Project.FileCollection.BuildableFilesToResponseFile)
			{
				// only add files that found a moduleto be part of (since we can't build without the build settings that come from a module)
				if (Pair.Value.ContainsKey(Platform))
				{
					SourcesBuildPhase.AddFile(Pair.Key);
				}
			}
		}

		public override void WriteXcconfigFile(ILogger Logger)
		{
			// write out settings that for compiling natively
			Xcconfig!.AppendLine("CLANG_CXX_LANGUAGE_STANDARD = c++20");
			Xcconfig.AppendLine("GCC_WARN_INHIBIT_ALL_WARNINGS = YES");
			Xcconfig.AppendLine("GCC_PRECOMPILE_PREFIX_HEADER = YES");
			Xcconfig.AppendLine("GCC_OPTIMIZATION_LEVEL = 0");
			Xcconfig.AppendLine($"PRODUCT_NAME = {Name}");
			Xcconfig.AppendLine("SYMROOT = build");
			Xcconfig.AppendLine("USE_HEADERMAP = NO");
			Xcconfig.AppendLine("WARNING_CFLAGS = -Wno-c++11-narrowing");
			Xcconfig.Write();
		}
	}

	class XcodeProject : XcodeProjectNode
	{
		// a null platform here means all platforms like the old way
		public UnrealTargetPlatform? Platform;

		// the blob of data coming from unreal that we can pass around
		public UnrealData UnrealData;

		// container for all files and groups
		public XcodeFileCollection FileCollection;

		// Guid for the project node
		public string Guid = XcodeProjectFileGenerator.MakeXcodeGuid();
		private readonly string ProvisioningStyle;

		public List<XcodeRunTarget> RunTargets = new();
		public List<XcodeAppExtRunTarget> AppExtRunTargets = new();

		public XcodeBuildConfigList ProjectBuildConfigs;

		public XcodeProject(UnrealTargetPlatform? Platform, UnrealData UnrealData, XcodeFileCollection FileCollection, XcodeProjectFile ProjectFile, bool bIsStubEditor, ILogger Logger)
		{
			this.Platform = Platform;
			this.UnrealData = UnrealData;
			this.FileCollection = FileCollection;

			ProvisioningStyle = UnrealData.bUseAutomaticSigning ? "Automatic" : "Manual";

			// if we are run-only, then we don't need a build target (this is shared between platforms if we are doing multi-target)
			XcodeBuildTarget? BuildTarget = null;
			if (!XcodeProjectFileGenerator.bGenerateRunOnlyProject && !UnrealData.bIsStubProject && !bIsStubEditor)
			{
				BuildTarget = new XcodeBuildTarget(this);
			}

			if (!bIsStubEditor)
			{
				// create one run target for each platform if our platform is null (ie XcodeProjectGenerator.PerPlatformMode is RunTargetPerPlatform)
				List<UnrealTargetPlatform> TargetPlatforms = Platform == null ? XcodeProjectFileGenerator.XcodePlatforms : new() { Platform.Value };
				foreach (UnrealTargetPlatform TargetPlatform in TargetPlatforms)
				{
					XcodeRunTarget RunTarget = new XcodeRunTarget(this, UnrealData.ProductName, UnrealData.AllConfigs[0].ProjectTarget!.TargetRules!.Type, TargetPlatform, BuildTarget, ProjectFile, Logger);
					RunTargets.Add(RunTarget);
					References.Add(RunTarget);
				}
			}

			ProjectBuildConfigs = new XcodeBuildConfigList(bIsStubEditor ? null : Platform, UnrealData.ProductName, UnrealData, bIncludeAllPlatformsInConfig: true);
			References.Add(ProjectBuildConfigs);

			// make an indexing target if we aren't just a run-only project, and it has buildable source files
			if (!XcodeProjectFileGenerator.bGenerateRunOnlyProject)
			{
				if (Platform != null)
				{
					// Per platform project, check if we have files to index for that platform
					if (UnrealData.BatchedFiles.ContainsKey(Platform.Value) && UnrealData.BatchedFiles[Platform.Value].Count > 0)
					{
						XcodeIndexTarget IndexTarget = new XcodeIndexTarget(this, Platform.Value);
						References.Add(IndexTarget);
					}
				}
				else
				{
					// Shared platform project, just try to index Mac target
					if (UnrealData.BatchedFiles[UnrealTargetPlatform.Mac].Count > 0)
					{
						XcodeIndexTarget IndexTarget = new XcodeIndexTarget(this, UnrealTargetPlatform.Mac);
						References.Add(IndexTarget);
					}
				}
			}
		}

		public override void Write(StringBuilder Content)
		{
			Content.WriteLine("/* Begin PBXProject section */");

			Content.WriteLine(2, $"{Guid} /* Project object */ = {{");
			Content.WriteLine(3, "isa = PBXProject;");
			Content.WriteLine(3, "attributes = {");
			Content.WriteLine(4, "LastUpgradeCheck = 2000;");
			Content.WriteLine(4, "ORGANIZATIONNAME = \"Epic Games, Inc.\";");
			Content.WriteLine(4, "TargetAttributes = {");
			foreach (XcodeRunTarget RunTarget in RunTargets)
			{
				Content.WriteLine(5, $"{RunTarget.Guid} = {{");
				Content.WriteLine(6, $"ProvisioningStyle = {ProvisioningStyle};");
				Content.WriteLine(5, "};");
			}
			foreach (XcodeAppExtRunTarget RunTarget in AppExtRunTargets)
			{
				Content.WriteLine(5, $"{RunTarget.Guid} = {{");
				Content.WriteLine(6, $"ProvisioningStyle = {ProvisioningStyle};");
				Content.WriteLine(5, "};");
			}
			Content.WriteLine(4, "};");
			Content.WriteLine(3, "};");
			Content.WriteLine(3, $"buildConfigurationList = {ProjectBuildConfigs.Guid} /* Build configuration list for PBXProject \"{ProjectBuildConfigs.TargetName}\" */;");
			Content.WriteLine(3, "compatibilityVersion = \"Xcode 8.0\";");
			Content.WriteLine(3, "developmentRegion = English;");
			Content.WriteLine(3, "hasScannedForEncodings = 0;");
			Content.WriteLine(3, "knownRegions = (");
			Content.WriteLine(4, "en");
			Content.WriteLine(3, ");");
			Content.WriteLine(3, $"mainGroup = {FileCollection.MainGroupGuid};");
			// for stub editor projects, we don't have a run target, so we don't have a product folder
			if (RunTargets.Count > 0 || AppExtRunTargets.Count > 0)
			{
				Content.WriteLine(3, $"productRefGroup = {FileCollection.GetProductGroupGuid()};");
			}
			Content.WriteLine(3, "projectDirPath = \"\";");
			Content.WriteLine(3, "projectRoot = \"\";");
			Content.WriteLine(3, "targets = (");
			foreach (XcodeTarget Target in XcodeProjectNode.GetNodesOfType<XcodeTarget>(this))
			{
				Content.WriteLine(4, $"{Target.Guid} /* {Target.Name} */,");
			}
			Content.WriteLine(3, ");");
			Content.WriteLine(2, "};");

			Content.WriteLine("/* End PBXProject section */");
		}
	}

	class XcodeProjectFile : ProjectFile
	{

		/// <summary>
		/// Constructs a new project file object
		/// </summary>
		/// <param name="InitFilePath">The path to the project file on disk</param>
		/// <param name="BaseDir">The base directory for files within this project</param>
		/// <param name="bIsForDistribution">True for distribution builds</param>
		/// <param name="bForceNoEntitlements">True to force disable entitlements</param>
		/// <param name="BundleID">Override option for bundle identifier</param>
		/// <param name="AppName"></param>
		/// <param name="bMakeProjectPerTarget"></param>
		/// <param name="SingleTargetName"></param>
		public XcodeProjectFile(FileReference InitFilePath, DirectoryReference BaseDir, bool bIsForDistribution, bool bForceNoEntitlements, string BundleID, string AppName, bool bMakeProjectPerTarget, string? SingleTargetName)
			: base(InitFilePath, BaseDir)
		{
			UnrealData = new UnrealData(InitFilePath, bIsForDistribution, bForceNoEntitlements, BundleID, AppName, bMakeProjectPerTarget);

			// create the container for all the files that will 
			SharedFileCollection = new XcodeFileCollection(this);
			FileCollection = SharedFileCollection;
			this.SingleTargetName = SingleTargetName;
		}

		public UnrealData UnrealData;

		/// <summary>
		///  The PBXPRoject node, root of everything
		/// </summary>
		public Dictionary<XcodeProject, UnrealTargetPlatform?> RootProjects = new();

		private XcodeProjectLegacy.XcodeProjectFile? LegacyProjectFile = null;
		private bool bHasCheckedForLegacy = false;
		public bool bHasLegacyProject => LegacyProjectFile != null;

		/// <summary>
		/// Gathers the files and generates project sections
		/// </summary>
		private readonly XcodeFileCollection SharedFileCollection;
		public XcodeFileCollection FileCollection;

		// if set, only this will be written
		private readonly string? SingleTargetName;

		/// <summary>
		/// Allocates a generator-specific source file object
		/// </summary>
		/// <param name="InitFilePath">Path to the source file on disk</param>
		/// <param name="InitProjectSubFolder">Optional sub-folder to put the file in.  If empty, this will be determined automatically from the file's path relative to the project file</param>
		/// <returns>The newly allocated source file object</returns>
		public override SourceFile? AllocSourceFile(FileReference InitFilePath, DirectoryReference? InitProjectSubFolder)
		{
			if (InitFilePath.GetFileName().StartsWith('.'))
			{
				return null;
			}
			return new XcodeSourceFile(InitFilePath, InitProjectSubFolder);
		}

		public void ConditionalCreateLegacyProject()
		{
			if (!bHasCheckedForLegacy)
			{
				bHasCheckedForLegacy = true;
				if (ProjectTargets.Count == 0)
				{
					throw new BuildException("Expected to have a target before AddModuleForIntelliSense is called");
				}

				UnrealData.InitializeUProjectFileLocation(this);
				if (AppleExports.CreatingAppOnWindows(UnrealData.UProjectFileLocation))
				{
					LegacyProjectFile = new XcodeProjectLegacy.XcodeProjectFile(ProjectFilePath, BaseDir, UnrealData.bForDistribution, UnrealData.BundleIdentifier, UnrealData.AppName, UnrealData.bMakeProjectPerTarget);
					LegacyProjectFile.ProjectTargets.AddRange(ProjectTargets);
					LegacyProjectFile.SourceFiles.AddRange(SourceFiles);
					LegacyProjectFile.IsGeneratedProject = IsGeneratedProject;
					LegacyProjectFile.IsStubProject = IsStubProject;
					LegacyProjectFile.IsForeignProject = IsForeignProject;
				}
			}
		}

		public override void AddModuleForIntelliSense(UEBuildModuleCPP Module, CppCompileEnvironment CompileEnvironment)
		{
			ConditionalCreateLegacyProject();

			if (LegacyProjectFile != null)// && Module.Rules.Target.Platform == UnrealTargetPlatform.Mac)
			{
				LegacyProjectFile.AddModuleForIntelliSense(Module, CompileEnvironment);
				return;
			}

			UnrealData.AddModuleForIntelliSense(Module, CompileEnvironment);
		}

		/// <summary>
		/// Generates bodies of all sections that contain a list of source files plus a dictionary of project navigator groups.
		/// </summary>
		private void ProcessSourceFiles()
		{
			// process the files that came from UE/cross-platform land
			SourceFiles.SortBy(x => x.Reference.FullName);

			ConfigHierarchy PlatformIni = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, UnrealData.ConfigDirectory, UnrealTargetPlatform.Mac);

			IReadOnlyList<string>? ExcludedDirs;
			if (!PlatformIni.TryGetValues("/Script/MacTargetPlatform.XcodeProjectSettings", "XCProjectExcludedDirs", out ExcludedDirs))
			{
				ExcludedDirs = new List<string>();
			}

			Dictionary<DirectoryReference, int> BuildFileMap = new();
			foreach (XcodeSourceFile SourceFile in SourceFiles.OfType<XcodeSourceFile>())
			{
				if (!ExcludedDirs.Any(dir => SourceFile.Reference.IsUnderDirectory(DirectoryReference.Combine(SharedFileCollection.RootDirectory, dir))))
				{
					SharedFileCollection.ProcessFile(SourceFile, bIsForBuild: IsGeneratedProject, bIsFolder: false, SourceToBuildFileMap: BuildFileMap);
				}
			}

			// cache the main group
			SharedFileCollection.MainGroupGuid = XcodeFileCollection.GetRootGroupGuid(SharedFileCollection.Groups, UnrealData.XcodeProjectFileLocation);

			// filter each file into the appropriate batch
			foreach (XcodeSourceFile File in SharedFileCollection.BuildableFilesToResponseFile.Keys)
			{
				AddFileToBatch(File, SharedFileCollection);
			}

			// write out the response files for each batch now that everything is done
			foreach (KeyValuePair<UnrealTargetPlatform, List<UnrealBatchedFiles>> Pair in UnrealData.BatchedFiles)
			{
				foreach(UnrealBatchedFiles Batch in Pair.Value)
				{
					Batch.GenerateResponseFile();
				}
			}
		}

		private void AddFileToBatch(XcodeSourceFile File, XcodeFileCollection FileCollection)
		{
			foreach (KeyValuePair<UnrealTargetPlatform, List<UnrealBatchedFiles>> Pair in UnrealData.BatchedFiles)
			{
				foreach (UnrealBatchedFiles Batch in Pair.Value)
				{
					if (Batch.Module.ContainsFile(File.Reference))
					{
						Batch.Files.Add(File);
						FileCollection.BuildableFilesToResponseFile[File][Pair.Key] = Batch.ResponseFile;
						break;
					}
				}
			}
		}

		public FileReference ProjectFilePathForPlatform(UnrealTargetPlatform? Platform)
		{
			return new FileReference(XcodeUtils.ProjectDirPathForPlatform(UnrealData.XcodeProjectFileLocation, Platform).FullName);
		}

		public FileReference PBXFilePathForPlatform(UnrealTargetPlatform? Platform)
		{
			return FileReference.Combine(XcodeUtils.ProjectDirPathForPlatform(UnrealData.XcodeProjectFileLocation, Platform), "project.pbxproj");
		}

		/// Implements Project interface
		public override bool WriteProjectFile(List<UnrealTargetPlatform> InPlatforms, List<UnrealTargetConfiguration> InConfigurations, PlatformProjectGeneratorCollection PlatformProjectGenerators, ILogger Logger)
		{
			// if we don't want this one, just skip
			if (SingleTargetName != null && !ProjectFilePath.GetFileNameWithoutAnyExtensions().Equals(SingleTargetName, StringComparison.OrdinalIgnoreCase))
			{
				return true;
			}

			ConditionalCreateLegacyProject();

			if (LegacyProjectFile != null)
			{
				return LegacyProjectFile.WriteProjectFile(InPlatforms, InConfigurations, PlatformProjectGenerators, Logger);
			}

			if (UnrealData.Initialize(this, InConfigurations, Logger) == false)
			{
				// if we failed to initialize, we silently return to move on (it's not an error, it's a project with nothing to do)
				return true;
			}

			// look for an existing project to use as a template (if none found, create one from scratch)
			DirectoryReference BuildDirLocation = UnrealData.UProjectFileLocation == null ? Unreal.EngineDirectory : UnrealData.UProjectFileLocation.Directory;
			string ExistingProjectName = UnrealData.ProductName;
			FileReference TemplateProject = FileReference.Combine(BuildDirLocation, $"Build/IOS/{UnrealData.ProductName}.xcodeproj/project.pbxproj");

			// @todo this for per-platform!
			UnrealData.bIsMergingProjects = FileReference.Exists(TemplateProject);
			UnrealData.bWriteCodeSigningSettings = !UnrealData.bIsMergingProjects;

			// turn all UE files into internal representation
			ProcessSourceFiles();

			bool bSuccess = true;
			foreach (UnrealTargetPlatform? Platform in XcodeProjectFileGenerator.WorkspacePlatforms)
			{
				bool bAddStubEditor = Platform != UnrealTargetPlatform.Mac && UnrealData.ProductName == "UnrealEditor";
				// skip the platform if the project has no configurations for it
				if (!bAddStubEditor && !UnrealData.AllConfigs.Any(x => x.Supports(Platform)))
				{
					continue;
				}
				FileReference PBXFilePath = PBXFilePathForPlatform(Platform);

				// now create the xcodeproject elements (project -> target -> buildconfigs, etc)
				FileCollection = new XcodeFileCollection(SharedFileCollection);
				XcodeProject RootProject = new XcodeProject(Platform, UnrealData, FileCollection, this, bAddStubEditor, Logger);
				RootProjects[RootProject] = Platform;

				if (FileReference.Exists(TemplateProject))
				{
					// Merging was removed in CL 23517683 in 2022-12, when adding support for 'modern' Xcode projects.
					continue;
				}
				else
				{
					// write metadata now so we can add them to the FileCollection
					ConditionalWriteMetadataFiles();

					StringBuilder Content = new StringBuilder();

					Content.WriteLine(0, "// !$*UTF8*$!");
					Content.WriteLine(0, "{");
					Content.WriteLine(1, "archiveVersion = 1;");
					Content.WriteLine(1, "classes = {");
					Content.WriteLine(1, "};");
					Content.WriteLine(1, "objectVersion = 46;");
					Content.WriteLine(1, "objects = {");

					// write out the list of files and groups
					FileCollection.Write(Content, Platform);

					// now write out the project node and its recursive dependent nodes
					XcodeProjectNode.WriteNodeAndReferences(Content, RootProject, Logger);

					Content.WriteLine(1, "};");
					Content.WriteLine(1, $"rootObject = {RootProject.Guid} /* Project object */;");
					Content.WriteLine(0, "}");

					// finally write out the pbxproj file!
					bSuccess = ProjectFileGenerator.WriteFileIfChanged(PBXFilePath.FullName, Content.ToString(), Logger, new UTF8Encoding()) && bSuccess;
				}

				bool bNeedScheme = !bAddStubEditor && XcodeUtils.ShouldIncludeProjectInWorkspace(this);
				if (bNeedScheme)
				{
					if (bSuccess)
					{
						string ProjectName = ProjectFilePathForPlatform(Platform).GetFileNameWithoutAnyExtensions();
						string? BuildTargetGuid = XcodeProjectNode.GetNodesOfType<XcodeBuildTarget>(RootProject).FirstOrDefault()?.Guid;
						string? IndexTargetGuid = XcodeProjectNode.GetNodesOfType<XcodeIndexTarget>(RootProject).FirstOrDefault()?.Guid;
						XcodeSchemeFile.WriteSchemeFile(UnrealData, Platform, ProjectName, RootProject.RunTargets, BuildTargetGuid, IndexTargetGuid);
					}
				}
				else
				{
					XcodeSchemeFile.CleanSchemeFile(UnrealData, Platform);
				}
			}
			return bSuccess;
		}

		private void ConditionalWriteMetadataFiles()
		{
			// we now use templates or premade, no writing out here
			foreach (MetadataItem Data in UnrealData.Metadata!.PlistFiles.Values)
			{
				if (Data.XcodeProjectRelative != null)
				{
					FileCollection.AddFileReference(XcodeProjectFileGenerator.MakeXcodeGuid(), Data.XcodeProjectRelative, "explicitFileType", "text.plist", "\"<group>\"", "Metadata");
				}
			}
			foreach (MetadataItem Data in UnrealData.Metadata.EntitlementsFiles.Values)
			{
				if (Data.XcodeProjectRelative != null && Data.Mode == MetadataMode.UsePremade)
				{
					FileCollection.AddFileReference(XcodeProjectFileGenerator.MakeXcodeGuid(), Data.XcodeProjectRelative, "explicitFileType", "text.plist", "\"<group>\"", "Metadata");
				}
			}
		}
	}
}
