// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	enum XcodePerPlatformMode
	{
		OneWorkspacePerPlatform,
		OneTargetPerPlatform,
	}

	/// <summary>
	/// Xcode project file generator implementation
	/// </summary>
	class XcodeProjectFileGenerator : ProjectFileGenerator
	{
		/// <summary>
		///  Xcode makes a project per target, for modern to be able to pull in Cooked data
		/// </summary>
		protected override bool bMakeProjectPerTarget => true;
		protected override bool bAllowContentOnlyProjects => true;
		protected override bool bAllowMultiModuleReference => true;

		public DirectoryReference? XCWorkspace;

		// always seed the random number the same, so multiple runs of the generator will generate the same project
		static readonly Random Rand = new Random(0);

		// how to handle multiple platforms
		public static XcodePerPlatformMode PerPlatformMode = XcodePerPlatformMode.OneWorkspacePerPlatform;

		/// <summary>
		/// Mark for distribution builds
		/// </summary>
		readonly bool bForDistribution = false;

		/// <summary>
		/// Don't use entitlements
		/// </summary>
		readonly bool bNoEntitlements = false;

		/// <summary>
		/// Override BundleID
		/// </summary>
		readonly string BundleIdentifier = "";

		/// <summary>
		/// Override AppName
		/// </summary>
		readonly string AppName = "";

		/// <summary>
		/// Store the single game project (when using -game -project=...) to a place that XcodeProjectLegacy can easily retrieve it
		/// </summary>
		public static FileReference? SingleGameProject = null;

		/// <summary>
		/// Shared file that the project agnostic projects can point to to get content only projects working nicely in Xcode
		/// </summary>
		public static FileReference ContentOnlySettingsFile = FileReference.Combine(Unreal.EngineDirectory, "Build/Xcode/ContentOnlySettings.xcconfig");

		/// <summary>
		/// A static copy of ProjectPlatforms from the base class
		/// </summary>
		public static List<UnrealTargetPlatform> XcodePlatforms = new();

		public static List<UnrealTargetPlatform?> WorkspacePlatforms = new();
		public static List<UnrealTargetPlatform?> RunTargetPlatforms = new();
		public static List<UnrealTargetPlatform?> NullPlatformList = new() { null };
		internal static Dictionary<Tuple<ProjectFile, UnrealTargetPlatform>, IEnumerable<UEBuildFramework>> TargetFrameworks = new();
		internal static Dictionary<Tuple<ProjectFile, UnrealTargetPlatform>, IEnumerable<UEBuildBundleResource>> TargetBundles = new();
		internal static Dictionary<Tuple<ProjectFile, UnrealTargetPlatform>, IEnumerable<ModuleRules.RuntimeDependency>> TargetRawDylibs = new();

		/// <summary>
		/// Should we generate only a run project (no build/index targets)
		/// </summary>
		public static bool bGenerateRunOnlyProject = false;

		/// <summary>
		/// Platforms we should generate IntelliSense data for, Xcode projects should index for each Apple platform.
		/// </summary>
		protected override List<UnrealTargetPlatform> GetIntelliSensePlatforms()
		{
			if (bMakeProjectPerTarget)
			{
				// For now just allow indexing to work on Mac and IOS, enabling too many will make GPF and workspaces even slower
				return new() { UnrealTargetPlatform.Mac, UnrealTargetPlatform.IOS };
			}
			else
			{
				// If sharing projects across platform targets, we are unable to index separately (due to generated headers being platform dependent)
				// Choose Mac to be the platform for indexing
				return new() { UnrealTargetPlatform.Mac };
			}
		}

		public XcodeProjectFileGenerator(FileReference? InOnlyGameProject, CSharpProjectCache CSharpProjCache, CommandLineArguments CommandLine)
			: base(InOnlyGameProject, CSharpProjCache)
		{
			SingleGameProject = InOnlyGameProject;
			if (CommandLine.HasOption("-distribution"))
			{
				bForDistribution = true;
			}
			if (CommandLine.HasOption("-noEntitlements"))
			{
				bNoEntitlements = true;
			}
			if (CommandLine.HasValue("-bundleID="))
			{
				BundleIdentifier = CommandLine.GetString("-bundleID=");
			}

			if (CommandLine.HasValue("-appname="))
			{
				AppName = CommandLine.GetString("-appname=");
			}

			// make sure only one Target writes the file
			lock (ContentOnlySettingsFile)
			{
				if (OnlyGameProject == null && !FileReference.Exists(ContentOnlySettingsFile))
				{
					DirectoryReference.CreateDirectory(ContentOnlySettingsFile.Directory);
					File.WriteAllLines(ContentOnlySettingsFile.FullName, new string[]
					{
							"// Enter the settings for your active code only project here. Note you will need to cook and stage your project before running.",
							"// Since we use the variables directly, we cannot take a single path to a uproject file, it must be split over two variables",
							"",
							"// Enter the path to the root directory of your project (use the full path - you can get it by selecting the folder in Finder and pressing Option-Command-C, then pasting here:",
							"UE_CONTENTONLY_PROJECT_DIR=",
							"",
							"// Enter the name of the project (which is your uproject filename without the extension)",
							"UE_CONTENTONLY_PROJECT_NAME=",
							"",
							"// Uncomment this line if you want the Editor to launch with your Content Only project above",
							"// UE_CONTENTONLY_EDITOR_STARTUP_PROJECT=$(UE_CONTENTONLY_PROJECT_DIR)/$(UE_CONTENTONLY_PROJECT_NAME).uproject",
							"",
							"",
							"",
							"",
							"",
							"// This is the default location for your Staged data, only change this if you know that you need to:",
							"UE_OVERRIDE_STAGE_DIR = $(UE_CONTENTONLY_PROJECT_DIR)/Saved/StagedBuilds/$(UE_TARGET_PLATFORM_NAME)",
					});
				}
			}
		}

		/// <summary>
		/// Make a random Guid string usable by Xcode (24 characters exactly)
		/// </summary>
		public static string MakeXcodeGuid()
		{
			string Guid = "";

			byte[] Randoms = new byte[12];
			lock(Rand)
			{
#pragma warning disable CA5394 // Random Guids need to be stable so Random is fine
			Rand.NextBytes(Randoms);
#pragma warning restore CA5394
			}
			for (int Index = 0; Index < 12; Index++)
			{
				Guid += Randoms[Index].ToString("X2");
			}

			return Guid;
		}

		/// File extension for project files we'll be generating (e.g. ".vcxproj")
		public override string ProjectFileExtension => ".xcodeproj";

		/// <summary>
		/// </summary>
		public override void CleanProjectFiles(DirectoryReference InPrimaryProjectDirectory, string InPrimaryProjectName, DirectoryReference InIntermediateProjectFilesPath, ILogger Logger)
		{
			foreach (DirectoryReference ProjectFile in DirectoryReference.EnumerateDirectories(InPrimaryProjectDirectory, $"{InPrimaryProjectName}*.xcworkspace"))
			{
				DirectoryReference.Delete(ProjectFile, true);
			}

			// Delete the project files folder
			if (DirectoryReference.Exists(InIntermediateProjectFilesPath))
			{
				try
				{
					DirectoryReference.Delete(InIntermediateProjectFilesPath, true);
				}
				catch (Exception Ex)
				{
					Logger.LogInformation("Error while trying to clean project files path {InIntermediateProjectFilesPath}. Ignored.", InIntermediateProjectFilesPath);
					Logger.LogInformation("\t{Ex}", Ex.Message);
				}
			}
		}

		/// <summary>
		/// Allocates a generator-specific project file object
		/// </summary>
		/// <param name="InitFilePath">Path to the project file</param>
		/// <param name="BaseDir">The base directory for files within this project</param>
		/// <returns>The newly allocated project file object</returns>
		protected override ProjectFile AllocateProjectFile(FileReference InitFilePath, DirectoryReference BaseDir)
		{
			// this may internally (later) make a Legacy project object if the unreal project wants old behavior
			// unfortunately, we can't read the project configs now because we don't have enough information to 
			// find the .uproject file for that would make this project (we could change the high level to pass it 
			// down but it would touch all project generators - not worth it if we end up removing the legacy)
			return new XcodeProjectXcconfig.XcodeProjectFile(InitFilePath, BaseDir, bForDistribution, bNoEntitlements, BundleIdentifier, AppName, bMakeProjectPerTarget, SingleTargetName);
		}

		private static bool WriteWorkspaceSettingsFile(string Path, ILogger Logger)
		{
			StringBuilder WorkspaceSettingsContent = new StringBuilder();
			WorkspaceSettingsContent.Append("<?xml version=\"1.0\" encoding=\"UTF-8\"?>" + ProjectFileGenerator.NewLine);
			WorkspaceSettingsContent.Append("<!DOCTYPE plist PUBLIC \"-//Apple//DTD PLIST 1.0//EN\" \"http://www.apple.com/DTDs/PropertyList-1.0.dtd\">" + ProjectFileGenerator.NewLine);
			WorkspaceSettingsContent.Append("<plist version=\"1.0\">" + ProjectFileGenerator.NewLine);
			WorkspaceSettingsContent.Append("<dict>" + ProjectFileGenerator.NewLine);
			// @todo when we move to xcode 14, we remove these next 4 keys
			WorkspaceSettingsContent.Append("\t<key>BuildSystemType</key>" + ProjectFileGenerator.NewLine);
			WorkspaceSettingsContent.Append("\t<string>Original</string>" + ProjectFileGenerator.NewLine);
			WorkspaceSettingsContent.Append("\t<key>BuildLocationStyle</key>" + ProjectFileGenerator.NewLine);
			WorkspaceSettingsContent.Append("\t<string>UseTargetSettings</string>" + ProjectFileGenerator.NewLine);
			WorkspaceSettingsContent.Append("\t<key>CustomBuildLocationType</key>" + ProjectFileGenerator.NewLine);
			WorkspaceSettingsContent.Append("\t<string>RelativeToDerivedData</string>" + ProjectFileGenerator.NewLine);
			WorkspaceSettingsContent.Append("\t<key>DerivedDataLocationStyle</key>" + ProjectFileGenerator.NewLine);
			WorkspaceSettingsContent.Append("\t<string>Default</string>" + ProjectFileGenerator.NewLine);
			WorkspaceSettingsContent.Append("\t<key>IssueFilterStyle</key>" + ProjectFileGenerator.NewLine);
			WorkspaceSettingsContent.Append("\t<string>ShowAll</string>" + ProjectFileGenerator.NewLine);
			WorkspaceSettingsContent.Append("\t<key>LiveSourceIssuesEnabled</key>" + ProjectFileGenerator.NewLine);
			WorkspaceSettingsContent.Append("\t<true/>" + ProjectFileGenerator.NewLine);
			WorkspaceSettingsContent.Append("\t<key>SnapshotAutomaticallyBeforeSignificantChanges</key>" + ProjectFileGenerator.NewLine);
			WorkspaceSettingsContent.Append("\t<true/>" + ProjectFileGenerator.NewLine);
			WorkspaceSettingsContent.Append("\t<key>SnapshotLocationStyle</key>" + ProjectFileGenerator.NewLine);
			WorkspaceSettingsContent.Append("\t<string>Default</string>" + ProjectFileGenerator.NewLine);
			WorkspaceSettingsContent.Append("</dict>" + ProjectFileGenerator.NewLine);
			WorkspaceSettingsContent.Append("</plist>" + ProjectFileGenerator.NewLine);
			return WriteFileIfChanged(Path, WorkspaceSettingsContent.ToString(), Logger, new UTF8Encoding());
		}

		private static bool WriteWorkspaceSharedSettingsFile(string Path, ILogger Logger)
		{
			StringBuilder WorkspaceSettingsContent = new StringBuilder();
			WorkspaceSettingsContent.Append("<?xml version=\"1.0\" encoding=\"UTF-8\"?>" + ProjectFileGenerator.NewLine);
			WorkspaceSettingsContent.Append("<!DOCTYPE plist PUBLIC \"-//Apple//DTD PLIST 1.0//EN\" \"http://www.apple.com/DTDs/PropertyList-1.0.dtd\">" + ProjectFileGenerator.NewLine);
			WorkspaceSettingsContent.Append("<plist version=\"1.0\">" + ProjectFileGenerator.NewLine);
			WorkspaceSettingsContent.Append("<dict>" + ProjectFileGenerator.NewLine);
			// @todo when we move to xcode 14, we remove these next 2 keys
			WorkspaceSettingsContent.Append("\t<key>DisableBuildSystemDeprecationWarning</key>" + ProjectFileGenerator.NewLine);
			WorkspaceSettingsContent.Append("\t<true/>" + ProjectFileGenerator.NewLine);
			WorkspaceSettingsContent.Append("\t<key>DisableBuildSystemDeprecationDiagnostic</key>" + ProjectFileGenerator.NewLine);
			WorkspaceSettingsContent.Append("\t<true/>" + ProjectFileGenerator.NewLine);
			WorkspaceSettingsContent.Append("\t<key>IDEWorkspaceSharedSettings_AutocreateContextsIfNeeded</key>" + ProjectFileGenerator.NewLine);
			WorkspaceSettingsContent.Append("\t<false/>" + ProjectFileGenerator.NewLine);
			WorkspaceSettingsContent.Append("</dict>" + ProjectFileGenerator.NewLine);
			WorkspaceSettingsContent.Append("</plist>" + ProjectFileGenerator.NewLine);
			return WriteFileIfChanged(Path, WorkspaceSettingsContent.ToString(), Logger, new UTF8Encoding());
		}

		private string PrimaryProjectNameForPlatform(UnrealTargetPlatform? Platform)
		{
			string ProjectName = PrimaryProjectName;
			if (!String.IsNullOrEmpty(SingleTargetName))
			{
				ProjectName = $"{ProjectName}_{SingleTargetName}";
			}
			// if there are projectplatforms, then there is a platform name already in the name
			if (Platform != null && ProjectPlatforms.Count == 0)
			{
				ProjectName = $"{ProjectName} ({Platform})";
			}
			return ProjectName;
		}

		private bool WriteXcodeWorkspace(ILogger Logger)
		{
			bool bSuccess = true;

			// loop opver all projects to see if at least one is modern (if not, we don't bother splitting up by platform)
			bool bHasModernProjects = false;
			Action<List<PrimaryProjectFolder>>? FindModern = null;
			FindModern = (FolderList) =>
			{
				foreach (PrimaryProjectFolder CurFolder in FolderList)
				{
					ProjectFile? Modern = CurFolder.ChildProjects.FirstOrDefault(P =>
						P.GetType() == typeof(XcodeProjectXcconfig.XcodeProjectFile) &&
						!((XcodeProjectXcconfig.XcodeProjectFile)P).bHasLegacyProject);
					if (Modern != null)
					{
						//Logger.LogWarning($"Project {Modern.ProjectFilePath} is modern");
						bHasModernProjects = true;
					}
					if (!bHasModernProjects)
					{
						FindModern!(CurFolder.SubFolders);
					}
				}
			};
			FindModern(RootFolder.SubFolders);

			// if we want one workspace with multiple platforms, and we have at least one modern project, then process each platform individually
			// otherwise use null as Platform which means to merge all platforms
			List<UnrealTargetPlatform?> PlatformsToProcess = bHasModernProjects ? WorkspacePlatforms : NullPlatformList;
			foreach (UnrealTargetPlatform? Platform in PlatformsToProcess)
			{
				StringBuilder WorkspaceDataContent = new();
				WorkspaceDataContent.Append("<?xml version=\"1.0\" encoding=\"UTF-8\"?>" + ProjectFileGenerator.NewLine);
				WorkspaceDataContent.Append("<Workspace" + ProjectFileGenerator.NewLine);
				WorkspaceDataContent.Append("   version = \"1.0\">" + ProjectFileGenerator.NewLine);

				List<XcodeProjectXcconfig.XcodeProjectFile> BuildableProjects = new();

				System.Action<List<PrimaryProjectFolder> /* Folders */, string /* Ident */ >? AddProjectsFunction = null;
				AddProjectsFunction = (FolderList, Ident) =>
					{
						foreach (PrimaryProjectFolder CurFolder in FolderList)
						{
							WorkspaceDataContent.Append(Ident + "   <Group" + ProjectFileGenerator.NewLine);
							WorkspaceDataContent.Append(Ident + "      location = \"container:\"      name = \"" + CurFolder.FolderName + "\">" + ProjectFileGenerator.NewLine);

							// add a reference to the file used for launching content only game projects (don't need the file if we are making a project for a single speciic project)
							if (bHasModernProjects && OnlyGameProject == null && CurFolder.FolderName == "Engine")
							{
								WorkspaceDataContent.Append("     <FileRef" + ProjectFileGenerator.NewLine);
								WorkspaceDataContent.Append("       location = \"group:" + ContentOnlySettingsFile.MakeRelativeTo(ProjectFileGenerator.PrimaryProjectPath) + "\">" + ProjectFileGenerator.NewLine);
								WorkspaceDataContent.Append("     </FileRef>" + ProjectFileGenerator.NewLine);
							}

							AddProjectsFunction!(CurFolder.SubFolders, Ident + "   ");

							// Filter out anything that isn't an XC project, and that shouldn't be in the workspace
							IEnumerable<XcodeProjectXcconfig.XcodeProjectFile> SupportedProjects =
									CurFolder.ChildProjects.Where(P => P.GetType() == typeof(XcodeProjectXcconfig.XcodeProjectFile))
										.Select(P => (XcodeProjectXcconfig.XcodeProjectFile)P)
										.Where(P => XcodeProjectXcconfig.XcodeUtils.ShouldIncludeProjectInWorkspace(P))
										// @todo - still need to handle legacy project getting split up?
										.Where(P => P.RootProjects.Count == 0 || P.RootProjects.ContainsValue(Platform))
										.OrderBy(P => P.ProjectFilePath.GetFileName());

							foreach (XcodeProjectXcconfig.XcodeProjectFile XcodeProject in SupportedProjects)
							{
								// if we are only generating a single target project, skip any others now
								if (!String.IsNullOrEmpty(SingleTargetName) && XcodeProject.ProjectFilePath.GetFileNameWithoutAnyExtensions() != SingleTargetName)
								{
									continue;
								}

								// we have to re-check for each project - if it's a legacy project, even if we wanted it split, it won't be, so always point to 
								// the shared legacy project
								FileReference PathToProject = XcodeProject.ProjectFilePath;
								if (!XcodeProject.bHasLegacyProject && PerPlatformMode == XcodePerPlatformMode.OneWorkspacePerPlatform)
								{
									PathToProject = XcodeProject.ProjectFilePathForPlatform(Platform);
								}
								WorkspaceDataContent.Append(Ident + "      <FileRef" + ProjectFileGenerator.NewLine);
								WorkspaceDataContent.Append(Ident + "         location = \"group:" + PathToProject.MakeRelativeTo(ProjectFileGenerator.PrimaryProjectPath) + "\">" + ProjectFileGenerator.NewLine);
								WorkspaceDataContent.Append(Ident + "      </FileRef>" + ProjectFileGenerator.NewLine);
							}

							BuildableProjects.AddRange(SupportedProjects);

							WorkspaceDataContent.Append(Ident + "   </Group>" + ProjectFileGenerator.NewLine);
						}
					};
				AddProjectsFunction(RootFolder.SubFolders, "");

				WorkspaceDataContent.Append("</Workspace>" + ProjectFileGenerator.NewLine);

				// Also, update project's schemes index so that the schemes are in a sensible order
				// (Game, Editor, Client, Server, Programs)
				int SchemeIndex = 0;
				BuildableProjects.Sort((ProjA, ProjB) =>
				{

					ProjectTarget TargetA = ProjA.ProjectTargets.OfType<ProjectTarget>().OrderBy(T => T.TargetRules!.Type).First();
					ProjectTarget TargetB = ProjB.ProjectTargets.OfType<ProjectTarget>().OrderBy(T => T.TargetRules!.Type).First();

					TargetType TypeA = TargetA.TargetRules!.Type;
					TargetType TypeB = TargetB.TargetRules!.Type;

					if (TypeA != TypeB)
					{
						return TypeA.CompareTo(TypeB);
					}

					return String.Compare(TargetA.Name, TargetB.Name, StringComparison.Ordinal);
				});

				foreach (XcodeProjectXcconfig.XcodeProjectFile XcodeProject in BuildableProjects)
				{
					FileReference SchemeManagementFile = XcodeProject.ProjectFilePathForPlatform(Platform) + "/xcuserdata/" + Environment.UserName + ".xcuserdatad/xcschemes/xcschememanagement.plist";
					if (FileReference.Exists(SchemeManagementFile))
					{
						string SchemeManagementContent = FileReference.ReadAllText(SchemeManagementFile);
						SchemeManagementContent = SchemeManagementContent.Replace(
							"<key>orderHint</key>\n\t\t\t<integer>1</integer>",
							"<key>orderHint</key>\n\t\t\t<integer>" + SchemeIndex.ToString() + "</integer>",
							StringComparison.Ordinal);
						FileReference.WriteAllText(SchemeManagementFile, SchemeManagementContent);
						SchemeIndex++;
					}
				}

				string ProjectName = PrimaryProjectNameForPlatform(Platform);
				string WorkspaceDataFilePath = PrimaryProjectPath + "/" + ProjectName + ".xcworkspace/contents.xcworkspacedata";
				Logger.LogInformation("Writing xcode workspace {WorkspaceDataPath}", Path.GetDirectoryName(WorkspaceDataFilePath));
				bSuccess = WriteFileIfChanged(WorkspaceDataFilePath, WorkspaceDataContent.ToString(), Logger, new UTF8Encoding());
				if (bSuccess)
				{
					string WorkspaceSettingsFilePath = PrimaryProjectPath + "/" + ProjectName + ".xcworkspace/xcuserdata/" + Environment.UserName + ".xcuserdatad/WorkspaceSettings.xcsettings";
					bSuccess = WriteWorkspaceSettingsFile(WorkspaceSettingsFilePath, Logger);
					string WorkspaceSharedSettingsFilePath = PrimaryProjectPath + "/" + ProjectName + ".xcworkspace/xcshareddata/WorkspaceSettings.xcsettings";
					bSuccess = WriteWorkspaceSharedSettingsFile(WorkspaceSharedSettingsFilePath, Logger);

					// cache the location of the workspace, for users of this to know where the final workspace is
					XCWorkspace = new FileReference(WorkspaceDataFilePath).Directory;
				}
			}

			// delete outdated workspace files, reduce confusion (only for real workspaces, not stub ones)
			if (!bGenerateRunOnlyProject)
			{
				if (bHasModernProjects)
				{
					DirectoryReference OutdatedWorkspaceDirectory = new DirectoryReference(PrimaryProjectPath + "/" + PrimaryProjectNameForPlatform(null) + ".xcworkspace");
					if (DirectoryReference.Exists(OutdatedWorkspaceDirectory))
					{
						DirectoryReference.Delete(OutdatedWorkspaceDirectory, true);
					}
				}
				else
				{
					foreach (UnrealTargetPlatform? Platform in WorkspacePlatforms)
					{
						DirectoryReference OutdatedWorkspaceDirectory = new DirectoryReference(PrimaryProjectPath + "/" + PrimaryProjectNameForPlatform(Platform) + ".xcworkspace");
						if (DirectoryReference.Exists(OutdatedWorkspaceDirectory))
						{
							DirectoryReference.Delete(OutdatedWorkspaceDirectory, true);
						}
					}
				}
			}

			return bSuccess;
		}

		protected override bool WritePrimaryProjectFile(ProjectFile? UBTProject, PlatformProjectGeneratorCollection PlatformProjectGenerators, ILogger Logger)
		{
			return WriteXcodeWorkspace(Logger);
		}

		/// <inheritdoc/>
		protected override void ConfigureProjectFileGeneration(string[] Arguments, ref bool IncludeAllPlatforms, ILogger Logger)
		{
			// Call parent implementation first
			base.ConfigureProjectFileGeneration(Arguments, ref IncludeAllPlatforms, Logger);

			// reset some statics in case it runs twice
			XcodeProjectFileGenerator.XcodePlatforms.Clear();
			XcodeProjectFileGenerator.WorkspacePlatforms.Clear();
			XcodeProjectFileGenerator.RunTargetPlatforms.Clear();
			XcodeProjectFileGenerator.TargetFrameworks.Clear();
			XcodeProjectFileGenerator.TargetBundles.Clear();
			XcodeProjectFileGenerator.TargetRawDylibs.Clear();
			XcodeProjectFileGenerator.bGenerateRunOnlyProject = false;

			if (ProjectPlatforms.Count > 0)
			{
				XcodePlatforms.AddRange(ProjectPlatforms);
			}
			else
			{
				// add platforms that have synced platform support
				foreach (UnrealTargetPlatform ApplePlatform in UEBuildPlatform.GetPlatformsInGroup(UnrealPlatformGroup.Apple))
				{
					if (InstalledPlatformInfo.IsValidPlatform(ApplePlatform, EProjectType.Code) && UEBuildPlatform.TryGetBuildPlatform(ApplePlatform, out _))
					{
						XcodePlatforms.Add(ApplePlatform);
					}
				}
			}

			if (PerPlatformMode == XcodePerPlatformMode.OneWorkspacePerPlatform)
			{
				WorkspacePlatforms = XcodePlatforms.Select<UnrealTargetPlatform, UnrealTargetPlatform?>(x => x).ToList();
			}
			else
			{
				WorkspacePlatforms = NullPlatformList;
			}

			foreach (string CurArgument in Arguments)
			{
				if (CurArgument.Contains("-iOSDeployOnly", StringComparison.InvariantCultureIgnoreCase) ||
					CurArgument.Contains("-tvOSDeployOnly", StringComparison.InvariantCultureIgnoreCase) ||
					CurArgument.Contains("-DeployOnly", StringComparison.InvariantCultureIgnoreCase))
				{
					bGenerateRunOnlyProject = true;
					break;
				}
			}

			if (bGenerateRunOnlyProject)
			{
				bIncludeConfigFiles = false;
				bIncludeDocumentation = false;
				bIncludeShaderSource = false;
				bIncludeEngineSource = true;
				bGeneratingTemporaryProjects = true;

				// generate just the engine project
				if (OnlyGameProject != null)
				{
					bGeneratingGameProjectFiles = true;
				}
			}
		}

		protected override void AddAdditionalNativeTargetInformation(PlatformProjectGeneratorCollection PlatformProjectGenerators, List<(ProjectFile, ProjectTarget)> Targets, ILogger Logger)
		{
			using ITimelineEvent Timer = Timeline.ScopeEvent("Adding additional Xcode project info", Logger);
			DateTime MainStart = DateTime.UtcNow;

			ParallelOptions Options = new ParallelOptions
			{
				//MaxDegreeOfParallelism=1,
			};
			Parallel.ForEach(Targets, Options, TargetPair =>
			{
				// don't bother if we aren't interested in this target
				if (SingleTargetName != null && !TargetPair.Item2.Name.Equals(SingleTargetName, StringComparison.OrdinalIgnoreCase))
				{
					return;
				}

				ProjectFile TargetProjectFile = TargetPair.Item1;
				if (TargetProjectFile.IsContentOnlyProject)
				{
					return;
				}

				// don't do this for legacy projects, for speed
				((XcodeProjectXcconfig.XcodeProjectFile)TargetProjectFile).ConditionalCreateLegacyProject();
				if (((XcodeProjectXcconfig.XcodeProjectFile)TargetProjectFile).bHasLegacyProject)
				{
					return;
				}

				ProjectTarget CurTarget = TargetPair.Item2;

				HashSet<UnrealTargetPlatform> PlatformsToGenerate = [UnrealTargetPlatform.Mac, UnrealTargetPlatform.IOS];
				PlatformsToGenerate.IntersectWith(CurTarget.SupportedPlatforms);

				foreach (UnrealTargetPlatform Platform in PlatformsToGenerate)
				{
					UnrealArch Arch = UnrealArch.Arm64;
					TargetDescriptor TargetDesc = new TargetDescriptor(CurTarget.UnrealProjectFilePath, CurTarget.Name, Platform, UnrealTargetConfiguration.Development,
						new UnrealArchitectures(Arch), null);
					TargetDesc.IntermediateEnvironment = UnrealIntermediateEnvironment.GenerateProjectFiles;
					DateTime Start = DateTime.UtcNow;

					try
					{
						// Create the target
						UEBuildTarget Target = UEBuildTarget.Create(TargetDesc, true, false, bUsePrecompiled, Logger);

						List<UEBuildFramework> Frameworks = new();
						List<UEBuildBundleResource> Bundles = new();
						List<ModuleRules.RuntimeDependency> Dylibs = new();
						// Generate a compile environment for each module in the binary, and cache the one per Target, since that is the slowest one
						CppCompileEnvironment GlobalCompileEnvironment = Target.CreateCompileEnvironmentForProjectFiles(Logger);
						string cacheKey = MakeGlobalCompileEnvironmentCacheKey(CurTarget.UnrealProjectFilePath, CurTarget.Name, Platform, UnrealTargetConfiguration.Development, new UnrealArchitectures(Arch));
						GlobalCompileEnvironmentCache.TryAdd(cacheKey, GlobalCompileEnvironment);

						foreach (UEBuildBinary Binary in Target.Binaries)
						{
							CppCompileEnvironment BinaryCompileEnvironment = Binary.CreateBinaryCompileEnvironment(GlobalCompileEnvironment);
							foreach (UEBuildModuleCPP Module in Binary.Modules.OfType<UEBuildModuleCPP>())
							{
								CppCompileEnvironment CompileEnvironment = Module.CreateModuleCompileEnvironment(Target.Rules, BinaryCompileEnvironment, Logger);
								Frameworks.AddRange(CompileEnvironment.AdditionalFrameworks);
							}
							// walk over CPP and External modules looking for dylibs that need to be copied into the .app directly (not in a framework)
							foreach (UEBuildModule Module in Binary.Modules)
							{
								Dylibs.AddRange(Module.Rules.RuntimeDependencies.Inner.Where(x => x.Path.StartsWith("$(BinaryOutputDir)", StringComparison.Ordinal) && Path.GetExtension(x.SourcePath) == ".dylib"));
								Bundles.AddRange(Module.Rules.AdditionalBundleResources.Select(x => new UEBuildBundleResource(x)));
							}
						}

						// track frameworks if we found any
						if (Frameworks.Count > 0)
						{
							lock (TargetFrameworks)
							{
								TargetFrameworks.Add(Tuple.Create(TargetProjectFile, Platform), Frameworks.Distinct());
							}
						}
						if (Bundles.Count > 0)
						{
							lock (TargetBundles)
							{
								TargetBundles.Add(Tuple.Create(TargetProjectFile, Platform), Bundles.Distinct());
							}
						}
						if (Dylibs.Count > 0)
						{
							lock (TargetRawDylibs)
							{
								TargetRawDylibs.Add(Tuple.Create(TargetProjectFile, Platform), Dylibs);
							}
						}
					}
					catch (Exception Ex)
					{
						Logger.LogDebug("Failed to build target {Target} for {Platform}. Skipping it for GettingNativeInfo. Exception:\n{Exception}", TargetProjectFile.ProjectFilePath.GetFileNameWithoutAnyExtensions(), Platform, Ex.Message);
					}

					Logger.LogDebug("GettingNativeInfo [{Project} / {Platform}] {TimeMs}ms", TargetProjectFile.ProjectFilePath.GetFileNameWithoutAnyExtensions(), Platform, (DateTime.UtcNow - Start).TotalMilliseconds);
				}
			});
			Timer.Finish();
		}
	}
}
