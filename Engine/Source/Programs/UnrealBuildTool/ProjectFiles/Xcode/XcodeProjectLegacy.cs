// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;
using UnrealBuildTool.XcodeProjectXcconfig;

namespace UnrealBuildTool.XcodeProjectLegacy
{
	/// <summary>
	/// Represents a group of files shown in Xcode's project navigator as a folder
	/// </summary>
	class XcodeFileGroup
	{
		public XcodeFileGroup(string InName, string InPath, bool InIsReference)
		{
			GroupName = InName;
			GroupPath = InPath;
			GroupGuid = XcodeProjectFileGenerator.MakeXcodeGuid();
			bIsReference = InIsReference;
		}

		public string GroupGuid;
		public string GroupName;
		public string GroupPath;
		public Dictionary<string, XcodeFileGroup> Children = new Dictionary<string, XcodeFileGroup>();
		public List<XcodeSourceFile> Files = new List<XcodeSourceFile>();
		public bool bIsReference;
	}

	class XcodeBuildConfig
	{
		public XcodeBuildConfig(string InDisplayName, string InBuildTarget, FileReference? InMacExecutablePath, FileReference? InIOSExecutablePath, FileReference? InTVOSExecutablePath,
			ProjectTarget? InProjectTarget, UnrealTargetConfiguration InBuildConfig)
		{
			DisplayName = InDisplayName;
			MacExecutablePath = InMacExecutablePath;
			IOSExecutablePath = InIOSExecutablePath;
			TVOSExecutablePath = InTVOSExecutablePath;
			BuildTarget = InBuildTarget;
			ProjectTarget = InProjectTarget;
			BuildConfig = InBuildConfig;
		}

		public string DisplayName;
		public FileReference? MacExecutablePath;
		public FileReference? IOSExecutablePath;
		public FileReference? TVOSExecutablePath;
		public string BuildTarget;
		public ProjectTarget? ProjectTarget;
		public UnrealTargetConfiguration BuildConfig;
	};

	class XcodeExtensionInfo
	{
		public XcodeExtensionInfo(string InName)
		{
			Name = InName;
			TargetDependencyGuid = XcodeProjectFileGenerator.MakeXcodeGuid();
			TargetProxyGuid = XcodeProjectFileGenerator.MakeXcodeGuid();
			TargetGuid = XcodeProjectFileGenerator.MakeXcodeGuid();
			ProductGuid = XcodeProjectFileGenerator.MakeXcodeGuid();
			ResourceBuildPhaseGuid = XcodeProjectFileGenerator.MakeXcodeGuid();
			ConfigListGuid = XcodeProjectFileGenerator.MakeXcodeGuid();
			AllConfigs = new Dictionary<string, XcodeBuildConfig>();
		}

		public string Name;
		public string TargetDependencyGuid;
		public string TargetProxyGuid;
		public string TargetGuid;
		public string ProductGuid;
		public string ResourceBuildPhaseGuid;
		public string ConfigListGuid;
		public Dictionary<string, XcodeBuildConfig> AllConfigs;

		public string? ConfigurationContents;
	}

	class XcodeProjectFile : ProjectFile
	{
		Dictionary<string, XcodeFileGroup> Groups = new Dictionary<string, XcodeFileGroup>();

		/// <summary>
		/// Constructs a new project file object
		/// </summary>
		/// <param name="InitFilePath">The path to the project file on disk</param>
		/// <param name="BaseDir">The base directory for files within this project</param>
		/// <param name="IsForDistribution">True for distribution builds</param>
		/// <param name="BundleID">Override option for bundle identifier</param>
		/// <param name="InAppName"></param>
		/// <param name="bMakeProjectPerTarget"></param>
		public XcodeProjectFile(FileReference InitFilePath, DirectoryReference BaseDir, bool IsForDistribution, string BundleID, string InAppName, bool bMakeProjectPerTarget)
			: base(InitFilePath, BaseDir)
		{
			bForDistribution = IsForDistribution;
			BundleIdentifier = BundleID;
			AppName = InAppName;
			this.bMakeProjectPerTarget = bMakeProjectPerTarget;
		}

		public override string ToString()
		{
			return ProjectFilePath.GetFileNameWithoutExtension();
		}

		/// <summary>
		/// Temporary for developing this feature
		/// </summary>
		readonly bool bMakeProjectPerTarget;

		/// <summary>
		///  Used to mark the project for distribution (some platforms require this)
		/// </summary>
		readonly bool bForDistribution = false;

		/// <summary>
		/// Override for bundle identifier
		/// </summary>
		readonly string BundleIdentifier = "";

		/// <summary>
		/// Override AppName
		/// </summary>
		readonly string AppName = "";

		/// <summary>
		/// Architectures supported for iOS
		/// </summary>
		readonly UnrealArch[] SupportedIOSArchitectures = { UnrealArch.Arm64 };

		/// <summary>
		/// Gets Xcode file category based on its extension
		/// </summary>
		private static string GetFileCategory(string Extension)
		{
			// @todo Mac: Handle more categories
			switch (Extension)
			{
				case ".framework":
					return "Frameworks";
				default:
					return "Sources";
			}
		}

		/// <summary>
		/// Gets Xcode file type based on its extension
		/// </summary>
		private static string GetFileType(string Extension)
		{
			// @todo Mac: Handle more file types
			switch (Extension)
			{
				case ".c":
				case ".m":
					return "sourcecode.c.objc";
				case ".cc":
				case ".cpp":
				case ".cxx":
				case ".mm":
					return "sourcecode.cpp.objcpp";
				case ".h":
				case ".inl":
				case ".pch":
					return "sourcecode.c.h";
				case ".framework":
					return "wrapper.framework";
				case ".plist":
					return "text.plist.xml";
				case ".png":
					return "image.png";
				case ".icns":
					return "image.icns";
				default:
					return "file.text";
			}
		}

		/// <summary>
		/// Returns true if Extension is a known extension for files containing source code
		/// </summary>
		private static bool IsSourceCode(string Extension)
		{
			return Extension == ".c" || Extension == ".cc" || Extension == ".cxx" || Extension == ".cpp" || Extension == ".m" || Extension == ".mm";
		}

		private static bool ShouldIncludeFileInBuildPhaseSection(XcodeSourceFile SourceFile)
		{
			string FileExtension = SourceFile.Reference.GetExtension();

			if (IsSourceCode(FileExtension))
			{
				foreach (string PlatformName in UnrealTargetPlatform.GetValidPlatformNames())
				{
					string AltName = PlatformName == "Win64" ? "windows" : PlatformName;
					if ((SourceFile.Reference.FullName.Contains($"/{PlatformName}/", StringComparison.OrdinalIgnoreCase) || SourceFile.Reference.FullName.Contains($"/{AltName}/", StringComparison.OrdinalIgnoreCase))
						&& PlatformName != "Mac" && PlatformName != "IOS" && PlatformName != "TVOS")
					{
						// Build phase is used for indexing only and indexing currently works only with files that can be compiled for Mac, so skip files for other platforms
						return false;
					}
				}

				return true;
			}

			return false;
		}

		/// <summary>
		/// Returns a project navigator group to which the file should belong based on its path.
		/// Creates a group tree if it doesn't exist yet.
		/// </summary>
		private static XcodeFileGroup? FindGroupByAbsolutePath(ref Dictionary<string, XcodeFileGroup> Groups, string AbsolutePath)
		{
			string[] Parts = AbsolutePath.Split(Path.DirectorySeparatorChar);
			string CurrentPath = "/";
			Dictionary<string, XcodeFileGroup> CurrentSubGroups = Groups;

			for (int Index = 1; Index < Parts.Length; ++Index)
			{
				string Part = Parts[Index];

				if (CurrentPath.Length > 1)
				{
					CurrentPath += Path.DirectorySeparatorChar;
				}

				CurrentPath += Part;

				XcodeFileGroup CurrentGroup;
				if (!CurrentSubGroups.ContainsKey(CurrentPath))
				{
					CurrentGroup = new XcodeFileGroup(Path.GetFileName(CurrentPath), CurrentPath, CurrentPath.EndsWith(".xcassets", StringComparison.Ordinal));
					CurrentSubGroups.Add(CurrentPath, CurrentGroup);
				}
				else
				{
					CurrentGroup = CurrentSubGroups[CurrentPath];
				}

				if (CurrentPath == AbsolutePath)
				{
					return CurrentGroup;
				}

				CurrentSubGroups = CurrentGroup.Children;
			}

			return null;
		}

		/// <summary>
		/// Convert all paths to Apple/Unix format (with forward slashes)
		/// </summary>
		/// <param name="InPath">The path to convert</param>
		/// <returns>The normalized path</returns>
		private static string ConvertPath(string InPath)
		{
			return InPath.Replace("\\", "/", StringComparison.Ordinal);
		}

		/// <summary>
		/// Allocates a generator-specific source file object
		/// </summary>
		/// <param name="InitFilePath">Path to the source file on disk</param>
		/// <param name="InitProjectSubFolder">Optional sub-folder to put the file in.  If empty, this will be determined automatically from the file's path relative to the project file</param>
		/// <returns>The newly allocated source file object</returns>
		public override SourceFile? AllocSourceFile(FileReference InitFilePath, DirectoryReference? InitProjectSubFolder)
		{
			if (InitFilePath.GetFileName().StartsWith('.'))
			{
				return null;
			}
			return new XcodeSourceFile(InitFilePath, InitProjectSubFolder);
		}

		/// <summary>
		/// Generates bodies of all sections that contain a list of source files plus a dictionary of project navigator groups.
		/// </summary>
		private void GenerateSectionsWithSourceFiles(StringBuilder PBXBuildFileSection, StringBuilder PBXFileReferenceSection, StringBuilder PBXSourcesBuildPhaseSection, string TargetAppGuid, string TargetName, bool bIsAppBundle)
		{
			SourceFiles.Sort((x, y) => { return x.Reference.CompareTo(y.Reference); });

			foreach (XcodeSourceFile SourceFile in SourceFiles.OfType<XcodeSourceFile>())
			{
				string FileName = SourceFile.Reference.GetFileName();
				string FileExtension = Path.GetExtension(FileName);
				string FilePath = SourceFile.Reference.MakeRelativeTo(ProjectFilePath.Directory);
				string FilePathMac = Utils.CleanDirectorySeparators(FilePath, '/');

				if (IsGeneratedProject)
				{
					PBXBuildFileSection.Append(String.Format("\t\t{0} /* {1} in {2} */ = {{isa = PBXBuildFile; fileRef = {3} /* {1} */; }};" + ProjectFileGenerator.NewLine,
						SourceFile.FileGuid,
						FileName,
						GetFileCategory(FileExtension),
						SourceFile.FileRefGuid));
				}

				PBXFileReferenceSection.Append(String.Format("\t\t{0} /* {1} */ = {{isa = PBXFileReference; explicitFileType = {2}; name = \"{1}\"; path = \"{3}\"; sourceTree = SOURCE_ROOT; }};" + ProjectFileGenerator.NewLine,
					SourceFile.FileRefGuid,
					FileName,
					GetFileType(FileExtension),
					FilePathMac));

				if (ShouldIncludeFileInBuildPhaseSection(SourceFile))
				{
					PBXSourcesBuildPhaseSection.Append("\t\t\t\t" + SourceFile.FileGuid + " /* " + FileName + " in Sources */," + ProjectFileGenerator.NewLine);
				}

				XcodeFileGroup? Group = FindGroupByAbsolutePath(ref Groups, SourceFile.Reference.Directory.FullName);
				Group?.Files.Add(SourceFile);
			}

			PBXFileReferenceSection.Append(String.Format("\t\t{0} /* {1} */ = {{isa = PBXFileReference; explicitFileType = {2}; path = {1}; sourceTree = BUILT_PRODUCTS_DIR; }};" + ProjectFileGenerator.NewLine, TargetAppGuid, TargetName, bIsAppBundle ? "wrapper.application" : "\"compiled.mach-o.executable\""));
		}

		private void GenerateSectionsWithExtensions(StringBuilder PBXBuildFileSection, StringBuilder PBXFileReferenceSection, StringBuilder PBXCopyFilesBuildPhaseSection, StringBuilder PBXResourcesBuildPhaseSection,
													List<XcodeExtensionInfo> AllExtensions, FileReference? UProjectPath, List<XcodeBuildConfig> BuildConfigs)
		{
			if (UProjectPath != null)
			{
				string ProjectExtensionsDir = Path.Combine(Path.GetDirectoryName(UProjectPath.FullName)!, "Build/IOS/Extensions");
				//string ProjectIntermediateDir = Path.Combine(Path.GetDirectoryName(UProjectPath.FullName), "Intermediate/IOS/Extensions");

				if (Directory.Exists(ProjectExtensionsDir))
				{
					foreach (DirectoryInfo DI in new System.IO.DirectoryInfo(ProjectExtensionsDir).EnumerateDirectories())
					{
						Console.WriteLine("  Project {0} has Extension {1}!", UProjectPath, DI);

						// assume each Extension in here will create a resulting Extension.appex
						string Extension = DI.Name + ".appex";

						string ExtensionGuid = XcodeProjectFileGenerator.MakeXcodeGuid();

						// make an extension info object
						XcodeExtensionInfo ExtensionInfo = new XcodeExtensionInfo(DI.Name);
						AllExtensions.Add(ExtensionInfo);

						PBXBuildFileSection.Append(String.Format("\t\t{0} /* {1} in Embed App Extensions */ = {{isa = PBXBuildFile; fileRef = {2} /* {1} */; settings = {{ATTRIBUTES = (RemoveHeadersOnCopy, ); }}; }};" + ProjectFileGenerator.NewLine,
							ExtensionGuid,
							Extension,
							ExtensionInfo.ProductGuid));

						PBXFileReferenceSection.Append(String.Format("\t\t{0} /* {1} */ = {{isa = PBXFileReference; explicitFileType = wrapper.app-extension; path = \"{1}\"; sourceTree = BUILT_PRODUCTS_DIR; }};" + ProjectFileGenerator.NewLine,
							ExtensionInfo.ProductGuid,
							Extension));

						PBXCopyFilesBuildPhaseSection.Append(String.Format("\t\t\t\t{0} /* {1} in Embed App Extensions */," + ProjectFileGenerator.NewLine,
							ExtensionGuid,
							Extension));

						PBXResourcesBuildPhaseSection.Append("/* Begin PBXResourcesBuildPhase section */" + ProjectFileGenerator.NewLine);
						PBXResourcesBuildPhaseSection.Append("\t\t" + ExtensionInfo.ResourceBuildPhaseGuid + " /* Resources */ = {" + ProjectFileGenerator.NewLine);
						PBXResourcesBuildPhaseSection.Append("\t\t\tisa = PBXResourcesBuildPhase;" + ProjectFileGenerator.NewLine);
						PBXResourcesBuildPhaseSection.Append("\t\t\tbuildActionMask = 2147483647;" + ProjectFileGenerator.NewLine);
						PBXResourcesBuildPhaseSection.Append("\t\t\tfiles = (" + ProjectFileGenerator.NewLine);
						if (Directory.Exists(Path.Combine(DI.FullName, "Resources")))
						{
							DirectoryInfo ResourceDir = new System.IO.DirectoryInfo(Path.Combine(DI.FullName, "Resources"));
							foreach (FileSystemInfo FSI in ResourceDir.EnumerateFileSystemInfos())
							{
								if (FSI.Name.StartsWith('.'))
								{
									continue;
								}
								// for each resource, put it into the File/FileRef section, and into the CopyResuorceBuildPhase
								string ResourceGuid = XcodeProjectFileGenerator.MakeXcodeGuid();
								string ResourceRefGuid = XcodeProjectFileGenerator.MakeXcodeGuid();
								PBXBuildFileSection.Append(String.Format("\t\t{0} /* {1} in Embed App Extensions */ = {{isa = PBXBuildFile; fileRef = {2} /* {1} */; }};" + ProjectFileGenerator.NewLine,
									ResourceGuid,
									FSI.Name,
									ResourceRefGuid));

								// lastKnownFileType = wrapper.app-extension; 
								PBXFileReferenceSection.Append(String.Format("\t\t{0} /* {1} */ = {{isa = PBXFileReference; lastKnownFileType = folder.assetcatalog; path = \"{2}\"; sourceTree = \"<absolute>\"; }};" + ProjectFileGenerator.NewLine,
									ResourceRefGuid,
									FSI.Name,
									// @todo: make this relative path!! 
									FSI.FullName));

								PBXResourcesBuildPhaseSection.Append("\t\t\t\t" + ResourceGuid + " /* " + FSI.Name + " in " + ResourceDir.Name + " */," + ProjectFileGenerator.NewLine);
							}
						}
						PBXResourcesBuildPhaseSection.Append("\t\t\t);" + ProjectFileGenerator.NewLine);
						PBXResourcesBuildPhaseSection.Append("\t\t\trunOnlyForDeploymentPostprocessing = 0;" + ProjectFileGenerator.NewLine);
						PBXResourcesBuildPhaseSection.Append("\t\t};" + ProjectFileGenerator.NewLine);
						PBXResourcesBuildPhaseSection.Append("/* End PBXResourcesBuildPhase section */" + ProjectFileGenerator.NewLine + ProjectFileGenerator.NewLine);

						StringBuilder ConfigSection = new StringBuilder();
						// copy over the configs from the general project to the extension
						foreach (XcodeBuildConfig Configuration in BuildConfigs)
						{
							string ConfigGuid = XcodeProjectFileGenerator.MakeXcodeGuid();
							string ConfigName = Configuration.DisplayName;

							ConfigSection.Append("\t\t" + ConfigGuid + " /* " + ConfigName + " */ = {" + ProjectFileGenerator.NewLine);
							ConfigSection.Append("\t\t\tisa = XCBuildConfiguration;" + ProjectFileGenerator.NewLine);
							ConfigSection.Append("\t\t\tbuildSettings = {" + ProjectFileGenerator.NewLine);
							ConfigSection.Append("\t\t\t\tASSETCATALOG_COMPILER_APPICON_NAME = \"iMessage App Icon\";" + ProjectFileGenerator.NewLine);
							ConfigSection.Append("\t\t\t\tINFOPLIST_FILE = \"" + Path.Combine(DI.FullName, "Info.plist") + "\";" + ProjectFileGenerator.NewLine);
							ConfigSection.Append("\t\t\t\tSKIP_INSTALL = YES;" + ProjectFileGenerator.NewLine);
							ConfigSection.Append("\t\t\t\tPRODUCT_NAME = \"$(TARGET_NAME)\";" + ProjectFileGenerator.NewLine);

							bool bSupportIOS = true;
							bool bSupportTVOS = true;
							if (bSupportIOS && InstalledPlatformInfo.IsValidPlatform(UnrealTargetPlatform.IOS, EProjectType.Code))
							{
								IOSPlatform IOSPlatform = ((IOSPlatform)UEBuildPlatform.GetBuildPlatform(UnrealTargetPlatform.IOS));
								IOSProjectSettings ProjectSettings = IOSPlatform.ReadProjectSettings(UProjectPath);
								ConfigSection.Append("\t\t\t\t\"PRODUCT_BUNDLE_IDENTIFIER[sdk=iphoneos*]\" = " + ProjectSettings.BundleIdentifier + "." + ExtensionInfo.Name + ";" + ProjectFileGenerator.NewLine);
							}

							if (bSupportTVOS && InstalledPlatformInfo.IsValidPlatform(UnrealTargetPlatform.TVOS, EProjectType.Code))
							{
								TVOSPlatform TVOSPlatform = ((TVOSPlatform)UEBuildPlatform.GetBuildPlatform(UnrealTargetPlatform.TVOS));
								TVOSProjectSettings ProjectSettings = TVOSPlatform.ReadProjectSettings(UProjectPath);
								ConfigSection.Append("\t\t\t\t\"PRODUCT_BUNDLE_IDENTIFIER[sdk=appletvos*]\" = " + ProjectSettings.BundleIdentifier + "." + ExtensionInfo.Name + ";" + ProjectFileGenerator.NewLine);
							}

							string? IOSRuntimeVersion, TVOSRuntimeVersion;
							AppendPlatformConfiguration(ConfigSection, Configuration, ExtensionInfo.Name, UProjectPath, false, bSupportIOS, bSupportTVOS, out IOSRuntimeVersion, out TVOSRuntimeVersion);

							ConfigSection.Append("\t\t\t};" + ProjectFileGenerator.NewLine);
							ConfigSection.Append("\t\t\tname = \"" + ConfigName + "\";" + ProjectFileGenerator.NewLine);
							ConfigSection.Append("\t\t};" + ProjectFileGenerator.NewLine);

							XcodeBuildConfig Config = new XcodeBuildConfig(ConfigName, ExtensionInfo.Name, null, null, null, null, Configuration.BuildConfig);
							ExtensionInfo.AllConfigs.Add(ConfigGuid, Config);
						}

						ExtensionInfo.ConfigurationContents = ConfigSection.ToString();
					}
				}
			}
		}

		private static void AppendGroup(XcodeFileGroup Group, StringBuilder Content)
		{
			if (!Group.bIsReference)
			{
				Content.Append(String.Format("\t\t{0} = {{{1}", Group.GroupGuid, ProjectFileGenerator.NewLine));
				Content.Append("\t\t\tisa = PBXGroup;" + ProjectFileGenerator.NewLine);
				Content.Append("\t\t\tchildren = (" + ProjectFileGenerator.NewLine);

				foreach (XcodeFileGroup ChildGroup in Group.Children.Values)
				{
					Content.Append(String.Format("\t\t\t\t{0} /* {1} */,{2}", ChildGroup.GroupGuid, ChildGroup.GroupName, ProjectFileGenerator.NewLine));
				}

				foreach (XcodeSourceFile File in Group.Files)
				{
					Content.Append(String.Format("\t\t\t\t{0} /* {1} */,{2}", File.FileRefGuid, File.Reference.GetFileName(), ProjectFileGenerator.NewLine));
				}

				Content.Append("\t\t\t);" + ProjectFileGenerator.NewLine);
				Content.Append("\t\t\tname = \"" + Group.GroupName + "\";" + ProjectFileGenerator.NewLine);
				Content.Append("\t\t\tpath = \"" + Group.GroupPath + "\";" + ProjectFileGenerator.NewLine);
				Content.Append("\t\t\tsourceTree = \"<absolute>\";" + ProjectFileGenerator.NewLine);
				Content.Append("\t\t};" + ProjectFileGenerator.NewLine);

				foreach (XcodeFileGroup ChildGroup in Group.Children.Values)
				{
					AppendGroup(ChildGroup, Content);
				}
			}
		}

		private static void AppendBuildFileSection(StringBuilder Content, StringBuilder SectionContent)
		{
			Content.Append("/* Begin PBXBuildFile section */" + ProjectFileGenerator.NewLine);
			Content.Append(SectionContent);
			Content.Append("/* End PBXBuildFile section */" + ProjectFileGenerator.NewLine + ProjectFileGenerator.NewLine);
		}

		private static void AppendFileReferenceSection(StringBuilder Content, StringBuilder SectionContent)
		{
			Content.Append("/* Begin PBXFileReference section */" + ProjectFileGenerator.NewLine);
			Content.Append(SectionContent);
			Content.Append("/* End PBXFileReference section */" + ProjectFileGenerator.NewLine + ProjectFileGenerator.NewLine);
		}

		private static void AppendSourcesBuildPhaseSection(StringBuilder Content, StringBuilder SectionContent, string SourcesBuildPhaseGuid)
		{
			Content.Append("/* Begin PBXSourcesBuildPhase section */" + ProjectFileGenerator.NewLine);
			Content.Append(String.Format("\t\t{0} = {{{1}", SourcesBuildPhaseGuid, ProjectFileGenerator.NewLine));
			Content.Append("\t\t\tisa = PBXSourcesBuildPhase;" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tbuildActionMask = 2147483647;" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tfiles = (" + ProjectFileGenerator.NewLine);
			Content.Append(SectionContent);
			Content.Append("\t\t\t);" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\trunOnlyForDeploymentPostprocessing = 0;" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t};" + ProjectFileGenerator.NewLine);
			Content.Append("/* End PBXSourcesBuildPhase section */" + ProjectFileGenerator.NewLine + ProjectFileGenerator.NewLine);
		}

		private static XcodeFileGroup? FindRootFileGroup(Dictionary<string, XcodeFileGroup> GroupsDict)
		{
			foreach (XcodeFileGroup Group in GroupsDict.Values)
			{
				if (Group.Children.Count > 1 || Group.Files.Count > 0)
				{
					return Group;
				}
				else
				{
					XcodeFileGroup? Found = FindRootFileGroup(Group.Children);
					if (Found != null)
					{
						return Found;
					}
				}
			}
			return null;
		}

		private static void AppendCopyExtensionsBuildPhaseSection(StringBuilder Content, StringBuilder SectionContent, string CopyFilesBuildPhaseGuid)
		{
			Content.Append("/* Begin PBXCopyFilesBuildPhase section */" + ProjectFileGenerator.NewLine);
			Content.Append(String.Format("\t{0} /* Embed App Extensions */ = {{{1}", CopyFilesBuildPhaseGuid, ProjectFileGenerator.NewLine));
			Content.Append("\t\tisa = PBXCopyFilesBuildPhase;" + ProjectFileGenerator.NewLine);
			Content.Append("\t\tbuildActionMask = 2147483647;" + ProjectFileGenerator.NewLine);
			Content.Append("\t\tdstPath = \"\";" + ProjectFileGenerator.NewLine);
			Content.Append("\t\tdstSubfolderSpec = 13;" + ProjectFileGenerator.NewLine);
			Content.Append("\t\tfiles = (" + ProjectFileGenerator.NewLine);
			Content.Append(SectionContent);
			Content.Append("\t\t);" + ProjectFileGenerator.NewLine);
			Content.Append("\t\tname = \"Embed App Extensions\";" + ProjectFileGenerator.NewLine);
			Content.Append("\t\trunOnlyForDeploymentPostprocessing = 0;" + ProjectFileGenerator.NewLine);
			Content.Append("\t};" + ProjectFileGenerator.NewLine);
			Content.Append("/* End PBXCopyFilesBuildPhase section */" + ProjectFileGenerator.NewLine);
		}

		private void AppendGroupSection(StringBuilder Content, string MainGroupGuid, string ProductRefGroupGuid, string TargetAppGuid, string TargetName, List<XcodeExtensionInfo> AllExtensions)
		{
			XcodeFileGroup? RootGroup = FindRootFileGroup(Groups);
			if (RootGroup == null)
			{
				return;
			}

			Content.Append("/* Begin PBXGroup section */" + ProjectFileGenerator.NewLine);

			// Main group
			Content.Append(String.Format("\t\t{0} = {{{1}", MainGroupGuid, ProjectFileGenerator.NewLine));
			Content.Append("\t\t\tisa = PBXGroup;" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tchildren = (" + ProjectFileGenerator.NewLine);

			foreach (XcodeFileGroup Group in RootGroup.Children.Values)
			{
				Content.Append(String.Format("\t\t\t\t{0} /* {1} */,{2}", Group.GroupGuid, Group.GroupName, ProjectFileGenerator.NewLine));
			}

			foreach (XcodeSourceFile File in RootGroup.Files)
			{
				Content.Append(String.Format("\t\t\t\t{0} /* {1} */,{2}", File.FileRefGuid, File.Reference.GetFileName(), ProjectFileGenerator.NewLine));
			}

			Content.Append(String.Format("\t\t\t\t{0} /* Products */,{1}", ProductRefGroupGuid, ProjectFileGenerator.NewLine));
			Content.Append("\t\t\t);" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tsourceTree = \"<group>\";" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t};" + ProjectFileGenerator.NewLine);

			// Sources groups
			foreach (XcodeFileGroup Group in RootGroup.Children.Values)
			{
				AppendGroup(Group, Content);
			}

			// Products group
			Content.Append(String.Format("\t\t{0} /* Products */ = {{{1}", ProductRefGroupGuid, ProjectFileGenerator.NewLine));
			Content.Append("\t\t\tisa = PBXGroup;" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tchildren = (" + ProjectFileGenerator.NewLine);
			Content.Append(String.Format("\t\t\t\t{0} /* {1} */,{2}", TargetAppGuid, TargetName, ProjectFileGenerator.NewLine));
			foreach (XcodeExtensionInfo EI in AllExtensions)
			{
				Content.Append(String.Format("\t\t\t\t{0} /* {1} */,{2}", EI.ProductGuid, EI.Name, ProjectFileGenerator.NewLine));
			}
			Content.Append("\t\t\t);" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tname = Products;" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tsourceTree = \"<group>\";" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t};" + ProjectFileGenerator.NewLine);

			Content.Append("/* End PBXGroup section */" + ProjectFileGenerator.NewLine + ProjectFileGenerator.NewLine);
		}

		private static void AppendLegacyTargetSection(StringBuilder Content, string TargetName, string TargetGuid, string TargetBuildConfigGuid, FileReference? UProjectPath)
		{
			string UEDir = ConvertPath(Path.GetFullPath(Directory.GetCurrentDirectory() + "../../.."));
			string BuildToolPath = UEDir + "/Engine/Build/BatchFiles/Mac/XcodeBuild.sh";

			Content.Append("/* Begin PBXLegacyTarget section */" + ProjectFileGenerator.NewLine);

			Content.Append("\t\t" + TargetGuid + " /* " + TargetName + " */ = {" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tisa = PBXLegacyTarget;" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tbuildArgumentsString = \"$(ACTION) $(UE_BUILD_TARGET_NAME) $(PLATFORM_NAME) $(UE_BUILD_TARGET_CONFIG)"
				+ (UProjectPath == null ? "" : " \\\"" + UProjectPath.FullName + "\\\"")
				+ "\";" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tbuildConfigurationList = " + TargetBuildConfigGuid + " /* Build configuration list for PBXLegacyTarget \"" + TargetName + "\" */;" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tbuildPhases = (" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\t);" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tbuildToolPath = \"" + BuildToolPath + "\";" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tbuildWorkingDirectory = \"" + UEDir + "\";" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tdependencies = (" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\t);" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tname = \"" + TargetName + "\";" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tpassBuildSettingsInEnvironment = 1;" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tproductName = \"" + TargetName + "\";" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t};" + ProjectFileGenerator.NewLine);

			Content.Append("/* End PBXLegacyTarget section */" + ProjectFileGenerator.NewLine + ProjectFileGenerator.NewLine);
		}

		private static void AppendRunTargetSection(StringBuilder Content, string TargetName, string TargetGuid, string TargetBuildConfigGuid, string TargetDependencyGuid,
				string TargetAppGuid, string CopyExtensionsBuildPhaseGuid, string ShellScriptSectionGuid, List<XcodeExtensionInfo> AllExtensions, bool bIsAppBundle)
		{
			List<string> DependencyGuids = new List<string>();
			// depends on the Run target if we want one
			if (!XcodeProjectFileGenerator.bGenerateRunOnlyProject)
			{
				DependencyGuids.Add(TargetDependencyGuid);
			}
			// make sure extensions get built
			foreach (XcodeExtensionInfo EI in AllExtensions)
			{
				DependencyGuids.Add(EI.TargetDependencyGuid);
			}

			Dictionary<string, string> BuildPhases = new Dictionary<string, string>();
			// add optional build phases
			if (!String.IsNullOrEmpty(CopyExtensionsBuildPhaseGuid))
			{
				BuildPhases.Add(CopyExtensionsBuildPhaseGuid, "Embed App Extensions");
			}
			if (!String.IsNullOrEmpty(ShellScriptSectionGuid))
			{
				BuildPhases.Add(ShellScriptSectionGuid, "Shell Script");
			}

			// use generica target section function for an application type
			AppendGenericTargetSection(Content, TargetName, TargetGuid, bIsAppBundle ? "com.apple.product-type.application" : "com.apple.product-type.tool", TargetBuildConfigGuid, TargetAppGuid, DependencyGuids, BuildPhases);
		}

		private static void AppendGenericTargetSection(StringBuilder Content, string TargetName, string TargetGuid, string TargetType, string TargetBuildConfigGuid, string TargetAppGuid, IEnumerable<string>? TargetDependencyGuids, Dictionary<string, string> BuildPhases)
		{
			Content.Append("/* Begin PBXNativeTarget section */" + ProjectFileGenerator.NewLine);

			Content.Append("\t\t" + TargetGuid + " /* " + TargetName + " */ = {" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tisa = PBXNativeTarget;" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tbuildConfigurationList = " + TargetBuildConfigGuid + " /* Build configuration list for PBXNativeTarget \"" + TargetName + "\" */;" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tbuildPhases = (" + ProjectFileGenerator.NewLine);
			if (BuildPhases != null)
			{
				foreach (KeyValuePair<string, string> BuildPhasePair in BuildPhases)
				{
					Content.Append("\t\t\t\t" + BuildPhasePair.Key + " /* " + BuildPhasePair.Value + " */, " + ProjectFileGenerator.NewLine);
				}
			}
			Content.Append("\t\t\t);" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tdependencies = (" + ProjectFileGenerator.NewLine);
			if (TargetDependencyGuids != null)
			{
				foreach (string DependencyGuid in TargetDependencyGuids)
				{
					Content.Append("\t\t\t\t" + DependencyGuid + " /* PBXTargetDependency */," + ProjectFileGenerator.NewLine);
				}
			}
			Content.Append("\t\t\t);" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tname = \"" + TargetName + "\";" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tpassBuildSettingsInEnvironment = 1;" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tproductName = \"" + TargetName + "\";" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tproductReference = \"" + TargetAppGuid + "\";" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tproductType = \"" + TargetType + "\";" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t};" + ProjectFileGenerator.NewLine);

			Content.Append("/* End PBXNativeTarget section */" + ProjectFileGenerator.NewLine + ProjectFileGenerator.NewLine);
		}

		private static void AppendIndexTargetSection(StringBuilder Content, string TargetName, string TargetGuid, string TargetBuildConfigGuid, string SourcesBuildPhaseGuid)
		{
			Content.Append("/* Begin PBXNativeTarget section */" + ProjectFileGenerator.NewLine);

			Content.Append("\t\t" + TargetGuid + " /* " + TargetName + " */ = {" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tisa = PBXNativeTarget;" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tbuildConfigurationList = " + TargetBuildConfigGuid + " /* Build configuration list for PBXNativeTarget \"" + TargetName + "\" */;" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tbuildPhases = (" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\t\t" + SourcesBuildPhaseGuid + " /* Sources */," + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\t);" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tdependencies = (" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\t);" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tname = \"" + TargetName + "\";" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tpassBuildSettingsInEnvironment = 1;" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tproductName = \"" + TargetName + "\";" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tproductType = \"com.apple.product-type.library.static\";" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t};" + ProjectFileGenerator.NewLine);

			Content.Append("/* End PBXNativeTarget section */" + ProjectFileGenerator.NewLine + ProjectFileGenerator.NewLine);
		}

		private static void AppendShellScriptSection(StringBuilder Content, string ShellScriptGuid, FileReference? UProjectPath)
		{
			StringBuilder FrameworkScript = new StringBuilder();

			if (UProjectPath != null)
			{

				// @todo: look also in Project/Build/Frameworks directory!
				ProjectDescriptor Project = ProjectDescriptor.FromFile(UProjectPath);
				List<PluginInfo> AvailablePlugins = Plugins.ReadAvailablePlugins(Unreal.EngineDirectory, DirectoryReference.FromFile(UProjectPath), Project.AdditionalPluginDirectories);

				// look in each plugin for frameworks
				// @todo: Cache this kind of things since every target will re-do this work!
				foreach (PluginInfo PI in AvailablePlugins)
				{
					if (!Plugins.IsPluginEnabledForTarget(PI, Project, UnrealTargetPlatform.IOS, UnrealTargetConfiguration.Development, TargetRules.TargetType.Game))
					{
						continue;
					}

					// for now, we copy and code sign all *.framework.zip, even if the have no code (non-code frameworks are assumed to be *.embeddedframework.zip
					DirectoryReference FrameworkDir = DirectoryReference.Combine(PI.Directory, "Source/Frameworks");
					if (!DirectoryReference.Exists(FrameworkDir))
					{
						FrameworkDir = DirectoryReference.Combine(PI.Directory, "Frameworks");
					}
					if (DirectoryReference.Exists(FrameworkDir))
					{
						// look at each zip
						foreach (FileInfo FI in new System.IO.DirectoryInfo(FrameworkDir.FullName).EnumerateFiles("*.framework.zip"))
						{
							//string Guid = XcodeProjectFileGenerator.MakeXcodeGuid();
							//string RefGuid = XcodeProjectFileGenerator.MakeXcodeGuid();

							// for FI of foo.framework.zip, this will give us foo.framework
							//string Framework = Path.GetFileNameWithoutExtension(FI.FullName);

							// unzip the framework right into the .app
							FrameworkScript.AppendFormat("\\techo Unzipping {0}...\\n", FI.FullName);
							FrameworkScript.AppendFormat("\\tunzip -o -q {0} -d ${{FRAMEWORK_DIR}} -x \\\"__MACOSX/*\\\" \\\"*/.DS_Store\\\"\\n", FI.FullName);
						}
					}
				}
			}

			string ShellScript = "set -e\\n\\nIFS=$'\\\\n'\\n\\n" +
				"if [ $PLATFORM_NAME = iphoneos ] || [ $PLATFORM_NAME = tvos ]; then \\n" +
				"\\tFRAMEWORK_DIR=$TARGET_BUILD_DIR/$EXECUTABLE_FOLDER_PATH/Frameworks\\n" +
				FrameworkScript.ToString() +
				// and now code sign anything that has been unzipped above
				"\\tfor FRAMEWORK in ${FRAMEWORK_DIR}/*.framework; do\\n" +
					"\\t\\t[ -d \\\"${FRAMEWORK}\\\" ] || continue\\n" +
					"\\t\\techo Codesigning ${FRAMEWORK}\\n" +
					"\\t\\tcodesign --force --sign ${EXPANDED_CODE_SIGN_IDENTITY} --generate-entitlement-der --verbose --preserve-metadata=identifier,entitlements,flags --timestamp=none \\\"${FRAMEWORK}\\\"\\n" +
				"\\tdone\\n" +
				"fi\\n";

			Content.Append("/* Begin PBXShellScriptBuildPhase section */" + ProjectFileGenerator.NewLine);
			Content.Append(String.Format("\t\t{0} /* Sign Frameworks */ = {{" + ProjectFileGenerator.NewLine, ShellScriptGuid));
			Content.Append("\t\t\tisa = PBXShellScriptBuildPhase;" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tbuildActionMask = 2147483647;" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tfiles = (" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\t);" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tinputPaths = (" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\t);" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\toutputPaths = (" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\t\t/dev/null" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\t);" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tname = \"Sign Manual Frameworks\";" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\trunOnlyForDeploymentPostprocessing = 0;" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tshellPath = /bin/sh;" + ProjectFileGenerator.NewLine);
			Content.Append(String.Format("\t\t\tshellScript = \"{0}\";" + ProjectFileGenerator.NewLine, ShellScript));
			Content.Append("\t\t};" + ProjectFileGenerator.NewLine);
			Content.Append("/* End PBXShellScriptBuildPhase section */" + ProjectFileGenerator.NewLine);
		}

		private static void AppendExtensionTargetSections(StringBuilder ProjectFileContent, List<XcodeExtensionInfo> AllExtensions)
		{
			foreach (XcodeExtensionInfo EI in AllExtensions)
			{
				Dictionary<string, string> BuildPhases = new Dictionary<string, string>();
				BuildPhases.Add(EI.ResourceBuildPhaseGuid, "Resources");

				AppendGenericTargetSection(ProjectFileContent, EI.Name, EI.TargetGuid, "com.apple.product-type.app-extension.messages-sticker-pack", EI.ConfigListGuid, EI.ProductGuid, null, BuildPhases);
			}
		}

		private static void AppendProjectSection(StringBuilder Content, string TargetName, string TargetGuid, string BuildTargetName, string BuildTargetGuid, string IndexTargetName, string IndexTargetGuid, string MainGroupGuid, string ProductRefGroupGuid, string ProjectGuid, string ProjectBuildConfigGuid, FileReference? ProjectFile, List<XcodeExtensionInfo> AllExtensions)
		{
			Content.Append("/* Begin PBXProject section */" + ProjectFileGenerator.NewLine);

			Content.Append("\t\t" + ProjectGuid + " /* Project object */ = {" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tisa = PBXProject;" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tattributes = {" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\t\tLastUpgradeCheck = 2000;" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\t\tORGANIZATIONNAME = \"Epic Games, Inc.\";" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\t\tTargetAttributes = {" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\t\t\t" + TargetGuid + " = {" + ProjectFileGenerator.NewLine);

			bool bAutomaticSigning = false;
			if (InstalledPlatformInfo.IsValidPlatform(UnrealTargetPlatform.IOS, EProjectType.Code))
			{
				IOSPlatform IOSPlatform = ((IOSPlatform)UEBuildPlatform.GetBuildPlatform(UnrealTargetPlatform.IOS));
				IOSProjectSettings ProjectSettings = IOSPlatform.ReadProjectSettings(ProjectFile);
				bAutomaticSigning = ProjectSettings.bAutomaticSigning;
			}

			if (InstalledPlatformInfo.IsValidPlatform(UnrealTargetPlatform.TVOS, EProjectType.Code))
			{
				TVOSPlatform TVOSPlatform = ((TVOSPlatform)UEBuildPlatform.GetBuildPlatform(UnrealTargetPlatform.TVOS));
				TVOSProjectSettings ProjectSettings = TVOSPlatform.ReadProjectSettings(ProjectFile);
				//TVOSProvisioningData ProvisioningData = TVOSPlatform.ReadProvisioningData(ProjectSettings, bForDistribution);
				bAutomaticSigning = ProjectSettings.bAutomaticSigning;
			}

			if (bAutomaticSigning)
			{
				Content.Append("\t\t\t\t\t\tProvisioningStyle = Automatic;" + ProjectFileGenerator.NewLine);
			}
			else
			{
				Content.Append("\t\t\t\t\t\tProvisioningStyle = Manual;" + ProjectFileGenerator.NewLine);
			}
			Content.Append("\t\t\t\t\t};" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\t\t};" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\t};" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tbuildConfigurationList = " + ProjectBuildConfigGuid + " /* Build configuration list for PBXProject \"" + TargetName + "\" */;" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tcompatibilityVersion = \"Xcode 8.0\";" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tdevelopmentRegion = English;" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\thasScannedForEncodings = 0;" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tknownRegions = (" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\t\ten" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\t);" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tmainGroup = " + MainGroupGuid + ";" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tproductRefGroup = " + ProductRefGroupGuid + ";" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tprojectDirPath = \"\";" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tprojectRoot = \"\";" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\ttargets = (" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\t" + TargetGuid + " /* " + TargetName + " */," + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\t" + BuildTargetGuid + " /* " + BuildTargetName + " */," + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\t" + IndexTargetGuid + " /* " + IndexTargetName + " */," + ProjectFileGenerator.NewLine);
			foreach (XcodeExtensionInfo EI in AllExtensions)
			{
				Content.Append("\t\t\t" + EI.TargetGuid + " /* " + EI.Name + " */," + ProjectFileGenerator.NewLine);
			}
			Content.Append("\t\t\t);" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t};" + ProjectFileGenerator.NewLine);

			Content.Append("/* End PBXProject section */" + ProjectFileGenerator.NewLine + ProjectFileGenerator.NewLine);
		}

		private static void AppendContainerItemProxySection(StringBuilder Content, string TargetName, string TargetGuid, string TargetProxyGuid, string ProjectGuid)
		{
			Content.Append("/* Begin PBXContainerItemProxy section */" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t" + TargetProxyGuid + " /* PBXContainerItemProxy */ = {" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tisa = PBXContainerItemProxy;" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tcontainerPortal = " + ProjectGuid + " /* Project object */;" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tproxyType = 1;" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tremoteGlobalIDString = " + TargetGuid + ";" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tremoteInfo = \"" + TargetName + "\";" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t};" + ProjectFileGenerator.NewLine);
			Content.Append("/* End PBXContainerItemProxy section */" + ProjectFileGenerator.NewLine + ProjectFileGenerator.NewLine);
		}

		private static void AppendTargetDependencySection(StringBuilder Content, string TargetName, string TargetGuid, string TargetDependencyGuid, string TargetProxyGuid)
		{
			Content.Append("/* Begin PBXTargetDependency section */" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t" + TargetDependencyGuid + " /* PBXTargetDependency */ = {" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tisa = PBXTargetDependency;" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\ttarget = " + TargetGuid + " /* " + TargetName + " */;" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\ttargetProxy = " + TargetProxyGuid + " /* PBXContainerItemProxy */;" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t};" + ProjectFileGenerator.NewLine);
			Content.Append("/* End PBXTargetDependency section */" + ProjectFileGenerator.NewLine + ProjectFileGenerator.NewLine);
		}

		private void AppendProjectBuildConfiguration(StringBuilder Content, string ConfigName, string ConfigGuid)
		{
			Content.Append("\t\t" + ConfigGuid + " /* \"" + ConfigName + "\" */ = {" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tisa = XCBuildConfiguration;" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tbuildSettings = {" + ProjectFileGenerator.NewLine);

			Content.Append("\t\t\t\tGCC_PREPROCESSOR_DEFINITIONS = (" + ProjectFileGenerator.NewLine);
			foreach (string Definition in IntelliSensePreprocessorDefinitions)
			{
				Content.Append("\t\t\t\t\t\"" + Definition.Replace("\"", "", StringComparison.Ordinal).Replace("\\", "", StringComparison.Ordinal) + "\"," + ProjectFileGenerator.NewLine);
			}
			Content.Append("\t\t\t\t\t\"__INTELLISENSE__\"," + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\t\t\t\"MONOLITHIC_BUILD=1\"," + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\t\t);" + ProjectFileGenerator.NewLine);

			Content.Append("\t\t\t\tHEADER_SEARCH_PATHS = (" + ProjectFileGenerator.NewLine);
			foreach (string SearchPath in IntelliSenseSystemIncludeSearchPaths)
			{
				string Path = SearchPath.Contains(' ', StringComparison.Ordinal) ? "\\\"" + SearchPath + "\\\"" : SearchPath;
				Content.Append("\t\t\t\t\t\"" + Path + "\"," + ProjectFileGenerator.NewLine);
			}
			Content.Append("\t\t\t\t);" + ProjectFileGenerator.NewLine);

			Content.Append("\t\t\t\tUSER_HEADER_SEARCH_PATHS = (" + ProjectFileGenerator.NewLine);
			foreach (string SearchPath in IntelliSenseIncludeSearchPaths)
			{
				string Path = SearchPath.Contains(' ', StringComparison.Ordinal) ? "\\\"" + SearchPath + "\\\"" : SearchPath;
				Content.Append("\t\t\t\t\t\"" + Path + "\"," + ProjectFileGenerator.NewLine);
			}
			Content.Append("\t\t\t\t);" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\t\tONLY_ACTIVE_ARCH = YES;" + ProjectFileGenerator.NewLine);
			if (ConfigName == "Debug")
			{
				Content.Append("\t\t\t\tENABLE_TESTABILITY = YES;" + ProjectFileGenerator.NewLine);
			}
			Content.Append("\t\t\t\tALWAYS_SEARCH_USER_PATHS = NO;" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\t\tCLANG_CXX_LANGUAGE_STANDARD = \"c++14\";" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\t\tGCC_ENABLE_CPP_RTTI = NO;" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\t\tGCC_WARN_CHECK_SWITCH_STATEMENTS = NO;" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\t\tUSE_HEADERMAP = NO;" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\t};" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tname = \"" + ConfigName + "\";" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t};" + ProjectFileGenerator.NewLine);
		}

		// cache for the below function
		readonly Dictionary<string, UnrealArchitectures> CachedMacProjectArcitectures = new();

		/// <summary>
		/// Returns the Mac architectures that should be configured for the provided target. If the target has a project we'll adhere
		/// to whether it's set as Intel/Universal/Apple unless the type is denied (pretty much just Editor)
		/// 
		/// If the target has no project we'll support allow-listed targets for installed builds and all non-editor architectures 
		/// for source builds. Not all programs are going to compile for Apple Silicon, but being able to build and fail is useful...
		/// </summary>
		/// <param name="Config">Build config for the target we're generating</param>
		/// <param name="InProjectFile">Path to the project file, or null if the target has no project</param>
		/// <returns></returns>
		UnrealArchitectures GetSupportedMacArchitectures(XcodeBuildConfig Config, FileReference? InProjectFile)
		{
			// All architectures supported
			UnrealArchitectures AllArchitectures = new(new[] { UnrealArch.Arm64, UnrealArch.X64 });

			// Add a way on the command line of forcing a project file with all architectures (there isn't a good way to let this be
			// set and checked where we can access it).
			bool ForceAllArchitectures = Environment.GetCommandLineArgs().Contains("AllArchitectures", StringComparer.OrdinalIgnoreCase);

			if (ForceAllArchitectures)
			{
				return AllArchitectures;
			}

			string TargetName = Config.BuildTarget;

			// First time seeing this target?
			if (!CachedMacProjectArcitectures.ContainsKey(TargetName))
			{
				CachedMacProjectArcitectures[TargetName] = UnrealArchitectureConfig.ForPlatform(UnrealTargetPlatform.Mac).ProjectSupportedArchitectures(InProjectFile, TargetName);
			}

			return CachedMacProjectArcitectures[TargetName];
		}

		private void AppendPlatformConfiguration(StringBuilder Content, XcodeBuildConfig Config, string TargetName, FileReference? ProjectFile, bool bSupportMac, bool bSupportIOS, bool bSupportTVOS, out string? IOSRunTimeVersion, out string? TVOSRunTimeVersion, string BinariesSubDir = "/Payload")
		{
			FileReference MacExecutablePath = Config.MacExecutablePath!;

			string UEDir = ConvertPath(Path.GetFullPath(Directory.GetCurrentDirectory() + "../../.."));
			string MacExecutableDir = bSupportMac ? ConvertPath(MacExecutablePath.Directory.FullName) : "";
			string MacExecutableFileName = bSupportMac ? MacExecutablePath.GetFileName() : "";

			// Get Mac architectures supported by this project
			UnrealArchitectures SupportedMacArchitectures = GetSupportedMacArchitectures(Config, ProjectFile);

			IOSRunTimeVersion = null;
			TVOSRunTimeVersion = null;

			bool bIsUnrealGame = TargetName.Equals("UnrealGame", StringComparison.OrdinalIgnoreCase);
			bool bIsUnrealClient = TargetName.Equals("UnrealClient", StringComparison.OrdinalIgnoreCase);
			DirectoryReference? GameDir = ProjectFile?.Directory;
			string? GamePath = GameDir != null ? ConvertPath(GameDir.FullName) : null;

			string? IOSRunTimeDevices = null;
			string? TVOSRunTimeDevices = null;
			string SupportedPlatforms = bSupportMac ? "macosx" : "";

			bool bAutomaticSigning = false;
			string? UUID_IOS = "";
			string? UUID_TVOS = "";
			string? TEAM_IOS = "";
			string? TEAM_TVOS = "";
			string? IOS_CERT = "iPhone Developer";
			string? TVOS_CERT = "iPhone Developer";
			string IOS_BUNDLE = "";
			string TVOS_BUNDLE = "";
			if (bSupportIOS && InstalledPlatformInfo.IsValidPlatform(UnrealTargetPlatform.IOS, EProjectType.Code))
			{
				IOSPlatform IOSPlatform = ((IOSPlatform)UEBuildPlatform.GetBuildPlatform(UnrealTargetPlatform.IOS));
				IOSProjectSettings ProjectSettings = IOSPlatform.ReadProjectSettings(ProjectFile);
				IOSProvisioningData ProvisioningData = IOSPlatform.ReadProvisioningData(ProjectSettings, bForDistribution);
				IOSRunTimeVersion = ProjectSettings.RuntimeVersion;
				IOSRunTimeDevices = ProjectSettings.RuntimeDevices;
				SupportedPlatforms += " iphoneos";
				bAutomaticSigning = ProjectSettings.bAutomaticSigning;
				if (!bAutomaticSigning)
				{
					UUID_IOS = ProvisioningData.MobileProvisionUUID;
					IOS_CERT = ProvisioningData.SigningCertificate;
				}
				TEAM_IOS = ProvisioningData.TeamUUID;
				IOS_BUNDLE = ProjectSettings.BundleIdentifier;
			}

			if (bSupportTVOS && InstalledPlatformInfo.IsValidPlatform(UnrealTargetPlatform.TVOS, EProjectType.Code))
			{
				TVOSPlatform TVOSPlatform = ((TVOSPlatform)UEBuildPlatform.GetBuildPlatform(UnrealTargetPlatform.TVOS));
				TVOSProjectSettings ProjectSettings = TVOSPlatform.ReadProjectSettings(ProjectFile);
				TVOSProvisioningData ProvisioningData = TVOSPlatform.ReadProvisioningData(ProjectSettings, bForDistribution);
				TVOSRunTimeVersion = ProjectSettings.RuntimeVersion;
				TVOSRunTimeDevices = ProjectSettings.RuntimeDevices;
				SupportedPlatforms += " appletvos";
				if (!bAutomaticSigning)
				{
					UUID_TVOS = ProvisioningData.MobileProvisionUUID;
					TVOS_CERT = ProvisioningData.SigningCertificate;
				}
				TEAM_TVOS = ProvisioningData.TeamUUID;
				TVOS_BUNDLE = ProjectSettings.BundleIdentifier;
			}

			Content.Append("\t\t\t\tSUPPORTED_PLATFORMS = \"" + SupportedPlatforms.Trim() + "\";" + ProjectFileGenerator.NewLine);
			if (bAutomaticSigning)
			{
				Content.Append("\t\t\t\tCODE_SIGN_STYLE = Automatic;" + ProjectFileGenerator.NewLine);
			}
			if (IOSRunTimeVersion != null)
			{
				Content.Append("\t\t\t\t\"VALID_ARCHS[sdk=iphoneos*]\" = \"" + String.Join(" ", SupportedIOSArchitectures) + "\";" + ProjectFileGenerator.NewLine);
				Content.Append("\t\t\t\tIPHONEOS_DEPLOYMENT_TARGET = " + IOSRunTimeVersion + ";" + ProjectFileGenerator.NewLine);
				Content.Append("\t\t\t\t\"PRODUCT_NAME[sdk=iphoneos*]\" = \"" + TargetName + "\";" + ProjectFileGenerator.NewLine); // @todo: change to Path.GetFileName(Config.IOSExecutablePath) when we stop using payload
				Content.Append("\t\t\t\t\"TARGETED_DEVICE_FAMILY[sdk=iphoneos*]\" = \"" + IOSRunTimeDevices + "\";" + ProjectFileGenerator.NewLine);
				if (!String.IsNullOrEmpty(TEAM_IOS))
				{
					Content.Append("\t\t\t\t\"DEVELOPMENT_TEAM[sdk=iphoneos*]\" = " + TEAM_IOS + ";" + ProjectFileGenerator.NewLine);
				}
				Content.Append("\t\t\t\t\"CODE_SIGN_IDENTITY[sdk=iphoneos*]\" = \"" + IOS_CERT + "\";" + ProjectFileGenerator.NewLine);
				if (!bAutomaticSigning && !String.IsNullOrEmpty(UUID_IOS))
				{
					Content.Append("\t\t\t\t\"PROVISIONING_PROFILE_SPECIFIER[sdk=iphoneos*]\" = \"" + UUID_IOS + "\";" + ProjectFileGenerator.NewLine);
				}
				if (ProjectFile != null)
				{
					Content.Append("\t\t\t\t\"PRODUCT_BUNDLE_IDENTIFIER[sdk=iphoneos*]\" = " + IOS_BUNDLE + ";" + ProjectFileGenerator.NewLine);
				}
			}
			if (TVOSRunTimeVersion != null)
			{
				Content.Append("\t\t\t\t\"VALID_ARCHS[sdk=appletvos*]\" = \"" + String.Join(" ", SupportedIOSArchitectures) + "\";" + ProjectFileGenerator.NewLine);
				Content.Append("\t\t\t\tTVOS_DEPLOYMENT_TARGET = " + TVOSRunTimeVersion + ";" + ProjectFileGenerator.NewLine);
				Content.Append("\t\t\t\t\"PRODUCT_NAME[sdk=appletvos*]\" = \"" + TargetName + "\";" + ProjectFileGenerator.NewLine); // @todo: change to Path.GetFileName(Config.TVOSExecutablePath) when we stop using payload
				Content.Append("\t\t\t\t\"TARGETED_DEVICE_FAMILY[sdk=appletvos*]\" = \"" + TVOSRunTimeDevices + "\";" + ProjectFileGenerator.NewLine);
				if (!String.IsNullOrEmpty(TEAM_TVOS))
				{
					Content.Append("\t\t\t\t\"DEVELOPMENT_TEAM[sdk=appletvos*]\" = " + TEAM_TVOS + ";" + ProjectFileGenerator.NewLine);
				}
				Content.Append("\t\t\t\t\"CODE_SIGN_IDENTITY[sdk=appletvos*]\" = \"" + TVOS_CERT + "\";" + ProjectFileGenerator.NewLine);
				if (!bAutomaticSigning && !String.IsNullOrEmpty(UUID_TVOS))
				{
					Content.Append("\t\t\t\t\"PROVISIONING_PROFILE_SPECIFIER[sdk=appletvos*]\" = \"" + UUID_TVOS + "\";" + ProjectFileGenerator.NewLine);
				}
				if (ProjectFile != null)
				{
					Content.Append("\t\t\t\t\"PRODUCT_BUNDLE_IDENTIFIER[sdk=appletvos*]\" = " + TVOS_BUNDLE + ";" + ProjectFileGenerator.NewLine);
				}
			}
			if (bSupportMac)
			{
				Content.Append("\t\t\t\t\"VALID_ARCHS[sdk=macosx*]\" = \"" + String.Join(" ", SupportedMacArchitectures.Architectures.Select(x => x.AppleName)) + "\";" + ProjectFileGenerator.NewLine);
				Content.Append("\t\t\t\t\"PRODUCT_NAME[sdk=macosx*]\" = \"" + MacExecutableFileName + "\";" + ProjectFileGenerator.NewLine);
				Content.Append("\t\t\t\t\"CONFIGURATION_BUILD_DIR[sdk=macosx*]\" = \"" + MacExecutableDir + "\";" + ProjectFileGenerator.NewLine);

				// MacToolchain uses the IOS bundle identifier
				ConfigHierarchy IOSIni = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, ProjectFile?.Directory, UnrealTargetPlatform.IOS);
				string IOSBundleIdentifier;
				IOSIni.GetString("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "BundleIdentifier", out IOSBundleIdentifier!);

				// we are going to load/sign the UnrealEditor.app when doing modular editor builds, so reuse the shared identifier
				bool bBuildingEditor = Config.ProjectTarget!.TargetRules!.Type == TargetType.Editor && Config.ProjectTarget!.TargetRules!.LinkType == TargetLinkType.Modular;
				string MacBundleIdentifier = bBuildingEditor ? ("com.epicgames.UnrealEditor") : (IOSBundleIdentifier.Replace("[PROJECT_NAME]", TargetName, StringComparison.Ordinal).Replace("_", "", StringComparison.Ordinal));
				// Xcode 14 wants this to match what is in the final plist
				Content.Append("\t\t\t\t\"PRODUCT_BUNDLE_IDENTIFIER[sdk=macosx*]\" = " + MacBundleIdentifier + ";" + ProjectFileGenerator.NewLine);
			}

			if (IOSRunTimeVersion != null)
			{
				Content.Append("\t\t\t\t\"CONFIGURATION_BUILD_DIR[sdk=iphoneos*]\" = \"" + Config.IOSExecutablePath!.Directory!.FullName + BinariesSubDir + "\";" + ProjectFileGenerator.NewLine);
			}
			if (TVOSRunTimeVersion != null)
			{
				Content.Append("\t\t\t\t\"CONFIGURATION_BUILD_DIR[sdk=appletvos*]\" = \"" + Config.TVOSExecutablePath!.Directory!.FullName + BinariesSubDir + "\";" + ProjectFileGenerator.NewLine);
			}
		}

		private void AppendNativeTargetBuildConfiguration(StringBuilder Content, XcodeBuildConfig Config, string ConfigGuid, FileReference? ProjectFile, ILogger Logger)
		{
			bool bMacOnly = true;
			if (Config.ProjectTarget!.TargetRules != null)
			{
				if (XcodeProjectFileGenerator.XcodePlatforms.Contains(UnrealTargetPlatform.IOS) && Config.ProjectTarget.SupportedPlatforms.Contains(UnrealTargetPlatform.IOS))
				{
					bMacOnly = false;
				}
				if (XcodeProjectFileGenerator.XcodePlatforms.Contains(UnrealTargetPlatform.TVOS) && Config.ProjectTarget.SupportedPlatforms.Contains(UnrealTargetPlatform.TVOS))
				{
					bMacOnly = false;
				}
			}

			Content.Append("\t\t" + ConfigGuid + " /* \"" + Config.DisplayName + "\" */ = {" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tisa = XCBuildConfiguration;" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tbuildSettings = {" + ProjectFileGenerator.NewLine);

			string UEDir = ConvertPath(Path.GetFullPath(Directory.GetCurrentDirectory() + "../../.."));
			//string MacExecutableDir = ConvertPath(Config.MacExecutablePath.Directory.FullName);
			string MacExecutableFileName = Config.MacExecutablePath!.GetFileName();

			string? IOSRunTimeVersion, TVOSRunTimeVersion;
			AppendPlatformConfiguration(Content, Config, Config.BuildTarget, ProjectFile, true, !bMacOnly, !bMacOnly, out IOSRunTimeVersion, out TVOSRunTimeVersion);

			bool bIsUnrealGame = Config.BuildTarget.Equals("UnrealGame", StringComparison.OrdinalIgnoreCase);
			bool bIsUnrealClient = Config.BuildTarget.Equals("UnrealClient", StringComparison.OrdinalIgnoreCase);

			DirectoryReference? GameDir = ProjectFile?.Directory;
			string? GamePath = GameDir != null ? ConvertPath(GameDir.FullName) : null;

			string IOSInfoPlistPath;
			string TVOSInfoPlistPath;
			string MacInfoPlistPath;
			string? IOSEntitlementPath = null;
			string? TVOSEntitlementPath = null;
			if (bIsUnrealGame)
			{
				IOSInfoPlistPath = UEDir + "/Engine/Intermediate/IOS/" + Config.BuildTarget + "-Info.plist";
				TVOSInfoPlistPath = UEDir + "/Engine/Intermediate/TVOS/" + Config.BuildTarget + "-Info.plist";
				MacInfoPlistPath = UEDir + "/Engine/Intermediate/Mac/" + MacExecutableFileName + "-Info.plist";
				IOSEntitlementPath = "";
				TVOSEntitlementPath = "";
			}
			else if (bIsUnrealClient)
			{
				IOSInfoPlistPath = UEDir + "/Engine/Intermediate/IOS/UnrealGame-Info.plist";
				TVOSInfoPlistPath = UEDir + "/Engine/Intermediate/TVOS/UnrealGame-Info.plist";
				MacInfoPlistPath = UEDir + "/Engine/Intermediate/Mac/" + MacExecutableFileName + "-Info.plist";
				IOSEntitlementPath = "";
				TVOSEntitlementPath = "";
			}
			else if (ProjectFile != null)
			{
				IOSInfoPlistPath = GamePath + "/Intermediate/IOS/" + Config.BuildTarget + "-Info.plist";
				TVOSInfoPlistPath = GamePath + "/Intermediate/TVOS/" + Config.BuildTarget + "-Info.plist";
				MacInfoPlistPath = GamePath + "/Intermediate/Mac/" + MacExecutableFileName + "-Info.plist";
				IOSEntitlementPath = GamePath + "/Intermediate/IOS/" + Config.BuildTarget + ".entitlements";
				TVOSEntitlementPath = GamePath + "/Intermediate/TVOS/" + Config.BuildTarget + ".entitlements";
			}
			else
			{
				if (GamePath == null)
				{
					IOSInfoPlistPath = UEDir + "/Engine/Intermediate/IOS/" + Config.BuildTarget + "-Info.plist";
					TVOSInfoPlistPath = UEDir + "/Engine/Intermediate/TVOS/" + Config.BuildTarget + "-Info.plist";
					MacInfoPlistPath = UEDir + "/Engine/Intermediate/Mac/" + MacExecutableFileName + "-Info.plist";
				}
				else
				{
					IOSInfoPlistPath = GamePath + "/Intermediate/IOS/" + Config.BuildTarget + "-Info.plist";
					TVOSInfoPlistPath = GamePath + "/Intermediate/TVOS/" + Config.BuildTarget + "-Info.plist";
					MacInfoPlistPath = GamePath + "/Intermediate/Mac/" + MacExecutableFileName + "-Info.plist";
				}
			}

			if (XcodeProjectFileGenerator.bGenerateRunOnlyProject)
			{
				if (XcodeProjectFileGenerator.XcodePlatforms.Contains(UnrealTargetPlatform.IOS))
				{
					Content.Append("\t\t\t\tINFOPLIST_FILE = \"" + IOSInfoPlistPath + "\";" + ProjectFileGenerator.NewLine);
					Content.Append("\t\t\t\tCODE_SIGN_ENTITLEMENTS = \"" + IOSEntitlementPath + "\";" + ProjectFileGenerator.NewLine);
				}
				else
				{
					Content.Append("\t\t\t\tINFOPLIST_FILE = \"" + TVOSInfoPlistPath + "\";" + ProjectFileGenerator.NewLine);
					Content.Append("\t\t\t\tCODE_SIGN_ENTITLEMENTS = \"" + TVOSEntitlementPath + "\";" + ProjectFileGenerator.NewLine);
				}
			}
			else
			{
				Content.Append("\t\t\t\t\"INFOPLIST_FILE[sdk=macosx*]\" = \"" + MacInfoPlistPath + "\";" + ProjectFileGenerator.NewLine);
				if (IOSRunTimeVersion != null)
				{
					Content.Append("\t\t\t\t\"INFOPLIST_FILE[sdk=iphoneos*]\" = \"" + IOSInfoPlistPath + "\";" + ProjectFileGenerator.NewLine);
					Content.Append("\t\t\t\t\"CODE_SIGN_ENTITLEMENTS[sdk=iphoneos*]\" = \"" + IOSEntitlementPath + "\";" + ProjectFileGenerator.NewLine);
				}
				if (TVOSRunTimeVersion != null)
				{
					Content.Append("\t\t\t\t\"INFOPLIST_FILE[sdk=appletvos*]\" = \"" + TVOSInfoPlistPath + "\";" + ProjectFileGenerator.NewLine);
					Content.Append("\t\t\t\t\"CODE_SIGN_ENTITLEMENTS[sdk=appletvos*]\" = \"" + TVOSEntitlementPath + "\";" + ProjectFileGenerator.NewLine);
				}
			}

			// Prepare a temp Info.plist file so Xcode has some basic info about the target immediately after opening the project.
			// This is needed for the target to pass the settings validation before code signing. UBT will overwrite this plist file later, with proper contents.
			if (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Mac)
			{
				bool bCreateMacInfoPlist = !File.Exists(MacInfoPlistPath);
				bool bCreateIOSInfoPlist = !File.Exists(IOSInfoPlistPath) && IOSRunTimeVersion != null;
				bool bCreateTVOSInfoPlist = !File.Exists(TVOSInfoPlistPath) && TVOSRunTimeVersion != null;
				if (bCreateMacInfoPlist || bCreateIOSInfoPlist || bCreateTVOSInfoPlist)
				{
					DirectoryReference? ProjectPath = GameDir;
					DirectoryReference EngineDir = DirectoryReference.Combine(new DirectoryReference(UEDir), "Engine");
					string GameName = Config.BuildTarget;
					bool bIsClient = false;
					if (ProjectPath == null)
					{
						ProjectPath = EngineDir;
					}
					if (bIsUnrealGame)
					{
						ProjectPath = EngineDir;
						GameName = "UnrealGame";
						bIsClient = (AppName == "UnrealClient");
					}

					if (bCreateMacInfoPlist)
					{
						Directory.CreateDirectory(Path.GetDirectoryName(MacInfoPlistPath)!);
						UEDeployMac.GeneratePList(ProjectPath.FullName, bIsUnrealGame, GameName, Config.BuildTarget, EngineDir.FullName, MacExecutableFileName, Config.ProjectTarget!.TargetRules!.Type);
					}
					if (bCreateIOSInfoPlist)
					{
						// get the receipt
						FileReference ReceiptFilename;
						UnrealArchitectures Architectures = UnrealArchitectureConfig.ForPlatform(UnrealTargetPlatform.IOS).ActiveArchitectures(ProjectFile, GameName);
						if (bIsUnrealGame)
						{
							ReceiptFilename = TargetReceipt.GetDefaultPath(Unreal.EngineDirectory, "UnrealGame", UnrealTargetPlatform.IOS, Config.BuildConfig, Architectures);
						}
						else
						{
							ReceiptFilename = TargetReceipt.GetDefaultPath(ProjectPath, GameName, UnrealTargetPlatform.IOS, Config.BuildConfig, Architectures);
						}
						Directory.CreateDirectory(Path.GetDirectoryName(IOSInfoPlistPath)!);
						TargetReceipt? Receipt;
						TargetReceipt.TryRead(ReceiptFilename, out Receipt);
						bool bBuildAsFramework = UEDeployIOS.GetCompileAsDll(Receipt);
						UEDeployIOS.GenerateIOSPList(ProjectFile, Config.BuildConfig, ProjectPath.FullName, bIsUnrealGame, GameName, bIsClient, Config.BuildTarget, EngineDir.FullName, ProjectPath + "/Binaries/IOS/Payload", null, BundleIdentifier, bBuildAsFramework, Logger);
					}
					if (bCreateTVOSInfoPlist)
					{
						Directory.CreateDirectory(Path.GetDirectoryName(TVOSInfoPlistPath)!);
						AppleOnWindowsAppMaker.GenerateTVOSPList(ProjectPath.FullName, bIsUnrealGame, GameName, bIsClient, Config.BuildTarget, EngineDir.FullName, ProjectPath + "/Binaries/TVOS/Payload", BundleIdentifier);
					}
				}
			}

			// #jira UE-143619: Pre Monterey macOS requires this option for a packaged app to run on iOS15 due to new code signature format. Could be removed once Monterey is minimum OS.
			Content.Append("\t\t\t\tOTHER_CODE_SIGN_FLAGS = \"--generate-entitlement-der --deep\";" + ProjectFileGenerator.NewLine);

			Content.Append("\t\t\t\tMACOSX_DEPLOYMENT_TARGET = " + MacToolChainSettings.MinMacDeploymentVersion(Config.ProjectTarget!.TargetRules!.Type) + ";" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\t\tINFOPLIST_OUTPUT_FORMAT = xml;" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\t\tCOMBINE_HIDPI_IMAGES = YES;" + ProjectFileGenerator.NewLine);

			//#jira UE-50382 Xcode Address Sanitizer feature does not work on iOS
			// address sanitizer dylib loader depends on the SDKROOT parameter. For macosx or default (missing, translated as macosx), the path is incorrect for iphone/appletv
			if (XcodeProjectFileGenerator.bGenerateRunOnlyProject)
			{
				if (XcodeProjectFileGenerator.XcodePlatforms.Contains(UnrealTargetPlatform.IOS))
				{
					Content.Append("\t\t\t\tSDKROOT = iphoneos;" + ProjectFileGenerator.NewLine);
				}
				else
				{
					Content.Append("\t\t\t\tSDKROOT = appletvos;" + ProjectFileGenerator.NewLine);
				}
			}
			else
			{
				Content.Append("\t\t\t\tSDKROOT = macosx;" + ProjectFileGenerator.NewLine);
			}
			Content.Append("\t\t\t\tGCC_PRECOMPILE_PREFIX_HEADER = YES;" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\t\tGCC_PREFIX_HEADER = \"" + UEDir + "/Engine/Source/Editor/UnrealEd/Public/UnrealEd.h\";" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\t};" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tname = \"" + Config.DisplayName + "\";" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t};" + ProjectFileGenerator.NewLine);
		}

		private void AppendLegacyTargetBuildConfiguration(StringBuilder Content, XcodeBuildConfig Config, string ConfigGuid)
		{
			bool bMacOnly = true;
			if (Config.ProjectTarget!.TargetRules != null)
			{
				if (XcodeProjectFileGenerator.XcodePlatforms.Contains(UnrealTargetPlatform.IOS) && Config.ProjectTarget.SupportedPlatforms.Contains(UnrealTargetPlatform.IOS))
				{
					bMacOnly = false;
				}
				if (XcodeProjectFileGenerator.XcodePlatforms.Contains(UnrealTargetPlatform.TVOS) && Config.ProjectTarget.SupportedPlatforms.Contains(UnrealTargetPlatform.TVOS))
				{
					bMacOnly = false;
				}
			}

			// Get Mac architectures supported by this project
			UnrealArchitectures SupportedMacArchitectures = GetSupportedMacArchitectures(Config, ProjectFilePath);

			Content.Append("\t\t" + ConfigGuid + " /* \"" + Config.DisplayName + "\" */ = {" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tisa = XCBuildConfiguration;" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tbuildSettings = {" + ProjectFileGenerator.NewLine);
			if (bMacOnly)
			{
				Content.Append("\t\t\t\tVALID_ARCHS = \"" + String.Join(" ", SupportedMacArchitectures.Architectures.Select(x => x.AppleName)) + "\";" + ProjectFileGenerator.NewLine);
				Content.Append("\t\t\t\tSUPPORTED_PLATFORMS = \"macosx\";" + ProjectFileGenerator.NewLine);
			}
			else
			{
				IEnumerable<UnrealArch> ValidArchs = SupportedMacArchitectures.Architectures;
				string SupportedPlatforms = "macosx";
				if (InstalledPlatformInfo.IsValidPlatform(UnrealTargetPlatform.IOS, EProjectType.Code))
				{
					ValidArchs = ValidArchs.Union(SupportedIOSArchitectures);
					SupportedPlatforms += " iphoneos";
				}
				if (InstalledPlatformInfo.IsValidPlatform(UnrealTargetPlatform.TVOS, EProjectType.Code))
				{
					ValidArchs = ValidArchs.Union(SupportedIOSArchitectures);
					SupportedPlatforms += " appletvos";
				}
				Content.Append("\t\t\t\tVALID_ARCHS = \"" + String.Join(" ", ValidArchs.Select(x => x.AppleName)) + "\";" + ProjectFileGenerator.NewLine);
				Content.Append("\t\t\t\tSUPPORTED_PLATFORMS = \"" + SupportedPlatforms + "\";" + ProjectFileGenerator.NewLine);
			}
			Content.Append("\t\t\t\tGCC_PREPROCESSOR_DEFINITIONS = ();" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\t\tHEADER_SEARCH_PATHS = ();" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\t\tUSER_HEADER_SEARCH_PATHS = ();" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\t\tUE_BUILD_TARGET_NAME = \"" + Config.BuildTarget + "\";" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\t\tUE_BUILD_TARGET_CONFIG = \"" + Config.BuildConfig + "\";" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\t};" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tname = \"" + Config.DisplayName + "\";" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t};" + ProjectFileGenerator.NewLine);
		}

		private void AppendXCBuildConfigurationSection(StringBuilder Content, Dictionary<string, XcodeBuildConfig> ProjectBuildConfigs, Dictionary<string, XcodeBuildConfig> TargetBuildConfigs,
				Dictionary<string, XcodeBuildConfig> BuildTargetBuildConfigs, Dictionary<string, XcodeBuildConfig> IndexTargetBuildConfigs, FileReference? GameProjectPath,
				List<XcodeExtensionInfo> AllExtensions, ILogger Logger)
		{
			Content.Append("/* Begin XCBuildConfiguration section */" + ProjectFileGenerator.NewLine);

			foreach (KeyValuePair<string, XcodeBuildConfig> Config in ProjectBuildConfigs)
			{
				AppendProjectBuildConfiguration(Content, Config.Value.DisplayName, Config.Key);
			}

			foreach (KeyValuePair<string, XcodeBuildConfig> Config in TargetBuildConfigs)
			{
				AppendNativeTargetBuildConfiguration(Content, Config.Value, Config.Key, GameProjectPath, Logger);
			}

			foreach (KeyValuePair<string, XcodeBuildConfig> Config in BuildTargetBuildConfigs)
			{
				AppendLegacyTargetBuildConfiguration(Content, Config.Value, Config.Key);
			}

			foreach (KeyValuePair<string, XcodeBuildConfig> Config in IndexTargetBuildConfigs)
			{
				AppendNativeTargetBuildConfiguration(Content, Config.Value, Config.Key, GameProjectPath, Logger);
			}

			foreach (XcodeExtensionInfo EI in AllExtensions)
			{
				Content.Append(EI.ConfigurationContents);
			}

			Content.Append("/* End XCBuildConfiguration section */" + ProjectFileGenerator.NewLine + ProjectFileGenerator.NewLine);
		}

		private static void AppendXCConfigurationList(StringBuilder Content, string TypeName, string TargetName, string ConfigListGuid, Dictionary<string, XcodeBuildConfig> BuildConfigs, string Default = "Development")
		{
			Content.Append("\t\t" + ConfigListGuid + " /* Build configuration list for " + TypeName + " \"" + TargetName + "\" */ = {" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tisa = XCConfigurationList;" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tbuildConfigurations = (" + ProjectFileGenerator.NewLine);
			foreach (KeyValuePair<string, XcodeBuildConfig> Config in BuildConfigs)
			{
				Content.Append("\t\t\t\t" + Config.Key + " /* \"" + Config.Value.DisplayName + "\" */," + ProjectFileGenerator.NewLine);
			}
			Content.Append("\t\t\t);" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tdefaultConfigurationIsVisible = 0;" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\tdefaultConfigurationName = " + Default + ";" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t};" + ProjectFileGenerator.NewLine);
		}

		private static void AppendXCConfigurationListSection(StringBuilder Content, string TargetName, string BuildTargetName, string IndexTargetName, string ProjectConfigListGuid,
			Dictionary<string, XcodeBuildConfig> ProjectBuildConfigs, string TargetConfigListGuid, Dictionary<string, XcodeBuildConfig> TargetBuildConfigs,
			string BuildTargetConfigListGuid, Dictionary<string, XcodeBuildConfig> BuildTargetBuildConfigs,
			string IndexTargetConfigListGuid, Dictionary<string, XcodeBuildConfig> IndexTargetBuildConfigs,
			List<XcodeExtensionInfo> AllExtensions)
		{
			Content.Append("/* Begin XCConfigurationList section */" + ProjectFileGenerator.NewLine);

			AppendXCConfigurationList(Content, "PBXProject", TargetName, ProjectConfigListGuid, ProjectBuildConfigs);
			AppendXCConfigurationList(Content, "PBXLegacyTarget", BuildTargetName, BuildTargetConfigListGuid, BuildTargetBuildConfigs);
			AppendXCConfigurationList(Content, "PBXNativeTarget", TargetName, TargetConfigListGuid, TargetBuildConfigs);
			AppendXCConfigurationList(Content, "PBXNativeTarget", IndexTargetName, IndexTargetConfigListGuid, IndexTargetBuildConfigs);

			foreach (XcodeExtensionInfo EI in AllExtensions)
			{
				AppendXCConfigurationList(Content, "PBXNativeTarget", EI.Name, EI.ConfigListGuid, EI.AllConfigs);
			}

			Content.Append("/* End XCConfigurationList section */" + ProjectFileGenerator.NewLine);
		}

		private List<XcodeBuildConfig> GetSupportedBuildConfigs(List<UnrealTargetPlatform> Platforms, List<UnrealTargetConfiguration> Configurations)
		{
			List<XcodeBuildConfig> BuildConfigs = new List<XcodeBuildConfig>();

			//string ProjectName = ProjectFilePath.GetFileNameWithoutExtension();

			foreach (UnrealTargetConfiguration Configuration in Configurations)
			{
				if (InstalledPlatformInfo.IsValidConfiguration(Configuration, EProjectType.Code))
				{
					foreach (UnrealTargetPlatform Platform in Platforms)
					{
						if (InstalledPlatformInfo.IsValidPlatform(Platform, EProjectType.Code) && (Platform == UnrealTargetPlatform.Mac || Platform == UnrealTargetPlatform.IOS || Platform == UnrealTargetPlatform.TVOS)) // @todo support other platforms
						{
							UEBuildPlatform? BuildPlatform;
							if (UEBuildPlatform.TryGetBuildPlatform(Platform, out BuildPlatform) && (BuildPlatform.HasRequiredSDKsInstalled() == SDKStatus.Valid))
							{
								// Check we have targets (Expected to be no Engine targets when generating for a single .uproject)
								if (ProjectTargets.Count == 0 && BaseDir != Unreal.EngineDirectory)
								{
									throw new BuildException("Expecting at least one ProjectTarget to be associated with project '{0}' in the TargetProjects list ", ProjectFilePath);
								}

								// Now go through all of the target types for this project
								foreach (ProjectTarget ProjectTarget in ProjectTargets.OfType<ProjectTarget>())
								{
									if (MSBuildProjectFile.IsValidProjectPlatformAndConfiguration(ProjectTarget, Platform, Configuration))
									{
										// Figure out if this is a monolithic build
										bool bShouldCompileMonolithic = BuildPlatform.ShouldCompileMonolithicBinary(Platform);
										try
										{
											bShouldCompileMonolithic |= (ProjectTarget.CreateRulesDelegate(Platform, Configuration).LinkType == TargetLinkType.Monolithic);
										}
										catch (BuildException)
										{
										}

										string ConfigName = Configuration.ToString();
										if (!bMakeProjectPerTarget && ProjectTarget.TargetRules!.Type != TargetType.Game && ProjectTarget.TargetRules.Type != TargetType.Program)
										{
											ConfigName += " " + ProjectTarget.TargetRules.Type.ToString();
										}

										if (BuildConfigs.Where(Config => Config.DisplayName == ConfigName).ToList().Count == 0)
										{
											string TargetName = ProjectTarget.TargetFilePath.GetFileNameWithoutAnyExtensions();
											// Get the .uproject directory
											DirectoryReference? UProjectDirectory = DirectoryReference.FromFile(ProjectTarget.UnrealProjectFilePath);

											// Get the output directory
											DirectoryReference RootDirectory;
											if (UProjectDirectory != null &&
												(bShouldCompileMonolithic || ProjectTarget.TargetRules!.BuildEnvironment == TargetBuildEnvironment.Unique) &&
												ProjectTarget.TargetRules!.File!.IsUnderDirectory(UProjectDirectory))
											{
												RootDirectory = UEBuildTarget.GetOutputDirectoryForExecutable(UProjectDirectory, ProjectTarget.TargetRules.File!);
											}
											else
											{
												RootDirectory = UEBuildTarget.GetOutputDirectoryForExecutable(Unreal.EngineDirectory, ProjectTarget.TargetRules!.File!);
											}

											string ExeName = TargetName;
											if (!bShouldCompileMonolithic && ProjectTarget.TargetRules.Type != TargetType.Program)
											{
												// Figure out what the compiled binary will be called so that we can point the IDE to the correct file
												if (ProjectTarget.TargetRules.Type != TargetType.Game)
												{
													// Only if shared - unique retains the Target Name
													if (ProjectTarget.TargetRules.BuildEnvironment == TargetBuildEnvironment.Shared)
													{
														ExeName = "Unreal" + ProjectTarget.TargetRules.Type.ToString();
													}
												}
											}

											// Get the output directory
											DirectoryReference OutputDirectory = DirectoryReference.Combine(RootDirectory, "Binaries");
											DirectoryReference MacBinaryDir = DirectoryReference.Combine(OutputDirectory, "Mac");
											DirectoryReference IOSBinaryDir = DirectoryReference.Combine(OutputDirectory, "IOS");
											DirectoryReference TVOSBinaryDir = DirectoryReference.Combine(OutputDirectory, "TVOS");
											if (!String.IsNullOrEmpty(ProjectTarget.TargetRules.ExeBinariesSubFolder))
											{
												MacBinaryDir = DirectoryReference.Combine(MacBinaryDir, ProjectTarget.TargetRules.ExeBinariesSubFolder);
												IOSBinaryDir = DirectoryReference.Combine(IOSBinaryDir, ProjectTarget.TargetRules.ExeBinariesSubFolder);
												TVOSBinaryDir = DirectoryReference.Combine(TVOSBinaryDir, ProjectTarget.TargetRules.ExeBinariesSubFolder);
											}

											if (BuildPlatform.Platform == UnrealTargetPlatform.Mac)
											{
												string MacExecutableName = MakeExecutableFileName(ExeName, UnrealTargetPlatform.Mac, Configuration, ProjectTarget.TargetRules.Architectures, ProjectTarget.TargetRules.UndecoratedConfiguration);
												string IOSExecutableName = MacExecutableName.Replace("-Mac-", "-IOS-", StringComparison.Ordinal);
												string TVOSExecutableName = MacExecutableName.Replace("-Mac-", "-TVOS-", StringComparison.Ordinal);
												BuildConfigs.Add(new XcodeBuildConfig(ConfigName, TargetName, FileReference.Combine(MacBinaryDir, MacExecutableName), FileReference.Combine(IOSBinaryDir, IOSExecutableName), FileReference.Combine(TVOSBinaryDir, TVOSExecutableName), ProjectTarget, Configuration));
											}
											else if (BuildPlatform.Platform == UnrealTargetPlatform.IOS || BuildPlatform.Platform == UnrealTargetPlatform.TVOS)
											{
												string IOSExecutableName = MakeExecutableFileName(ExeName, UnrealTargetPlatform.IOS, Configuration, ProjectTarget.TargetRules.Architectures, ProjectTarget.TargetRules.UndecoratedConfiguration);
												string TVOSExecutableName = IOSExecutableName.Replace("-IOS-", "-TVOS-", StringComparison.Ordinal);
												//string MacExecutableName = IOSExecutableName.Replace("-IOS-", "-Mac-", StringComparison.Ordinal);
												BuildConfigs.Add(new XcodeBuildConfig(ConfigName, TargetName, FileReference.Combine(MacBinaryDir, IOSExecutableName), FileReference.Combine(IOSBinaryDir, IOSExecutableName), FileReference.Combine(TVOSBinaryDir, TVOSExecutableName), ProjectTarget, Configuration));
											}
										}
									}
								}
							}
						}
					}
				}
			}

			return BuildConfigs;
		}

		private static string MakeExecutableFileName(string BinaryName, UnrealTargetPlatform Platform, UnrealTargetConfiguration Configuration, UnrealArchitectures Architectures, UnrealTargetConfiguration UndecoratedConfiguration)
		{
			StringBuilder Result = new StringBuilder();

			Result.Append(BinaryName);

			if (Configuration != UndecoratedConfiguration)
			{
				Result.AppendFormat("-{0}-{1}", Platform.ToString(), Configuration.ToString());
			}

			if (UnrealArchitectureConfig.ForPlatform(Platform).RequiresArchitectureFilenames(Architectures))
			{
				Result.Append(Architectures.ToString());
			}

			return Result.ToString();
		}

		private FileReference GetUserSchemeManagementFilePath()
		{
			return new FileReference(ProjectFilePath.FullName + "/xcuserdata/" + Environment.UserName + ".xcuserdatad/xcschemes/xcschememanagement.plist");
		}

		private DirectoryReference GetProjectSchemeDirectory()
		{
			return new DirectoryReference(ProjectFilePath.FullName + "/xcshareddata/xcschemes");
		}

		private FileReference GetProjectSchemeFilePathForTarget(string TargetName)
		{
			return FileReference.Combine(GetProjectSchemeDirectory(), TargetName + ".xcscheme");
		}

		private void WriteSchemeFile(string TargetName, string TargetGuid, string BuildTargetGuid, string IndexTargetGuid, bool bHasEditorConfiguration, string GameProjectPath)
		{

			FileReference SchemeFilePath = GetProjectSchemeFilePathForTarget(TargetName);

			DirectoryReference.CreateDirectory(SchemeFilePath.Directory);

			string? OldCommandLineArguments = null;
			if (FileReference.Exists(SchemeFilePath))
			{
				string OldContents = File.ReadAllText(SchemeFilePath.FullName);
				int OldCommandLineArgumentsStart = OldContents.IndexOf("<CommandLineArguments>", StringComparison.Ordinal) + "<CommandLineArguments>".Length;
				int OldCommandLineArgumentsEnd = OldContents.IndexOf("</CommandLineArguments>", StringComparison.Ordinal);
				if (OldCommandLineArgumentsStart != -1 && OldCommandLineArgumentsEnd != -1)
				{
					OldCommandLineArguments = OldContents.Substring(OldCommandLineArgumentsStart, OldCommandLineArgumentsEnd - OldCommandLineArgumentsStart);
				}
			}

			string DefaultConfiguration = bMakeProjectPerTarget && bHasEditorConfiguration && !XcodeProjectFileGenerator.bGenerateRunOnlyProject ? "Development Editor" : "Development";

			StringBuilder Content = new StringBuilder();

			Content.Append("<?xml version=\"1.0\" encoding=\"UTF-8\"?>" + ProjectFileGenerator.NewLine);
			Content.Append("<Scheme" + ProjectFileGenerator.NewLine);
			Content.Append("   LastUpgradeVersion = \"2000\"" + ProjectFileGenerator.NewLine);
			Content.Append("   version = \"1.3\">" + ProjectFileGenerator.NewLine);
			Content.Append("   <BuildAction" + ProjectFileGenerator.NewLine);
			Content.Append("      parallelizeBuildables = \"YES\"" + ProjectFileGenerator.NewLine);
			Content.Append("      buildImplicitDependencies = \"YES\">" + ProjectFileGenerator.NewLine);
			Content.Append("      <BuildActionEntries>" + ProjectFileGenerator.NewLine);
			Content.Append("         <BuildActionEntry" + ProjectFileGenerator.NewLine);
			Content.Append("            buildForTesting = \"YES\"" + ProjectFileGenerator.NewLine);
			Content.Append("            buildForRunning = \"YES\"" + ProjectFileGenerator.NewLine);
			Content.Append("            buildForProfiling = \"YES\"" + ProjectFileGenerator.NewLine);
			Content.Append("            buildForArchiving = \"YES\"" + ProjectFileGenerator.NewLine);
			Content.Append("            buildForAnalyzing = \"YES\">" + ProjectFileGenerator.NewLine);
			Content.Append("            <BuildableReference" + ProjectFileGenerator.NewLine);
			Content.Append("               BuildableIdentifier = \"primary\"" + ProjectFileGenerator.NewLine);
			Content.Append("               BlueprintIdentifier = \"" + TargetGuid + "\"" + ProjectFileGenerator.NewLine);
			Content.Append("               BuildableName = \"" + TargetName + ".app\"" + ProjectFileGenerator.NewLine);
			Content.Append("               BlueprintName = \"" + TargetName + "\"" + ProjectFileGenerator.NewLine);
			Content.Append("               ReferencedContainer = \"container:" + TargetName + ".xcodeproj\">" + ProjectFileGenerator.NewLine);
			Content.Append("            </BuildableReference>" + ProjectFileGenerator.NewLine);
			Content.Append("         </BuildActionEntry>" + ProjectFileGenerator.NewLine);
			Content.Append("      </BuildActionEntries>" + ProjectFileGenerator.NewLine);
			Content.Append("   </BuildAction>" + ProjectFileGenerator.NewLine);
			Content.Append("   <TestAction" + ProjectFileGenerator.NewLine);
			Content.Append("      buildConfiguration = \"" + DefaultConfiguration + "\"" + ProjectFileGenerator.NewLine);
			Content.Append("      selectedDebuggerIdentifier = \"Xcode.DebuggerFoundation.Debugger.LLDB\"" + ProjectFileGenerator.NewLine);
			Content.Append("      selectedLauncherIdentifier = \"Xcode.DebuggerFoundation.Launcher.LLDB\"" + ProjectFileGenerator.NewLine);
			Content.Append("      shouldUseLaunchSchemeArgsEnv = \"YES\">" + ProjectFileGenerator.NewLine);
			Content.Append("      <Testables>" + ProjectFileGenerator.NewLine);
			Content.Append("      </Testables>" + ProjectFileGenerator.NewLine);
			Content.Append("      <MacroExpansion>" + ProjectFileGenerator.NewLine);
			Content.Append("            <BuildableReference" + ProjectFileGenerator.NewLine);
			Content.Append("               BuildableIdentifier = \"primary\"" + ProjectFileGenerator.NewLine);
			Content.Append("               BlueprintIdentifier = \"" + TargetGuid + "\"" + ProjectFileGenerator.NewLine);
			Content.Append("               BuildableName = \"" + TargetName + ".app\"" + ProjectFileGenerator.NewLine);
			Content.Append("               BlueprintName = \"" + TargetName + "\"" + ProjectFileGenerator.NewLine);
			Content.Append("               ReferencedContainer = \"container:" + TargetName + ".xcodeproj\">" + ProjectFileGenerator.NewLine);
			Content.Append("            </BuildableReference>" + ProjectFileGenerator.NewLine);
			Content.Append("      </MacroExpansion>" + ProjectFileGenerator.NewLine);
			Content.Append("      <AdditionalOptions>" + ProjectFileGenerator.NewLine);
			Content.Append("      </AdditionalOptions>" + ProjectFileGenerator.NewLine);
			Content.Append("   </TestAction>" + ProjectFileGenerator.NewLine);
			Content.Append("   <LaunchAction" + ProjectFileGenerator.NewLine);
			Content.Append("      buildConfiguration = \"" + DefaultConfiguration + "\"" + ProjectFileGenerator.NewLine);
			Content.Append("      selectedDebuggerIdentifier = \"Xcode.DebuggerFoundation.Debugger.LLDB\"" + ProjectFileGenerator.NewLine);
			Content.Append("      selectedLauncherIdentifier = \"Xcode.DebuggerFoundation.Launcher.LLDB\"" + ProjectFileGenerator.NewLine);
			Content.Append("      launchStyle = \"0\"" + ProjectFileGenerator.NewLine);
			Content.Append("      useCustomWorkingDirectory = \"NO\"" + ProjectFileGenerator.NewLine);
			Content.Append("      ignoresPersistentStateOnLaunch = \"NO\"" + ProjectFileGenerator.NewLine);
			Content.Append("      debugDocumentVersioning = \"NO\"" + ProjectFileGenerator.NewLine);
			Content.Append("      debugServiceExtension = \"internal\"" + ProjectFileGenerator.NewLine);
			Content.Append("      allowLocationSimulation = \"YES\">" + ProjectFileGenerator.NewLine);
			Content.Append("      <BuildableProductRunnable" + ProjectFileGenerator.NewLine);
			Content.Append("         runnableDebuggingMode = \"0\">" + ProjectFileGenerator.NewLine);
			Content.Append("            <BuildableReference" + ProjectFileGenerator.NewLine);
			Content.Append("               BuildableIdentifier = \"primary\"" + ProjectFileGenerator.NewLine);
			Content.Append("               BlueprintIdentifier = \"" + TargetGuid + "\"" + ProjectFileGenerator.NewLine);
			Content.Append("               BuildableName = \"" + TargetName + ".app\"" + ProjectFileGenerator.NewLine);
			Content.Append("               BlueprintName = \"" + TargetName + "\"" + ProjectFileGenerator.NewLine);
			Content.Append("               ReferencedContainer = \"container:" + TargetName + ".xcodeproj\">" + ProjectFileGenerator.NewLine);
			Content.Append("            </BuildableReference>" + ProjectFileGenerator.NewLine);
			Content.Append("      </BuildableProductRunnable>" + ProjectFileGenerator.NewLine);
			if (String.IsNullOrEmpty(OldCommandLineArguments))
			{
				if (bHasEditorConfiguration && !String.IsNullOrEmpty(GameProjectPath))
				{
					Content.Append("      <CommandLineArguments>" + ProjectFileGenerator.NewLine);
					if (IsForeignProject)
					{
						Content.Append("         <CommandLineArgument" + ProjectFileGenerator.NewLine);
						Content.Append("            argument = \"&quot;" + GameProjectPath + "&quot;\"" + ProjectFileGenerator.NewLine);
						Content.Append("            isEnabled = \"YES\">" + ProjectFileGenerator.NewLine);
						Content.Append("         </CommandLineArgument>" + ProjectFileGenerator.NewLine);
					}
					else
					{
						Content.Append("         <CommandLineArgument" + ProjectFileGenerator.NewLine);
						Content.Append("            argument = \"" + Path.GetFileNameWithoutExtension(GameProjectPath) + "\"" + ProjectFileGenerator.NewLine);
						Content.Append("            isEnabled = \"YES\">" + ProjectFileGenerator.NewLine);
						Content.Append("         </CommandLineArgument>" + ProjectFileGenerator.NewLine);
					}
					// Always add a configuration argument
					Content.Append("         <CommandLineArgument" + ProjectFileGenerator.NewLine);
					Content.Append("            argument = \"-RunConfig=$(Configuration)\"" + ProjectFileGenerator.NewLine);
					Content.Append("            isEnabled = \"YES\">" + ProjectFileGenerator.NewLine);
					Content.Append("         </CommandLineArgument>" + ProjectFileGenerator.NewLine);
					Content.Append("      </CommandLineArguments>" + ProjectFileGenerator.NewLine);
				}
			}
			else
			{
				Content.Append("      <CommandLineArguments>" + OldCommandLineArguments + "</CommandLineArguments>");
			}
			Content.Append("      <AdditionalOptions>" + ProjectFileGenerator.NewLine);
			Content.Append("      </AdditionalOptions>" + ProjectFileGenerator.NewLine);
			Content.Append("   </LaunchAction>" + ProjectFileGenerator.NewLine);
			Content.Append("   <ProfileAction" + ProjectFileGenerator.NewLine);
			Content.Append("      buildConfiguration = \"" + DefaultConfiguration + "\"" + ProjectFileGenerator.NewLine);
			Content.Append("      shouldUseLaunchSchemeArgsEnv = \"YES\"" + ProjectFileGenerator.NewLine);
			Content.Append("      savedToolIdentifier = \"\"" + ProjectFileGenerator.NewLine);
			Content.Append("      useCustomWorkingDirectory = \"NO\"" + ProjectFileGenerator.NewLine);
			Content.Append("      debugDocumentVersioning = \"NO\">" + ProjectFileGenerator.NewLine);
			Content.Append("      <BuildableProductRunnable" + ProjectFileGenerator.NewLine);
			Content.Append("         runnableDebuggingMode = \"0\">" + ProjectFileGenerator.NewLine);
			Content.Append("            <BuildableReference" + ProjectFileGenerator.NewLine);
			Content.Append("               BuildableIdentifier = \"primary\"" + ProjectFileGenerator.NewLine);
			Content.Append("               BlueprintIdentifier = \"" + TargetGuid + "\"" + ProjectFileGenerator.NewLine);
			Content.Append("               BuildableName = \"" + TargetName + ".app\"" + ProjectFileGenerator.NewLine);
			Content.Append("               BlueprintName = \"" + TargetName + "\"" + ProjectFileGenerator.NewLine);
			Content.Append("               ReferencedContainer = \"container:" + TargetName + ".xcodeproj\">" + ProjectFileGenerator.NewLine);
			Content.Append("            </BuildableReference>" + ProjectFileGenerator.NewLine);
			Content.Append("      </BuildableProductRunnable>" + ProjectFileGenerator.NewLine);
			Content.Append("   </ProfileAction>" + ProjectFileGenerator.NewLine);
			Content.Append("   <AnalyzeAction" + ProjectFileGenerator.NewLine);
			Content.Append("      buildConfiguration = \"" + DefaultConfiguration + "\">" + ProjectFileGenerator.NewLine);
			Content.Append("   </AnalyzeAction>" + ProjectFileGenerator.NewLine);
			Content.Append("   <ArchiveAction" + ProjectFileGenerator.NewLine);
			Content.Append("      buildConfiguration = \"" + DefaultConfiguration + "\"" + ProjectFileGenerator.NewLine);
			Content.Append("      revealArchiveInOrganizer = \"YES\">" + ProjectFileGenerator.NewLine);
			Content.Append("   </ArchiveAction>" + ProjectFileGenerator.NewLine);
			Content.Append("</Scheme>" + ProjectFileGenerator.NewLine);

			File.WriteAllText(SchemeFilePath.FullName, Content.ToString(), new UTF8Encoding());

			Content.Clear();

			Content.Append("<?xml version=\"1.0\" encoding=\"UTF-8\"?>" + ProjectFileGenerator.NewLine);
			Content.Append("<!DOCTYPE plist PUBLIC \"-//Apple//DTD PLIST 1.0//EN\" \"http://www.apple.com/DTDs/PropertyList-1.0.dtd\">" + ProjectFileGenerator.NewLine);
			Content.Append("<plist version=\"1.0\">" + ProjectFileGenerator.NewLine);
			Content.Append("<dict>" + ProjectFileGenerator.NewLine);
			Content.Append("\t<key>SchemeUserState</key>" + ProjectFileGenerator.NewLine);
			Content.Append("\t<dict>" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t<key>" + TargetName + ".xcscheme_^#shared#^_</key>" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t<dict>" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\t<key>orderHint</key>" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\t<integer>1</integer>" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t</dict>" + ProjectFileGenerator.NewLine);
			Content.Append("\t</dict>" + ProjectFileGenerator.NewLine);
			Content.Append("\t<key>SuppressBuildableAutocreation</key>" + ProjectFileGenerator.NewLine);
			Content.Append("\t<dict>" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t<key>" + TargetGuid + "</key>" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t<dict>" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\t<key>primary</key>" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\t<true/>" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t</dict>" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t<key>" + BuildTargetGuid + "</key>" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t<dict>" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\t<key>primary</key>" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\t<true/>" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t</dict>" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t<key>" + IndexTargetGuid + "</key>" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t<dict>" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\t<key>primary</key>" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t\t<true/>" + ProjectFileGenerator.NewLine);
			Content.Append("\t\t</dict>" + ProjectFileGenerator.NewLine);
			Content.Append("\t</dict>" + ProjectFileGenerator.NewLine);
			Content.Append("</dict>" + ProjectFileGenerator.NewLine);
			Content.Append("</plist>" + ProjectFileGenerator.NewLine);

			FileReference ManagementFile = GetUserSchemeManagementFilePath();
			if (!DirectoryReference.Exists(ManagementFile.Directory))
			{
				DirectoryReference.CreateDirectory(ManagementFile.Directory);
			}

			File.WriteAllText(ManagementFile.FullName, Content.ToString(), new UTF8Encoding());
		}

		public static IEnumerable<UnrealTargetConfiguration> GetSupportedConfigurations()
		{
			return new UnrealTargetConfiguration[] {
				UnrealTargetConfiguration.Debug,
				UnrealTargetConfiguration.DebugGame,
				UnrealTargetConfiguration.Development,
				UnrealTargetConfiguration.Test,
				UnrealTargetConfiguration.Shipping
			};
		}

		public bool CanBuildProjectLocally()
		{
			foreach (Project ProjectTarget in ProjectTargets)
			{
				foreach (UnrealTargetPlatform Platform in XcodeProjectFileGenerator.XcodePlatforms)
				{
					foreach (UnrealTargetConfiguration Config in GetSupportedConfigurations())
					{
						if (MSBuildProjectFile.IsValidProjectPlatformAndConfiguration(ProjectTarget, Platform, Config))
						{
							return true;
						}
					}
				}
			}

			return false;
		}

		/// Implements Project interface
		public override bool WriteProjectFile(List<UnrealTargetPlatform> InPlatforms, List<UnrealTargetConfiguration> InConfigurations, PlatformProjectGeneratorCollection PlatformProjectGenerators, ILogger Logger)
		{
			bool bSuccess = true;

			string TargetName = ProjectFilePath.GetFileNameWithoutExtension();
			string TargetGuid = XcodeProjectFileGenerator.MakeXcodeGuid();
			string TargetConfigListGuid = XcodeProjectFileGenerator.MakeXcodeGuid();
			string TargetDependencyGuid = XcodeProjectFileGenerator.MakeXcodeGuid();
			string TargetProxyGuid = XcodeProjectFileGenerator.MakeXcodeGuid();
			string TargetAppGuid = XcodeProjectFileGenerator.MakeXcodeGuid();
			string BuildTargetName = TargetName + "_Build";
			string BuildTargetGuid = XcodeProjectFileGenerator.MakeXcodeGuid();
			string BuildTargetConfigListGuid = XcodeProjectFileGenerator.MakeXcodeGuid();
			string IndexTargetName = TargetName + "_Index";
			string IndexTargetGuid = XcodeProjectFileGenerator.MakeXcodeGuid();
			string IndexTargetConfigListGuid = XcodeProjectFileGenerator.MakeXcodeGuid();
			string ProjectGuid = XcodeProjectFileGenerator.MakeXcodeGuid();
			string ProjectConfigListGuid = XcodeProjectFileGenerator.MakeXcodeGuid();
			string MainGroupGuid = XcodeProjectFileGenerator.MakeXcodeGuid();
			string ProductRefGroupGuid = XcodeProjectFileGenerator.MakeXcodeGuid();
			string SourcesBuildPhaseGuid = XcodeProjectFileGenerator.MakeXcodeGuid();
			string CopyExtensionsBuildPhaseGuid = XcodeProjectFileGenerator.MakeXcodeGuid();
			string ShellScriptSectionGuid = XcodeProjectFileGenerator.MakeXcodeGuid();

			// Figure out all the desired configurations
			List<XcodeBuildConfig> BuildConfigs = GetSupportedBuildConfigs(InPlatforms, InConfigurations);
			if (BuildConfigs.Count == 0)
			{
				return true;
			}

			// if a single project was specified then start with that, this will allow content-only projects to have
			// their BundleID set in the stub xcode project when codesigning, which Xcode needs to have now)
			// note that usually SingleGameProject is null
			FileReference? GameProjectPath = XcodeProjectFileGenerator.SingleGameProject;
			foreach (Project Target in ProjectTargets)
			{
				if (Target.UnrealProjectFilePath != null)
				{
					GameProjectPath = Target.UnrealProjectFilePath;
					break;
				}
			}

			bool bHasEditorConfiguration = false;
			bool bIsAppBundle = true;

			Dictionary<string, XcodeBuildConfig> ProjectBuildConfigs = new Dictionary<string, XcodeBuildConfig>();
			Dictionary<string, XcodeBuildConfig> TargetBuildConfigs = new Dictionary<string, XcodeBuildConfig>();
			Dictionary<string, XcodeBuildConfig> BuildTargetBuildConfigs = new Dictionary<string, XcodeBuildConfig>();
			Dictionary<string, XcodeBuildConfig> IndexTargetBuildConfigs = new Dictionary<string, XcodeBuildConfig>();
			foreach (XcodeBuildConfig Config in BuildConfigs)
			{
				ProjectBuildConfigs[XcodeProjectFileGenerator.MakeXcodeGuid()] = Config;
				TargetBuildConfigs[XcodeProjectFileGenerator.MakeXcodeGuid()] = Config;
				BuildTargetBuildConfigs[XcodeProjectFileGenerator.MakeXcodeGuid()] = Config;
				IndexTargetBuildConfigs[XcodeProjectFileGenerator.MakeXcodeGuid()] = Config;

				if (Config.ProjectTarget!.TargetRules!.Type == TargetType.Editor)
				{
					bHasEditorConfiguration = true;
				}
				else if (Config.ProjectTarget.TargetRules.bIsBuildingConsoleApplication || Config.ProjectTarget.TargetRules.bShouldCompileAsDLL)
				{
					bIsAppBundle = false;
				}
			}

			StringBuilder PBXBuildFileSection = new StringBuilder();
			StringBuilder PBXFileReferenceSection = new StringBuilder();
			StringBuilder PBXSourcesBuildPhaseSection = new StringBuilder();
			StringBuilder PBXCopyExtensionsBuildPhaseSection = new StringBuilder();
			StringBuilder PBXResourcesBuildPhaseSection = new StringBuilder();
			List<XcodeExtensionInfo> AllExtensions = new List<XcodeExtensionInfo>();
			GenerateSectionsWithSourceFiles(PBXBuildFileSection, PBXFileReferenceSection, PBXSourcesBuildPhaseSection, TargetAppGuid, TargetName, bIsAppBundle);
			GenerateSectionsWithExtensions(PBXBuildFileSection, PBXFileReferenceSection, PBXCopyExtensionsBuildPhaseSection, PBXResourcesBuildPhaseSection, AllExtensions, GameProjectPath, BuildConfigs);

			StringBuilder ProjectFileContent = new StringBuilder();

			ProjectFileContent.Append("// !$*UTF8*$!" + ProjectFileGenerator.NewLine);
			ProjectFileContent.Append("{" + ProjectFileGenerator.NewLine);
			ProjectFileContent.Append("\tarchiveVersion = 1;" + ProjectFileGenerator.NewLine);
			ProjectFileContent.Append("\tclasses = {" + ProjectFileGenerator.NewLine);
			ProjectFileContent.Append("\t};" + ProjectFileGenerator.NewLine);
			ProjectFileContent.Append("\tobjectVersion = 46;" + ProjectFileGenerator.NewLine);
			ProjectFileContent.Append("\tobjects = {" + ProjectFileGenerator.NewLine + ProjectFileGenerator.NewLine);

			AppendBuildFileSection(ProjectFileContent, PBXBuildFileSection);
			AppendFileReferenceSection(ProjectFileContent, PBXFileReferenceSection);
			AppendSourcesBuildPhaseSection(ProjectFileContent, PBXSourcesBuildPhaseSection, SourcesBuildPhaseGuid);
			AppendCopyExtensionsBuildPhaseSection(ProjectFileContent, PBXCopyExtensionsBuildPhaseSection, CopyExtensionsBuildPhaseGuid);
			ProjectFileContent.Append(PBXResourcesBuildPhaseSection);
			AppendContainerItemProxySection(ProjectFileContent, BuildTargetName, BuildTargetGuid, TargetProxyGuid, ProjectGuid);
			if (!XcodeProjectFileGenerator.bGenerateRunOnlyProject)
			{
				AppendTargetDependencySection(ProjectFileContent, BuildTargetName, BuildTargetGuid, TargetDependencyGuid, TargetProxyGuid);
			}
			foreach (XcodeExtensionInfo EI in AllExtensions)
			{
				AppendContainerItemProxySection(ProjectFileContent, EI.Name, EI.TargetGuid, EI.TargetProxyGuid, ProjectGuid);
				AppendTargetDependencySection(ProjectFileContent, EI.Name, EI.TargetGuid, EI.TargetDependencyGuid, EI.TargetProxyGuid);
			}
			AppendGroupSection(ProjectFileContent, MainGroupGuid, ProductRefGroupGuid, TargetAppGuid, TargetName, AllExtensions);
			AppendLegacyTargetSection(ProjectFileContent, BuildTargetName, BuildTargetGuid, BuildTargetConfigListGuid, GameProjectPath);
			AppendRunTargetSection(ProjectFileContent, TargetName, TargetGuid, TargetConfigListGuid, TargetDependencyGuid, TargetAppGuid, CopyExtensionsBuildPhaseGuid, ShellScriptSectionGuid, AllExtensions, bIsAppBundle);
			AppendIndexTargetSection(ProjectFileContent, IndexTargetName, IndexTargetGuid, IndexTargetConfigListGuid, SourcesBuildPhaseGuid);
			AppendExtensionTargetSections(ProjectFileContent, AllExtensions);
			AppendProjectSection(ProjectFileContent, TargetName, TargetGuid, BuildTargetName, BuildTargetGuid, IndexTargetName, IndexTargetGuid, MainGroupGuid, ProductRefGroupGuid, ProjectGuid, ProjectConfigListGuid, GameProjectPath, AllExtensions);
			AppendXCBuildConfigurationSection(ProjectFileContent, ProjectBuildConfigs, TargetBuildConfigs, BuildTargetBuildConfigs, IndexTargetBuildConfigs, GameProjectPath, AllExtensions, Logger);
			AppendXCConfigurationListSection(ProjectFileContent, TargetName, BuildTargetName, IndexTargetName, ProjectConfigListGuid, ProjectBuildConfigs,
				TargetConfigListGuid, TargetBuildConfigs, BuildTargetConfigListGuid, BuildTargetBuildConfigs, IndexTargetConfigListGuid, IndexTargetBuildConfigs, AllExtensions);
			AppendShellScriptSection(ProjectFileContent, ShellScriptSectionGuid, GameProjectPath);

			ProjectFileContent.Append("\t};" + ProjectFileGenerator.NewLine);
			ProjectFileContent.Append("\trootObject = " + ProjectGuid + " /* Project object */;" + ProjectFileGenerator.NewLine);
			ProjectFileContent.Append("}" + ProjectFileGenerator.NewLine);

			if (bSuccess)
			{
				FileReference PBXProjFilePath = ProjectFilePath + "/project.pbxproj";
				bSuccess = ProjectFileGenerator.WriteFileIfChanged(PBXProjFilePath.FullName, ProjectFileContent.ToString(), Logger, new UTF8Encoding());
			}

			bool bNeedScheme = CanBuildProjectLocally();

			if (bNeedScheme)
			{
				if (bSuccess)
				{
					WriteSchemeFile(TargetName, TargetGuid, BuildTargetGuid, IndexTargetGuid, bHasEditorConfiguration, GameProjectPath != null ? GameProjectPath.FullName : "");
				}
			}
			else
			{
				// clean this up because we don't want it persisting if we narrow our project list
				DirectoryReference SchemeDir = GetProjectSchemeDirectory();

				if (DirectoryReference.Exists(SchemeDir))
				{
					DirectoryReference.Delete(SchemeDir, true);
				}
			}

			return bSuccess;
		}
	}
}
