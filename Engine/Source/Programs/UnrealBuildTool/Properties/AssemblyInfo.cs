// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Reflection;
using System.Runtime.CompilerServices;

// General Information about an assembly is controlled through the following 
// set of attributes. Change these attribute values to modify the information
// associated with an assembly.
[assembly: AssemblyTitle("UnrealBuildTool")]
[assembly: AssemblyDescription("A tool to build Unreal Engine")]
[assembly: AssemblyConfiguration("")]

[assembly: InternalsVisibleTo("UnrealBuildTool.Tests")]

// We are not CLS compliant, as we make extensive use of e.g. uint.
[assembly: CLSCompliant(false)]
