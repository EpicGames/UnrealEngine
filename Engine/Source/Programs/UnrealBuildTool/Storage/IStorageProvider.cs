// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using System.Threading;
using EpicGames.Core;

namespace UnrealBuildTool.Storage
{
	/// <summary>
	/// Represents a read/write store addressed by a hash representing the contents to be stored.
	/// </summary>
	interface IStorageProvider
	{
		/// <summary>
		/// Attempts to open a stream from the output cache for reading.
		/// </summary>
		/// <returns>A readable stream of the stored content for this hash, if it exists. <see langword="null" /> if it does not exist.</returns>
		Stream? CreateReader(IoHash hash);

		/// <summary>
		/// Opens a stream for writing into the cache. The stream is transactional and will be committed on dispose unless cancelled.
		/// </summary>
		/// <param name="hash">The hash addressing the contents to write.</param>
		/// <param name="cancellationToken">A cancellation token, which, if cancelled before the returned stream is disposed,
		/// will cancel the transaction.</param>
		Stream CreateWriter(IoHash hash, CancellationToken cancellationToken);
	}
}
