// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Threading;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool.Storage.Impl
{
	/// <summary>
	/// Storage provider which uses the filesystem.
	/// </summary>
	internal class FileSystemStorageProvider : IStorageProvider
	{
		/// <summary>
		/// Cached copy of the current process ID.
		/// </summary>
		private static readonly int ProcessId = Environment.ProcessId;

		/// <summary>
		/// Represents a write transaction to the cache. If not cancelled, the transaction will be committed on dispose.
		/// </summary>
		private class TransactionalWriterStream(FileReference finalLocation, FileReference tempLocation, CancellationToken cancellationToken)
			: FileStream(tempLocation.FullName, FileMode.Create, FileAccess.Write, FileShare.Read)
		{
			protected override void Dispose(bool disposing)
			{
				base.Dispose(disposing);

				if (disposing && !cancellationToken.IsCancellationRequested)
				{
					try
					{
						FileReference.Move(tempLocation, finalLocation, overwrite: true);
					}
					// Try to handle an odd bug where, on Mac OS specifically, the temp file seems to go missing *during* the move call.
					catch (FileNotFoundException ex) when (OperatingSystem.IsMacOS())
					{
						bool tempFileExists = FileReference.Exists(tempLocation);
						// Check to see if the destination file exists, in case that proves to be a pattern.
						bool destFileExists = FileReference.Exists(finalLocation);
						// This log also will give us a timestamp, to see whether there's a pattern as to when in the compile this happens
						// (right at the start or right at the end would be conspicuous).
						Log.Logger.LogInformation(
							ex,
							"Caught the macOS FileNotFoundException issue! Temp file path: {TempFilePath}, temp file exists: {TempFileExists}, dest file path: {DestFilePath}, dest file exists: {DestFileExists}",
							tempLocation,
							tempFileExists,
							finalLocation,
							destFileExists);
						// Deleting a nonexistent file is idempotent.
						FileReference.Delete(tempLocation);
						// Don't throw, as it's disrupting build health.
					}
					catch
					{
						FileReference.Delete(tempLocation);
						throw;
					}
				}
			}
		}

		/// <inheritdoc />
		public Stream? CreateReader(IoHash hash)
		{
			FileReference fileRef = GetFileForDigest(hash);
			if (FileReference.Exists(fileRef))
			{
				return FileReference.Open(fileRef, FileMode.Open, FileAccess.Read, FileShare.Read | FileShare.Delete);
			}

			return null;
		}

		/// <inheritdoc />
		public Stream CreateWriter(IoHash hash, CancellationToken cancellationToken)
		{
			FileReference finalLocation = GetFileForDigest(hash);
			FileReference tempLocation = new(String.Format("{0}.{1}", finalLocation.FullName, ProcessId));
			DirectoryReference.CreateDirectory(finalLocation.Directory);
			return new TransactionalWriterStream(finalLocation, tempLocation, cancellationToken);
		}

		/// <summary>
		/// Gets the filename on disk to use for a particular hash.
		/// </summary>
		/// <param name="hash">The hash to find a filename for</param>
		/// <returns>Filename to use for the given hash.</returns>
		private static FileReference GetFileForDigest(IoHash hash)
		{
			string DigestText = hash.ToString();
			return FileReference.Combine(Unreal.EngineDirectory, "Saved", "UnrealBuildTool", "Cache", String.Format("{0}/{1}/{2}/{3}.bin", DigestText[0], DigestText[1], DigestText[2], DigestText));
		}
	}
}
