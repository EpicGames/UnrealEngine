// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using System.Linq;
using System.Text;

namespace UnrealBuildTool
{
	/// <summary>
	/// Parser for ASN.1 BER formats (eg. iOS mobile provisions)
	/// </summary>
	static class Asn
	{
		public enum FieldTag
		{
			EOC = 0,
			BOOLEAN = 1,
			INTEGER = 2,
			OCTET_STRING = 4,
			NULL = 5,
			OBJECT_IDENTIFIER = 6,
			SEQUENCE = 16,
			SET = 17,
			PRINTABLE_STRING = 19,
		}

		public struct FieldInfo
		{
			public int TagClass;
			public bool bPrimitive;
			public FieldTag Tag;
			public int Length;
		}

		public static class ObjectIdentifier
		{
			public static readonly int[] CountryName = new int[] { 2, 5, 4, 6 };
			public static readonly int[] OrganizationName = new int[] { 2, 5, 4, 10 };
			public static readonly int[] Pkcs7_SignedData = new int[] { 1, 2, 840, 113549, 1, 7, 2 };
			public static readonly int[] Pkcs7_Data = new int[] { 1, 2, 840, 113549, 1, 7, 1 };
		}

		static readonly List<KeyValuePair<int[], string>> KnownObjectIdentifiers = new List<KeyValuePair<int[], string>>
		{
			new KeyValuePair<int[], string>(ObjectIdentifier.CountryName, "CountryName"),
			new KeyValuePair<int[], string>(ObjectIdentifier.OrganizationName, "OrganizationName"),
			new KeyValuePair<int[], string>(ObjectIdentifier.Pkcs7_SignedData, "Pkcs7-SignedData"),
			new KeyValuePair<int[], string>(ObjectIdentifier.Pkcs7_Data, "Pkcs7-Data"),
		};

		public static FieldInfo ReadField(BinaryReader Reader)
		{
			// Read the type and length
			int Type = Reader.ReadByte();
			int Length = ReadLength(Reader);

			// Unpack the type
			FieldInfo Field = new FieldInfo();
			Field.TagClass = (Type >> 6);
			Field.bPrimitive = (Type & 0x20) == 0;
			Field.Tag = (FieldTag)(Type & 0x1f);
			Field.Length = Length;
			return Field;
		}

		public static void SkipValue(BinaryReader Reader, FieldInfo Field)
		{
			if (Field.bPrimitive)
			{
				Reader.BaseStream.Seek(Field.Length, SeekOrigin.Current);
			}
		}

		public static int ReadInteger(BinaryReader Reader, int Length)
		{
			int Value = 0;
			for (int Idx = 0; Idx < Length; Idx++)
			{
				Value = (Value << 8) | (int)Reader.ReadByte();
			}
			return Value;
		}

		public static int ReadLength(BinaryReader Reader)
		{
			int Count = (int)Reader.ReadByte();
			if (Count <= 0x7f)
			{
				return Count;
			}
			else
			{
				return ReadInteger(Reader, Count & 0x7f);
			}
		}

		public static int[] ReadObjectIdentifier(BinaryReader Reader, int Length)
		{
			byte[] Data = Reader.ReadBytes(Length);

			List<int> Values = new List<int>();
			Values.Add((int)Data[0] / 40);
			Values.Add((int)Data[0] % 40);
			for (int Idx = 1; Idx < Data.Length; Idx++)
			{
				int Value = (int)Data[Idx] & 0x7f;
				while (((int)Data[Idx] & 0x80) != 0)
				{
					Value = (Value << 7) | ((int)Data[++Idx] & 0x7f);
				}
				Values.Add(Value);
			}
			return Values.ToArray();
		}

		public static string GetObjectIdentifierName(int[] Values)
		{
			string? Description;
			if (!TryGetObjectIdentifierName(Values, out Description))
			{
				Description = String.Format("Unknown ({0})", String.Join(", ", Values.Select(x => x.ToString())));
			}
			return Description;
		}

		public static bool TryGetObjectIdentifierName(int[] Values, [NotNullWhen(true)] out string? Name)
		{
			foreach (KeyValuePair<int[], string> KnownObjectIdentifier in KnownObjectIdentifiers)
			{
				if (Enumerable.SequenceEqual(Values, KnownObjectIdentifier.Key))
				{
					Name = KnownObjectIdentifier.Value;
					return true;
				}
			}

			Name = null;
			return false;
		}

		public static void Dump(BinaryReader Reader, string Indent)
		{
			FieldInfo Field = ReadField(Reader);

			// If it's a primitive type, unpack the value
			string DescriptionSuffix = "";
			if (Field.bPrimitive)
			{
				if (Field.Length > 0)
				{
					string Description;
					switch (Field.Tag)
					{
						case FieldTag.INTEGER:
							Description = String.Format("{0}", ReadInteger(Reader, Field.Length));
							break;
						case FieldTag.OBJECT_IDENTIFIER:
							Description = GetObjectIdentifierName(ReadObjectIdentifier(Reader, Field.Length));
							break;
						case FieldTag.PRINTABLE_STRING:
							Description = Encoding.ASCII.GetString(Reader.ReadBytes(Field.Length));
							break;
						default:
							Reader.ReadBytes(Field.Length);
							Description = "unknown";
							break;
					}
					DescriptionSuffix = String.Format(" = {0}", Description);
				}
			}

			// Print the contents/name of this element
			Console.WriteLine("{0}{1}{2}", Indent, Field.Tag, DescriptionSuffix);

			// If it's not a primitive, print the contents
			if (!Field.bPrimitive)
			{
				long EndOffset = Reader.BaseStream.Position + Field.Length;
				while (Reader.BaseStream.Position < EndOffset)
				{
					Dump(Reader, Indent + "  ");
				}
			}
		}
	}
}
