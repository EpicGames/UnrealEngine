// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Runtime.Versioning;
using EpicGames.Core;

namespace UnrealBuildTool
{
	/// <summary>
	/// The type of shell supported by this platform. Used to configure command line arguments.
	/// </summary>
	public enum ShellType
	{
		/// <summary>
		/// The Bourne shell
		/// </summary>
		Sh,

		/// <summary>
		/// Windows command interpreter
		/// </summary>
		Cmd,
	}

	/// <summary>
	/// Host platform abstraction
	/// </summary>
	public abstract class BuildHostPlatform
	{
		/// <summary>
		/// Host platform singleton.
		/// </summary>
		public static BuildHostPlatform Current => s_currentPlatform.Value;

		private static readonly Lazy<BuildHostPlatform> s_currentPlatform = new(() =>
		{
			if (OperatingSystem.IsWindows())
			{
				return new WindowsBuildHostPlatform();
			}
			else if (OperatingSystem.IsMacOS())
			{
				return new MacBuildHostPlatform();
			}
			else if (OperatingSystem.IsLinux())
			{
				return new LinuxBuildHostPlatform();
			}
			throw new PlatformNotSupportedException();
		});

		/// <summary>
		/// Gets the current host platform type.
		/// </summary>
		public abstract UnrealTargetPlatform Platform { get; }

		/// <summary>
		/// Gets the path to the shell for this platform
		/// </summary>
		public abstract FileReference Shell { get; }

		/// <summary>
		/// The type of shell returned by the Shell parameter
		/// </summary>
		public abstract ShellType ShellType { get; }

		/// <summary>
		/// The executable binary suffix for this platform
		/// </summary>
		public abstract string BinarySuffix { get; }

#pragma warning disable CA1034 // Nested types should not be visible - preserve API of APIs exposed to UAT
		/// <summary>
		/// Class that holds information about a running process
		/// </summary>
		[DebuggerDisplay("{PID}, {Name}, {Filename}")]
		public record ProcessInfo
		{
			/// <summary>
			/// Process ID
			/// </summary>
			public int PID { get; init; }

			/// <summary>
			/// Name of the process
			/// </summary>
			public string Name { get; init; }

			/// <summary>
			/// Filename of the process binary
			/// </summary>
			public string Filename { get; init; }

			/// <summary>
			/// Constructor
			/// </summary>
			/// <param name="pid">The process ID</param>
			/// <param name="name">The process name</param>
			/// <param name="filename">The process filename</param>
			public ProcessInfo(int pid, string name, string filename)
			{
				PID = pid;
				Name = name;
				Filename = filename;
			}

			/// <summary>
			/// Constructor
			/// </summary>
			/// <param name="proc">Process to take information from</param>
			public ProcessInfo(Process proc)
			{
				PID = proc.Id;
				Name = proc.ProcessName;
				Filename = proc.MainModule?.FileName != null ? Path.GetFullPath(proc.MainModule.FileName) : String.Empty;
			}

			/// <inheritdoc/>
			public override int GetHashCode() => HashCode.Combine(PID, Name, Filename);

			/// <inheritdoc/>
			public override string? ToString() => $"{Name}, {Filename}";
		}
#pragma warning restore CA1034

		/// <summary>
		/// Gets all currently running processes.
		/// </summary>
		/// <returns></returns>
		public virtual ProcessInfo[] GetProcesses()
		{
			Process[] AllProcesses = Process.GetProcesses();
			List<ProcessInfo> Result = new List<ProcessInfo>(AllProcesses.Length);
			foreach (Process Proc in AllProcesses)
			{
				try
				{
					if (!Proc.HasExited)
					{
						Result.Add(new(Proc));
					}
				}
				catch { }
			}
			return [.. Result];
		}

		/// <summary>
		/// Gets a process by name.
		/// </summary>
		/// <param name="name">Name of the process to get information for.</param>
		/// <returns></returns>
		public virtual ProcessInfo? GetProcessByName(string name) => GetProcesses().FirstOrDefault(x => x.Name == name);

		/// <summary>
		/// Gets processes by name.
		/// </summary>
		/// <param name="name">Name of the process to get information for.</param>
		/// <returns></returns>
		public virtual ProcessInfo[] GetProcessesByName(string name) => [.. GetProcesses().Where(x => x.Name == name)];

		/// <summary>
		/// Gets the filenames of all modules associated with a process
		/// </summary>
		/// <param name="pid">Process ID</param>
		/// <param name="filename">Filename of the binary associated with the process.</param>
		/// <returns>An array of all module filenames associated with the process. Can be empty of the process is no longer running.</returns>
		public virtual string[] GetProcessModules(int pid, string filename)
		{
			try
			{
				Process Proc = Process.GetProcessById(pid);
				if (Proc != null)
				{
					return [..Proc.Modules
						.Cast<ProcessModule>()
						.Where(module => module.FileName != null)
						.Select(module => Path.GetFullPath(module.FileName))
						.Order()
					];
				}
			}
			catch { }
			return [];
		}

		/// <summary>
		/// Determines the default project file formats for this platform
		/// </summary>
		/// <returns>Sequence of project file formats</returns>
		internal abstract IEnumerable<ProjectFileFormat> GetDefaultProjectFileFormats();
	}

	[SupportedOSPlatform("windows")]
	class WindowsBuildHostPlatform : BuildHostPlatform
	{
		public override UnrealTargetPlatform Platform => UnrealTargetPlatform.Win64;

		public override FileReference Shell { get; } = new(Environment.GetEnvironmentVariable("ComSpec") ?? Path.Combine(Environment.SystemDirectory, "cmd.exe"));

		public override ShellType ShellType => ShellType.Cmd;

		public override string BinarySuffix => ".exe";

		internal override IEnumerable<ProjectFileFormat> GetDefaultProjectFileFormats()
		{
			yield return ProjectFileFormat.VisualStudio;
#if __VPROJECT_AVAILABLE__
			yield return ProjectFileFormat.VProject;
#endif
		}
	}

	[SupportedOSPlatform("macos")]
	class MacBuildHostPlatform : BuildHostPlatform
	{
		public override UnrealTargetPlatform Platform => UnrealTargetPlatform.Mac;

		public override FileReference Shell => new("/bin/sh");

		public override ShellType ShellType => ShellType.Sh;

		public override string BinarySuffix => String.Empty;

		/// <summary>
		/// Currently returns incomplete list of modules for Process.Modules so we need to parse vmmap output.
		/// https://github.com/dotnet/runtime/issues/64042
		/// </summary>
		/// <param name="pid"></param>
		/// <param name="filename"></param>
		/// <returns></returns>
		public override string[] GetProcessModules(int pid, string filename)
		{
			// Add the process file name to the module list. This is to make it compatible with the results of Process.Modules on Windows.
			HashSet<string> Modules = [filename];

			ProcessStartInfo startInfo = new ProcessStartInfo();
			startInfo.FileName = "vmmap";
			startInfo.Arguments = $"{pid} -w";
			startInfo.CreateNoWindow = true;
			startInfo.UseShellExecute = false;
			startInfo.RedirectStandardOutput = true;

			Process process = new Process();
			process.StartInfo = startInfo;
			try
			{
				process.Start();
				// Start processing output before vmmap exits otherwise it's going to hang
				while (!process.WaitForExit(1))
				{
					ProcessVMMapOutput(process, Modules);
				}
				ProcessVMMapOutput(process, Modules);
			}
			catch { }
			return [.. Modules.Order()];
		}

		private static void ProcessVMMapOutput(Process process, HashSet<string> modules)
		{
			for (string? line = process.StandardOutput.ReadLine(); line != null; line = process.StandardOutput.ReadLine())
			{
				line = line.Trim();
				if (line.EndsWith(".dylib", StringComparison.Ordinal))
				{
					const int sharingModeLength = 6;
					int smStart = line.IndexOf("SM=", StringComparison.Ordinal);
					if (smStart != -1)
					{
						int pathStart = smStart + sharingModeLength;
						string module = line.Substring(pathStart).Trim();
						if (Path.Exists(module))
						{
							modules.Add(module);
						}
					}
				}
			}
		}

		internal override IEnumerable<ProjectFileFormat> GetDefaultProjectFileFormats()
		{
			yield return ProjectFileFormat.XCode;
#if __VPROJECT_AVAILABLE__
			yield return ProjectFileFormat.VProject;
#endif
		}
	}

	[SupportedOSPlatform("linux")]
	class LinuxBuildHostPlatform : BuildHostPlatform
	{
		public override UnrealTargetPlatform Platform => UnrealTargetPlatform.Linux;

		public override FileReference Shell => new("/bin/sh");

		public override ShellType ShellType => ShellType.Sh;

		public override string BinarySuffix => String.Empty;

		/// <summary>
		/// Currently returns incomplete list of modules for Process.Modules so we need to parse /proc/PID/maps.
		/// (also, locks up during process traversal sometimes, trying to open /dev/snd/pcm*)
		/// https://github.com/dotnet/runtime/issues/64042
		/// </summary>
		/// <param name="pid"></param>
		/// <param name="filename"></param>
		/// <returns></returns>
		public override string[] GetProcessModules(int pid, string filename)
		{
			// Add the process file name to the module list. This is to make it compatible with the results of Process.Modules on Windows.
			HashSet<string> modules = [filename];
			// @TODO: Implement for Linux
			return [.. modules.Order()];
		}

		internal override IEnumerable<ProjectFileFormat> GetDefaultProjectFileFormats()
		{
			yield return ProjectFileFormat.Make;
			yield return ProjectFileFormat.VisualStudioCode;
#if __VPROJECT_AVAILABLE__
			yield return ProjectFileFormat.VProject;
#endif
		}
	}
}
