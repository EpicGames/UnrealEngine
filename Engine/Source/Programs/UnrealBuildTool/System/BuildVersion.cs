// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// Holds information about the current engine version
	/// </summary>
	[Serializable]
	public class BuildVersion
	{
		/// <summary>
		/// The major engine version (5 for UE5)
		/// </summary>
		public int MajorVersion;

		/// <summary>
		/// The minor engine version
		/// </summary>
		public int MinorVersion;

		/// <summary>
		/// The hotfix/patch version
		/// </summary>
		public int PatchVersion;

		/// <summary>
		/// The changelist that the engine is being built from
		/// </summary>
		public int Changelist;

		/// <summary>
		/// The changelist that the engine maintains compatibility with
		/// </summary>
		public int CompatibleChangelist;

		/// <summary>
		/// Whether the changelist numbers are a licensee changelist
		/// </summary>
		public bool IsLicenseeVersion;

		/// <summary>
		/// Whether the current build is a promoted build, that is, built strictly from a clean sync of the given changelist
		/// </summary>
		public bool IsPromotedBuild;

		/// <summary>
		/// Name of the current branch, with '/' characters escaped as '+'
		/// </summary>
		public string? BranchName;

		/// <summary>
		/// The current build id. This will be generated automatically whenever engine binaries change if not set in the default Engine/Build/Build.version.
		/// </summary>
		public string? BuildId;

		/// <summary>
		/// The build version string
		/// </summary>
		public string? BuildVersionString;

		/// <summary>
		/// Optional URL for a continuous integration job associated with this build version. (e.g. the job that build a set of binaries)
		/// </summary>
		public string? BuildURL;

		/// <summary>
		/// Returns the value which can be used as the compatible changelist. Requires that the regular changelist is also set, and defaults to the 
		/// regular changelist if a specific compatible changelist is not set.
		/// </summary>
		public int EffectiveCompatibleChangelist => (Changelist != 0 && CompatibleChangelist != 0) ? CompatibleChangelist : Changelist;

		/// <summary>
		/// Try to read a version file from disk
		/// </summary>
		/// <param name="FileName">Path to the version file</param>
		/// <param name="Version">The version information</param>
		/// <returns>True if the version was read successfully, false otherwise</returns>
		public static bool TryRead(FileReference FileName, [NotNullWhen(true)] out BuildVersion? Version)
		{
			JsonObject? Object;
			if (!JsonObject.TryRead(FileName, out Object))
			{
				Version = null;
				return false;
			}
			return TryParse(Object, out Version);
		}

		/// <summary>
		/// Get the default path to the build.version file on disk
		/// </summary>
		/// <returns>Path to the Build.version file</returns>
		public static FileReference GetDefaultFileName()
		{
			return FileReference.Combine(Unreal.EngineDirectory, "Build", "Build.version");
		}

		/// <summary>
		/// Get the default path for a target's version file.
		/// </summary>
		/// <param name="OutputDirectory">The output directory for the executable. For MacOS, this is the directory containing the app bundle.</param>
		/// <param name="TargetName">Name of the target being built</param>
		/// <param name="Platform">Platform the target is being built for</param>
		/// <param name="Configuration">The configuration being built</param>
		/// <param name="Architectures">Architecture of the target being built</param>
		/// <returns>Path to the target's version file</returns>
		public static FileReference GetFileNameForTarget(DirectoryReference OutputDirectory, string TargetName, UnrealTargetPlatform Platform, UnrealTargetConfiguration Configuration, UnrealArchitectures Architectures)
		{
			// Get the architecture suffix. Platforms have the option of overriding whether to include this string in filenames.
			string ArchitectureSuffix = "";
			if (UnrealArchitectureConfig.ForPlatform(Platform).RequiresArchitectureFilenames(Architectures))
			{
				ArchitectureSuffix = Architectures.ToString();
			}

			// Build the output filename
			if (String.IsNullOrEmpty(ArchitectureSuffix) && Configuration == UnrealTargetConfiguration.Development)
			{
				return FileReference.Combine(OutputDirectory, String.Format("{0}.version", TargetName));
			}
			else
			{
				return FileReference.Combine(OutputDirectory, String.Format("{0}-{1}-{2}{3}.version", TargetName, Platform.ToString(), Configuration.ToString(), ArchitectureSuffix));
			}
		}

		/// <summary>
		/// Parses a build version from a JsonObject
		/// </summary>
		/// <param name="Object">The object to read from</param>
		/// <param name="Version">The resulting version field</param>
		/// <returns>True if the build version could be read, false otherwise</returns>
		public static bool TryParse(JsonObject Object, [NotNullWhen(true)] out BuildVersion? Version)
		{
			BuildVersion NewVersion = new BuildVersion();
			if (!Object.TryGetIntegerField("MajorVersion", out NewVersion.MajorVersion) || !Object.TryGetIntegerField("MinorVersion", out NewVersion.MinorVersion) || !Object.TryGetIntegerField("PatchVersion", out NewVersion.PatchVersion))
			{
				Version = null;
				return false;
			}

			Object.TryGetIntegerField("Changelist", out NewVersion.Changelist);
			Object.TryGetIntegerField("CompatibleChangelist", out NewVersion.CompatibleChangelist);

			int IsLicenseeVersionInt;
			Object.TryGetIntegerField("IsLicenseeVersion", out IsLicenseeVersionInt);
			NewVersion.IsLicenseeVersion = IsLicenseeVersionInt != 0;

			int IsPromotedBuildInt;
			Object.TryGetIntegerField("IsPromotedBuild", out IsPromotedBuildInt);
			NewVersion.IsPromotedBuild = IsPromotedBuildInt != 0;

			Object.TryGetStringField("BranchName", out NewVersion.BranchName);
			Object.TryGetStringField("BuildId", out NewVersion.BuildId);
			Object.TryGetStringField("BuildVersion", out NewVersion.BuildVersionString);
			Object.TryGetStringField("BuildURL", out NewVersion.BuildURL);

			Version = NewVersion;
			return true;
		}

		/// <summary>
		/// Exports this object as Json
		/// </summary>
		/// <param name="FileName">The filename to write to</param>
		public void Write(FileReference FileName)
		{
			using (StreamWriter Writer = new StreamWriter(FileName.FullName))
			{
				Write(Writer);
			}
		}

		/// <summary>
		/// Exports this object as Json
		/// </summary>
		/// <param name="FileName">The filename to write to</param>
		/// <param name="Logger">Logger for output</param>
		public void WriteIfModified(FileReference FileName, ILogger Logger)
		{
			using StringWriter Writer = new StringWriter();
			Write(Writer);
			Utils.WriteFileIfChanged(FileName, Writer.ToString(), Logger);
		}

		/// <summary>
		/// Exports this object as Json
		/// </summary>
		/// <param name="Writer">Writer for output text</param>
		public void Write(TextWriter Writer)
		{
			using (JsonWriter OtherWriter = new JsonWriter(Writer))
			{
				OtherWriter.WriteObjectStart();
				WriteProperties(OtherWriter);
				OtherWriter.WriteObjectEnd();
			}
		}

		/// <summary>
		/// Exports this object as Json
		/// </summary>
		/// <param name="Writer">The json writer to receive the version settings</param>
		/// <returns>True if the build version could be read, false otherwise</returns>
		public void WriteProperties(JsonWriter Writer)
		{
			Writer.WriteValue("MajorVersion", MajorVersion);
			Writer.WriteValue("MinorVersion", MinorVersion);
			Writer.WriteValue("PatchVersion", PatchVersion);
			Writer.WriteValue("Changelist", Changelist);
			Writer.WriteValue("CompatibleChangelist", CompatibleChangelist);
			Writer.WriteValue("IsLicenseeVersion", IsLicenseeVersion ? 1 : 0);
			Writer.WriteValue("IsPromotedBuild", IsPromotedBuild ? 1 : 0);
			Writer.WriteValue("BranchName", BranchName);
			if (!String.IsNullOrEmpty(BuildId))
			{
				Writer.WriteValue("BuildId", BuildId);
			}
			if (!String.IsNullOrEmpty(BuildVersionString))
			{
				Writer.WriteValue("BuildVersion", BuildVersionString);
			}
			if (!String.IsNullOrEmpty(BuildURL))
			{
				Writer.WriteValue("BuildURL", BuildURL);
			}
		}
	}

	/// <summary>
	/// Read-only wrapper for a BuildVersion instance
	/// </summary>
	public class ReadOnlyBuildVersion
	{
		/// <summary>
		/// The inner build version
		/// </summary>
		private readonly BuildVersion Inner;

		/// <summary>
		/// Cached copy of the current build version
		/// </summary>
		private static ReadOnlyBuildVersion? CurrentCached;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="Inner">The writable build version instance</param>
		public ReadOnlyBuildVersion(BuildVersion Inner)
		{
			this.Inner = Inner;
		}

		/// <summary>
		/// Gets the current build version
		/// </summary>
		public static ReadOnlyBuildVersion Current
		{
			get
			{
				if (CurrentCached == null)
				{
					FileReference File = BuildVersion.GetDefaultFileName();
					if (!FileReference.Exists(File))
					{
						throw new BuildException("Version file is missing ({0})", File);
					}

					BuildVersion? Version;
					if (!BuildVersion.TryRead(File, out Version))
					{
						throw new BuildException("Unable to read version file ({0}). Check that this file is present and well-formed JSON.", File);
					}

					CurrentCached = new ReadOnlyBuildVersion(Version);
				}
				return CurrentCached;
			}
		}

		/// <summary>
		/// Accessors for fields on the inner BuildVersion instance
		/// </summary>
		#region Read-only accessor properties 
#pragma warning disable CS1591

		public int MajorVersion => Inner.MajorVersion;

		public int MinorVersion => Inner.MinorVersion;

		public int PatchVersion => Inner.PatchVersion;

		public int Changelist => Inner.Changelist;

		public int CompatibleChangelist => Inner.CompatibleChangelist;

		public int EffectiveCompatibleChangelist => Inner.EffectiveCompatibleChangelist;

		public bool IsLicenseeVersion => Inner.IsLicenseeVersion;

		public bool IsPromotedBuild => Inner.IsPromotedBuild;

		public string? BranchName => Inner.BranchName;

		public string? BuildVersionString => Inner.BuildVersionString;
#pragma warning disable CA1056 // URI properties should not be strings - can't break user API
		public string? BuildURL => Inner.BuildURL;
#pragma warning restore CA1056

#pragma warning restore C1591
		#endregion
	}
}
