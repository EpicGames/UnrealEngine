// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.Linq;
using System.Reflection;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace UnrealBuildTool
{
	/// <summary>
	/// Functions for parsing command line arguments
	/// </summary>
	public static class CommandLine
	{
		/// <summary>
		/// Stores information about the field to receive command line arguments
		/// </summary>
		class Parameter
		{
			/// <summary>
			/// Prefix for the argument.
			/// </summary>
			public string Prefix;

			/// <summary>
			/// Field to receive values for this argument
			/// </summary>
			public FieldInfo FieldInfo;

			/// <summary>
			/// The attribute containing this argument's info
			/// </summary>
			public CommandLineAttribute Attribute;

			/// <summary>
			/// Constructor
			/// </summary>
			/// <param name="Prefix"></param>
			/// <param name="FieldInfo"></param>
			/// <param name="Attribute"></param>
			public Parameter(string Prefix, FieldInfo FieldInfo, CommandLineAttribute Attribute)
			{
				this.Prefix = Prefix;
				this.FieldInfo = FieldInfo;
				this.Attribute = Attribute;
			}
		}

		/// <summary>
		/// Parse the given list of arguments and apply them to the given object
		/// </summary>
		/// <param name="Arguments">List of arguments. Parsed arguments will be removed from this list when the function returns.</param>
		/// <param name="TargetObject">Object to receive the parsed arguments. Fields in this object should be marked up with CommandLineArgumentAttribute's to indicate how they should be parsed.</param>
		public static void ParseArguments(IEnumerable<string> Arguments, object TargetObject) => ParseArguments(Arguments, TargetObject, Log.Logger);

		/// <summary>
		/// Parse the given list of arguments and apply them to the given object
		/// </summary>
		/// <param name="Arguments">List of arguments. Parsed arguments will be removed from this list when the function returns.</param>
		/// <param name="TargetObject">Object to receive the parsed arguments. Fields in this object should be marked up with CommandLineArgumentAttribute's to indicate how they should be parsed.</param>
		/// <param name="Logger">Logger for output</param>
		public static void ParseArguments(IEnumerable<string> Arguments, object TargetObject, ILogger Logger)
		{
			ParseAndRemoveArguments(Arguments.ToList(), TargetObject, Logger);
		}

		private static Type NonNullableType(Type type)
		{
			Type? UnderlyingType = Nullable.GetUnderlyingType(type);
			if (UnderlyingType != null)
			{
				return UnderlyingType;
			}
			return type;
		}

		private static readonly SearchValues<char> ArgumentSeparators = SearchValues.Create("=:");

		/// <summary>
		/// Parse the given list of arguments, and remove any that are parsed successfully
		/// </summary>
		/// <param name="Arguments">List of arguments. Parsed arguments will be removed from this list when the function returns.</param>
		/// <param name="TargetObject">Object to receive the parsed arguments. Fields in this object should be marked up with CommandLineArgumentAttribute's to indicate how they should be parsed.</param>
		/// <param name="Logger">Logger for output</param>
		public static void ParseAndRemoveArguments(List<string> Arguments, object TargetObject, ILogger Logger)
		{
			// Build a mapping from name to field and attribute for this object
			Dictionary<string, Parameter> PrefixToParameter = new Dictionary<string, Parameter>(StringComparer.InvariantCultureIgnoreCase);
			for (Type TargetType = TargetObject.GetType(); TargetType != typeof(object); TargetType = TargetType.BaseType!)
			{
				foreach (FieldInfo FieldInfo in TargetType.GetFields(BindingFlags.Instance | BindingFlags.GetField | BindingFlags.GetProperty | BindingFlags.Public | BindingFlags.NonPublic | BindingFlags.DeclaredOnly))
				{
					IEnumerable<CommandLineAttribute> Attributes = FieldInfo.GetCustomAttributes<CommandLineAttribute>();
					foreach (CommandLineAttribute Attribute in Attributes)
					{
						string? Prefix = Attribute.Prefix;
						if (Prefix == null)
						{
							if (NonNullableType(FieldInfo.FieldType) == typeof(bool))
							{
								Prefix = String.Format("-{0}", FieldInfo.Name);
							}
							else
							{
								Prefix = String.Format("-{0}=", FieldInfo.Name);
							}
						}
						else
						{
							if (NonNullableType(FieldInfo.FieldType) != typeof(bool) && Attribute.Value == null && !Prefix.EndsWith('=') && !Prefix.EndsWith(':'))
							{
								Prefix += "=";
							}
						}
						PrefixToParameter.Add(Prefix, new Parameter(Prefix, FieldInfo, Attribute));
					}
				}
			}

			// Step through the arguments, and remove those that we can parse
			Dictionary<FieldInfo, Parameter> AssignedFieldToParameter = new Dictionary<FieldInfo, Parameter>();
			for (int Idx = 0; Idx < Arguments.Count; Idx++)
			{
				string Argument = Arguments[Idx];
				if (Argument.Length > 0 && Argument[0] == '-')
				{
					// Get the length of the argument prefix
					int EqualsIdx = Argument.AsSpan().IndexOfAny(ArgumentSeparators);
					string Prefix = (EqualsIdx == -1) ? Argument : Argument.Substring(0, EqualsIdx + 1);

					// Check if there's a matching argument registered
					Parameter? Parameter;
					if (PrefixToParameter.TryGetValue(Prefix, out Parameter))
					{
						int NextIdx = Idx + 1;

						// Parse the value
						if (Parameter.Attribute.Value != null)
						{
							if (EqualsIdx != -1)
							{
								Logger.LogWarning("Cannot specify a value for {ParameterPrefix}", Parameter.Prefix);
							}
							else
							{
								AssignValue(Parameter, Parameter.Attribute.Value, TargetObject, AssignedFieldToParameter, Logger);
							}
						}
						else if (EqualsIdx != -1)
						{
							AssignValue(Parameter, Argument.Substring(EqualsIdx + 1), TargetObject, AssignedFieldToParameter, Logger);
						}
						else if (NonNullableType(Parameter.FieldInfo.FieldType) == typeof(bool))
						{
							AssignValue(Parameter, "true", TargetObject, AssignedFieldToParameter, Logger);
						}
						else
						{
							Logger.LogWarning("Missing value for {ParameterPrefix}", Parameter.Prefix);
						}

						// Remove the argument from the list
						Arguments.RemoveRange(Idx, NextIdx - Idx);
						Idx--;
					}
				}
			}

			// Make sure there are no required parameters that are missing
			Dictionary<FieldInfo, Parameter> MissingFieldToParameter = new Dictionary<FieldInfo, Parameter>();
			foreach (Parameter Parameter in PrefixToParameter.Values)
			{
				if (Parameter.Attribute.Required && !AssignedFieldToParameter.ContainsKey(Parameter.FieldInfo) && !MissingFieldToParameter.ContainsKey(Parameter.FieldInfo))
				{
					MissingFieldToParameter.Add(Parameter.FieldInfo, Parameter);
				}
			}
			if (MissingFieldToParameter.Count > 0)
			{
				if (MissingFieldToParameter.Count == 1)
				{
					throw new BuildException("Missing {0} argument", MissingFieldToParameter.First().Value.Prefix.Replace("=", "=...", StringComparison.Ordinal));
				}
				else
				{
					throw new BuildException("Missing {0} arguments", StringUtils.FormatList(MissingFieldToParameter.Values.Select(x => x.Prefix.Replace("=", "=...", StringComparison.Ordinal))));
				}
			}
		}

		/// <summary>
		/// Checks that the list of arguments is empty. If not, throws an exception listing them.
		/// </summary>
		/// <param name="RemainingArguments">List of remaining arguments</param>
		/// <param name="Logger">Logger for output</param>
		public static void CheckNoRemainingArguments(List<string> RemainingArguments, ILogger Logger)
		{
			if (RemainingArguments.Count > 0)
			{
				if (RemainingArguments.Count == 1)
				{
					Logger.LogWarning("Invalid argument: {Arg}", RemainingArguments[0]);
				}
				else
				{
					Logger.LogWarning("Invalid arguments:\n{Args}", String.Join("\n", RemainingArguments));
				}
			}
		}

		/// <summary>
		/// Parses and assigns a value to a field
		/// </summary>
		/// <param name="Parameter">The parameter being parsed</param>
		/// <param name="Text">Argument text</param>
		/// <param name="TargetObject">The target object to assign values to</param>
		/// <param name="AssignedFieldToParameter">Maps assigned fields to the parameter that wrote to it. Used to detect duplicate and conflicting arguments.</param>
		/// <param name="Logger">Logger for output</param>
		static void AssignValue(Parameter Parameter, string Text, object TargetObject, Dictionary<FieldInfo, Parameter> AssignedFieldToParameter, ILogger Logger)
		{
			// Check if the field type implements ICollection<>. If so, we can take multiple values.
			Type? CollectionType = null;
			foreach (Type InterfaceType in Parameter.FieldInfo.FieldType.GetInterfaces())
			{
				if (InterfaceType.IsGenericType && InterfaceType.GetGenericTypeDefinition() == typeof(ICollection<>))
				{
					CollectionType = InterfaceType;
					break;
				}
			}

			// Try to assign values to the target field
			if (CollectionType == null)
			{
				// Try to parse the value
				object? Value;
				if (!TryParseValue(Parameter.FieldInfo.FieldType, Text, out Value))
				{
					Logger.LogWarning("Invalid value for {ParameterPrefix}... - ignoring {Text}", Parameter.Prefix, Text);
					return;
				}

				// Check if this field has already been assigned to. Output a warning if the previous value is in conflict with the new one.
				Parameter? PreviousParameter;
				if (AssignedFieldToParameter.TryGetValue(Parameter.FieldInfo, out PreviousParameter))
				{
					object? PreviousValue = Parameter.FieldInfo.GetValue(TargetObject);
					if (!Object.Equals(PreviousValue, Value))
					{
						if (PreviousParameter.Prefix == Parameter.Prefix)
						{
							Logger.LogWarning("Conflicting {ParameterPrefix} arguments - ignoring", Parameter.Prefix);
						}
						else
						{
							Logger.LogWarning("{ParameterPrefix} conflicts with {PreviousParameterPrefix} - ignoring", Parameter.Prefix, PreviousParameter.Prefix);
						}
					}
					return;
				}

				// Set the value on the target object
				Parameter.FieldInfo.SetValue(TargetObject, Value);
				AssignedFieldToParameter.Add(Parameter.FieldInfo, Parameter);
			}
			else
			{
				// Split the text into an array of values if necessary
				string[] ItemArray;
				if (Parameter.Attribute.ListSeparator == 0)
				{
					ItemArray = new string[] { Text };
				}
				else
				{
					ItemArray = Text.Split(Parameter.Attribute.ListSeparator);
				}

				// Parse each of the argument values separately
				foreach (string Item in ItemArray)
				{
					object? Value;
					if (TryParseValue(CollectionType.GenericTypeArguments[0], Item, out Value))
					{
						CollectionType.InvokeMember("Add", BindingFlags.InvokeMethod, null, Parameter.FieldInfo.GetValue(TargetObject), new object[] { Value });
					}
					else
					{
						Logger.LogWarning("'{Item}' is not a valid value for -{ParameterPrefix}=... - ignoring", Item, Parameter.Prefix);
					}
				}
			}
		}

		/// <summary>
		/// Attempts to parse the given string to a value
		/// </summary>
		/// <param name="FieldType">Type of the field to convert to</param>
		/// <param name="Text">The value text</param>
		/// <param name="Value">On success, contains the parsed object</param>
		/// <returns>True if the text could be parsed, false otherwise</returns>
		static bool TryParseValue(Type FieldType, string Text, [NotNullWhen(true)] out object? Value)
		{
			FieldType = NonNullableType(FieldType);
			if (FieldType.IsEnum)
			{
				// Special handling for enums; parse the value ignoring case.
				try
				{
					Value = Enum.Parse(FieldType, Text, true);
					return true;
				}
				catch (ArgumentException)
				{
					Value = null;
					return false;
				}
			}
			else if (FieldType == typeof(FileReference))
			{
				// Construct a file reference from the string
				try
				{
					Value = new FileReference(Text);
					return true;
				}
				catch
				{
					Value = null;
					return false;
				}
			}
			else
			{
				// Otherwise let the framework convert between types
				try
				{
					Value = Convert.ChangeType(Text, FieldType);
					return true;
				}
				catch (InvalidCastException)
				{
					Value = null;
					return false;
				}
			}
		}
	}
}
