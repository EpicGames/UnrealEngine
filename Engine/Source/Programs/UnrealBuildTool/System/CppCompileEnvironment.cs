// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO.Enumeration;
using System.Linq;
using EpicGames.Core;
using UnrealBuildBase;

namespace UnrealBuildTool
{

	/// <summary>
	/// Compiler configuration. This controls whether to use define debug macros and other compiler settings. Note that optimization level should be based on the bOptimizeCode variable rather than
	/// this setting, so it can be modified on a per-module basis without introducing an incompatibility between object files or PCHs.
	/// </summary>
	enum CppConfiguration
	{
		Debug,
		Development,
		Shipping
	}

	/// <summary>
	/// Specifies which language standard to use. This enum should be kept in order, so that toolchains can check whether the requested setting is >= values that they support.
	/// </summary>
#pragma warning disable CA1027 // Mark enums with FlagsAttribute
	public enum CppStandardVersion
#pragma warning restore CA1027 // Mark enums with FlagsAttribute
	{
		/// <summary>
		/// Obsolete C++14 standard. No longer allowed, code will not compile with this version.
		/// </summary>
		[Obsolete("Deprecated in UE5.8. C++14 is no longer allowed, code will not compile with this version")]
		Cpp14,

		/// <summary>
		/// Obsolete C++17 standard. No longer allowed, code will not compile with this version.
		/// </summary>
		[Obsolete("Deprecated in UE5.8. C++17 is no longer allowed, code will not compile with this version")]
		Cpp17,

		/// <summary>
		/// C++20 standard
		/// </summary>
		Cpp20,

		/// <summary>
		/// C++23 standard. Experimental, compiler support is currently limited.
		/// </summary>
		Cpp23,

		/// <summary>
		/// Latest standard supported by the compiler
		/// </summary>
		Latest,

		/// <summary>
		/// Use the default standard version (BuildSettingsVersion.V1-V3: Cpp17, V4: Cpp20)
		/// </summary>
		Default = Cpp20,

		/// <summary>
		/// Use the default standard version for engine modules
		/// </summary>
		EngineDefault = Cpp20,

		/// <summary>
		/// The minimum allowed C++ standard version. Lower values than this will cause a build error.
		/// </summary>
		Minimum = Cpp20,
	}

	/// <summary>
	/// Specifies which C language standard to use. This enum should be kept in order, so that toolchains can check whether the requested setting is >= values that they support.
	/// </summary>
#pragma warning disable CA1027 // Mark enums with FlagsAttribute
	public enum CStandardVersion
#pragma warning restore CA1027 // Mark enums with FlagsAttribute
	{
		/// <summary>
		/// Supports no additional standard version flag
		/// </summary>
		None,

		/// <summary>
		/// Supports C89
		/// </summary>
		C89,

		/// <summary>
		/// Supports C99
		/// </summary>
		C99,

		/// <summary>
		/// Supports C11
		/// </summary>
		C11,

		/// <summary>
		/// Supports C17
		/// </summary>
		C17,

		/// <summary>
		/// Latest standard supported by the compiler
		/// </summary>
		Latest,

		/// <summary>
		/// Use the default standard version
		/// </summary>
		Default = None,
	}

	/// <summary>
	/// Specifies the architecture for code generation on x64 for windows platforms.
	/// Note that by enabling this you are changing the minspec for the PC platform, and the resultant executable will crash on machines without AVX support.
	/// For more details please see https://learn.microsoft.com/en-us/cpp/build/reference/arch-x64
	/// </summary>
#pragma warning disable CA1707 // Identifiers should not contain underscores - public facing API
#pragma warning disable CA1027 // Mark enums with FlagsAttribute
	public enum MinimumCpuArchitectureX64
#pragma warning restore CA1027 // Mark enums with FlagsAttribute
	{
		/// <summary>
		/// No minimum architecture
		/// </summary>
		None,

		/// <summary>
		/// Enables the use of Intel Advanced Vector Extensions instructions
		/// </summary>
		AVX,

		/// <summary>
		/// Enables the use of Intel Advanced Vector Extensions 2 instructions
		/// </summary>
		AVX2,

		/// <summary>
		/// Enables the use of Intel Advanced Vector Extensions 512 instructions
		/// </summary>
		AVX512,

		/// <summary>
		/// Enables the use of Intel Advanced Vector Extensions 10 version 1 instructions
		/// </summary>
		AVX10_1,

		/// <summary>
		/// Enables the use of Intel Advanced Vector Extensions 10 version 2 instructions
		/// </summary>
		AVX10_2,

		/// <summary>
		/// Use the default minimum architecture
		/// </summary>
		Default = None,
	}
#pragma warning restore CA1707

	/// <summary>
	/// Target ARM64 CPU for codegen. This defines different ARM ISA and extension targets
	/// </summary>
	public enum Arm64TargetCpuType
	{
		/// <summary>
		/// Default ARMv8.0-A ISA target CPU, supported by every 64 bit ARM CPU
		/// </summary>
		Default,

		/// <summary>
		/// AWS Graviton2 CPU - ARM Neoverse N1 uArch with ARMv8.2-A ISA + FP16 + rcpc + dotprod + crypto extensions
		/// </summary>
		Graviton2,

		/// <summary>
		/// AWS Graviton SVE (Graviton3+) - targets Neoverse 512TVB with SVE + bf16 + crypto.
		/// Cross-generation target compatible with Graviton3, Graviton4, and future SVE-capable
		/// Graviton processors. NOT compatible with Graviton2 (no SVE support).
		/// For Graviton2, use the Graviton2 target. For full Graviton4 optimizations (SVE2,
		/// ARMv9.0-A), use the Graviton4 target.
		/// </summary>
		GravitonSVE,

		/// <summary>
		/// AWS Graviton3 CPU - ARM Neoverse V1 uArch with ARMv8.4-A ISA + SVE + bf16 + rng + crypto extensions
		/// </summary>
		Graviton3,

		/// <summary>
		/// AWS Graviton4 CPU - ARM Neoverse V2 uArch with ARMv9.0-A ISA + SVE2 + bf16 + rng + crypto extensions
		/// </summary>
		Graviton4,
	}

	/// <summary>
	/// The optimization level that may be compilation targets for C# files.
	/// </summary>
	enum CSharpTargetConfiguration
	{
		Debug,
		Development,
	}

	/// <summary>
	/// The possible interactions between a precompiled header and a C++ file being compiled.
	/// </summary>
	enum PrecompiledHeaderAction
	{
		None,
		Include,
		Create
	}

	/// <summary>
	/// Encapsulates the compilation output of compiling a set of C++ files.
	/// </summary>
	class CPPOutput
	{
		public List<FileItem> ObjectFiles = [];
		public List<FileItem> CompiledModuleInterfaces = [];
		public List<FileItem> GeneratedHeaderFiles = [];
		public FileItem? PrecompiledHeaderFile = null;
		public Dictionary<UnrealArch, FileItem> PerArchPrecompiledHeaderFiles = [];

		public void Merge(CPPOutput Other, UnrealArch Architecture)
		{
			ObjectFiles.AddRange(Other.ObjectFiles);
			CompiledModuleInterfaces.AddRange(Other.CompiledModuleInterfaces);
			GeneratedHeaderFiles.AddRange(Other.GeneratedHeaderFiles);
			ObjectFiles.AddRange(Other.ObjectFiles);
			if (Other.PrecompiledHeaderFile != null)
			{
				PerArchPrecompiledHeaderFiles[Architecture] = Other.PrecompiledHeaderFile;
			}
		}

		public FileItem? GetPrecompiledHeaderFile(UnrealArch Architecture)
		{
			if (PerArchPrecompiledHeaderFiles != null)
			{
				PerArchPrecompiledHeaderFiles.TryGetValue(Architecture, out FileItem? PerArchPrecompiledHeaderFile);
				if (PerArchPrecompiledHeaderFile != null)
				{
					return PerArchPrecompiledHeaderFile;
				}
			}
			return PrecompiledHeaderFile;
		}
	}

	/// <summary>
	/// Encapsulates the environment that a C++ file is compiled in.
	/// </summary>
	class CppCompileEnvironment
	{
		/// <summary>
		/// The platform to be compiled/linked for.
		/// </summary>
		public readonly UnrealTargetPlatform Platform;

		/// <summary>
		/// The configuration to be compiled/linked for.
		/// </summary>
		public readonly CppConfiguration Configuration;

		/// <summary>
		/// The architecture that is being compiled/linked (empty string by default)
		/// </summary>
		public UnrealArchitectures Architectures;

		/// <summary>
		/// Gets the Architecture in the normal case where there is a single Architecture in Architectures
		/// </summary>
		public UnrealArch Architecture => Architectures.SingleArchitecture;

		/// <summary>
		/// Cache of source file metadata
		/// </summary>
		public readonly SourceFileMetadataCache MetadataCache;

		/// <summary>
		/// Root paths for this environment. Used for Clang VFS and UBA
		/// </summary>
		public readonly CppRootPaths RootPaths;

		/// <summary>
		/// Templates for shared precompiled headers
		/// </summary>
		public readonly List<PrecompiledHeaderTemplate> SharedPCHs;

		/// <summary>
		/// The name of the header file which is precompiled.
		/// </summary>
		public FileReference? PrecompiledHeaderIncludeFilename = null;

		/// <summary>
		/// Whether the compilation should create, use, or do nothing with the precompiled header.
		/// </summary>
		public PrecompiledHeaderAction PrecompiledHeaderAction = PrecompiledHeaderAction.None;

		/// <summary>
		/// Will replace pch with ifc and use header units instead
		/// </summary>
		public bool bUseHeaderUnitsForPch = false;

		/// <summary>
		/// Whether artifacts from this compile are shared with other targets. If so, we should not apply any target-wide modifications to the compile environment.
		/// </summary>
		public bool bUseSharedBuildEnvironment;

		/// <summary>
		/// Use run time type information
		/// </summary>
		public bool bUseRTTI = false;

		/// <summary>
		/// Whether to direct MSVC to remove unreferenced COMDAT functions and data.
		/// </summary>
		/// <seealso href="https://learn.microsoft.com/en-us/cpp/build/reference/zc-inline-remove-unreferenced-comdat">zc-inline-remove-unreferenced-comdat</seealso>
		public bool bVcRemoveUnreferencedComdat = true;

		/// <summary>
		/// Use Position Independent Executable (PIE). Has an overhead cost
		/// </summary>
		public bool bUsePIE = false;

		/// <summary>
		/// Use Stack Protection. Has an overhead cost
		/// </summary>
		public bool bUseStackProtection = false;

		/// <summary>
		/// Enable inlining.
		/// </summary>
		public bool bUseInlining = false;

		/// <summary>
		/// Whether to compile ISPC files.
		/// </summary>
		public bool bCompileISPC = false;

		/// <summary>
		/// Enable buffer security checks.   This should usually be enabled as it prevents severe security risks.
		/// </summary>
		public bool bEnableBufferSecurityChecks = true;

		/// <summary>
		/// Whether the AutoRTFM compiler is being used or not.
		/// </summary>
		public bool bUseAutoRTFMCompiler = false;

		/// <summary>
		/// Disables AutoRTFM instrumentation to this cpp file only when AutoRTFMCompiler is enabled
		/// </summary>
		public bool bDisableAutoRTFMInstrumentation = false;

		/// <summary>
		/// Whether AutoRTFM instrumentation is enabled or not.
		/// </summary>
		public bool bEnableAutoRTFMInstrumentation => bUseAutoRTFMCompiler && !bDisableAutoRTFMInstrumentation;

		/// <summary>
		/// Whether AutoRTFM verification is enabled or not.
		/// </summary>
		public bool bEnableAutoRTFMVerification = false;

		/// <summary>
		/// AutoRTFM external open -> closed mapping files
		/// </summary>
		public HashSet<FileItem> AutoRTFMExternalMappingFiles;

		/// <summary>
		/// Whether we should run an extra LLVM verification pass after the AutoRTFM compiler pass.
		/// </summary>
		public bool bAutoRTFMVerify = false;

		/// <summary>
		/// Whether we should link closed function declarations statically.
		/// </summary>
		public bool bAutoRTFMClosedStaticLinkage = false;

		/// <summary>
		/// Whether we should instrument all memory accesses for redirection
		/// </summary>
		public bool bAutoRTFMRedirectionInstrumentation = false;

		/// <summary>
		/// If unity builds are enabled this can be used to override if this specific module will build using Unity.
		/// This is set using the per module configurations in BuildConfiguration.
		/// </summary>
		public bool bUseUnity = false;

		/// <summary>
		/// The number of source files in this module before unity build will be activated for that module.  If set to
		/// anything besides -1, will override the default setting which is controlled by MinGameModuleSourceFilesForUnityBuild
		/// </summary>
		public int MinSourceFilesForUnityBuildOverride = 0;

		/// <summary>
		/// The minimum number of files that must use a pre-compiled header before it will be created and used.
		/// </summary>
		public int MinFilesUsingPrecompiledHeaderOverride = 0;

		/// <summary>
		/// Module uses a #import so must be built locally when compiling with SN-DBS
		/// </summary>
		public bool bBuildLocallyWithSNDBS = false;

		/// <summary>
		/// Whether to retain frame pointers
		/// </summary>
		public bool bRetainFramePointers = true;

		/// <summary>
		/// Enable exception handling
		/// </summary>
		public bool bEnableExceptions = false;

		/// <summary>
		/// Enable objective C exception handling
		/// </summary>
		public bool bEnableObjCExceptions = false;

		/// <summary>
		/// Enable objective C automatic reference counting (ARC)
		/// If you set this to true you should not use shared PCHs for this module. The engine won't be extensively using ARC in the short term  
		/// Not doing this will result in a compile errors because shared PCHs were compiled with different flags than consumer
		/// </summary>
		public bool bEnableObjCAutomaticReferenceCounting = false;

		/// <summary>
		/// How to treat any warnings in the code
		/// </summary>
		public WarningLevel DefaultWarningLevel = WarningLevel.Warning;

		/// <summary>
		/// The compiler warnings associated with the compile environment.
		/// </summary>
		public CppCompileWarnings CppCompileWarnings;

		/// <summary>
		/// Whether to treat all warnings as errors
		/// </summary>
		public bool bWarningsAsErrors = false;

		/// <summary>
		/// Whether to disable all static analysis - Clang, MSVC, PVS-Studio.
		/// </summary>
		public bool bDisableStaticAnalysis = false;

		/// <summary>
		/// Enable additional analyzer extension warnings using the EspXEngine plugin. This is only supported for MSVC.
		/// See https://learn.microsoft.com/en-us/cpp/code-quality/using-the-cpp-core-guidelines-checkers
		/// This will add a large number of warnings by default. It's recommended to use StaticAnalyzerRulesets if this is enabled.
		/// </summary>
		public bool bStaticAnalyzerExtensions = false;

		/// <summary>
		/// The static analyzer rulesets that should be used to filter warnings. This is only supported for MSVC.
		/// See https://learn.microsoft.com/en-us/cpp/code-quality/using-rule-sets-to-specify-the-cpp-rules-to-run
		/// </summary>
		public HashSet<FileReference> StaticAnalyzerRulesets;

		/// <summary>
		/// The static analyzer checkers that should be enabled rather than the defaults. This is only supported for Clang.
		/// </summary>
		public HashSet<string> StaticAnalyzerCheckers;

		/// <summary>
		/// The static analyzer default checkers that should be disabled. Unused if StaticAnalyzerCheckers is populated. This is only supported for Clang.
		/// </summary>
		public HashSet<string> StaticAnalyzerDisabledCheckers;

		/// <summary>
		/// The static analyzer non-default checkers that should be enabled. Unused if StaticAnalyzerCheckers is populated. This is only supported for Clang.
		/// </summary>
		public HashSet<string> StaticAnalyzerAdditionalCheckers;

		/// <summary>
		/// The PVS Studio analysis warnings that should be disabled.
		/// </summary>
		public HashSet<string> StaticAnalyzerPVSDisabledErrors;

		/// <summary>
		/// Whether to disable all dynamic analysis (such as Address Sanitizer)
		/// </summary>
		public bool bDisableDynamicAnalysis = false;

		/// <summary>
		/// True if compiler optimizations should be enabled. This setting is distinct from the configuration (see CPPTargetConfiguration).
		/// </summary>
		public bool bOptimizeCode = false;

		/// <summary>
		/// True if the compilation should produce tracing output for code coverage.
		/// </summary>
		public bool bCodeCoverage = false;

		/// <summary>
		/// Allows to fine tune optimizations level for speed and\or code size
		/// </summary>
		public OptimizationMode OptimizationLevel = OptimizationMode.Speed;

		/// <summary>
		/// Determines the FP semantics.
		/// </summary>
		public FPSemanticsMode FPSemantics = FPSemanticsMode.Default;

		/// <summary>
		/// True if debug info should be created.
		/// </summary>
		public bool bCreateDebugInfo = true;

		/// <summary>
		/// True if debug info should only generate line number tables (clang)
		/// </summary>
		public bool bDebugLineTablesOnly = false;

		/// <summary>
		/// If true, omits line tables for inlined functions at their callsites, and only includes line information 
		/// for the caller.
		/// </summary>
		public bool bDebugNoInlineLineTables = false;

		/// <summary>
		/// If true, use simple names for templates in debug type info. i.e. just TArray without template parameters.
		/// Template argument information is still present in the type information. 
		/// </summary>
		public bool bDebugSimpleTemplateNames = false;

		/// <summary>
		/// True if we're compiling .cpp files that will go into a library (.lib file)
		/// </summary>
		public bool bIsBuildingLibrary = false;

		/// <summary>
		/// True if we're compiling a DLL
		/// </summary>
		public bool bIsBuildingDLL = false;

		/// <summary>
		/// Whether we should compile using the statically-linked CRT. This is not widely supported for the whole engine, but is required for programs that need to run without dependencies.
		/// </summary>
		public bool bUseStaticCRT = false;

		/// <summary>
		/// Whether to use the debug CRT in debug configurations
		/// </summary>
		public bool bUseDebugCRT = false;

		/// <summary>
		/// Whether to omit frame pointers or not. Disabling is useful for e.g. memory profiling on the PC
		/// </summary>
		public bool bOmitFramePointers = true;

		/// <summary>
		/// Whether we should compile with support for OS X 10.9 Mavericks. Used for some tools that we need to be compatible with this version of OS X.
		/// </summary>
		public bool bEnableOSX109Support = false;

		/// <summary>
		/// Whether PDB files should be used for Visual C++ builds.
		/// </summary>
		public bool bUsePDBFiles = false;

		/// <summary>
		/// Whether to just preprocess source files
		/// </summary>
		public bool bPreprocessOnly = false;

		/// <summary>
		/// Should an assembly file be generated while compiling. Works exclusively on MSVC compilers for now.
		/// </summary>
		public bool bWithAssembly = false;

		/// <summary>
		/// Whether to support edit and continue.  Only works on Microsoft compilers in 32-bit compiles.
		/// </summary>
		public bool bSupportEditAndContinue;

		/// <summary>
		/// Whether to use incremental linking or not.
		/// </summary>
		public bool bUseIncrementalLinking;

		/// <summary>
		/// Whether to allow the use of LTCG (link time code generation) 
		/// </summary>
		public bool bAllowLTCG;

		/// <summary>
		/// Whether to enable Profile Guided Optimization (PGO) instrumentation in this build.
		/// </summary>
		public bool bPGOProfile;

		/// <summary>
		/// Whether to optimize this build with Profile Guided Optimization (PGO).
		/// </summary>
		public bool bPGOOptimize;

		/// <summary>
		/// Platform specific directory where PGO profiling data is stored.
		/// </summary>
		public string? PGODirectory;

		/// <summary>
		/// Platform specific filename where PGO profiling data is saved.
		/// </summary>
		public string? PGOFilenamePrefix;

		/// <summary>
		/// Whether to log detailed timing info from the compiler
		/// </summary>
		public bool bPrintTimingInfo;

		/// <summary>
		/// When enabled, allows XGE to compile pre-compiled header files on remote machines.  Otherwise, PCHs are always generated locally.
		/// </summary>
		public bool bAllowRemotelyCompiledPCHs = false;

		/// <summary>
		/// Ordered list of include paths for the module
		/// </summary>
		public HashSet<DirectoryReference> UserIncludePaths;

		/// <summary>
		/// The include paths where changes to contained files won't cause dependent C++ source files to
		/// be recompiled, unless BuildConfiguration.bCheckSystemHeadersForModification==true.
		/// </summary>
		public HashSet<DirectoryReference> SystemIncludePaths;

		/// <summary>
		/// The include paths which were previously in UserIncludePaths, but are now in a shared response file, persisted in the environment for validation.
		/// Do not add to this set unless a shared response is in use, and only when removing those headers from UserIncludePaths.
		/// </summary>
		public HashSet<DirectoryReference> SharedUserIncludePaths;

		/// <summary>
		/// The include paths which were previously in SystemIncludePaths, but are now in a shared response file, persisted in the environment for validation.
		/// Do not add to this set unless a shared response is in use, and only when removing those headers from SystemIncludePaths.
		/// </summary>
		public HashSet<DirectoryReference> SharedSystemIncludePaths;

		/// <summary>
		/// Enumerable of all possible include paths
		/// </summary>
		public IEnumerable<DirectoryReference> AllIncludePath => SharedUserIncludePaths.Concat(UserIncludePaths).Concat(SharedSystemIncludePaths).Concat(SystemIncludePaths);

		/// <summary>
		/// List of paths to search for compiled module interface (*.ifc) files
		/// </summary>
		public HashSet<DirectoryReference> ModuleInterfacePaths;

		/// <summary>
		/// Whether headers in system paths should be checked for modification when determining outdated actions.
		/// </summary>
		public bool bCheckSystemHeadersForModification;

		/// <summary>
		/// List of header files to force include
		/// </summary>
		public List<FileItem> ForceIncludeFiles;

		/// <summary>
		/// List of files that need to be up to date before compile can proceed
		/// </summary>
		public List<FileItem> AdditionalPrerequisites;

		/// <summary>
		/// A dictionary of the source file items and the inlined gen.cpp files contained in it
		/// </summary>
		public Dictionary<FileItem, HashSet<FileItem>> FileInlineSourceMap;
		
		/// <summary>
		/// Non-default naming conventions that generated CPP files can have (this uses a filter to check the file name for matches) 
		/// </summary>
		public List<string> ExtraGeneratedCPPFileTypes;

		/// <summary>
		/// FileItems with colliding names. (Which means they would overwrite each other in intermediate folder
		/// </summary>
		public HashSet<FileItem>? CollidingNames;

		/// <summary>
		/// The C++ preprocessor definitions to use.
		/// </summary>
		public List<string> Definitions;

		/// <summary>
		/// Additional response files that will be used by main response file
		/// </summary>
		public List<FileItem> AdditionalResponseFiles;

		/// <summary>
		/// Whether the compile environment has a response file in AdditionalResponseFiles that contains global compiler arguments.
		/// </summary>
		public bool bHasSharedResponseFile = false;

		/// <summary>
		/// Additional arguments to pass to the compiler.
		/// </summary>
		public string AdditionalArguments = "";

		/// <summary>
		/// A list of additional frameworks whose include paths are needed.
		/// </summary>
		public List<UEBuildFramework> AdditionalFrameworks;

		/// <summary>
		/// A dictionary of PCH files for multiple architectures
		/// </summary>
		public Dictionary<UnrealArch, FileItem>? PerArchPrecompiledHeaderFiles => PCHInstance?.Output.PerArchPrecompiledHeaderFiles;

		/// <summary>
		/// The instance containing the precompiled header data.
		/// </summary>
		public PrecompiledHeaderInstance? PCHInstance = null;

		/// <summary>
		/// The file containing the precompiled header data.
		/// </summary>
		public FileItem? PrecompiledHeaderFile => GetPrecompiledHeaderFile(PCHInstance);

		/// <summary>
		/// The parent PCH instance used when creating this PCH.
		/// </summary>
		public PrecompiledHeaderInstance? ParentPCHInstance = null;

		/// <summary>
		/// The parent's PCH header file.
		/// </summary>
		public FileItem? ParentPrecompiledHeaderFile => GetPrecompiledHeaderFile(ParentPCHInstance);

		/// <summary>
		/// True if a single PRecompiledHeader exists, or at least one PerArchPrecompiledHeaderFile exists
		/// </summary>
		public bool bHasPrecompiledHeader => PrecompiledHeaderAction == PrecompiledHeaderAction.Include;

		/// <summary>
		/// Whether or not UHT is being built
		/// </summary>
		public bool bHackHeaderGenerator;

		/// <summary>
		/// Whether to hide symbols by default
		/// </summary>
		public bool bHideSymbolsByDefault = true;

		/// <summary>
		/// Whether this environment should be treated as an engine module.
		/// </summary>
		public bool bTreatAsEngineModule;

		/// <summary>
		/// Which C++ standard to support for engine modules. CppStandard will be set to this for engine modules and CppStandardEngine should not be checked in any toolchain. May not be compatible with all platforms.
		/// </summary>
		public CppStandardVersion CppStandardEngine = CppStandardVersion.EngineDefault;

		/// <summary>
		/// Which C++ standard to support. May not be compatible with all platforms.
		/// </summary>
		public CppStandardVersion CppStandard = CppStandardVersion.Default;

		/// <summary>
		/// Which C standard to support. May not be compatible with all platforms.
		/// </summary>
		public CStandardVersion CStandard = CStandardVersion.Default;

		/// <summary>
		/// Direct the compiler to generate AVX instructions wherever SSE or AVX intrinsics are used.
		/// Note that by enabling this you are changing the minspec for the PC platform, and the resultant executable will crash on machines without AVX support.
		/// </summary>
		public MinimumCpuArchitectureX64 MinCpuArchX64 = MinimumCpuArchitectureX64.Default;

		/// <summary>
		/// Direct the compiler to compile with APX support for X64.
		/// Note that by enabling this you are changing the minspec for the PC platform, and the resultant executable will crash on machines without APX support.
		/// </summary>
		public bool bMinCpuArchAPX = false;

		/// <summary>
		/// Select a minimum ARM64 target CPU. This will change what ARMv8(9)-A ISA + extension code will be compiled for.
		/// Changing it from default will guarantee not to work on all ARM64 CPUs.
		/// Currently supported by clang compiler only
		/// </summary>
		public Arm64TargetCpuType MinArm64CpuTarget = Arm64TargetCpuType.Default;

		/// <summary>
		/// The amount of the stack usage to report static analysis warnings.
		/// </summary>
		public int AnalyzeStackSizeWarning = 300000;

		/// <summary>
		/// Enable C++ coroutine support. 
		/// For MSVC, adds "/await:strict" to the command line. Program should #include &lt;coroutine&gt;
		/// For Clang, adds "-fcoroutines-ts" to the command line. Program should #include &lt;experimental/coroutine&gt; (not supported in every clang toolchain)
		/// </summary>
		public bool bEnableCoroutines = false;

		/// <summary>
		/// What version of include order specified by the module rules. Used to determine shared PCH variants.
		/// </summary>
		public EngineIncludeOrderVersion IncludeOrderVersion = EngineIncludeOrderVersion.Latest;

		/// <summary>
		/// Set flags for deterministic compiles.
		/// </summary>
		public bool bDeterministic;

		/// <summary>
		/// Emits compilation errors for incorrect UE_LOG format strings.
		/// </summary>
		public bool bValidateFormatStrings = true;

		/// <summary>
		/// Emits deprecated warnings\errors for internal API usage for non-engine modules.
		/// </summary>
		public bool bValidateInternalApi = false;

		/// <summary>
		/// Emits deprecated warnings\errors for experimental API usage for non-engine modules.
		/// </summary>
		public bool bValidateExperimentalApi = false;

		/// <summary>
		/// Experimental: Whether this module requires platform specific SDK files. Not supported on all platforms yet.
		/// </summary>
		public bool bRequiresPlatformSDK = false;

		/// <summary>
		/// Directory where to put crash report files for platforms that support it
		/// </summary>
		public string? CrashDiagnosticDirectory;

		/// <summary>
		/// File to use as an LLVM FPass Plugin:
		///    'clang -fpass-plugin=[FPassPlugin]' 
		/// </summary>
		public List<string> FPassPlugins;

		/// <summary>
		/// Default constructor.
		/// </summary>
		public CppCompileEnvironment(UnrealTargetPlatform Platform, CppConfiguration Configuration, UnrealArchitectures Architectures, SourceFileMetadataCache MetadataCache)
		{
			this.Platform = Platform;
			this.Configuration = Configuration;
			this.Architectures = Architectures;
			this.MetadataCache = MetadataCache;
			RootPaths = new();
			SharedPCHs = [];
			UserIncludePaths = [];
			SystemIncludePaths = [];
			SharedUserIncludePaths = [];
			SharedSystemIncludePaths = [];
			ModuleInterfacePaths = [];
			ForceIncludeFiles = [];
			AdditionalPrerequisites = [];
			FileInlineSourceMap = [];
			Definitions = [];
			AdditionalResponseFiles = [];
			AdditionalFrameworks = [];
			AutoRTFMExternalMappingFiles = [];
			ExtraGeneratedCPPFileTypes = [];
			StaticAnalyzerRulesets = [];
			StaticAnalyzerCheckers = [];
			StaticAnalyzerDisabledCheckers = [];
			StaticAnalyzerAdditionalCheckers = [];
			StaticAnalyzerPVSDisabledErrors = [];
			FPassPlugins = [];
			CppCompileWarnings = new CppCompileWarnings();
		}

		/// <summary>
		/// Copy constructor.
		/// </summary>
		/// <param name="Other">Environment to copy settings from</param>
		public CppCompileEnvironment(CppCompileEnvironment Other)
		{
			Platform = Other.Platform;
			Configuration = Other.Configuration;
			Architectures = new(Other.Architectures);
			MetadataCache = Other.MetadataCache;
			RootPaths = new(Other.RootPaths);
			SharedPCHs = Other.SharedPCHs;
			PrecompiledHeaderIncludeFilename = Other.PrecompiledHeaderIncludeFilename;
			PrecompiledHeaderAction = Other.PrecompiledHeaderAction;
			bUseSharedBuildEnvironment = Other.bUseSharedBuildEnvironment;
			bUseRTTI = Other.bUseRTTI;
			bVcRemoveUnreferencedComdat = Other.bVcRemoveUnreferencedComdat;
			bUsePIE = Other.bUsePIE;
			bUseStackProtection = Other.bUseStackProtection;
			bUseInlining = Other.bUseInlining;
			bCompileISPC = Other.bCompileISPC;
			bEnableBufferSecurityChecks = Other.bEnableBufferSecurityChecks;
			bUseUnity = Other.bUseUnity;
			MinSourceFilesForUnityBuildOverride = Other.MinSourceFilesForUnityBuildOverride;
			MinFilesUsingPrecompiledHeaderOverride = Other.MinFilesUsingPrecompiledHeaderOverride;
			bBuildLocallyWithSNDBS = Other.bBuildLocallyWithSNDBS;
			bRetainFramePointers = Other.bRetainFramePointers;
			bEnableExceptions = Other.bEnableExceptions;
			bEnableObjCExceptions = Other.bEnableObjCExceptions;
			bEnableObjCAutomaticReferenceCounting = Other.bEnableObjCAutomaticReferenceCounting;
			DefaultWarningLevel = Other.DefaultWarningLevel;
			CppCompileWarnings = CppCompileWarnings.CreateShallowCopy(Other.CppCompileWarnings);
			bWarningsAsErrors = Other.bWarningsAsErrors;
			bDisableStaticAnalysis = Other.bDisableStaticAnalysis;
			StaticAnalyzerCheckers = new(Other.StaticAnalyzerCheckers);
			StaticAnalyzerDisabledCheckers = new(Other.StaticAnalyzerDisabledCheckers);
			StaticAnalyzerAdditionalCheckers = new(Other.StaticAnalyzerAdditionalCheckers);
			StaticAnalyzerPVSDisabledErrors = new(Other.StaticAnalyzerPVSDisabledErrors);
			bStaticAnalyzerExtensions = Other.bStaticAnalyzerExtensions;
			StaticAnalyzerRulesets = new(Other.StaticAnalyzerRulesets);
			bOptimizeCode = Other.bOptimizeCode;
			bUseAutoRTFMCompiler = Other.bUseAutoRTFMCompiler;
			bDisableAutoRTFMInstrumentation = Other.bDisableAutoRTFMInstrumentation;
			bEnableAutoRTFMVerification = Other.bEnableAutoRTFMVerification;
			AutoRTFMExternalMappingFiles = new(Other.AutoRTFMExternalMappingFiles);
			bAutoRTFMVerify = Other.bAutoRTFMVerify;
			bAutoRTFMClosedStaticLinkage = Other.bAutoRTFMClosedStaticLinkage;
			bAutoRTFMRedirectionInstrumentation = Other.bAutoRTFMRedirectionInstrumentation;
			bCodeCoverage = Other.bCodeCoverage;
			OptimizationLevel = Other.OptimizationLevel;
			FPSemantics = Other.FPSemantics;
			bCreateDebugInfo = Other.bCreateDebugInfo;
			bDebugLineTablesOnly = Other.bDebugLineTablesOnly;
			bDebugNoInlineLineTables = Other.bDebugNoInlineLineTables;
			bDebugSimpleTemplateNames = Other.bDebugSimpleTemplateNames;
			bIsBuildingLibrary = Other.bIsBuildingLibrary;
			bIsBuildingDLL = Other.bIsBuildingDLL;
			bUseStaticCRT = Other.bUseStaticCRT;
			bUseDebugCRT = Other.bUseDebugCRT;
			bOmitFramePointers = Other.bOmitFramePointers;
			bEnableOSX109Support = Other.bEnableOSX109Support;
			bUsePDBFiles = Other.bUsePDBFiles;
			bPreprocessOnly = Other.bPreprocessOnly;
			bWithAssembly = Other.bWithAssembly;
			bSupportEditAndContinue = Other.bSupportEditAndContinue;
			bUseIncrementalLinking = Other.bUseIncrementalLinking;
			bAllowLTCG = Other.bAllowLTCG;
			bPGOOptimize = Other.bPGOOptimize;
			bPGOProfile = Other.bPGOProfile;
			PGOFilenamePrefix = Other.PGOFilenamePrefix;
			PGODirectory = Other.PGODirectory;
			bPrintTimingInfo = Other.bPrintTimingInfo;
			bAllowRemotelyCompiledPCHs = Other.bAllowRemotelyCompiledPCHs;
			bHasSharedResponseFile = Other.bHasSharedResponseFile;
			bUseHeaderUnitsForPch = Other.bUseHeaderUnitsForPch;
			UserIncludePaths = new(Other.UserIncludePaths);
			SystemIncludePaths = new(Other.SystemIncludePaths);
			SharedUserIncludePaths = new(Other.SharedUserIncludePaths);
			SharedSystemIncludePaths = new(Other.SharedSystemIncludePaths);
			ModuleInterfacePaths = new(Other.ModuleInterfacePaths);
			bCheckSystemHeadersForModification = Other.bCheckSystemHeadersForModification;
			ForceIncludeFiles = new(Other.ForceIncludeFiles);
			AdditionalPrerequisites = new(Other.AdditionalPrerequisites);
			ExtraGeneratedCPPFileTypes = Other.ExtraGeneratedCPPFileTypes;
			CollidingNames = Other.CollidingNames;
			FileInlineSourceMap = new(Other.FileInlineSourceMap);
			Definitions = new(Other.Definitions);
			AdditionalResponseFiles = new(Other.AdditionalResponseFiles);
			AdditionalArguments = Other.AdditionalArguments;
			AdditionalFrameworks = new(Other.AdditionalFrameworks);
			PCHInstance = Other.PCHInstance;
			ParentPCHInstance = Other.ParentPCHInstance;
			bHackHeaderGenerator = Other.bHackHeaderGenerator;
			bHideSymbolsByDefault = Other.bHideSymbolsByDefault;
			bTreatAsEngineModule = Other.bTreatAsEngineModule;
			CppStandardEngine = Other.CppStandardEngine;
			CppStandard = Other.CppStandard;
			CStandard = Other.CStandard;
			MinCpuArchX64 = Other.MinCpuArchX64;
			bMinCpuArchAPX = Other.bMinCpuArchAPX;
			MinArm64CpuTarget = Other.MinArm64CpuTarget;
			bEnableCoroutines = Other.bEnableCoroutines;
			IncludeOrderVersion = Other.IncludeOrderVersion;
			bDeterministic = Other.bDeterministic;
			CrashDiagnosticDirectory = Other.CrashDiagnosticDirectory;
			bValidateFormatStrings = Other.bValidateFormatStrings;
			bValidateInternalApi = Other.bValidateInternalApi;
			bValidateExperimentalApi = Other.bValidateExperimentalApi;
			bRequiresPlatformSDK = Other.bRequiresPlatformSDK;
			FPassPlugins = Other.FPassPlugins;
		}

		public CppCompileEnvironment(CppCompileEnvironment Other, UnrealArch OverrideArchitecture)
			: this(Other)
		{
			Architectures = new UnrealArchitectures(OverrideArchitecture);
		}

		private FileItem? GetPrecompiledHeaderFile(PrecompiledHeaderInstance? Instance)
		{
			return Instance?.Output.GetPrecompiledHeaderFile(Architectures.SingleArchitecture);
		}
		
		public bool FileMatchesExtraGeneratedCPPTypes(string FileName)
		{
			foreach (string fileFilter in ExtraGeneratedCPPFileTypes)
			{
				if (FileSystemName.MatchesSimpleExpression(fileFilter, FileName, true))
				{
					return true;
				}
			}

			return false;
		}
	}
}
