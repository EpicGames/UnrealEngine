// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// Reads the contents of C++ dependency files, and caches them for future iterations.
	/// </summary>
	class CppDependencyCache
	{
		/// <summary>
		/// Contents of a single dependency file
		/// </summary>
		internal class DependencyInfo
		{
			public long LastWriteTimeUtc;
			public string? ProducedModule;
			public List<(string Name, string BMI)>? ImportedModules;
			public List<FileItem> Files;

			public DependencyInfo(long LastWriteTimeUtc, string? ProducedModule, List<(string, string)>? ImportedModules, List<FileItem> Files)
			{
				this.LastWriteTimeUtc = LastWriteTimeUtc;
				this.ProducedModule = ProducedModule;
				this.ImportedModules = ImportedModules;
				this.Files = Files;
			}

			public static DependencyInfo Read(BinaryArchiveReader Reader)
			{
				long LastWriteTimeUtc = Reader.ReadLong();
				string? ProducedModule = Reader.ReadString();
				List<(string, string)>? ImportedModules = Reader.ReadList(() =>
				{
					return (Reader.ReadString(), Reader.ReadString());
				})!;
				List<FileItem> Files = Reader.ReadList(() => Reader.ReadCompactFileItem())!;
				return new DependencyInfo(LastWriteTimeUtc, ProducedModule, ImportedModules, Files);
			}

			public void Write(BinaryArchiveWriter Writer)
			{
				Writer.WriteLong(LastWriteTimeUtc);
				Writer.WriteString(ProducedModule);
				Writer.WriteList(ImportedModules, (Module) =>
				{
					Writer.WriteString(Module.Name);
					Writer.WriteString(Module.BMI);
				});
				Writer.WriteList<FileItem>(Files, File => Writer.WriteCompactFileItem(File));
			}
		}

		class CachePartition
		{
			/// <summary>
			/// The current file version
			/// </summary>
			public const int CurrentVersion = 4;

			/// <summary>
			/// Location of this dependency cache
			/// </summary>
			public FileReference Location;

			/// <summary>
			/// Map from file item to dependency info
			/// </summary>
			public ConcurrentDictionary<FileItem, DependencyInfo> DependencyFileToInfo = new ConcurrentDictionary<FileItem, DependencyInfo>();

			/// <summary>
			/// Whether the cache has been modified and needs to be saved
			/// </summary>
			public bool bModified;

			/// <summary>
			/// Whether the partition has been loaded or not
			/// </summary>
			public bool Loaded;

			/// <summary>
			/// Constructs a dependency cache. This method is private; call CppDependencyCache.Create() to create a cache hierarchy for a given project.
			/// </summary>
			/// <param name="Location">File to store the cache</param>
			public CachePartition(FileReference Location)
			{
				this.Location = Location;
			}

			/// <summary>
			/// Reads data for this dependency cache from disk
			/// </summary>
			public void Read(ILogger Logger)
			{
				try
				{
					using (BinaryArchiveReader Reader = new BinaryArchiveReader(Location))
					{
						int Version = Reader.ReadInt();
						if (Version != CurrentVersion)
						{
							Logger.LogDebug("Unable to read dependency cache from {File}; version {Version} vs current {CurrentVersion}", Location, Version, CurrentVersion);
							return;
						}

						int Count = Reader.ReadInt();
						for (int Idx = 0; Idx < Count; Idx++)
						{
							FileItem File = Reader.ReadFileItem()!;
							DependencyFileToInfo[File] = DependencyInfo.Read(Reader);
						}
					}
				}
				catch (Exception Ex)
				{
					Logger.LogWarning("Unable to read {Location}. See log for additional information.", Location);
					Logger.LogDebug("{Ex}", ExceptionUtils.FormatExceptionDetails(Ex));
				}
			}

			/// <summary>
			/// Writes data for this dependency cache to disk
			/// </summary>
			public void Write()
			{
				DirectoryReference.CreateDirectory(Location.Directory);
				using (FileStream Stream = File.Open(Location.FullName, FileMode.Create, FileAccess.Write, FileShare.Read))
				{
					using (BinaryArchiveWriter Writer = new BinaryArchiveWriter(Stream))
					{
						Writer.WriteInt(CurrentVersion);

						Writer.WriteInt(DependencyFileToInfo.Count);
						foreach (KeyValuePair<FileItem, DependencyInfo> Pair in DependencyFileToInfo)
						{
							Writer.WriteFileItem(Pair.Key);
							Pair.Value.Write(Writer);
						}
					}
				}
				bModified = false;
			}
		}

		/// <summary>
		/// List of partitions
		/// </summary>
		readonly List<(DirectoryReference Root, List<(string Filter, CachePartition Partition)>)> GlobalPartitions = [];

		readonly List<Task> GlobalTasks = [];

		/// <summary>
		/// Minimum version of a MSVC source dependency json
		/// </summary>
		static readonly VersionNumber MinVCDependencyVersion = new VersionNumber(1, 0);

		/// <summary>
		/// Minimum version of a MSVC source dependency json that supports additional module info
		/// </summary>
		static readonly VersionNumber VCDependencyAdditionalModuleInfoVersion = new VersionNumber(1, 1);

		/// <summary>
		/// Constructs a dependency cache. This method is private; call CppDependencyCache.Create() to create a cache hierarchy for a given project.
		/// </summary>
		public CppDependencyCache()
		{
		}

		/// <summary>
		/// Gets the produced module from a dependencies file
		/// </summary>
		/// <param name="InputFile">The dependencies file</param>
		/// <param name="Logger">Logger for output</param>
		/// <param name="OutModule">The produced module name</param>
		/// <returns>True if a produced module was found</returns>
		public bool TryGetProducedModule(FileItem InputFile, ILogger Logger, [NotNullWhen(true)] out string? OutModule)
		{
			DependencyInfo? Info;
			if (TryGetDependencyInfo(InputFile, Logger, out Info) && Info.ProducedModule != null)
			{
				OutModule = Info.ProducedModule;
				return true;
			}
			else
			{
				OutModule = null;
				return false;
			}
		}

		/// <summary>
		/// Attempts to get a list of imported modules for the given file
		/// </summary>
		/// <param name="InputFile">The dependency file to query</param>
		/// <param name="Logger">Logger for output</param>
		/// <param name="OutImportedModules">List of imported modules</param>
		/// <returns>True if a list of imported modules was obtained</returns>
		public bool TryGetImportedModules(FileItem InputFile, ILogger Logger, [NotNullWhen(true)] out List<(string Name, string BMI)>? OutImportedModules)
		{
			DependencyInfo? Info;
			if (TryGetDependencyInfo(InputFile, Logger, out Info))
			{
				OutImportedModules = Info.ImportedModules;
				return OutImportedModules != null;
			}
			else
			{
				OutImportedModules = null;
				return false;
			}
		}

		/// <summary>
		/// Attempts to read the dependencies from the given input file
		/// </summary>
		/// <param name="InputFile">File to be read</param>
		/// <param name="Logger">Logger for output</param>
		/// <param name="OutDependencyItems">Receives a list of output items</param>
		/// <returns>True if the input file exists and the dependencies were read</returns>
		public bool TryGetDependencies(FileItem InputFile, ILogger Logger, [NotNullWhen(true)] out List<FileItem>? OutDependencyItems)
		{
			DependencyInfo? Info;
			if (TryGetDependencyInfo(InputFile, Logger, out Info))
			{
				OutDependencyItems = Info.Files;
				return true;
			}
			else
			{
				OutDependencyItems = null;
				return false;
			}
		}

		readonly string Intermediate = $"{Path.DirectorySeparatorChar}Intermediate";
		readonly string External = $"{Path.DirectorySeparatorChar}External";
		readonly string Build = $"{Path.DirectorySeparatorChar}Build{Path.DirectorySeparatorChar}";

		/// <summary>
		/// Attempts to read dependencies from the given file.
		/// </summary>
		/// <param name="InputFile">File to be read</param>
		/// <param name="Logger">Logger</param>
		/// <param name="OutInfo">The dependency info</param>
		/// <returns>True if the input file exists and the dependencies were read</returns>
		private bool TryGetDependencyInfo(FileItem InputFile, ILogger Logger, [NotNullWhen(true)] out DependencyInfo? OutInfo)
		{
			if (!InputFile.Exists)
			{
				OutInfo = null;
				return false;
			}

			ReadOnlySpan<char> FullName = InputFile.FullName;
			int index = FullName.IndexOf(Intermediate, FileReference.Comparison);
			if (index == -1)
			{
				Logger.LogWarning("Path {InputPath} is not under intermediate folder", InputFile);
				OutInfo = null;
				return false;
			}

			ReadOnlySpan<char> filterPath = FullName.Slice(index + Intermediate.Length);

			if (filterPath.StartsWith(External))
			{
				filterPath = filterPath.Slice(External.Length);
			}

			if (!filterPath.StartsWith(Build))
			{
				Logger.LogWarning("Path {InputPath} is not under intermediate build folder", InputFile);
				OutInfo = null;
				return false;
			}
			filterPath = filterPath.Slice(Build.Length);

			CachePartition? partition = null;

			foreach ((DirectoryReference Root, List<(string Filter, CachePartition Partition)>) root in GlobalPartitions)
			{
				if (!InputFile.Location.IsUnderDirectory(root.Root))
				{
					continue;
				}

				foreach ((string Filter, CachePartition Partition) entry in root.Item2)
				{
					if (filterPath.StartsWith(entry.Filter, FileReference.Comparison))
					{
						partition = entry.Partition;
						break;
					}
				}
				break;

			}

			if (partition == null)
			{
				Logger.LogWarning("CppDependencyCache - There is no partition covering path to {InputPath}!", InputFile);
				OutInfo = null;
				return false;
			}

			DependencyInfo? Info;
			if (!partition.DependencyFileToInfo.TryGetValue(InputFile, out Info) || InputFile.LastWriteTimeUtc.Ticks > Info.LastWriteTimeUtc)
			{
				try
				{
					Info = ReadDependencyInfo(InputFile);
					partition.DependencyFileToInfo.AddOrUpdate(InputFile, Info, (k, v) => Info);
					partition.bModified = true;
				}
				catch (Exception Ex)
				{
					Logger.LogDebug("Unable to read {File}:\n{Ex}", InputFile, ExceptionUtils.FormatExceptionDetails(Ex));
					OutInfo = null;
					return false;
				}
			}

			OutInfo = Info;
			return true;
		}

		/// <summary>
		/// Attempts to read the dependencies from the given input file. This static version will not use any caches.
		/// </summary>
		/// <param name="InputFile">File to be read</param>
		/// <param name="OutDependencyItems">Receives a list of output items</param>
		/// <returns>True if the input file exists and the dependencies were read</returns>
		public static bool TryGetDependenciesUncached(FileItem InputFile, [NotNullWhen(true)] out List<FileItem>? OutDependencyItems)
		{
			if (!InputFile.Exists)
			{
				OutDependencyItems = null;
				return false;
			}

			OutDependencyItems = ReadDependencyInfo(InputFile).Files;
			return true;
		}

		/// <summary>
		/// Creates a cache hierarchy for a particular target
		/// </summary>
		/// <param name="TargetDescriptor">Target descriptor being built</param>
		/// <param name="TargetType">The target type</param>
		/// <param name="Logger">Logger for output</param>
		/// <returns>Dependency cache hierarchy for the given project</returns>
		public void Mount(TargetDescriptor TargetDescriptor, TargetType TargetType, ILogger Logger)
		{

			string AppName = TargetDescriptor.Name;
			if (TargetDescriptor.ProjectFile == null || !Unreal.IsEngineInstalled())
			{
				if (TargetType == TargetType.Program)
				{
					AppName = TargetDescriptor.Name;
				}
				else
				{
					AppName = UEBuildTarget.GetAppNameForTargetType(TargetType);
				}
			}
			Mount(TargetDescriptor.ProjectFile, AppName, TargetDescriptor.Platform, TargetDescriptor.Configuration, TargetDescriptor.Architectures, TargetDescriptor.IntermediateEnvironment, Logger);
		}

		/// <summary>
		/// Same as Mount but in a task
		/// </summary>
		public void MountAsync(TargetDescriptor descriptor, string AppName, UnrealIntermediateEnvironment intermediateEnvironment, ILogger logger)
		{
			lock (GlobalTasks)
			{
				GlobalTasks.Add(Task.Run(() =>
				{
					Mount(descriptor.ProjectFile, AppName, descriptor.Platform, descriptor.Configuration, descriptor.Architectures, intermediateEnvironment, logger);
				}));
			}
		}

		public void WaitAsync()
		{
			Task[] tasks;
			lock (GlobalTasks)
			{
				tasks = GlobalTasks.ToArray();
				GlobalTasks.Clear();
			}

			using ITimelineEvent ActionGraphTimer = Timeline.ScopeEvent($"CppDependencyCacheWait");
			Task.WaitAll(tasks);
		}

		/// <summary>
		/// Creates a cache hierarchy for a particular target
		/// </summary>
		/// <param name="projectFile">Project file for the target being built</param>
		/// <param name="appName">Name of the app.</param>
		/// <param name="platform">Platform being built</param>
		/// <param name="configuration">Configuration being built</param>
		/// <param name="architectures">The target architectures</param>
		/// <param name="intermediateEnvironment">Intermediate environment to use</param>
		/// <param name="logger">Logger for output</param>
		/// <returns>Dependency cache hierarchy for the given project</returns>
		public void Mount(FileReference? projectFile, string appName, UnrealTargetPlatform platform, UnrealTargetConfiguration configuration, UnrealArchitectures architectures, UnrealIntermediateEnvironment intermediateEnvironment, ILogger logger)
		{
			using ITimelineEvent ActionGraphTimer = Timeline.ScopeEvent($"CppDependencyCache");

			appName = UEBuildTarget.GetTargetIntermediateFolderName(appName, intermediateEnvironment);

			string GetFilterPath(UnrealArch arch, UnrealTargetConfiguration configuration)
			{
				StringBuilder sb = new();
				sb.Append(platform).Append(Path.DirectorySeparatorChar).Append(UnrealArchitectureConfig.ForPlatform(platform).GetFolderNameForArchitecture(arch)).Append(Path.DirectorySeparatorChar);
				sb.Append(appName).Append(Path.DirectorySeparatorChar).Append(configuration);
				return sb.ToString();
			}

			void AddPartition(DirectoryReference rootPath, string filterPath)
			{
				CachePartition? partition = null;
				lock (GlobalPartitions)
				{
					(DirectoryReference Root, List<(string Filter, CachePartition Partition)>) root = GlobalPartitions.FirstOrDefault(e => e.Root == rootPath);
					if (root.Root == null)
					{
						root = new(rootPath, new());
						GlobalPartitions.Add(root);
					}
					List<(string Filter, CachePartition Partition)> filters = root.Item2;
					(string Filter, CachePartition Partition) filter = filters.FirstOrDefault(f => f.Filter == filterPath);
					partition = filter.Partition;
					if (partition == null)
					{
						FileReference location = FileReference.Combine(rootPath, "Intermediate", "Build", filterPath, "DependencyCache.bin");
						partition = new CachePartition(location);
						filters.Add((filterPath, partition));
					}
				}

				lock (partition)
				{
					if (!partition.Loaded)
					{
						partition.Loaded = true;
						if (FileReference.Exists(partition.Location))
						{
							partition.Read(logger);
						}
					}
				}
			}

			foreach (UnrealArch arch in architectures.Architectures)
			{
				AddPartition(Unreal.WritableEngineDirectory, GetFilterPath(arch, configuration));
				if (configuration == UnrealTargetConfiguration.DebugGame)
				{
					AddPartition(Unreal.WritableEngineDirectory, GetFilterPath(arch, UnrealTargetConfiguration.Development));
				}

				if (projectFile != null)
				{
					AddPartition(projectFile.Directory, GetFilterPath(arch, configuration));

					// Only needed for targets that has unique build environment
					if (configuration == UnrealTargetConfiguration.DebugGame)
					{
						AddPartition(projectFile.Directory, GetFilterPath(arch, UnrealTargetConfiguration.Development));
					}
				}
			}
		}

		/// <summary>
		/// Save all the caches that have been modified
		/// </summary>
		public void SaveAll()
		{
			Parallel2.ForEach(GlobalPartitions.SelectMany(p => p.Item2).Select(f => f.Partition), Cache =>
			{
				if (Cache.bModified)
				{
					Cache.Write();
				}
			});
		}

		/// <summary>
		/// Reads dependencies from the given file.
		/// </summary>
		/// <param name="InputFile">The file to read from</param>
		/// <returns>List of included dependencies</returns>
		internal static DependencyInfo ReadDependencyInfo(FileItem InputFile)
		{
			if (InputFile.HasExtension(".d"))
			{
				string Text = FileReference.ReadAllText(InputFile.Location);

				List<string> Tokens = new List<string>();

				StringBuilder Token = new StringBuilder();
				for (int Idx = 0; TryReadMakefileToken(Text, ref Idx, Token);)
				{
					Tokens.Add(Token.ToString());
				}

				int TokenIdx = 0;
				while (TokenIdx < Tokens.Count && Tokens[TokenIdx] == "\n")
				{
					TokenIdx++;
				}

				if (TokenIdx + 1 >= Tokens.Count || Tokens[TokenIdx + 1] != ":")
				{
					throw new BuildException($"Unable to parse dependency file {InputFile.Location}");
				}

				TokenIdx += 2;

				// TODO: Move this to the provided from the outside somewhere.
				string CurrentDir = String.Empty;

				List<FileItem> NewDependencyFiles = new List<FileItem>();
				for (; TokenIdx < Tokens.Count && Tokens[TokenIdx] != "\n"; TokenIdx++)
				{
					string FilePath = Tokens[TokenIdx];
					if (!Path.IsPathFullyQualified(FilePath))
					{
						if (CurrentDir.Length == 0)
						{
							CurrentDir = Directory.GetCurrentDirectory(); // Get current directory has crazy contention. So we only want to call it once
						}
						FilePath = Path.Combine(CurrentDir, FilePath);
					}
					NewDependencyFiles.Add(FileItem.GetItemByPath(FilePath));
				}

				while (TokenIdx < Tokens.Count && Tokens[TokenIdx] == "\n")
				{
					TokenIdx++;
				}

				if (TokenIdx != Tokens.Count)
				{
					throw new BuildException($"Unable to parse dependency file {InputFile.Location}");
				}

				return new DependencyInfo(InputFile.LastWriteTimeUtc.Ticks, null, null, NewDependencyFiles);
			}
			else if (InputFile.HasExtension(".txt"))
			{
				string[] Lines = FileReference.ReadAllLines(InputFile.Location);

				HashSet<FileItem> DependencyItems = new HashSet<FileItem>();
				foreach (string Line in Lines)
				{
					if (Line.Length > 0)
					{
						// Ignore *.tlh and *.tli files generated by the compiler from COM DLLs
						if (!Line.EndsWith(".tlh", StringComparison.OrdinalIgnoreCase) && !Line.EndsWith(".tli", StringComparison.OrdinalIgnoreCase))
						{
							string FixedLine = Line.Replace("\\\\", "\\", StringComparison.Ordinal); // ISPC outputs files with escaped slashes
							DependencyItems.Add(FileItem.GetItemByPath(FixedLine));
						}
					}
				}
				return new DependencyInfo(InputFile.LastWriteTimeUtc.Ticks, null, null, DependencyItems.ToList());
			}
			else if (InputFile.HasExtension(".json"))
			{
				// https://docs.microsoft.com/en-us/cpp/build/reference/sourcedependencies?view=msvc-160&viewFallbackFrom=vs-2019

				JsonObject Object = JsonObject.Read(InputFile.Location);

				if (!Object.TryGetStringField("Version", out string? VersionString))
				{
					throw new BuildException(
						$"Dependency file \"{InputFile.Location}\" does not have have a \"Version\" field.");
				}

				if (!VersionNumber.TryParse(VersionString, out VersionNumber? Version) || Version == null)
				{
					throw new BuildException(
						$"Dependency file \"{InputFile.Location}\" does not have have a valid \"Version\" field (\"{VersionString}\").");
				}

				if (Version < MinVCDependencyVersion)
				{
					throw new BuildException(
						$"Dependency file \"{InputFile.Location}\" version (\"{Version}\") is not supported version");
				}

				JsonObject? Data;
				if (!Object.TryGetObjectField("Data", out Data))
				{
					throw new BuildException("Missing 'Data' field in {0}", InputFile);
				}

				Data.TryGetStringField("ProvidedModule", out string? ProducedModule);

				List<(string Name, string BMI)>? ImportedModules = null;

				if (Version >= VCDependencyAdditionalModuleInfoVersion && !InputFile.HasExtension(".md.json"))
				{
					if (Data.TryGetObjectArrayField("ImportedModules", out JsonObject[]? ImportedModulesJson))
					{
						if (ImportedModulesJson.Length > 0)
						{
							ImportedModules = new List<(string Name, string BMI)>();

							foreach (JsonObject ImportedModule in ImportedModulesJson)
							{
								ImportedModule.TryGetStringField("Name", out string? Name);
								ImportedModule.TryGetStringField("BMI", out string? BMI);

								ImportedModules.Add((Name!, BMI!));
							}
						}
					}
				}
				else
				{
					if (Data.TryGetStringArrayField("ImportedModules", out string[]? ImportedModuleArray) && ImportedModuleArray.Length > 0)
					{
						ImportedModules =
							new List<(string Name, string BMI)>(ImportedModuleArray.ConvertAll(x => (x, "")));
					}
				}

				List<FileItem> Files = new List<FileItem>();
				{
					Data.TryGetStringArrayField("Includes", out string[]? Includes);

					if (Includes != null)
					{
						foreach (string Include in Includes)
						{
							Files.Add(FileItem.GetItemByPath(Include));
						}
					}
				}

				return new DependencyInfo(InputFile.LastWriteTimeUtc.Ticks, ProducedModule, ImportedModules, Files);
			}
			else
			{
				throw new BuildException("Unknown dependency list file type: {0}", InputFile);
			}
		}

		/// <summary>
		/// Attempts to read a single token from a makefile
		/// </summary>
		/// <param name="Text">Text to read from</param>
		/// <param name="RefIdx">Current position within the file</param>
		/// <param name="Token">Receives the token characters</param>
		/// <returns>True if a token was read, false if the end of the buffer was reached</returns>
		static bool TryReadMakefileToken(string Text, ref int RefIdx, StringBuilder Token)
		{
			Token.Clear();

			int Idx = RefIdx;
			for (; ; )
			{
				if (Idx == Text.Length)
				{
					return false;
				}

				// Skip whitespace
				while (Text[Idx] == ' ' || Text[Idx] == '\t')
				{
					if (++Idx == Text.Length)
					{
						return false;
					}
				}

				// Colon token
				if (Text[Idx] == ':')
				{
					Token.Append(':');
					RefIdx = Idx + 1;
					return true;
				}

				// Check for a newline
				if (Text[Idx] == '\r' || Text[Idx] == '\n')
				{
					Token.Append('\n');
					RefIdx = Idx + 1;
					return true;
				}

				// Check for an escaped newline
				if (Text[Idx] == '\\' && Idx + 1 < Text.Length)
				{
					if (Text[Idx + 1] == '\n')
					{
						Idx += 2;
						continue;
					}
					if (Text[Idx + 1] == '\r' && Idx + 2 < Text.Length && Text[Idx + 2] == '\n')
					{
						Idx += 3;
						continue;
					}
				}

				// Read a token. Special handling for drive letters on Windows!
				for (; Idx < Text.Length; Idx++)
				{
					if (Text[Idx] == ' ' || Text[Idx] == '\t' || Text[Idx] == '\r' || Text[Idx] == '\n')
					{
						break;
					}
					if (Text[Idx] == ':' && Token.Length > 1)
					{
						break;
					}
					if (Text[Idx] == '\\' && Idx + 1 < Text.Length && Text[Idx + 1] == ' ')
					{
						Idx++;
					}
					Token.Append(Text[Idx]);
				}

				RefIdx = Idx;
				return true;
			}
		}
	}
}
