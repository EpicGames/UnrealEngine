// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Diagnostics;
using System.Linq;
using System.Text;
using System.Text.RegularExpressions;
using EpicGames.Core;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// Builds a reverse lookup table for finding all source files that includes a set of headers
	/// </summary>
	class CppIncludeLookup
	{
		[DebuggerDisplay("{File}")]
		class CppIncludeFileInfo
		{
			public FileItem File { get; }
			public List<CppIncludeNameInfo>? IncludedNames { get; set; }
			public DateTime LastWriteTimeUtc { get; set; }

			public CppIncludeFileInfo(FileItem File) => this.File = File;
		}

		[DebuggerDisplay("{Name}")]
		class CppIncludeNameInfo
		{
			public string Name { get; }
			public List<CppIncludeFileInfo> Files { get; set; } = new List<CppIncludeFileInfo>();
			public HashSet<CppIncludeNameInfo> IncludedByNames { get; set; } = new HashSet<CppIncludeNameInfo>();
			public int Index { get; set; }

			public CppIncludeNameInfo(string Name) => this.Name = Name;
		}

		const int CurrentVersion = 1;

		static readonly Regex SourceFileRegex = new Regex(@"\.(?:c|cpp|cxx)$");
		static readonly Regex SourceOrHeaderFileRegex = new Regex(@"\.(?:c|h|cpp|hpp|cxx|hxx)$");

		readonly FileReference Location;
		Dictionary<string, CppIncludeNameInfo> NameToInfo = new Dictionary<string, CppIncludeNameInfo>(StringComparer.OrdinalIgnoreCase);

		public CppIncludeLookup(FileReference Location)
		{
			this.Location = Location;
		}

		public void Load()
		{
			if (FileReference.Exists(Location))
			{
				using (BinaryArchiveReader Reader = new BinaryArchiveReader(Location))
				{
					int Version = Reader.ReadInt();
					if (Version == CurrentVersion)
					{
						CppIncludeNameInfo[] NameInfos = Reader.ReadArray(() => new CppIncludeNameInfo(Reader.ReadString()!))!;
						foreach (CppIncludeNameInfo NameInfo in NameInfos)
						{
							int NumFiles = Reader.ReadInt();
							for (int Idx = 0; Idx < NumFiles; Idx++)
							{
								CppIncludeFileInfo FileInfo = new CppIncludeFileInfo(Reader.ReadCompactFileItem()!);

								int[]? NameIndexes = Reader.ReadIntArray();
								if (NameIndexes != null)
								{
									FileInfo.IncludedNames = NameIndexes.Select(x => NameInfos[x]).ToList();
									FileInfo.LastWriteTimeUtc = new DateTime(Reader.ReadLong(), DateTimeKind.Utc);
								}

								NameInfo.Files.Add(FileInfo);
							}
						}
						NameToInfo = NameInfos.ToDictionary(x => x.Name, x => x, StringComparer.OrdinalIgnoreCase);
					}
				}
			}
		}

		public void Save()
		{
			using (BinaryArchiveWriter Writer = new BinaryArchiveWriter(Location))
			{
				List<CppIncludeNameInfo> NameInfos = NameToInfo.Values.ToList();
				Writer.WriteInt(CurrentVersion);

				Dictionary<CppIncludeNameInfo, int> NameToIndex = new Dictionary<CppIncludeNameInfo, int>(NameInfos.Count);
				for (int Idx = 0; Idx < NameInfos.Count; Idx++)
				{
					NameToIndex[NameInfos[Idx]] = Idx;
				}

				Writer.WriteList(NameInfos, x => Writer.WriteString(x.Name));
				foreach (CppIncludeNameInfo NameInfo in NameInfos)
				{
					Writer.WriteInt(NameInfo.Files.Count);
					foreach (CppIncludeFileInfo FileInfo in NameInfo.Files)
					{
						Writer.WriteCompactFileItem(FileInfo.File);
						if (FileInfo.IncludedNames == null)
						{
							Writer.WriteIntArray(null);
						}
						else
						{
							Writer.WriteIntArray(FileInfo.IncludedNames.Select(x => NameToIndex[x]).ToArray());
							Writer.WriteLong(FileInfo.LastWriteTimeUtc.Ticks);
						}
					}
				}
			}
		}

		public IEnumerable<FileItem> FindFiles(IEnumerable<string> Names)
		{
			List<CppIncludeNameInfo> NameInfoList = new List<CppIncludeNameInfo>();
			foreach (string Name in Names)
			{
				CppIncludeNameInfo? NameInfo;
				if (NameToInfo.TryGetValue(Name, out NameInfo))
				{
					NameInfoList.Add(NameInfo);
				}
			}

			HashSet<CppIncludeNameInfo> NameInfoSet = new HashSet<CppIncludeNameInfo>(NameInfoList);
			for (int Idx = 0; Idx < NameInfoList.Count; Idx++)
			{
				CppIncludeNameInfo NameInfo = NameInfoList[Idx];
				foreach (CppIncludeNameInfo IncludedByName in NameInfo.IncludedByNames)
				{
					if (NameInfoSet.Add(IncludedByName))
					{
						NameInfoList.Add(IncludedByName);
					}
				}
			}

			return NameInfoSet.SelectMany(x => x.Files).Select(x => x.File);
		}

		public void Update(IEnumerable<DirectoryReference> RootDirs, IEnumerable<FileItem> AdditionalFiles)
		{
			// Find all the source files in the given root directory
			ConcurrentBag<FileItem> KeepFileItems = new ConcurrentBag<FileItem>(AdditionalFiles);
			using (ThreadPoolWorkQueue Queue = new ThreadPoolWorkQueue())
			{
				foreach (DirectoryReference RootDir in RootDirs)
				{
					DirectoryItem RootDirItem = DirectoryItem.GetItemByDirectoryReference(RootDir);
					Queue.Enqueue(() => ScanRootDir(RootDirItem, KeepFileItems, Queue));
				}
			}

			// Remove any files which no longer exist
			HashSet<FileItem> FileItemsSet = new HashSet<FileItem>(KeepFileItems);
			foreach (CppIncludeNameInfo NameInfo in NameToInfo.Values)
			{
				NameInfo.Files.RemoveAll(x => !FileItemsSet.Contains(x.File));
			}
			FileItemsSet.ExceptWith(NameToInfo.Values.SelectMany(x => x.Files).Select(x => x.File));

			// Add any new items
			foreach (FileItem FileItem in FileItemsSet)
			{
				CppIncludeNameInfo NameInfo = FindOrAddNameInfo(FileItem.Name);
				NameInfo.Files.Add(new CppIncludeFileInfo(FileItem));
			}

			// Update all the dependencies
			HashSet<CppIncludeFileInfo> Files = new HashSet<CppIncludeFileInfo>();
			using (ThreadPoolWorkQueue Queue = new ThreadPoolWorkQueue())
			{
				List<CppIncludeFileInfo> RootFiles = NameToInfo.Values.Where(x => SourceFileRegex.IsMatch(x.Name)).SelectMany(x => x.Files).ToList();
				foreach (CppIncludeFileInfo FileInfo in RootFiles)
				{
					Queue.Enqueue(() => ScanDependencies(FileInfo, Files, Queue));
				}
			}

			// Build the inverse lookup from name to files
			foreach (CppIncludeNameInfo NameInfo in NameToInfo.Values)
			{
				foreach (CppIncludeFileInfo FileInfo in NameInfo.Files)
				{
					if (FileInfo.IncludedNames != null)
					{
						foreach (CppIncludeNameInfo IncludeNameInfo in FileInfo.IncludedNames)
						{
							IncludeNameInfo.IncludedByNames.Add(NameInfo);
						}
					}
				}
			}
		}

		void ScanDependencies(CppIncludeFileInfo FileInfo, HashSet<CppIncludeFileInfo> Files, ThreadPoolWorkQueue Queue)
		{
			if (!FileInfo.File.Exists)
			{
				return;
			}
			// Update names for this file
			if (FileInfo.IncludedNames == null || FileInfo.File.LastWriteTimeUtc != FileInfo.LastWriteTimeUtc)
			{
				byte[] Data = FileReference.ReadAllBytes(FileInfo.File.Location);

				List<string> Names = new List<string>();
				FindIncludes(Data, Names);

				lock (NameToInfo)
				{
					FileInfo.IncludedNames = Names.ConvertAll(x => FindOrAddNameInfo(x));
					FileInfo.LastWriteTimeUtc = FileInfo.File.LastWriteTimeUtc;
				}
			}

			// Update all the files that 
			lock (Files)
			{
				foreach (CppIncludeFileInfo IncludedFile in FileInfo.IncludedNames.SelectMany(x => x.Files))
				{
					if (Files.Add(IncludedFile))
					{
						Queue.Enqueue(() => ScanDependencies(IncludedFile, Files, Queue));
					}
				}
			}
		}

		static void ScanRootDir(DirectoryItem DirItem, ConcurrentBag<FileItem> FileItems, ThreadPoolWorkQueue Queue)
		{
			foreach (DirectoryReference ExtensionDir in Unreal.GetExtensionDirs(DirItem.Location))
			{
				DirectoryItem ExtensionDirItem = DirectoryItem.GetItemByDirectoryReference(ExtensionDir);

				DirectoryItem PluginsDirItem = DirectoryItem.Combine(ExtensionDirItem, "Plugins");
				if (PluginsDirItem.Exists)
				{
					Queue.Enqueue(() => ScanPluginDir(PluginsDirItem, FileItems, Queue));
				}

				DirectoryItem SourceDirItem = DirectoryItem.Combine(ExtensionDirItem, "Source");
				if (SourceDirItem.Exists)
				{
					Queue.Enqueue(() => ScanSourceDir(SourceDirItem, FileItems, Queue));
				}
			}
		}

		static void ScanPluginDir(DirectoryItem DirItem, ConcurrentBag<FileItem> FileItems, ThreadPoolWorkQueue Queue)
		{
			if (DirItem.EnumerateFiles().Any(x => x.HasExtension(".uplugin")))
			{
				DirectoryItem SourceDirItem = DirectoryItem.Combine(DirItem, "Source");
				if (SourceDirItem.Exists)
				{
					Queue.Enqueue(() => ScanSourceDir(SourceDirItem, FileItems, Queue));
				}
			}
			else
			{
				foreach (DirectoryItem SubDirItem in DirItem.EnumerateDirectories())
				{
					Queue.Enqueue(() => ScanPluginDir(SubDirItem, FileItems, Queue));
				}
			}
		}

		static void ScanSourceDir(DirectoryItem DirItem, ConcurrentBag<FileItem> FileItems, ThreadPoolWorkQueue Queue)
		{
			foreach (DirectoryItem SubDirItem in DirItem.EnumerateDirectories())
			{
				Queue.Enqueue(() => ScanSourceDir(SubDirItem, FileItems, Queue));
			}
			foreach (FileItem FileItem in DirItem.EnumerateFiles().Where(x => SourceOrHeaderFileRegex.IsMatch(x.Name)))
			{
				FileItems.Add(FileItem);
			}
		}

		CppIncludeNameInfo FindOrAddNameInfo(string Name)
		{
			CppIncludeNameInfo? NameInfo;
			if (!NameToInfo.TryGetValue(Name, out NameInfo))
			{
				NameInfo = new CppIncludeNameInfo(Name);
				NameToInfo.Add(Name, NameInfo);
			}
			return NameInfo;
		}

		const string IncludeText = "include";
		static readonly ReadOnlyMemory<byte> IncludeBytes = Encoding.UTF8.GetBytes(IncludeText);
		static readonly sbyte[] SkipTable = CreateSkipTable();

		static sbyte[] CreateSkipTable()
		{
			sbyte[] Table = new sbyte[256];
			for (int Idx = 0; Idx < 256; Idx++)
			{
				Table[Idx] = (sbyte)IncludeText.Length;
			}
			for (int Idx = IncludeText.Length - 1; Idx >= 0; Idx--)
			{
				Table[IncludeText[Idx]] = (sbyte)-Idx;
			}
			return Table;
		}

		static void FindIncludes(ReadOnlySpan<byte> Span, List<string> Names)
		{
			ReadOnlySpan<byte> IncludeSpan = IncludeBytes.Span;

			int Idx = IncludeText.Length;
			while (Idx < Span.Length)
			{
				int Skip = SkipTable[Span[Idx]];
				Idx += Skip;

				if (Skip > 0)
				{
					continue;
				}

				byte[]? Name = ParseIncludeName(Span, Idx);
				if (Name != null)
				{
					Names.Add(Encoding.UTF8.GetString(Name));
				}

				Idx += IncludeSpan.Length;
			}
		}

		static byte[]? ParseIncludeName(ReadOnlySpan<byte> Span, int Idx)
		{
			ReadOnlySpan<byte> IncludeSpan = IncludeBytes.Span;
			if (!Span.Slice(Idx).StartsWith(IncludeSpan))
			{
				return null;
			}

			// Check it's an include directive
			int MinIdx = Idx;
			for (; ; )
			{
				byte Char = Span[--MinIdx];
				if (Char == '#')
				{
					break;
				}
				if (Char != ' ' && Char != '\t')
				{
					return null;
				}
				if (MinIdx == 1)
				{
					return null;
				}
			}
			for (; MinIdx > 0;)
			{
				byte Char = Span[--MinIdx];
				if (Char == '\n')
				{
					break;
				}
				if (Char != ' ' && Char != '\t')
				{
					return null;
				}
			}

			// Scan to the start of the filename
			int MinNameIdx = Idx + IncludeSpan.Length;
			if (MinNameIdx + 1 >= Span.Length)
			{
				return null;
			}
			for (; ; )
			{
				byte Char = Span[MinNameIdx++];
				if (Char == '\"' || Char == '<')
				{
					break;
				}
				if (Char != ' ' && Char != '\t')
				{
					return null;
				}
				if (MinNameIdx + 1 >= Span.Length)
				{
					return null;
				}
			}

			// Scan to the end of the included file. Update MinNameIndex to the start of the filename.
			int MaxNameIdx = MinNameIdx;
			for (; ; MaxNameIdx++)
			{
				byte Char = Span[MaxNameIdx];
				if (Char == '\"' || Char == '>')
				{
					break;
				}
				if (Char == '/' || Char == '\\')
				{
					MinNameIdx = MaxNameIdx + 1;
				}
				if (MaxNameIdx + 1 >= Span.Length)
				{
					return null;
				}
			}

			// Normalize the filename
			byte[] Name = new byte[MaxNameIdx - MinNameIdx];
			Span.Slice(MinNameIdx, MaxNameIdx - MinNameIdx).CopyTo(Name);
			return Name;
		}
	}
}
