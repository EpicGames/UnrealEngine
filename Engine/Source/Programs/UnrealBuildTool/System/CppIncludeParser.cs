// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Text;

namespace UnrealBuildTool
{
	/// <summary>
	/// Parses include directives from cpp source files, to make dedicated PCHs
	/// </summary>
	internal static class CppIncludeParser
	{
		/// <summary>
		/// Copies include directives from one file to another, until an unsafe directive (or non-#include line) is found.
		/// </summary>
		/// <param name="reader">Stream to read from</param>
		/// <param name="writer">Stream to write directives to</param>
		/// <param name="tempStorage">Buffer to use for parser token storage</param>
		internal static void CopyIncludeDirectives(StreamReader reader, StringBuilder writer, StringBuilder tempStorage)
		{
			while (TryReadToken(reader, tempStorage))
			{
				if (tempStorage.Length > 1 || tempStorage[0] != '\n')
				{
					if (tempStorage.Length != 1 || tempStorage[0] != '#')
					{
						break;
					}
					if (!TryReadToken(reader, tempStorage))
					{
						break;
					}

					ReadOnlySpan<char> directiveSpan = GetSpanFromStringBuilder(tempStorage);
					if (directiveSpan.Equals("pragma", StringComparison.Ordinal))
					{
						if (!TryReadToken(reader, tempStorage) || !GetSpanFromStringBuilder(tempStorage).Equals("once", StringComparison.Ordinal))
						{
							break;
						}
						if (!TryReadToken(reader, tempStorage) || tempStorage[0] != '\n')
						{
							break;
						}
					}
					else if (directiveSpan.Equals("include", StringComparison.Ordinal))
					{
						// @todo We support <> includes elsewhere but only "" ones here - should we add a check for < or remove the <> elsewhere?
						if (!TryReadToken(reader, tempStorage) || tempStorage[0] != '\"')
						{
							break;
						}

						ReadOnlySpan<char> tokenSpan = GetSpanFromStringBuilder(tempStorage);

						if (!tokenSpan.EndsWith(".h\"", StringComparison.Ordinal) && !tokenSpan.EndsWith(".h>", StringComparison.Ordinal))
						{
							break;
						}
						if (tokenSpan.Equals("\"RequiredProgramMainCPPInclude.h\"", StringComparison.OrdinalIgnoreCase))
						{
							break;
						}

						int writerLength = writer.Length;
						writer.Append("#include ").Append(tokenSpan).AppendLine();

						if (!TryReadToken(reader, tempStorage) || tempStorage[0] != '\n')
						{
							// Undo the previous append as the above condition negates it.
							writer.Length = writerLength;
							break;
						}
					}
					else
					{
						break;
					}
				}
			}
		}

		private static ReadOnlySpan<char> GetSpanFromStringBuilder(StringBuilder stringBuilder)
		{
			// If the headers fit into the initial allocation of the StringBuilder, we can just use its internal memory
			// rather than allocating more memory.
			StringBuilder.ChunkEnumerator ChunkEnumerator = stringBuilder.GetChunks();
			ChunkEnumerator.MoveNext();
			ReadOnlyMemory<char> MaybeOnlyChunk = ChunkEnumerator.Current;
			// MoveNext will be false if there's only one chunk i.e. our one chunk contains all the contents.
			ReadOnlySpan<char> TokenSpan = ChunkEnumerator.MoveNext() ? stringBuilder.ToString().AsSpan() : MaybeOnlyChunk.Span;
			return TokenSpan;
		}

		/// <summary>
		/// Reads an individual token from the input stream
		/// </summary>
		/// <param name="reader">Stream to read from</param>
		/// <param name="token">Buffer to store token read from the stream</param>
		/// <returns>True if a token was read, false otherwise</returns>
		private static bool TryReadToken(StreamReader reader, StringBuilder token)
		{
			token.Clear();

			int NextChar;
			for (; ; )
			{
				NextChar = reader.Read();
				if (NextChar == -1)
				{
					return false;
				}
				if (NextChar != ' ' && NextChar != '\t' && NextChar != '\r')
				{
					if (NextChar != '/')
					{
						break;
					}
					else if (reader.Peek() == '/')
					{
						reader.Read();
						for (; ; )
						{
							NextChar = reader.Read();
							if (NextChar == -1)
							{
								return false;
							}
							if (NextChar == '\n')
							{
								break;
							}
						}
					}
					else if (reader.Peek() == '*')
					{
						reader.Read();
						for (; ; )
						{
							NextChar = reader.Read();
							if (NextChar == -1)
							{
								return false;
							}
							if (NextChar == '*' && reader.Peek() == '/')
							{
								break;
							}
						}
						reader.Read();
					}
					else
					{
						break;
					}
				}
			}

			token.Append((char)NextChar);

			if (Char.IsLetterOrDigit((char)NextChar))
			{
				for (; ; )
				{
					NextChar = reader.Read();
					if (NextChar == -1 || !Char.IsLetterOrDigit((char)NextChar))
					{
						break;
					}
					token.Append((char)NextChar);
				}
			}
			else if (NextChar == '\"' || NextChar == '<')
			{
				for (; ; )
				{
					NextChar = reader.Read();
					if (NextChar == -1)
					{
						break;
					}
					token.Append((char)NextChar);
					if (NextChar == '\"' || NextChar == '>')
					{
						break;
					}
				}
			}

			return true;
		}
	}
}
