// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.Linq;
using EpicGames.Core;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// Named cpp root folders
	/// </summary>
	public enum CppRootPathFolder
	{
		/// <summary>
		/// Project folder. Can be both inside and outside Root so this must be first
		/// </summary>
		Project,

		/// <summary>
		/// UE Root folder 
		/// </summary>
		Root,

		/// <summary>
		/// Compiler folder
		/// </summary>
		Compiler,

		/// <summary>
		/// Toolchain folder (if needed)
		/// </summary>
		Toolchain,

		/// <summary>
		/// WindowsSDK folder (if needed)
		/// </summary>
		WinSDK,

		/// <summary>
		/// Additional platform SDK folder (if needed)
		/// </summary>
		PlatformSDK,

		/// <summary>
		/// Number of entries in enum
		/// </summary>
		Count,
	}

	/// <summary>
	/// Encapsulates the roots used for cache and vfs
	/// </summary>
	class CppRootPaths : IEnumerable<(uint id, DirectoryReference vfs, DirectoryReference local)>
	{
		private static readonly DirectoryReference _vfsRootPath = new(OperatingSystem.IsWindows() ? "Z:/UEVFS" : "/UEVFS");

		/// <summary>
		/// Default constructor
		/// </summary>
		public CppRootPaths()
		{
			_lookup = [];
			_extras = [];
			_folderNames = [];
		}

		/// <summary>
		/// Archive reader constructor
		/// </summary>
		public CppRootPaths(BinaryArchiveReader Reader)
		{
			{
				int count = Reader.ReadInt();
				_folderNames = [];
				for (int idx = 0; idx < count; idx++)
				{
					CppRootPathFolder folder = (CppRootPathFolder)Reader.ReadInt();
					string name = Reader.ReadString()!;
					_folderNames[folder] = name;
				}
			}
			{
				int count = Reader.ReadInt();
				_lookup = [];
				for (int idx = 0; idx < count; idx++)
				{
					CppRootPathFolder folder = (CppRootPathFolder)Reader.ReadInt();
					DirectoryReference local = Reader.ReadCompactDirectoryReference();
					_lookup.Add((folder, (AsVfsPath(folder), local)));
				}
				_lookup.Sort(static (a, b) => a.root.CompareTo(b.root));
			}
			{
				int count = Reader.ReadInt();
				_extras = [];
				for (int idx = 0; idx < count; idx++)
				{
					string id = Reader.ReadString()!;
					DirectoryReference vfs = DirectoryReference.Combine(_vfsRootPath, id);
					DirectoryReference local = Reader.ReadCompactDirectoryReference();
					_extras.Add(id, (vfs, local));
				}
			}

			bUseVfs = Reader.ReadBool();
		}

		/// <summary>
		/// Copy constructor
		/// </summary>
		public CppRootPaths(CppRootPaths Other)
		{
			_lookup = new(Other._lookup);
			_extras = new(Other._extras);
			_folderNames = new(Other._folderNames);
			bUseVfs = Other.bUseVfs;
		}

		/// <summary>
		/// Write root paths to archive
		/// </summary>
		public void Write(BinaryArchiveWriter Writer)
		{
			Writer.WriteDictionary(_folderNames, Key => Writer.WriteInt((int)Key), Value => Writer.WriteString(Value));
			Writer.WriteList(_lookup, item =>
			{
				Writer.WriteInt((int)item.root);
				Writer.WriteCompactDirectoryReference(item.Item2.local);
			});
			Writer.WriteDictionary(_extras, Key => Writer.WriteString(Key), Value => Writer.WriteCompactDirectoryReference(Value.local));
			Writer.WriteBool(bUseVfs);
		}

		/// <summary>
		/// Set path for named roots
		/// </summary>
		public DirectoryReference this[CppRootPathFolder key]
		{
			set
			{
				if (key == CppRootPathFolder.Project)
				{
					string projectName = value.GetDirectoryName();
					if (!projectName.StartsWith("Clang", StringComparison.Ordinal)) // Lol, there is a project called clang.. which collides with the compiler clang causing two roots to have the same path
					{
						AddFolderName(CppRootPathFolder.Project, value.GetDirectoryName());
					}
				}
				for (int i = 0; i != _lookup.Count; ++i)
				{
					if (_lookup[i].root == key)
					{
						_lookup[i] = (key, (AsVfsPath(key), value));
						return;
					}
				}
				_lookup.Add((key, (AsVfsPath(key), value)));
				_lookup.Sort(static (a, b) => a.root.CompareTo(b.root));
			}
		}

		/// <summary>
		/// Enumerator for all roots
		/// </summary>
		public IEnumerator<(uint id, DirectoryReference vfs, DirectoryReference local)> GetEnumerator()
		{
			foreach ((CppRootPathFolder root, (DirectoryReference vfs, DirectoryReference local)) in _lookup)
			{
				yield return ((uint)root, vfs, local);
			}

			uint index = (uint)CppRootPathFolder.Count; // Max of CppRootPathFolder
			foreach (KeyValuePair<string, (DirectoryReference vfs, DirectoryReference local)> kv in _extras)
			{
				yield return (index++, kv.Value.vfs, kv.Value.local);
			}
			yield break;
		}

		/// <summary>
		/// Enumerator for all roots
		/// </summary>
		System.Collections.IEnumerator System.Collections.IEnumerable.GetEnumerator()
		{
			return _lookup.GetEnumerator();
		}

		/// <summary>
		/// Adds extra root path to the RootPaths table
		/// </summary>
		public void AddExtraPath((string id, string path)? extra)
		{
			if (extra == null)
			{
				return;
			}

			DirectoryReference? path = DirectoryReference.FromString(extra.Value.path);
			if (path == null)
			{
				return;
			}

			string id = extra.Value.id;

			if (_extras.TryGetValue(id, out (DirectoryReference vfs, DirectoryReference local) root))
			{
				if (root.local != path)
				{
					throw new Exception($"Extra root path with id {id} already set. Current: {root.local.FullName} New: {path.FullName}");
				}
			}
			else
			{
				DirectoryReference vfs = DirectoryReference.Combine(_vfsRootPath, id);
				_extras[id] = (vfs, path);
			}
		}

		/// <summary>
		/// Merge in entries from another root paths object, discarding duplicates
		/// </summary>
		/// <param name="other">The paths to merge from</param>
		public void Merge(CppRootPaths other)
		{
			foreach ((CppRootPathFolder root, (DirectoryReference vfs, DirectoryReference local)) otherEntry in other._lookup)
			{
				bool found = false;
				foreach ((CppRootPathFolder root, (DirectoryReference vfs, DirectoryReference local)) in _lookup)
				{
					found |= root == otherEntry.root;
				}
				if (!found)
				{
					_lookup.Add(otherEntry);
				}
			}
			_lookup.Sort(static (a, b) => a.root.CompareTo(b.root));

			foreach (KeyValuePair<string, (DirectoryReference vfs, DirectoryReference local)> extra in other._extras.Where(x => !_extras.ContainsKey(x.Key)))
			{
				AddExtraPath((extra.Key, extra.Value.local.FullName));
			}
			foreach (KeyValuePair<CppRootPathFolder, string> name in other._folderNames.Where(x => !_folderNames.ContainsKey(x.Key)))
			{
				AddFolderName(name.Key, name.Value);
			}
		}

		/// <summary>
		/// Get a virtual filesystem overlay path if in use for a local path
		/// </summary>
		/// <param name="reference">The path to convert</param>
		/// <param name="vfsPath">Out param, the converted path</param>
		/// <returns>True if the local path can be mapped to a virtual path when enabled, false otherwise</returns>
		public bool GetVfsOverlayPath(FileSystemReference reference, [NotNullWhen(true)] out string? vfsPath)
		{
			vfsPath = null;
			if (!bUseVfs)
			{
				return false;
			}

			bool InternalGet(DirectoryReference vfs, DirectoryReference local, [NotNullWhen(true)]out string? vfsPath)
			{
				if (reference.IsUnderDirectory(local))
				{
					Span<char> initial = stackalloc char[512];
					using StackStringBuilder sb = new(initial);
					sb.Append(vfs.FullName);
					sb.Append(reference.FullName.AsSpan(local.FullName.Length));
					if (OperatingSystem.IsWindows())
					{
						sb.Replace('\\', '/');
					}
					vfsPath = sb.ToString();
					return true;
				}
				vfsPath = null;
				return false;
			}

			foreach ((CppRootPathFolder root, (DirectoryReference vfs, DirectoryReference local)) in _lookup)
			{
				if (InternalGet(vfs, local, out vfsPath))
				{
					return true;
				}
			}

			foreach ((DirectoryReference vfs, DirectoryReference local) in _extras.Values)
			{
				if (InternalGet(vfs, local, out vfsPath))
				{
					return true;
				}
			}

			return false;
		}

		/// <summary>
		/// Always return a local file reference (convert from vfs if needed)
		/// </summary>
		public FileReference GetLocalPath(FileReference reference)
		{
			if (!bUseVfs)
			{
				return reference;
			}

			foreach ((_, (DirectoryReference vfs, DirectoryReference local)) in _lookup)
			{
				if (reference.IsUnderDirectory(vfs))
				{
					return new(local.FullName + reference.FullName.Substring(vfs.FullName.Length));
				}
			}

			foreach ((DirectoryReference vfs, DirectoryReference local) in _extras.Values)
			{
				if (reference.IsUnderDirectory(vfs))
				{
					return new(local.FullName + reference.FullName.Substring(vfs.FullName.Length));
				}
			}

			return reference;
		}

		/// <summary>
		/// Populate from json objects
		/// </summary>
		public void Read(JsonObject Object)
		{
			if (Object.TryGetObjectArrayField("Lookup", out JsonObject[]? lookupItems))
			{
				foreach (JsonObject item in lookupItems)
				{
					DirectoryReference local = new(item.GetStringField("Path"));
					if (item.TryGetEnumField("Root", out CppRootPathFolder folder))
					{
						_lookup.Add((folder, (AsVfsPath(folder), local)));

						if (item.TryGetStringField("Root", out string? folderName))
						{
							_folderNames.Add(folder, folderName);
						}
					}
				}
				_lookup.Sort(static (a, b) => a.root.CompareTo(b.root));
			}

			if (Object.TryGetObjectArrayField("Extras", out JsonObject[]? extraItems))
			{
				foreach (JsonObject item in extraItems)
				{
					string id = item.GetStringField("Id");
					DirectoryReference local = new(item.GetStringField("Path"));
					_extras.Add(id, (AsVfsPath(id), local));
				}
			}

			if (Object.TryGetBoolField("bUseVfs", out bool useVfs))
			{
				bUseVfs = useVfs;
			}
		}

		/// <summary>
		/// Writing json objects to json writer
		/// </summary>
		public void Write(JsonWriter writer)
		{
			if (_lookup.Count > 0)
			{
				writer.WriteArrayStart("Lookup");
				foreach ((CppRootPathFolder root, (_, DirectoryReference local)) in _lookup)
				{
					writer.WriteObjectStart();
					writer.WriteEnumValue("Root", root);
					writer.WriteValue("Path", local.FullName);
					if (!_folderNames.TryGetValue(root, out string? folderName))
					{
						writer.WriteValue("Name", folderName);
					}
					writer.WriteObjectEnd();
				}
				writer.WriteArrayEnd();
			}

			if (_extras.Any())
			{
				writer.WriteArrayStart("Extras");
				foreach (KeyValuePair<string, (DirectoryReference vfs, DirectoryReference local)> item in _extras)
				{
					writer.WriteObjectStart();
					writer.WriteValue("Id", item.Key);
					writer.WriteValue("Path", item.Value.local.FullName);
					writer.WriteObjectEnd();
				}
				writer.WriteArrayEnd();
			}

			if (_folderNames.Any())
			{
				writer.WriteArrayStart("Names");
				foreach (KeyValuePair<CppRootPathFolder, string> item in _folderNames)
				{
					writer.WriteObjectStart();
					writer.WriteEnumValue("Root", item.Key);
					writer.WriteValue("Name", item.Value);
					writer.WriteObjectEnd();
				}
				writer.WriteArrayEnd();
			}

			writer.WriteValue("bUseVfs", bUseVfs);
		}

		/// <summary>
		/// Add an optional folder name to use in VFS paths rather than the enum name
		/// </summary>
		/// <param name="folder">The folder enum to update</param>
		/// <param name="name">The new string name</param>
		public void AddFolderName(CppRootPathFolder folder, string name)
		{
			_folderNames[folder] = name;
		}

		private DirectoryReference AsVfsPath(CppRootPathFolder folder)
		{
			string? folderName;
			if (!_folderNames.TryGetValue(folder, out folderName))
			{
				folderName = folder.ToString();
			}
			return DirectoryReference.Combine(_vfsRootPath, folderName);
		}
		private static DirectoryReference AsVfsPath(string id) => DirectoryReference.Combine(_vfsRootPath, id);

		private Dictionary<CppRootPathFolder, string> _folderNames { get; }
		private List<(CppRootPathFolder root, (DirectoryReference vfs, DirectoryReference local))> _lookup { get; } // We want this to be a list for perf reasons
		private SortedDictionary<string, (DirectoryReference vfs, DirectoryReference local)> _extras { get; }

		/// <summary>
		/// Set to true to make sure all created paths are using virtual path
		/// </summary>
		public bool bUseVfs { get; set; }
	}
}