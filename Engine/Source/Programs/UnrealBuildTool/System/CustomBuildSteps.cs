// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.Linq;
using EpicGames.Core;

namespace UnrealBuildTool
{
	/// <summary>
	/// Stores custom build steps to be executed by a project or plugin
	/// </summary>
	public class CustomBuildSteps
	{
		readonly Dictionary<UnrealTargetPlatform, string[]> HostPlatformToCommands = new Dictionary<UnrealTargetPlatform, string[]>();

		/// <summary>
		/// Construct a custom build steps object from a Json object.
		/// </summary>
		public CustomBuildSteps(JsonObject RawObject)
		{
			foreach (string HostPlatformName in RawObject.KeyNames)
			{
				UnrealTargetPlatform Platform;
				if (UnrealTargetPlatform.TryParse(HostPlatformName, out Platform))
				{
					HostPlatformToCommands.Add(Platform, RawObject.GetStringArrayField(HostPlatformName));
				}
			}
		}

		/// <summary>
		/// Reads a list of build steps from a Json project or plugin descriptor
		/// </summary>
		/// <param name="RawObject">The json descriptor object</param>
		/// <param name="FieldName">Name of the field to read</param>
		/// <param name="OutBuildSteps">Output variable to store the sorted dictionary that was read</param>
		/// <returns>True if the field was read (and OutBuildSteps is set), false otherwise.</returns>
		public static bool TryRead(JsonObject RawObject, string FieldName, [NotNullWhen(true)] out CustomBuildSteps? OutBuildSteps)
		{
			JsonObject? BuildStepsObject;
			if (RawObject.TryGetObjectField(FieldName, out BuildStepsObject))
			{
				OutBuildSteps = new CustomBuildSteps(BuildStepsObject);
				return true;
			}
			else
			{
				OutBuildSteps = null;
				return false;
			}
		}

		/// <summary>
		/// Reads a list of build steps from a Json project or plugin descriptor
		/// </summary>
		/// <param name="Writer">Writer to receive json output</param>
		/// <param name="FieldName">Name of the field to read</param>
		/// <returns>True if the field was read (and OutBuildSteps is set), false otherwise.</returns>
		public void Write(JsonWriter Writer, string FieldName)
		{
			Writer.WriteObjectStart(FieldName);
			foreach (KeyValuePair<UnrealTargetPlatform, string[]> Pair in HostPlatformToCommands.OrderBy(x => x.Key.ToString()))
			{
				Writer.WriteArrayStart(Pair.Key.ToString());
				foreach (string Line in Pair.Value)
				{
					Writer.WriteValue(Line);
				}
				Writer.WriteArrayEnd();
			}
			Writer.WriteObjectEnd();
		}

		/// <summary>
		/// Converts this object to a JsonObject to be saved or manipulated.
		/// </summary>
		/// <returns>The JsonObject representation of this object.</returns>
		public JsonObject ToJsonObject()
		{
			JsonObject CustomBuildStepObject = new JsonObject();
			foreach (KeyValuePair<UnrealTargetPlatform, string[]> Pair in HostPlatformToCommands.OrderBy(x => x.Key.ToString()))
			{
				CustomBuildStepObject.AddOrSetFieldValue(Pair.Key.ToString(), Pair.Value);
			}
			return CustomBuildStepObject;
		}

		/// <summary>
		/// Tries to get the commands for a given host platform
		/// </summary>
		/// <param name="HostPlatform">The host platform to look for</param>
		/// <param name="OutCommands">Array of commands</param>
		/// <returns>True if a list of commands was generated</returns>
		public bool TryGetCommands(UnrealTargetPlatform HostPlatform, [NotNullWhen(true)] out string[]? OutCommands)
		{
			string[]? Commands;
			if (HostPlatformToCommands.TryGetValue(HostPlatform, out Commands) && Commands.Length > 0)
			{
				OutCommands = Commands;
				return true;
			}
			else
			{
				OutCommands = null;
				return false;
			}
		}
	}
}
