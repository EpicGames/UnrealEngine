// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Threading;
using EpicGames.Core;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	///  Class to manage looking up data driven platform information (loaded from .ini files instead of in code)
	/// </summary>
	public static class DataDrivenPlatformInfo
	{
#pragma warning disable CA1034 // Nested types should not be visible - preserve API of APIs exposed to UAT
		/// <summary>
		/// All data driven information about a platform
		/// </summary>
		public class ConfigDataDrivenPlatformInfo
		{
			/// <summary>
			/// Is the platform a confidential ("console-style") platform
			/// </summary>
			public bool bIsConfidential = false;

			/// <summary>
			/// Is the platform enabled on this host platform
			/// </summary>
			public bool bIsEnabled = false;

			/// <summary>
			/// If true, this platform can stage the crashreporter
			/// </summary>
			public bool bCanUseCrashReporter = false;

			/// <summary>
			/// Additional restricted folders for this platform.
			/// </summary>
			public string[]? AdditionalRestrictedFolders = null;

			/// <summary>
			/// the compression format that this platform wants; overrides game unless bForceUseProjectCompressionFormat
			/// </summary>
			public string? HardwareCompressionFormat = null;

			/// <summary>
			/// The ICU data directory name this platform ships (e.g. "icudt64l", "icudt78l"). Null if unset.
			/// </summary>
			public string? ICUDataVersion = null;

			/// <summary>
			/// The online account can't be set from the command line
			/// </summary>
			public bool bNoAccountOverride = false;

			/// <summary>
			/// Entire ini parent chain, ending with this platform
			/// </summary>
			public string[]? IniParentChain = null;

			/// <summary>
			/// The raw DataDrivenPlatformInfo.ini file
			/// </summary>
			public ConfigFile? PlatformConfig = null;
		}
#pragma warning restore CA1034

		static IReadOnlyDictionary<string, ConfigDataDrivenPlatformInfo>? PlatformInfos = null;
		static readonly Lock LockObject = new();

		/// <summary>
		/// Return all data driven infos found
		/// </summary>
		/// <returns></returns>
		public static IReadOnlyDictionary<string, ConfigDataDrivenPlatformInfo> GetAllPlatformInfos()
		{
			// need to init?
			lock (LockObject)
			{
				if (PlatformInfos == null)
				{
					Dictionary<string, ConfigDataDrivenPlatformInfo> PlatformInfosDict = new Dictionary<string, ConfigDataDrivenPlatformInfo>(StringComparer.OrdinalIgnoreCase);
					PlatformInfos = PlatformInfosDict;
					Dictionary<string, string> IniParents = new Dictionary<string, string>(StringComparer.OrdinalIgnoreCase);

					// find all platform directories (skipping NFL/NoRedist)
					foreach (DirectoryReference EngineConfigDir in Unreal.GetExtensionDirs(Unreal.EngineDirectory, "Config", bIncludeRestrictedDirectories: false))
					{
						// look through all config dirs looking for the data driven ini file
						foreach (string FilePath in Directory.EnumerateFiles(EngineConfigDir.FullName, "DataDrivenPlatformInfo.ini", SearchOption.AllDirectories))
						{
							FileReference FileRef = new FileReference(FilePath);

							// get the platform name from the path
							string IniPlatformName;
							if (FileRef.IsUnderDirectory(DirectoryReference.Combine(Unreal.EngineDirectory, "Config")))
							{
								// Foo/Engine/Config/<Platform>/DataDrivenPlatformInfo.ini
								IniPlatformName = Path.GetFileName(Path.GetDirectoryName(FilePath))!;
							}
							else
							{
								// Foo/Engine/Platforms/<Platform>/Config/DataDrivenPlatformInfo.ini
								IniPlatformName = Path.GetFileName(Path.GetDirectoryName(Path.GetDirectoryName(FilePath)))!;
							}

							// load the DataDrivenPlatformInfo from the path (with Add support in a file that doesn't use +'s in the arrays for C++ usage)
							ConfigFile Config = new ConfigFile(FileRef, ConfigLineAction.Add);
							ConfigDataDrivenPlatformInfo NewInfo = new ConfigDataDrivenPlatformInfo();

							// we must have the key section 
							ConfigFileSection? Section = null;
							if (Config.TryGetSection("DataDrivenPlatformInfo", out Section))
							{
								ConfigHierarchySection ParsedSection = new ConfigHierarchySection(new List<ConfigFileSection>() { Section });

								// get string values
								string? IniParent;
								if (ParsedSection.TryGetValue("IniParent", out IniParent))
								{
									IniParents[IniPlatformName] = IniParent;
								}

								// slightly nasty bool parsing for bool values
								string? Temp;
								if (ParsedSection.TryGetValue("bIsConfidential", out Temp))
								{
									_ = ConfigHierarchy.TryParse(Temp, out NewInfo.bIsConfidential);
								}
								// unspecified means true for this property
								NewInfo.bCanUseCrashReporter = true;
								if (ParsedSection.TryGetValue("bCanUseCrashReporter", out Temp))
								{
									_ = ConfigHierarchy.TryParse(Temp, out NewInfo.bCanUseCrashReporter);
								}

								string HostKey = ConfigHierarchy.GetIniPlatformName(BuildHostPlatform.Current.Platform) + ":bIsEnabled";
								string NormalKey = "bIsEnabled";
								if (ParsedSection.TryGetValue(HostKey, out Temp) == true || ParsedSection.TryGetValue(NormalKey, out Temp) == true)
								{
									_ =ConfigHierarchy.TryParse(Temp, out NewInfo.bIsEnabled);
								}

								if (ParsedSection.TryGetValue("HardwareCompressionFormat", out NewInfo.HardwareCompressionFormat) == false)
								{
									NewInfo.HardwareCompressionFormat = null;
								}

								if (ParsedSection.TryGetValue("ICUDataVersion", out NewInfo.ICUDataVersion) == false)
								{
									NewInfo.ICUDataVersion = null;
								}

								NewInfo.bNoAccountOverride = false;
								if (ParsedSection.TryGetValue("bNoAccountOverride", out Temp))
								{
									_ = ConfigHierarchy.TryParse(Temp, out NewInfo.bNoAccountOverride);
								}

								// get a list of additional restricted folders
								IReadOnlyList<string>? AdditionalRestrictedFolders;
								if (ParsedSection.TryGetValues("AdditionalRestrictedFolders", out AdditionalRestrictedFolders) && AdditionalRestrictedFolders.Count > 0)
								{
									NewInfo.AdditionalRestrictedFolders = AdditionalRestrictedFolders.Select(x => x.Trim()).Where(x => x.Length > 0).ToArray();
								}

								// create cache it
								NewInfo.PlatformConfig = Config;
								PlatformInfosDict[IniPlatformName] = NewInfo;
							}
						}
					}

					// now that all are read in, calculate the ini parent chain, starting with parent-most
					foreach (KeyValuePair<string, ConfigDataDrivenPlatformInfo> Pair in PlatformInfos)
					{
						string? CurrentPlatform;

						// walk up the chain and build up the ini chain
						List<string> Chain = new List<string>();
						if (IniParents.TryGetValue(Pair.Key, out CurrentPlatform))
						{
							while (!String.IsNullOrEmpty(CurrentPlatform))
							{
								// insert at 0 to reverse the order
								Chain.Insert(0, CurrentPlatform);
								if (IniParents.TryGetValue(CurrentPlatform, out CurrentPlatform) == false)
								{
									break;
								}
							}
						}

						// bake it into the info
						if (Chain.Count > 0)
						{
							Pair.Value.IniParentChain = Chain.ToArray();
						}
					}
				}
			}
			return PlatformInfos;
		}

		/// <summary>
		/// Return the data driven info for the given platform name 
		/// </summary>
		/// <param name="PlatformName"></param>
		/// <returns></returns>
		public static ConfigDataDrivenPlatformInfo? GetDataDrivenInfoForPlatform(string PlatformName)
		{
			// lookup the platform name (which is not guaranteed to be there)
			ConfigDataDrivenPlatformInfo? Info;
			GetAllPlatformInfos().TryGetValue(PlatformName, out Info);

			// return what we found of null if nothing
			return Info;
		}

		/// <summary>
		/// Return the data driven info for the given UnrealTargetPlatform
		/// </summary>
		/// <param name="TargetPlatform"></param>
		/// <returns></returns>
		public static ConfigDataDrivenPlatformInfo? GetDataDrivenInfoForPlatform(UnrealTargetPlatform TargetPlatform)
		{
			string PlatformName = ConfigHierarchy.GetIniPlatformName(TargetPlatform);

			return GetDataDrivenInfoForPlatform(PlatformName);
		}
	}
}
