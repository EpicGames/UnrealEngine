// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// General deploy support functions, exported for UAT
	/// </summary>
	public static class DeployExports
	{
		/// <summary>
		/// The max length of the deploy folder name.
		/// </summary>
		public const int DeployFolderMaxLength = 50;

		/// <summary>
		/// Gets the default deploy folder name on the format ShortProjectName-RootDirectoryName-UserName,
		/// truncated to DeployFolderMaxLength characters, with dots '.' and whitespace ' ' stripped.
		/// </summary>
		/// <param name="ShortProjectName">Project to deploy</param>
		/// <returns>The default deploy folder name.</returns>
		public static string GetDefaultDeployFolder(string ShortProjectName)
		{
			string DeployFolder = String.Format("{0}-{1}-{2}",
				ShortProjectName,
				Unreal.RootDirectory.GetDirectoryName(),
				Environment.UserName);

			DeployFolder = DeployFolder.Replace(" ", "", StringComparison.Ordinal);
			DeployFolder = DeployFolder.Replace(".", "", StringComparison.Ordinal);

			if (DeployFolder.Length > DeployFolderMaxLength)
			{
				DeployFolder = DeployFolder.Substring(0, DeployFolderMaxLength);
				char[] CharsToTrim = { '-' };
				DeployFolder = DeployFolder.Trim(CharsToTrim);
			}

			return DeployFolder;
		}
	}
}
