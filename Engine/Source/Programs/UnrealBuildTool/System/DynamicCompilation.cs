// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Reflection;
using System.Text;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.LogMatchers;
using Microsoft.CodeAnalysis;
using Microsoft.CodeAnalysis.CSharp;
using Microsoft.CodeAnalysis.Emit;
using Microsoft.CodeAnalysis.Text;
using Microsoft.Extensions.Logging;
using OpenTracing.Util;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// Methods for dynamically compiling C# source files
	/// </summary>
	public static class DynamicCompilation
	{
		/// <summary>
		/// Checks to see if the assembly needs compilation
		/// </summary>
		/// <param name="SourceFiles">Set of source files</param>
		/// <param name="AssemblyManifestFilePath">File containing information about this assembly, like which source files it was built with and engine version</param>
		/// <param name="OutputAssemblyPath">Output path for the assembly</param>
		/// <param name="Logger">Logger for output</param>
		/// <returns>True if the assembly needs to be built</returns>
		private static bool RequiresCompilation(IEnumerable<FileReference> SourceFiles, FileReference AssemblyManifestFilePath, FileReference OutputAssemblyPath, ILogger Logger)
		{
			// Do not compile the file if it's installed
			if (Unreal.IsFileInstalled(OutputAssemblyPath))
			{
				Logger.LogDebug("Skipping {OutputAssemblyPath}: File is installed", OutputAssemblyPath);
				return false;
			}

			// Check to see if we already have a compiled assembly file on disk
			FileItem OutputAssemblyInfo = FileItem.GetItemByFileReference(OutputAssemblyPath);
			if (!OutputAssemblyInfo.Exists)
			{
				Logger.LogDebug("Compiling {OutputAssemblyPath}: Assembly does not exist", OutputAssemblyPath);
				return true;
			}

			// Check the time stamp of the UnrealBuildTool assembly.  If Unreal Build Tool was compiled more
			// recently than the dynamically-compiled assembly, then we'll always recompile it.  This is
			// because Unreal Build Tool's code may have changed in such a way that invalidate these
			// previously-compiled assembly files.
			FileItem UnrealBuildToolDllItem = FileItem.GetItemByFileReference(Unreal.UnrealBuildToolDllPath);
			if (UnrealBuildToolDllItem.LastWriteTimeUtc > OutputAssemblyInfo.LastWriteTimeUtc)
			{
				Logger.LogDebug("Compiling {OutputAssemblyPath}: {UnrealBuildToolDllItemName} is newer", OutputAssemblyPath, UnrealBuildToolDllItem.Name);
				return true;
			}

			// Make sure we have a manifest of source files used to compile the output assembly.  If it doesn't exist
			// for some reason (not an expected case) then we'll need to recompile.
			FileItem AssemblySourceListFile = FileItem.GetItemByFileReference(AssemblyManifestFilePath);
			if (!AssemblySourceListFile.Exists)
			{
				Logger.LogDebug("Compiling {OutputAssemblyPath}: Missing source file list ({AssemblyManifestFilePath})", OutputAssemblyPath, AssemblyManifestFilePath);
				return true;
			}

			JsonObject Manifest;
			try
			{
				Manifest = JsonObject.Read(AssemblyManifestFilePath);
			}
			catch (Exception)
			{
				Logger.LogDebug("Compiling {OutputAssemblyPath}: Error reading source file list ({AssemblyManifestFilePath})", OutputAssemblyPath, AssemblyManifestFilePath);
				return true;
			}

			// check if the engine version is different
			string EngineVersionManifest = Manifest.GetStringField("EngineVersion");
			string EngineVersionCurrent = FormatVersionNumber(ReadOnlyBuildVersion.Current);
			if (EngineVersionManifest != EngineVersionCurrent)
			{
				Logger.LogDebug("Compiling {OutputAssemblyPath}: Engine Version changed from {EngineVersionManifest} to {EngineVersionCurrent}", OutputAssemblyPath, EngineVersionManifest, EngineVersionCurrent);
				return true;
			}

			// Make sure the source files we're compiling are the same as the source files that were compiled
			// for the assembly that we want to load
			HashSet<FileItem> CurrentSourceFileItems = new HashSet<FileItem>();
			foreach (string Line in Manifest.GetStringArrayField("SourceFiles"))
			{
				CurrentSourceFileItems.Add(FileItem.GetItemByPath(Line));
			}

			// Get the new source files
			HashSet<FileItem> SourceFileItems = new HashSet<FileItem>();
			foreach (FileReference SourceFile in SourceFiles)
			{
				SourceFileItems.Add(FileItem.GetItemByFileReference(SourceFile));
			}

			// Check if there are any differences between the sets
			foreach (FileItem CurrentSourceFileItem in CurrentSourceFileItems)
			{
				if (!SourceFileItems.Contains(CurrentSourceFileItem))
				{
					Logger.LogDebug("Compiling {OutputAssemblyPath}: Removed source file ({CurrentSourceFileItem})", OutputAssemblyPath, CurrentSourceFileItem);
					return true;
				}
			}
			foreach (FileItem SourceFileItem in SourceFileItems)
			{
				if (!CurrentSourceFileItems.Contains(SourceFileItem))
				{
					Logger.LogDebug("Compiling {OutputAssemblyPath}: Added source file ({SourceFileItem})", OutputAssemblyPath, SourceFileItem);
					return true;
				}
			}

			// Check if any of the timestamps are newer
			foreach (FileItem SourceFileItem in SourceFileItems)
			{
				if (SourceFileItem.LastWriteTimeUtc > OutputAssemblyInfo.LastWriteTimeUtc)
				{
					Logger.LogDebug("Compiling {OutputAssemblyPath}: {SourceFileItem} is newer", OutputAssemblyPath, SourceFileItem);
					return true;
				}
			}

			return false;
		}

		private static void LogDiagnostics(IEnumerable<Diagnostic> Diagnostics, ILogger Logger)
		{
			using LogEventParser Parser = new LogEventParser(Logger);
			Parser.AddMatchers(LogMatcherCollections.UBTMatchers);

			foreach (Diagnostic Diag in Diagnostics)
			{
				switch (Diag.Severity)
				{
					// Diagnostics are pre-formatted suitable for Visual Studio consumption - print them without an additional severity prefix
					case DiagnosticSeverity.Error:
						{
							Parser.WriteLine(Diag.ToString());
							break;
						}
					case DiagnosticSeverity.Hidden:
						{
							break;
						}
					case DiagnosticSeverity.Warning:
						{
							Parser.WriteLine(Diag.ToString());
							break;
						}
					case DiagnosticSeverity.Info:
						{
							Parser.WriteLine(Diag.ToString());
							break;
						}
				}
			}
		}

		private static SyntaxTree ParseSyntaxTree(FileReference sourceFileName, CSharpParseOptions parseOptions, CancellationToken cancellationToken)
		{
			using FileStream fs = File.OpenRead(sourceFileName.FullName);
			SourceText text = SourceText.From(fs, encoding: Encoding.UTF8, canBeEmbedded: true);
			return CSharpSyntaxTree.ParseText(text, parseOptions, path: sourceFileName.FullName, cancellationToken: cancellationToken);
		}

		private static async Task<IEnumerable<SyntaxTree>> ParseSyntaxTreesAsync(IEnumerable<FileReference> sourceFileNames, IEnumerable<string>? preprocessorDefines, CancellationToken cancellationToken)
		{
			CSharpParseOptions parseOptions = new(
				 languageVersion: LanguageVersion.CSharp12, // Prefer specific version, faster than using Latest due to version probing
				 kind: SourceCodeKind.Regular,
				 preprocessorSymbols: preprocessorDefines,
				 documentationMode: DocumentationMode.None);

			ConcurrentBag<SyntaxTree> trees = new();
			int dop = Math.Max(1, Environment.ProcessorCount - 1);

			await Parallel.ForEachAsync(sourceFileNames, new ParallelOptions { MaxDegreeOfParallelism = dop, CancellationToken = cancellationToken },
				(f, token) =>
				{
					trees.Add(ParseSyntaxTree(f, parseOptions, token));
					return ValueTask.CompletedTask;
				});
			return trees.OrderBy(t => t.FilePath).ToArray();
		}

		private static async Task<Assembly?> CompileAssemblyAsync(FileReference OutputAssemblyPath, IEnumerable<FileReference> SourceFileNames, ILogger Logger, IEnumerable<string>? ReferencedAssembies, IEnumerable<string>? PreprocessorDefines = null, bool TreatWarningsAsErrors = false)
		{
			IEnumerable<SyntaxTree> SyntaxTrees = await ParseSyntaxTreesAsync(SourceFileNames, PreprocessorDefines, default);

			// Check for errors
			{
				IEnumerable<Diagnostic> syntaxTreeErrors = SyntaxTrees.SelectMany(x => x.GetDiagnostics());
				if (syntaxTreeErrors.Where(x => x.Severity == DiagnosticSeverity.Error).Any())
				{
					LogDiagnostics(syntaxTreeErrors, Logger);
					return null;
				}
			}

			// Create the output directory if it doesn't exist already
			DirectoryInfo DirInfo = new DirectoryInfo(OutputAssemblyPath.Directory.FullName);
			if (!DirInfo.Exists)
			{
				try
				{
					DirInfo.Create();
				}
				catch (Exception Ex)
				{
					throw new BuildException(Ex, "Unable to create directory '{0}' for intermediate assemblies (Exception: {1})", OutputAssemblyPath, Ex.Message);
				}
			}

			List<Type> ReferenceAssemblyTypes = [
				typeof(object),
				typeof(UnrealBuildTool),
				typeof(FileReference),
				typeof(UEBuildPlatformSDK),
			];

			List<string> ReferenceAssemblyStrings = [
				// system references,
				"System.Runtime",
				"System.CodeDom",
				"System.Collections",
				"System.IO",
				"System.IO.FileSystem",
				"System.Linq",
				"System.Text.Json",
				"System.Threading.Tasks",
				"System.Threading.Tasks.Parallel",
				"System.Collections.Concurrent",
				"System.Private.Uri",
				"System.Private.Xml",
				"System.Private.Xml.Linq",
				"System.Text.RegularExpressions",
				"Microsoft.CodeAnalysis.CSharp",
				"System.Console",
				"System.Runtime.Extensions",
				"Microsoft.Extensions.Logging.Abstractions",
				"netstandard",

				// process start dependencies
				"System.ComponentModel.Primitives",
				"System.Diagnostics.Process",

				// registry access
				"Microsoft.Win32.Registry",

				// RNGCryptoServiceProvider, used to generate random hex bytes
				"System.Security.Cryptography",
				"System.Security.Cryptography.Algorithms",
				"System.Security.Cryptography.Csp",
			];

			List<MetadataReference> MetadataReferences = [
				.. ReferencedAssembies?.Select(x => MetadataReference.CreateFromFile(x)) ?? [],
				.. ReferenceAssemblyTypes.Select(x => MetadataReference.CreateFromFile(x.Assembly.Location)),
				.. ReferenceAssemblyStrings.Select(x => MetadataReference.CreateFromFile(Assembly.Load(x).Location))
			];

			CSharpCompilationOptions CompilationOptions = new CSharpCompilationOptions(
				outputKind: OutputKind.DynamicallyLinkedLibrary,
#if DEBUG
				optimizationLevel: OptimizationLevel.Debug,
#else
				// Optimize the managed code in Development
				optimizationLevel: OptimizationLevel.Release,
#endif
				warningLevel: 4,
				deterministic: true,
				assemblyIdentityComparer: DesktopAssemblyIdentityComparer.Default,
				reportSuppressedDiagnostics: true
			);

			CSharpCompilation Compilation = CSharpCompilation.Create(
				assemblyName: OutputAssemblyPath.GetFileNameWithoutAnyExtensions(),
				syntaxTrees: SyntaxTrees,
				references: MetadataReferences,
				options: CompilationOptions
				);

			{
				using FileStream AssemblyStream = FileReference.Open(OutputAssemblyPath, FileMode.Create);
				using FileStream PdbStream = FileReference.Open(OutputAssemblyPath.ChangeExtension(".pdb"), FileMode.Create);

				EmitOptions EmitOptions = new(
					includePrivateMembers: true,
					debugInformationFormat: DebugInformationFormat.PortablePdb
				);

				EmitResult Result = Compilation.Emit(
					peStream: AssemblyStream,
					pdbStream: PdbStream,
					options: EmitOptions);
				LogDiagnostics(Result.Diagnostics, Logger);

				if (!Result.Success)
				{
					return null;
				}
			}

			return Assembly.LoadFrom(OutputAssemblyPath.FullName);
		}

		/// <summary>
		/// Dynamically compiles an assembly async for the specified source file and loads that assembly into the application's
		/// current domain.  If an assembly has already been compiled and is not out of date, then it will be loaded and
		/// no compilation is necessary.
		/// </summary>
		/// <param name="OutputAssemblyPath">Full path to the assembly to be created</param>
		/// <param name="SourceFileNames">List of source file name</param>
		/// <param name="Logger">Logger for output</param>
		/// <param name="ReferencedAssembies"></param>
		/// <param name="PreprocessorDefines"></param>
		/// <param name="DoNotCompile"></param>
		/// <param name="ForceCompile"></param>
		/// <param name="TreatWarningsAsErrors"></param>
		/// <returns>The assembly that was loaded</returns>
		public static async Task<Assembly?> CompileAndLoadAssemblyAsync(FileReference OutputAssemblyPath, IEnumerable<FileReference> SourceFileNames, ILogger Logger, IEnumerable<string>? ReferencedAssembies = null, IEnumerable<string>? PreprocessorDefines = null, bool DoNotCompile = false, bool ForceCompile = false, bool TreatWarningsAsErrors = false)
		{
			// Check to see if the resulting assembly is compiled and up to date
			FileReference AssemblyManifestFilePath = FileReference.Combine(OutputAssemblyPath.Directory, Path.GetFileNameWithoutExtension(OutputAssemblyPath.FullName) + "Manifest.json");

			bool bNeedsCompilation = ForceCompile;
			if (!DoNotCompile)
			{
				bNeedsCompilation = RequiresCompilation(SourceFileNames, AssemblyManifestFilePath, OutputAssemblyPath, Logger);
			}

			// Load the assembly to ensure it is correct
			Assembly? CompiledAssembly = null;
			if (!bNeedsCompilation)
			{
				try
				{
					// Load the previously-compiled assembly from disk
					CompiledAssembly = Assembly.LoadFrom(OutputAssemblyPath.FullName);
				}
				catch (FileLoadException Ex)
				{
					Logger.LogInformation("Unable to load the previously-compiled assembly file '{File}'.  Unreal Build Tool will try to recompile this assembly now.  (Exception: {Ex})", OutputAssemblyPath, Ex.Message);
					bNeedsCompilation = true;
				}
				catch (BadImageFormatException Ex)
				{
					Logger.LogInformation("Compiled assembly file '{File}' appears to be for a newer CLR version or is otherwise invalid.  Unreal Build Tool will try to recompile this assembly now.  (Exception: {Ex})", OutputAssemblyPath, Ex.Message);
					bNeedsCompilation = true;
				}
				catch (FileNotFoundException)
				{
					throw new BuildException("Precompiled rules assembly '{0}' does not exist.", OutputAssemblyPath);
				}
				catch (Exception Ex)
				{
					throw new BuildException(Ex, "Error while loading previously-compiled assembly file '{0}'.  (Exception: {1})", OutputAssemblyPath, Ex.Message);
				}
			}

			// Compile the assembly if me
			if (bNeedsCompilation)
			{
				using (GlobalTracer.Instance.BuildSpan(String.Format("Compiling rules assembly ({0})", OutputAssemblyPath.GetFileName())).StartActive())
				{
					CompiledAssembly = await CompileAssemblyAsync(OutputAssemblyPath, SourceFileNames, Logger, ReferencedAssembies, PreprocessorDefines, TreatWarningsAsErrors);
				}

				using (JsonWriter Writer = new JsonWriter(AssemblyManifestFilePath))
				{
					ReadOnlyBuildVersion Version = ReadOnlyBuildVersion.Current;

					Writer.WriteObjectStart();
					// Save out a list of all the source files we compiled.  This is so that we can tell if whole files were added or removed
					// since the previous time we compiled the assembly.  In that case, we'll always want to recompile it!
					Writer.WriteStringArrayField("SourceFiles", SourceFileNames.Select(x => x.FullName));
					Writer.WriteValue("EngineVersion", FormatVersionNumber(Version));
					Writer.WriteObjectEnd();
				}
			}

			return CompiledAssembly;
		}

		/// <summary>
		/// Dynamically compiles an assembly for the specified source file and loads that assembly into the application's
		/// current domain.  If an assembly has already been compiled and is not out of date, then it will be loaded and
		/// no compilation is necessary.
		/// </summary>
		/// <param name="OutputAssemblyPath">Full path to the assembly to be created</param>
		/// <param name="SourceFileNames">List of source file name</param>
		/// <param name="Logger">Logger for output</param>
		/// <param name="ReferencedAssembies"></param>
		/// <param name="PreprocessorDefines"></param>
		/// <param name="DoNotCompile"></param>
		/// <param name="ForceCompile"></param>
		/// <param name="TreatWarningsAsErrors"></param>
		/// <returns>The assembly that was loaded</returns>
		public static Assembly? CompileAndLoadAssembly(FileReference OutputAssemblyPath, IEnumerable<FileReference> SourceFileNames, ILogger Logger, IEnumerable<string>? ReferencedAssembies = null, IEnumerable<string>? PreprocessorDefines = null, bool DoNotCompile = false, bool ForceCompile = false, bool TreatWarningsAsErrors = false)
		{
			return CompileAndLoadAssemblyAsync(OutputAssemblyPath, SourceFileNames, Logger, ReferencedAssembies, PreprocessorDefines, DoNotCompile, ForceCompile, TreatWarningsAsErrors).Result;
		}

		private static string FormatVersionNumber(ReadOnlyBuildVersion Version) => $"{Version.MajorVersion}.{Version.MinorVersion}.{Version.PatchVersion}";
	}
}
