// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Linq;
using System.Text.Json;
using System.Text.RegularExpressions;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace UnrealBuildTool
{
	/// <summary>
	/// Helper functions for dealing with encryption and pak signing
	/// </summary>
	public static class EncryptionAndSigning
	{
#pragma warning disable CA1034 // Nested types should not be visible - preserve API of APIs exposed to UAT
		/// <summary>
		/// Wrapper class for a single RSA key
		/// </summary>
		public class SigningKey
		{
			/// <summary>
			/// Exponent
			/// </summary>
			public byte[]? Exponent { get; set; }

			/// <summary>
			/// Modulus
			/// </summary>
			public byte[]? Modulus { get; set; }

			/// <summary>
			/// Determine if this key is valid
			/// </summary>
			public bool IsValid()
			{
				return Exponent != null && Modulus != null && Exponent.Length > 0 && Modulus.Length > 0;
			}
		}

		/// <summary>
		/// Wrapper class for an RSA public/private key pair
		/// </summary>
		public class SigningKeyPair
		{
			/// <summary>
			/// Public key
			/// </summary>
			public SigningKey PublicKey { get; set; } = new SigningKey();

			/// <summary>
			/// Private key
			/// </summary>
			public SigningKey PrivateKey { get; set; } = new SigningKey();

			/// <summary>
			/// Determine if this key is valid
			/// </summary>
			public bool IsValid()
			{
				return PublicKey != null && PrivateKey != null && PublicKey.IsValid() && PrivateKey.IsValid();
			}

			/// <summary>
			/// Returns TRUE if this is a short key from the old 256-bit system
			/// </summary>
			public bool IsUnsecureLegacyKey()
			{
				int LongestKey = PublicKey.Exponent!.Length;
				LongestKey = Math.Max(LongestKey, PublicKey.Modulus!.Length);
				LongestKey = Math.Max(LongestKey, PrivateKey.Exponent!.Length);
				LongestKey = Math.Max(LongestKey, PrivateKey.Modulus!.Length);
				return LongestKey <= 64;
			}
		}

		/// <summary>
		/// Wrapper class for a 128 bit AES encryption key
		/// </summary>
		public class EncryptionKey
		{
			/// <summary>
			/// Optional name for this encryption key
			/// </summary>
			public string? Name { get; set; }
			/// <summary>
			/// Optional guid for this encryption key
			/// </summary>
			public string? Guid { get; set; }
			/// <summary>
			/// AES key
			/// </summary>
			public byte[]? Key { get; set; }
			/// <summary>
			/// Determine if this key is valid
			/// </summary>
			public bool IsValid()
			{
				return Key != null && Key.Length > 0 && Guid != null;
			}
		}

		/// <summary>
		/// Wrapper class for all crypto settings
		/// </summary>
		public class CryptoSettings
		{
			/// <summary>
			/// AES encyption key
			/// </summary>
			public EncryptionKey? EncryptionKey { get; set; } = null;

			/// <summary>
			/// RSA public/private key
			/// </summary>
			public SigningKeyPair? SigningKey { get; set; } = null;

			/// <summary>
			/// Enable pak signature checking
			/// </summary>
			public bool bEnablePakSigning { get; set; } = false;

			/// <summary>
			/// Encrypt the index of the pak file. Stops the pak file being easily accessible by unrealpak
			/// </summary>
			public bool bEnablePakIndexEncryption { get; set; } = false;

			/// <summary>
			/// Encrypt all ini files in the pak. Good for game data obsfucation
			/// </summary>
			public bool bEnablePakIniEncryption { get; set; } = false;

			/// <summary>
			/// Encrypt the uasset files in the pak file. After cooking, uasset files only contain package metadata / nametables / export and import tables. Provides good string data obsfucation without
			/// the downsides of full package encryption, with the security drawbacks of still having some data stored unencrypted 
			/// </summary>
			public bool bEnablePakUAssetEncryption { get; set; } = false;

			/// <summary>
			/// Encrypt all assets data files (including exp and ubulk) in the pak file. Likely to be slow, and to cause high data entropy (bad for delta patching)
			/// </summary>
			public bool bEnablePakFullAssetEncryption { get; set; } = false;

			/// <summary>
			/// Some platforms have their own data crypto systems, so allow the config settings to totally disable our own crypto
			/// </summary>
			public bool bDataCryptoRequired { get; set; } = false;

			/// <summary>
			/// Config setting to enable pak encryption
			/// </summary>
			public bool PakEncryptionRequired { get; set; } = true;

			/// <summary>
			/// Config setting to enable pak signing
			/// </summary>
			public bool PakSigningRequired { get; set; } = true;

			/// <summary>
			/// A set of named encryption keys that can be used to encrypt different sets of data with a different key that is delivered dynamically (i.e. not embedded within the game executable)
			/// </summary>
			public EncryptionKey[]? SecondaryEncryptionKeys { get; set; }

			/// <summary>
			/// 
			/// </summary>
			public bool IsAnyEncryptionEnabled()
			{
				return (EncryptionKey != null && EncryptionKey.IsValid()) && (bEnablePakFullAssetEncryption || bEnablePakUAssetEncryption || bEnablePakIndexEncryption || bEnablePakIniEncryption);
			}

			/// <summary>
			/// 
			/// </summary>
			public bool IsPakSigningEnabled()
			{
				return (SigningKey != null && SigningKey.IsValid()) && bEnablePakSigning;
			}

			/// <summary>
			/// 
			/// </summary>
			public void Save(FileReference InFile)
			{
				DirectoryReference.CreateDirectory(InFile.Directory);
				FileReference.WriteAllTextIfDifferent(InFile, JsonSerializer.Serialize(this));
			}

			/// <summary>
			/// Returns whether the specified encryption key exist.
			/// </summary>
			public bool ContainsEncryptionKey(string KeyGuid)
			{
				if (String.IsNullOrEmpty(KeyGuid))
				{
					return false;
				}

				if (EncryptionKey != null && EncryptionKey.Guid == KeyGuid)
				{
					return true;
				}

				if (SecondaryEncryptionKeys != null)
				{
					return SecondaryEncryptionKeys.Any(Key => Key.Guid == KeyGuid);
				}

				return false;
			}
		}
#pragma warning restore CA1034

		/// <summary>
		/// Helper class for formatting incoming hex signing key strings
		/// </summary>
		private static string ProcessSigningKeyInputStrings(string InString)
		{
			if (InString.StartsWith("0x", StringComparison.Ordinal))
			{
				InString = InString.Substring(2);
			}
			return InString.TrimStart('0');
		}

		/// <summary>
		/// Helper function for comparing two AES keys
		/// </summary>
		private static bool CompareKey(byte[] KeyA, byte[] KeyB)
		{
			if (KeyA.Length != KeyB.Length)
			{
				return false;
			}

			for (int Index = 0; Index < KeyA.Length; ++Index)
			{
				if (KeyA[Index] != KeyB[Index])
				{
					return false;
				}
			}

			return true;
		}

		static readonly ConcurrentDictionary<FileReference, Lazy<List<EncryptionKey>?>> s_encryptionKeyCache = [];

		/// <summary>
		/// Parse crypto settings from INI file
		/// </summary>
		public static CryptoSettings ParseCryptoSettings(DirectoryReference? InProjectDirectory, UnrealTargetPlatform InTargetPlatform, ILogger Logger)
		{
			CryptoSettings Settings = new CryptoSettings();

			ConfigHierarchy Ini = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, InProjectDirectory, InTargetPlatform);
			Ini.GetBool("PlatformCrypto", "PlatformRequiresDataCrypto", out bool bDataCryptoRequired);
			Settings.bDataCryptoRequired = bDataCryptoRequired;

			Ini.GetBool("PlatformCrypto", "PakSigningRequired", out bool PakSigningRequired);
			Settings.PakSigningRequired = PakSigningRequired;

			Ini.GetBool("PlatformCrypto", "PakEncryptionRequired", out bool PakEncryptionRequired);
			Settings.PakEncryptionRequired = PakEncryptionRequired;

			{
				// Start by parsing the legacy encryption.ini settings
				Ini = ConfigCache.ReadHierarchy(ConfigHierarchyType.Encryption, InProjectDirectory, InTargetPlatform);

				Ini.GetBool("Core.Encryption", "SignPak", out bool bEnablePakSigning);
				Settings.bEnablePakSigning = bEnablePakSigning;

				Ini.GetString("Core.Encryption", "rsa.privateexp", out string PrivateExponent);
				Ini.GetString("Core.Encryption", "rsa.modulus", out string Modulus);
				Ini.GetString("Core.Encryption", "rsa.publicexp", out string PublicExponent);

				if (PrivateExponent.Length > 0 && Modulus.Length > 0 && PublicExponent.Length > 0)
				{
					Settings.SigningKey = new SigningKeyPair();
					Settings.SigningKey.PrivateKey.Exponent = ParseHexStringToByteArray(ProcessSigningKeyInputStrings(PrivateExponent), 64);
					Settings.SigningKey.PrivateKey.Modulus = ParseHexStringToByteArray(ProcessSigningKeyInputStrings(Modulus), 64);
					Settings.SigningKey.PublicKey.Exponent = ParseHexStringToByteArray(ProcessSigningKeyInputStrings(PublicExponent), 64);
					Settings.SigningKey.PublicKey.Modulus = Settings.SigningKey.PrivateKey.Modulus;

					if ((Settings.SigningKey.PrivateKey.Exponent.Length > 64) ||
						(Settings.SigningKey.PrivateKey.Modulus.Length > 64) ||
						(Settings.SigningKey.PublicKey.Exponent.Length > 64) ||
						(Settings.SigningKey.PublicKey.Modulus.Length > 64))
					{
						throw new Exception(String.Format("[{0}] Signing keys parsed from encryption.ini are too long. They must be a maximum of 64 bytes long!", InProjectDirectory));
					}
				}

				Ini.GetBool("Core.Encryption", "EncryptPak", out bool bEnablePakIndexEncryption);
				Settings.bEnablePakIndexEncryption = bEnablePakIndexEncryption;

				Settings.bEnablePakFullAssetEncryption = false;
				Settings.bEnablePakUAssetEncryption = false;
				Settings.bEnablePakIniEncryption = Settings.bEnablePakIndexEncryption;

				string EncryptionKeyString;
				Ini.GetString("Core.Encryption", "aes.key", out EncryptionKeyString);
				Settings.EncryptionKey = new EncryptionKey();

				if (EncryptionKeyString.Length > 0)
				{
					if (EncryptionKeyString.Length < 32)
					{
						Logger.LogWarning("AES key parsed from encryption.ini is too short. It must be 32 bytes, so will be padded with 0s, giving sub-optimal security!");
					}
					else if (EncryptionKeyString.Length > 32)
					{
						Logger.LogWarning("AES key parsed from encryption.ini is too long. It must be 32 bytes, so will be truncated!");
					}

					Settings.EncryptionKey.Key = ParseAnsiStringToByteArray(EncryptionKeyString, 32);
				}
			}

			Ini = ConfigCache.ReadHierarchy(ConfigHierarchyType.Crypto, InProjectDirectory, InTargetPlatform);
			string SectionName = "/Script/CryptoKeys.CryptoKeysSettings";
			ConfigHierarchySection CryptoSection = Ini.FindSection(SectionName);

			// If we have new format crypto keys, read them in over the top of the legacy settings
			if (CryptoSection != null && CryptoSection.KeyNames.Any())
			{
				Ini.GetBool(SectionName, "bEnablePakSigning", out bool bEnablePakSigning);
				Settings.bEnablePakSigning = bEnablePakSigning;

				Ini.GetBool(SectionName, "bEncryptPakIniFiles", out bool bEnablePakIniEncryption);
				Settings.bEnablePakIniEncryption = bEnablePakIniEncryption;

				Ini.GetBool(SectionName, "bEncryptPakIndex", out bool bEnablePakIndexEncryption);
				Settings.bEnablePakIndexEncryption = bEnablePakIndexEncryption;

				Ini.GetBool(SectionName, "bEncryptUAssetFiles", out bool bEnablePakUAssetEncryption);
				Settings.bEnablePakUAssetEncryption = bEnablePakUAssetEncryption;

				Ini.GetBool(SectionName, "bEncryptAllAssetFiles", out bool bEnablePakFullAssetEncryption);
				Settings.bEnablePakFullAssetEncryption = bEnablePakFullAssetEncryption;

				// Parse encryption key
				string EncryptionKeyString;
				Ini.GetString(SectionName, "EncryptionKey", out EncryptionKeyString);
				if (!String.IsNullOrEmpty(EncryptionKeyString))
				{
					Settings.EncryptionKey = new EncryptionKey();
					Settings.EncryptionKey.Key = System.Convert.FromBase64String(EncryptionKeyString);
					Settings.EncryptionKey.Guid = Guid.Empty.ToString();
					Settings.EncryptionKey.Name = "Embedded";
				}

				// Parse secondary encryption keys
				List<EncryptionKey> SecondaryEncryptionKeys = new List<EncryptionKey>();
				List<string>? SecondaryEncryptionKeyStrings;

				if (Ini.GetArray(SectionName, "SecondaryEncryptionKeys", out SecondaryEncryptionKeyStrings))
				{
					foreach (string KeySource in SecondaryEncryptionKeyStrings)
					{
						EncryptionKey NewKey = new EncryptionKey();
						SecondaryEncryptionKeys.Add(NewKey);

						Regex Search = new Regex("\\(Guid=(?\'Guid\'.*),Name=\\\"(?\'Name\'.*)\\\",Key=\\\"(?\'Key\'.*)\\\"\\)");
						Match Match = Search.Match(KeySource);
						if (Match.Success)
						{
							foreach (string GroupName in Search.GetGroupNames())
							{
								string Value = Match.Groups[GroupName].Value;
								if (GroupName == "Guid")
								{
									NewKey.Guid = Value;
								}
								else if (GroupName == "Name")
								{
									NewKey.Name = Value;
								}
								else if (GroupName == "Key")
								{
									NewKey.Key = System.Convert.FromBase64String(Value);
								}
							}
						}
					}
				}

				Settings.SecondaryEncryptionKeys = SecondaryEncryptionKeys.ToArray();

				// Parse signing key
				string PrivateExponent, PublicExponent, Modulus;
				Ini.GetString(SectionName, "SigningPrivateExponent", out PrivateExponent);
				Ini.GetString(SectionName, "SigningModulus", out Modulus);
				Ini.GetString(SectionName, "SigningPublicExponent", out PublicExponent);

				if (!String.IsNullOrEmpty(PrivateExponent) && !String.IsNullOrEmpty(PublicExponent) && !String.IsNullOrEmpty(Modulus))
				{
					Settings.SigningKey = new SigningKeyPair();
					Settings.SigningKey.PublicKey.Exponent = System.Convert.FromBase64String(PublicExponent);
					Settings.SigningKey.PublicKey.Modulus = System.Convert.FromBase64String(Modulus);
					Settings.SigningKey.PrivateKey.Exponent = System.Convert.FromBase64String(PrivateExponent);
					Settings.SigningKey.PrivateKey.Modulus = Settings.SigningKey.PublicKey.Modulus;
				}
			}

			// Parse project dynamic keychain keys
			if (InProjectDirectory != null)
			{
				ConfigHierarchy GameIni = ConfigCache.ReadHierarchy(ConfigHierarchyType.Game, InProjectDirectory, InTargetPlatform);
				if (GameIni != null)
				{
					string Filename;
					if (GameIni.GetString("ContentEncryption", "ProjectKeyChain", out Filename))
					{
						FileReference ProjectKeyChainFile = FileReference.Combine(InProjectDirectory, "Content", Filename);
						List<EncryptionKey>? EncryptionKeysInFile = s_encryptionKeyCache.GetOrAdd(ProjectKeyChainFile, static ProjectKeyChainFile => new Lazy<List<EncryptionKey>?>(() =>
						{
							if (!FileReference.Exists(ProjectKeyChainFile))
							{
								return null;
							}

							List<EncryptionKey> EncryptionKeys = [];

							string[] Lines = FileReference.ReadAllLines(ProjectKeyChainFile);
							foreach (string Line in Lines)
							{
								string[] KeyParts = Line.Split(':');
								if (KeyParts.Length == 4)
								{
									EncryptionKey NewKey = new EncryptionKey();

									NewKey.Name = KeyParts[0];
									NewKey.Guid = KeyParts[2];
									NewKey.Key = System.Convert.FromBase64String(KeyParts[3]);

									EncryptionKey? ExistingKey = EncryptionKeys.Find((EncryptionKey OtherKey) => { return OtherKey.Guid == NewKey.Guid; });
									if (ExistingKey != null && !CompareKey(ExistingKey.Key!, NewKey.Key))
									{
										throw new Exception("Found multiple encryption keys with the same guid but different AES keys while merging secondary keys from the project key-chain!");
									}

									EncryptionKeys.Add(NewKey);
								}
							}

							return EncryptionKeys;
						})).Value;

						if (EncryptionKeysInFile != null)
						{
							List<EncryptionKey> EncryptionKeys = new();
							if (Settings.SecondaryEncryptionKeys != null)
							{
								EncryptionKeys.AddRange(Settings.SecondaryEncryptionKeys);
							}

							EncryptionKeys.AddRange(EncryptionKeysInFile);

							Settings.SecondaryEncryptionKeys = EncryptionKeys.ToArray();
						}
					}
				}
			}

			if (!Settings.bDataCryptoRequired)
			{
				CryptoSettings NewSettings = new CryptoSettings();
				NewSettings.SecondaryEncryptionKeys = Settings.SecondaryEncryptionKeys;
				Settings = NewSettings;
			}
			else
			{
				if (!Settings.PakSigningRequired)
				{
					Settings.bEnablePakSigning = false;
					Settings.SigningKey = null;
				}

				if (!Settings.PakEncryptionRequired)
				{
					Settings.bEnablePakFullAssetEncryption = false;
					Settings.bEnablePakIndexEncryption = false;
					Settings.bEnablePakIniEncryption = false;
					Settings.EncryptionKey = null;
					Settings.SigningKey = null;
				}
			}

			// Check if we have a valid signing key that is of the old short form
			if (Settings.SigningKey != null && Settings.SigningKey.IsValid() && Settings.SigningKey.IsUnsecureLegacyKey())
			{
				Log.TraceWarningOnce("Project signing keys found in '{0}' are of the old insecure short format. Please regenerate them using the project crypto settings panel in the editor!", InProjectDirectory);
			}

			// Validate the settings we have read
			if (Settings.bDataCryptoRequired && Settings.bEnablePakSigning && (Settings.SigningKey == null || !Settings.SigningKey.IsValid()))
			{
				Log.TraceWarningOnce("Pak signing is enabled, but no valid signing keys were found. Please generate a key in the editor project crypto settings. Signing will be disabled");
				Settings.bEnablePakSigning = false;
			}

			bool bEncryptionKeyValid = (Settings.EncryptionKey != null && Settings.EncryptionKey.IsValid());
			bool bAnyEncryptionRequested = Settings.bEnablePakFullAssetEncryption || Settings.bEnablePakIndexEncryption || Settings.bEnablePakIniEncryption || Settings.bEnablePakUAssetEncryption;
			if (Settings.bDataCryptoRequired && bAnyEncryptionRequested && !bEncryptionKeyValid)
			{
				Log.TraceWarningOnce("Pak encryption is enabled, but no valid encryption key was found. Please generate a key in the editor project crypto settings. Encryption will be disabled");
				Settings.bEnablePakUAssetEncryption = false;
				Settings.bEnablePakFullAssetEncryption = false;
				Settings.bEnablePakIndexEncryption = false;
				Settings.bEnablePakIniEncryption = false;
			}

			return Settings;
		}

		/// <summary>
		/// Take a hex string and parse into an array of bytes
		/// </summary>
		private static byte[] ParseHexStringToByteArray(string InString, int InMinimumLength)
		{
			if (InString.StartsWith("0x", StringComparison.Ordinal))
			{
				InString = InString.Substring(2);
			}

			List<byte> Bytes = new List<byte>();
			while (InString.Length > 0)
			{
				int CharsToParse = Math.Min(2, InString.Length);
				string Value = InString.Substring(InString.Length - CharsToParse);
				InString = InString.Substring(0, InString.Length - CharsToParse);
				Bytes.Add(Byte.Parse(Value, System.Globalization.NumberStyles.AllowHexSpecifier));
			}

			while (Bytes.Count < InMinimumLength)
			{
				Bytes.Add(0);
			}

			return Bytes.ToArray();
		}

		private static byte[] ParseAnsiStringToByteArray(string InString, int InRequiredLength)
		{
			List<byte> Bytes = new List<byte>();

			if (InString.Length > InRequiredLength)
			{
				InString = InString.Substring(0, InRequiredLength);
			}

			foreach (char C in InString)
			{
				Bytes.Add((byte)C);
			}

			while (Bytes.Count < InRequiredLength)
			{
				Bytes.Add(0);
			}

			return Bytes.ToArray();
		}
	}
}
