// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers.Binary;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.IO;
using System.IO.Hashing;
using System.Linq;
using System.Runtime.InteropServices;
using System.Text;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// A database storing hashes of files together with last written time.
	/// This is used to reduce number of reads just to check if file is up-to-date before writing it.
	/// </summary>
	class FileHashCache
	{
		/// <summary>
		/// Ctor
		/// </summary>
		/// <param name="warnOnNoPartition">Warn if file does not exist in a mounted partition</param>
		/// <param name="backupModifiedFiles">Backup old file before writing new changed file</param>
		/// <param name="errorOnRewrite">Report error if file is rewritten with different content, otherwise just warning</param>
		public FileHashCache(bool warnOnNoPartition, bool backupModifiedFiles, bool errorOnRewrite = false)
		{
			_warnOnNoPartition = warnOnNoPartition;
			_backupModifiedFiles = backupModifiedFiles;
			_errorOnRewrite = errorOnRewrite;
		}

		/// <summary>
		/// FileHashCache has reported at least one error
		/// </summary>
		public bool HasErrors => _hasErrors;

		/// <summary>
		/// Mount partition based on below parameters. Will filter out files into this partition based on root path and intermediate sub paths
		/// </summary>
		/// <param name="descriptor"></param>
		/// <param name="appName"></param>
		/// <param name="intermediateEnvironment"></param>
		/// <param name="logger"></param>
		public void Mount(TargetDescriptor descriptor, string appName, UnrealIntermediateEnvironment intermediateEnvironment, ILogger logger)
		{
			Mount(descriptor.ProjectFile, appName, descriptor.Platform, descriptor.Configuration, descriptor.Architectures, intermediateEnvironment, logger);
		}

		/// <summary>
		/// Mount partition based on below parameters. Will filter out files into this partition based on root path and intermediate sub paths
		/// </summary>
		public void Mount(FileReference? projectFile, string appName, UnrealTargetPlatform platform, UnrealTargetConfiguration? configuration, UnrealArchitectures? architectures, UnrealIntermediateEnvironment intermediateEnvironment, ILogger logger)
		{
			appName = UEBuildTarget.GetTargetIntermediateFolderName(appName, intermediateEnvironment);

			string GetFilterPath(string? archFolder, UnrealTargetConfiguration? configuration)
			{
				StringBuilder sb = new();
				sb.Append(platform).Append(Path.DirectorySeparatorChar);
				if (archFolder != null)
				{
					sb.Append(archFolder).Append(Path.DirectorySeparatorChar);
				}
				sb.Append(appName);
				if (configuration != null)
				{
					sb.Append(Path.DirectorySeparatorChar).Append(configuration);
				}
				return sb.ToString();
			}

			void AddPartitionByArch(string? arch, ILogger logger)
			{
				AddPartition(Unreal.WritableEngineDirectory, GetFilterPath(arch, configuration), logger);
				if (configuration == UnrealTargetConfiguration.DebugGame)
				{
					AddPartition(Unreal.WritableEngineDirectory, GetFilterPath(arch, UnrealTargetConfiguration.Development), logger);
				}

				if (projectFile != null)
				{
					AddPartition(projectFile.Directory, GetFilterPath(arch, configuration), logger);
					if (configuration == UnrealTargetConfiguration.DebugGame)
					{
						AddPartition(projectFile.Directory, GetFilterPath(arch, UnrealTargetConfiguration.Development), logger);
					}
				}
			}

			if (architectures != null)
			{
				foreach (UnrealArch arch in architectures.Architectures)
				{
					AddPartitionByArch(UnrealArchitectureConfig.ForPlatform(platform).GetFolderNameForArchitecture(arch), logger);
				}

				// If macos we also need to cover "arm64+x64" architecture
				if (OperatingSystem.IsMacOS() && architectures.Architectures.Count > 1)
				{
					AddPartitionByArch(architectures.ToString(), logger);
				}
			}
			else
			{
				AddPartitionByArch(null, logger);
			}
		}

		/// <summary>
		/// Mount a root directory directly
		/// </summary>
		public void Mount(DirectoryReference root, ILogger logger)
		{
			AddPartition(root, "", logger);
		}

		/// <summary>
		/// Catch all mount
		/// </summary>
		public void MountAll(DirectoryReference intermediateDir, ILogger logger)
		{
			FileReference file = FileReference.Combine(intermediateDir, "FileHashCache.bin");
			Partition partition = new(file);
			_roots.Add((new DirectoryReference("", DirectoryReference.Sanitize.None), [("", partition)]));
			ReadPartition(partition, logger);
		}

		/// <summary>
		/// Get hash of MemoryStream
		/// </summary>
		/// <param name="stream"></param>
		/// <returns></returns>
		/// <exception cref="Exception"></exception>
		public static UInt128 GetFileHash(MemoryStream stream)
		{
			XxHash128 xx = new();

			stream.Position = 0;
			xx.Append(stream);
			UInt128 newHash = ToUInt128LittleEndian(xx.GetHashAndReset());
			if (newHash == 0)
			{
				throw new Exception("Failed to calculate hash");
			}
			return newHash;
		}

		/// <summary>
		/// Write file if changed. 
		/// </summary>
		public Task? WriteFileIfChanged(FileItem fileItem, string content, ILogger logger, bool allowAsync)
		{
			UInt128 newHash = XxHash128.HashToUInt128(MemoryMarshal.AsBytes(content.AsSpan()));
			return InternalWriteFileIfChanged(fileItem, newHash, content, logger, allowAsync, false);
		}

		/// <summary>
		/// Write file if changed. 
		/// </summary>
		public Task? WriteFileIfChanged(FileItem fileItem, IEnumerable<string> content, ILogger logger, bool allowAsync)
		{
			XxHash128 xx = new();
			foreach (string str in content)
			{
				xx.Append(MemoryMarshal.AsBytes(str.AsSpan()));
			}
			UInt128 newHash = ToUInt128LittleEndian(xx.GetHashAndReset());
			return InternalWriteFileIfChanged(fileItem, newHash, content, logger, allowAsync, false);
		}

		/// <summary>
		/// Write file if changed. 
		/// </summary>
		public Task? WriteFileIfChanged(FileItem fileItem, MemoryStream data, ILogger logger, bool alwaysUpdateTimestamp = false)
		{
			UInt128 newHash = GetFileHash(data);
			return InternalWriteFileIfChanged(fileItem, newHash, data, logger, false, alwaysUpdateTimestamp);
		}

		/// <summary>
		/// Save all partitions
		/// </summary>
		public void SaveAll(ILogger logger)
		{
			foreach ((DirectoryReference root, List<(string filter, Partition partition)>) root in _roots)
			{
				foreach ((string filter, Partition partition) in root.Item2)
				{
					if (!partition.Modified)
					{
						continue;
					}

					List<KeyValuePair<FileItem, FileEntry>> files = new(partition.Files);
					files.Sort(static (a, b) => a.Key.Location.CompareTo(b.Key.Location));

					try
					{
						DirectoryReference.CreateDirectory(partition.Location.Directory);
						using FileStream stream = File.Open(partition.Location.FullName, FileMode.Create, FileAccess.Write, FileShare.Read);
						using BinaryWriter writer = new(stream, new UTF8Encoding(false), leaveOpen: false);
						writer.Write(Partition.CurrentVersion);
						writer.Write(partition.Files.Count);

						string lastFullName = "";

						foreach (KeyValuePair<FileItem, FileEntry> pair in files)
						{
							ReadOnlySpan<char> last = lastFullName;
							ReadOnlySpan<char> curr = pair.Key.FullName.AsSpan();
							ushort matchLength = (ushort)MemoryExtensions.CommonPrefixLength(last, curr);
							writer.Write(matchLength);
							writer.Write(curr.Slice(matchLength).ToString());
							lastFullName = pair.Key.FullName;

							writer.Write((ulong)pair.Value.hash);
							writer.Write((ulong)(pair.Value.hash >> 64));
							writer.Write(pair.Value.lastWriteTimeUtc);
						}
					}
					catch (Exception ex)
					{
						logger.LogInformation("Failed to save {Location} - {Message}", partition.Location, ex.Message);
					}
				}
			}
		}

		/// <summary>
		/// bytes array to 128 byte key
		/// </summary>
		public static UInt128 ToUInt128LittleEndian(ReadOnlySpan<byte> hash16)
		{
			ulong lo = BinaryPrimitives.ReadUInt64LittleEndian(hash16[..8]);
			ulong hi = BinaryPrimitives.ReadUInt64LittleEndian(hash16[8..]);
			return ((UInt128)hi << 64) | lo;
		}

		/// <summary>
		/// Get hash of file if it exists
		/// </summary>
		public bool TryGetHash(FileItem fileItem, ILogger logger, out UInt128 hash)
		{
			Partition? partition = GetPartition(fileItem.Location, logger);
			if (partition != null && partition.Files.TryGetValue(fileItem, out FileEntry fileEntry))
			{
				hash = fileEntry.hash;
				return true;
			}
			hash = 0;
			return false;
		}

		private Task? InternalWriteFileIfChanged(FileItem fileItem, UInt128 newHash, object data, ILogger logger, bool allowAsync, bool alwaysUpdateTimestamp)
		{
			Partition? partition = GetPartition(fileItem.Location, logger);

			bool writtenBefore = false;
			if (partition != null)
			{
				if (partition.Files.TryGetValue(fileItem, out FileEntry fileEntry))
				{
					writtenBefore = fileEntry.written;

					// If hash is the same we check last write time
					if (fileEntry.hash == newHash)
					{
						if (fileItem.Exists)
						{
							// if match we can early out
							if (fileItem.LastWriteTimeUtc.Ticks == fileEntry.lastWriteTimeUtc)
							{
								if (alwaysUpdateTimestamp)
								{
									DateTime now = DateTime.UtcNow;
									if (partition.Files.TryUpdate(fileItem, new() { hash = fileEntry.hash, lastWriteTimeUtc = now.Ticks, written = true }, fileEntry))
									{
										partition.Modified = true;
									}
									File.SetLastWriteTimeUtc(fileItem.FullName, now);
									fileItem.ResetCachedInfo();
								}
								else if (!writtenBefore)
								{
									partition.Files.TryUpdate(fileItem, new() { hash = fileEntry.hash, lastWriteTimeUtc = fileEntry.lastWriteTimeUtc, written = true }, fileEntry);
								}
								return null;
							}
						}
					}
				}
			}
			if (allowAsync)
			{
				return Task.Run(() => InternalWriteFileIfChanged2(writtenBefore, partition, fileItem, newHash, data, logger, alwaysUpdateTimestamp));
			}

			InternalWriteFileIfChanged2(writtenBefore, partition, fileItem, newHash, data, logger, alwaysUpdateTimestamp);
			return null;
		}

		private void InternalWriteFileIfChanged2(bool writtenBefore, Partition? partition, FileItem fileItem, UInt128 newHash, object data, ILogger logger, bool alwaysUpdateTimestamp)
		{
			if (fileItem.Exists)
			{
				try
				{
					bool isEqual = false;
					if (data is MemoryStream)
					{
						const int BufferSize = 1024 * 1024; // 1 MB
						byte[] buffer = new byte[BufferSize];
						using FileStream fs = new(fileItem.FullName, FileMode.Open, FileAccess.Read, FileShare.Read, BufferSize, FileOptions.SequentialScan);
						int read;

						XxHash128 xx = new();
						while ((read = fs.Read(buffer, 0, buffer.Length)) > 0)
						{
							xx.Append(buffer.AsSpan(0, read));
						}

						isEqual = newHash == ToUInt128LittleEndian(xx.GetHashAndReset());
					}
					else if (data is string content)
					{
						string oldContent = FileReference.ReadAllText(fileItem.Location, new UTF8Encoding(false));
						isEqual = String.Equals(oldContent, content, StringComparison.Ordinal);
					}
					else if (data is IEnumerable<string> contentLines)
					{
						string[] oldLines = FileReference.ReadAllLines(fileItem.Location, new UTF8Encoding(false));
						isEqual = oldLines.SequenceEqual(contentLines, StringComparer.Ordinal);
					}

					if (isEqual)
					{
						DateTime lastWrite = fileItem.LastWriteTimeUtc;
						if (alwaysUpdateTimestamp)
						{
							lastWrite = DateTime.UtcNow;
							File.SetLastWriteTimeUtc(fileItem.FullName, lastWrite);
							fileItem.ResetCachedInfo();
						}
						if (partition != null)
						{
							if (partition.Files.TryAdd(fileItem, new() { hash = newHash, lastWriteTimeUtc = lastWrite.Ticks }))
							{
								partition.Modified = true;
							}
						}
						return;
					}
				}
				catch
				{
				}
			}

			// Same file is written with different content multiple times
			if (writtenBefore)
			{

				logger.Log(_errorOnRewrite ? LogLevel.Error : LogLevel.Warning, "Re-writing a file that was previously written with different content: \"{File}\"", fileItem.Location);
				_hasErrors = _errorOnRewrite;
			}

			if (!fileItem.Exists)
			{
				if (!fileItem.Directory.Exists)
				{
					DirectoryReference.CreateDirectory(fileItem.Directory.Location);
				}
			}
			else if (_backupModifiedFiles)
			{
				// Make a copy of old file
				FileReference backupFile = new FileReference(fileItem.FullName + ".old");
				try
				{
					logger.LogDebug("Updating {File}: contents have changed. Saving previous version to {BackupFile}.", fileItem.Location, backupFile);
					FileReference.Delete(backupFile);
					FileReference.Move(fileItem.Location, backupFile);
				}
				catch (Exception Ex)
				{
					logger.LogWarning("Unable to rename {FileItem} to {BackupFile}", fileItem, backupFile);
					logger.LogDebug(Ex, "{Ex}", ExceptionUtils.FormatExceptionDetails(Ex));
				}
			}

			// When generating solutions there is a tiny chance that we will write the same file at the same time
			// .. so let's just handle that with retry.
			int retryIndex = 0;
			bool isLocked = false;
			while (true)
			{
				try
				{
					if (data is MemoryStream stream)
					{
						stream.Position = 0;
						using FileStream file = new(fileItem.FullName, FileMode.Create, FileAccess.Write, FileShare.None);
						stream.CopyTo(file);
					}
					else if (data is string content)
					{
						using FileStream fs = new(fileItem.FullName, FileMode.Create, FileAccess.Write, FileShare.Read, 1 << 20, FileOptions.SequentialScan);
						using StreamWriter writer = new(fs, new UTF8Encoding(false), 1 << 20, false);
						writer.Write(content);
					}
					else if (data is IEnumerable<string> contentLines)
					{
						string contents = String.Join(Environment.NewLine, contentLines);
						using FileStream fs = new(fileItem.FullName, FileMode.Create, FileAccess.Write, FileShare.Read, 1 << 20, FileOptions.SequentialScan);
						using StreamWriter writer = new(fs, new UTF8Encoding(false), 1 << 20, false);
						writer.Write(contents);
					}
					if (isLocked)
					{
						Monitor.Exit(fileItem);
					}
					break;
				}
				catch (Exception e)
				{
					if (retryIndex == 20)
					{
						if (isLocked)
						{
							Monitor.Exit(fileItem);
						}
						throw;
					}

					if (!isLocked)
					{
						logger.LogTrace("Get exception trying to write {Location} ({Message}). Will wait 100ms and try again", fileItem.Location, e.Message);
						Monitor.Enter(fileItem);
						isLocked = true;
					}
					Thread.Sleep(100);
					++retryIndex;
				}
			}

			fileItem.ResetCachedInfo();

			if (partition != null)
			{
				FileEntry fileEntry = new() { hash = newHash, lastWriteTimeUtc = fileItem.LastWriteTimeUtc.Ticks, written = true };
				partition.Files.AddOrUpdate(fileItem, fileEntry, (k,v) => fileEntry);
				partition.Modified = true;
			}
		}

		public void AddPartition(DirectoryReference rootPath, string filterPath, ILogger logger)
		{
			Partition? partition = null;
			lock (_roots)
			{
				(DirectoryReference root, List<(string filter, Partition partition)>) root = _roots.FirstOrDefault(e => e.root == rootPath);
				if (root.root == null)
				{
					root = new(rootPath, new());
					_roots.Add(root);
				}
				List<(string filter, Partition partition)> filters = root.Item2;
				(string filter, Partition partition) filter = filters.FirstOrDefault(f => f.filter == filterPath);
				partition = filter.partition;
				if (partition == null)
				{
					FileReference location;
					if (filterPath.Length != 0)
					{
						location = FileReference.Combine(rootPath, "Intermediate", "Build", filterPath, "FileHashCache.bin");
					}
					else
					{
						location = FileReference.Combine(rootPath, "FileHashCache.bin");
					}
					partition = new Partition(location);
					filters.Add((filterPath, partition));
				}
			}

			lock (partition)
			{
				if (!partition.Loaded)
				{
					partition.Loaded = true;
					if (FileReference.Exists(partition.Location))
					{
						ReadPartition(partition, logger);
					}
				}
			}
		}

		private readonly string Intermediate = $"{Path.DirectorySeparatorChar}Intermediate";
		private readonly string External = $"{Path.DirectorySeparatorChar}External";
		private readonly string Build = $"{Path.DirectorySeparatorChar}Build{Path.DirectorySeparatorChar}";

		private Partition? GetPartition(FileReference fileRef, ILogger logger)
		{
			ReadOnlySpan<char> fullName = fileRef.FullName;
			int index = fullName.IndexOf(Intermediate, FileReference.Comparison);
			if (index == -1)
			{
				if (_roots.Count == 1 && _roots[0].root.FullName.Length == 0) // MountAll
				{
					return _roots[0].Item2[0].partition;
				}

				if (_warnOnNoPartition)
				{
					logger.LogWarning("FileHashCache - {FilePath} is not under intermediate folder", fileRef);
				}
				return null;
			}

			ReadOnlySpan<char> filterPath = fullName.Slice(index + Intermediate.Length);

			if (filterPath.StartsWith(External))
			{
				filterPath = filterPath.Slice(External.Length);
			}

			if (filterPath.StartsWith(Build))
			{
				filterPath = filterPath.Slice(Build.Length);
			}

			Partition? partition = null;

			foreach ((DirectoryReference root, List<(string filter, Partition partition)>) root in _roots)
			{
				if (root.root.FullName.Length != 0 && !fileRef.IsUnderDirectory(root.root))
				{
					continue;
				}

				foreach ((string filter, Partition partition) entry in root.Item2)
				{
					if (filterPath.StartsWith(entry.filter, FileReference.Comparison))
					{
						partition = entry.partition;
						break;
					}
				}
				break;

			}

			if (partition == null)
			{
				if (_warnOnNoPartition)
				{
					logger.LogWarning("FileHashCache - There is no partition covering path to {FilePath}!", fileRef);
				}
				return null;
			}

			return partition;
		}
		private static void ReadPartition(Partition partition, ILogger logger)
		{
			try
			{
				using FileStream stream = File.Open(partition.Location.FullName, FileMode.Open, FileAccess.Read, FileShare.Read);
				using BinaryReader reader = new(stream, new UTF8Encoding(false), leaveOpen: false);

				int version = reader.ReadInt32();
				if (version != Partition.CurrentVersion)
				{
					logger.LogDebug(
						"Unable to read dependency cache from {PartitionPath}; version {ReadVersion} vs current {CurrentVersion}",
						partition.Location,
						version,
						Partition.CurrentVersion);
					return;
				}

				int count = reader.ReadInt32();
				partition.Files = new(-1, count);

				string lastFilename = "";
				StringBuilder sb = new(256);

				for (int Idx = 0; Idx < count; Idx++)
				{
					ushort matchLength = reader.ReadUInt16();
					string str = reader.ReadString();
					sb.Clear().Append(lastFilename, 0, matchLength).Append(str);

					ulong hashA = reader.ReadUInt64();
					UInt128 hashB = reader.ReadUInt64();

					UInt128 hash = (hashB << 64) + hashA;
					long lastWriteTimeUtc = reader.ReadInt64();

					string path = sb.ToString();
					lastFilename = path;
					partition.Files.TryAdd(FileItem.GetItemByPath(path), new() { hash = hash, lastWriteTimeUtc = lastWriteTimeUtc });
				}
			}
			catch (FileNotFoundException)
			{
			}
			catch (DirectoryNotFoundException)
			{
			}
			catch (Exception Ex)
			{
				logger.LogInformation("Unable to read {Location} - {Message}", partition.Location, Ex.Message);
			}
		}

		private struct FileEntry
		{
			internal UInt128 hash;
			internal long lastWriteTimeUtc;
			internal bool written;
		}

		private class Partition
		{
			internal Partition(FileReference location) { Location = location; }
			internal readonly FileReference Location;
			internal bool Loaded;
			internal bool Modified;
			internal const int CurrentVersion = 1;

			internal ConcurrentDictionary<FileItem, FileEntry> Files = [];
		}

		private readonly List<(DirectoryReference root, List<(string filter, Partition partition)>)> _roots = [];
		private readonly bool _warnOnNoPartition;
		private readonly bool _backupModifiedFiles;
		private readonly bool _errorOnRewrite;
		private bool _hasErrors;
	}
}
