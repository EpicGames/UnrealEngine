// Copyright Epic Games, Inc. All Rights Reserved.

#define UBT_PREFETCH_ENABLED

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Linq;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// Prefetches metadata from the filesystem, by populating FileItem and DirectoryItem objects for requested directory trees. Since 
	/// </summary>
	static class FileMetadataPrefetch
	{
		// TODO: This can never be cancelled, should it be wired up or removed?
		/// <summary>
		/// Used to cancel any queued tasks
		/// </summary>
		static readonly CancellationTokenSource CancelSource = new CancellationTokenSource();

		/// <summary>
		/// The cancellation token
		/// </summary>
		static readonly CancellationToken CancelToken = CancelSource.Token;

		/// <summary>
		/// Set of all the directory trees that have been queued up, to save adding any more than once.
		/// </summary>
		static readonly ConcurrentDictionary<DirectoryReference, Lazy<Task>> QueuedDirectories = new();

		/// <summary>
		/// Enqueue the engine directory for prefetching
		/// </summary>
		public static Task QueueEngineDirectory()
		{
#if UBT_PREFETCH_ENABLED
			return QueuedDirectories.GetOrAdd(Unreal.EngineDirectory, new Lazy<Task>(() =>
			{
				return Task.Run(() =>
				{
					//using ITimelineEvent _ = Timeline.ScopeEvent("PrefetchEngineDir");
					ParallelQueue queue = new();
					queue.Enqueue(() => ScanEngineDirectory(queue));
					queue.Drain();
				});
			})).Value;
#else
			return Task.CompletedTask;
#endif
		}

		/// <summary>
		/// Enqueue a project directory for prefetching
		/// </summary>
		/// <param name="ProjectDirectory">The project directory to prefetch</param>
		public static Task QueueProjectDirectory(DirectoryReference ProjectDirectory)
		{
#if UBT_PREFETCH_ENABLED
			return QueuedDirectories.GetOrAdd(ProjectDirectory, new Lazy<Task>(() =>
			{
				return Task.Run(() =>
				{
					//using ITimelineEvent _ = Timeline.ScopeEvent("PrefetchProjectDir");
					ParallelQueue queue = new();
					queue.Enqueue(() => ScanProjectDirectory(queue, DirectoryItem.GetItemByDirectoryReference(ProjectDirectory)));
					queue.Drain();
				});
			})).Value;
#else
			return Task.CompletedTask;
#endif
		}

		/// <summary>
		/// Enqueue a directory tree for prefetching
		/// </summary>
		/// <param name="Directories">Directory to start searching from</param>
		public static Task QueueDirectoryTrees(IEnumerable<DirectoryItem?> Directories)
		{
#if UBT_PREFETCH_ENABLED
			return Task.Run(() =>
			{
				//using ITimelineEvent _ = Timeline.ScopeEvent("PrefetchDir");
				ParallelQueue queue = new();
				foreach (DirectoryItem? dir in Directories)
				{
					if (dir != null)
					{
						queue.Enqueue(() => ScanDirectoryTree(queue, dir));
					}
				}
				queue.Drain();
			});
#else
			return Task.CompletedTask;
#endif
		}

		/// <summary>
		/// Stop prefetching items, and cancel all pending tasks. synchronous.
		/// </summary>
		public static void Stop()
		{
#if UBT_PREFETCH_ENABLED
			foreach (Lazy<Task> directory in QueuedDirectories.Values)
			{
				directory.Value.Wait();
			}
			QueuedDirectories.Clear();
#endif
		}

		/// <summary>
		/// Scans the engine directory, adding tasks for subdirectories
		/// </summary>
		static void ScanEngineDirectory(ParallelQueue queue)
		{
			foreach (DirectoryReference ExtensionDir in Unreal.GetExtensionDirs(Unreal.EngineDirectory))
			{
				DirectoryItem BaseDirectory = DirectoryItem.GetItemByDirectoryReference(ExtensionDir);
				BaseDirectory.CacheDirectories();

				DirectoryItem BasePluginsDirectory = DirectoryItem.Combine(BaseDirectory, "Plugins");
				queue.Enqueue(() => ScanPluginFolder(queue, BasePluginsDirectory));

				DirectoryItem BaseSourceDirectory = DirectoryItem.Combine(BaseDirectory, "Source");
				BaseSourceDirectory.CacheDirectories();

				DirectoryItem BaseSourceRuntimeDirectory = DirectoryItem.Combine(BaseSourceDirectory, "Runtime");
				queue.Enqueue(() => ScanDirectoryTree(queue, BaseSourceRuntimeDirectory));

				DirectoryItem BaseSourceDeveloperDirectory = DirectoryItem.Combine(BaseSourceDirectory, "Developer");
				queue.Enqueue(() => ScanDirectoryTree(queue, BaseSourceDeveloperDirectory));

				DirectoryItem BaseSourceEditorDirectory = DirectoryItem.Combine(BaseSourceDirectory, "Editor");
				queue.Enqueue(() => ScanDirectoryTree(queue, BaseSourceEditorDirectory));
			}
		}

		/// <summary>
		/// Scans a project directory, adding tasks for subdirectories
		/// </summary>
		/// <param name="queue">The project directory to search</param>
		/// <param name="ProjectDirectory">The project directory to search</param>
		static void ScanProjectDirectory(ParallelQueue queue, DirectoryItem ProjectDirectory)
		{
			foreach (DirectoryReference ExtensionDir in Unreal.GetExtensionDirs(ProjectDirectory.Location))
			{
				DirectoryItem BaseDirectory = DirectoryItem.GetItemByDirectoryReference(ExtensionDir);
				BaseDirectory.CacheDirectories();

				DirectoryItem BasePluginsDirectory = DirectoryItem.Combine(BaseDirectory, "Plugins");
				queue.Enqueue(() => ScanPluginFolder(queue, BasePluginsDirectory));

				DirectoryItem BaseSourceDirectory = DirectoryItem.Combine(BaseDirectory, "Source");
				queue.Enqueue(() => ScanDirectoryTree(queue, BaseSourceDirectory));
			}
		}

		/// <summary>
		/// Scans a plugin parent directory, adding tasks for subdirectories
		/// </summary>
		/// <param name="queue">Root of the directory tree</param>
		/// <param name="Directory">The directory which may contain plugin directories</param>
		static void ScanPluginFolder(ParallelQueue queue, DirectoryItem Directory)
		{
			if (CancelToken.IsCancellationRequested || Directory.TryGetFile(".ubtignore", out FileItem? _))
			{
				return;
			}

			foreach (DirectoryItem SubDirectory in Directory.EnumerateDirectories())
			{
				if (SubDirectory.EnumerateFiles().Any((fi) => fi.HasExtension(".uplugin")))
				{
					queue.Enqueue(() => ScanDirectoryTree(queue, DirectoryItem.Combine(SubDirectory, "Source")));
				}
				else if (!SubDirectory.TryGetFile(".ubtignore", out FileItem? OutFile))
				{
					queue.Enqueue(() => ScanPluginFolder(queue, SubDirectory));
				}
			}
		}

		/// <summary>
		/// Scans an arbitrary directory tree
		/// </summary>
		/// <param name="queue">Root of the directory tree</param>
		/// <param name="Directory">Root of the directory tree</param>
		static void ScanDirectoryTree(ParallelQueue queue, DirectoryItem Directory)
		{
			if (CancelToken.IsCancellationRequested || Directory.TryGetFile(".ubtignore", out FileItem? _))
			{
				return;
			}

			foreach (DirectoryItem SubDirectory in Directory.EnumerateDirectories())
			{
				queue.Enqueue(() => ScanDirectoryTree(queue, SubDirectory));
			}
		}
	}
}
