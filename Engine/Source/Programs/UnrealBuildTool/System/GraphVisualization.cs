// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Drawing;
using System.Globalization;
using System.Xml;
using EpicGames.Core;

namespace UnrealBuildTool
{
	/// <summary>
	/// A single "node" in a directed graph
	/// </summary>
	class GraphNode
	{
		public string Label;
		public Color Color;
		public float Size;
		public Dictionary<string, object> Attributes = new Dictionary<string, object>(StringComparer.InvariantCultureIgnoreCase);

		public GraphNode(string Label, float Size = 1.0f)
			: this(Label, Color.Black, Size)
		{
		}

		public GraphNode(string Label, Color Color, float Size = 1.0f)
		{
			this.Label = Label;
			this.Color = Color;
			this.Size = Size;
		}
	}

	/// <summary>
	/// Describes an edge in the directed graph
	/// </summary>
	class GraphEdge
	{
		public GraphNode Source;
		public GraphNode Target;
		public double Weight = 1.0f;
		public Color Color = Color.FromArgb(64, 0, 0, 0);
		public float Thickness = 0.1f;

		public GraphEdge(GraphNode Source, GraphNode Target)
		{
			this.Source = Source;
			this.Target = Target;
		}
	}

	static class GraphVisualization
	{
		/// <summary>
		/// Attribute for a graph node
		/// </summary>
		class GraphAttribute
		{
			/// Gexf ID for this attribute
			public int Id;

			/// Name of the attribute
			public string Name;

			/// Gexf type name
			public string TypeName;

			public GraphAttribute(int Id, string Name, string TypeName)
			{
				this.Id = Id;
				this.Name = Name;
				this.TypeName = TypeName;
			}
		}

		/// <summary>
		/// Writes a GEXF graph file for the specified graph nodes and edges
		/// </summary>
		/// <param name="Filename">The file name to write</param>
		/// <param name="Description">The description to include in the graph file's metadata</param>
		/// <param name="GraphNodes">List of all graph nodes.  Index order is important and must match with the individual node Id members!</param>
		/// <param name="GraphEdges">List of all graph edges.  Index order is important and must match with the individual edge Id members!</param>
		public static void WriteGraphFile(FileReference Filename, string Description, List<GraphNode> GraphNodes, List<GraphEdge> GraphEdges)
		{
			CultureInfo OriginalCulture = CultureInfo.CurrentCulture;

			try
			{
				// export graph using invariant culture so "." is used as a decimal separator
				CultureInfo.CurrentCulture = CultureInfo.InvariantCulture;

				XmlWriterSettings Settings = new XmlWriterSettings();
				Settings.Indent = true;
				Settings.IndentChars = "    ";

				// Figure out all of the custom attribute types we're dealing with
				Dictionary<string, GraphAttribute> AllAttributes = new Dictionary<string, GraphAttribute>(StringComparer.InvariantCultureIgnoreCase);
				foreach (GraphNode GraphNode in GraphNodes)
				{
					foreach ((string AttributeName, object AttributeValue) in GraphNode.Attributes)
					{
						string AttributeType = GetAttributeType(AttributeValue);

						GraphAttribute? Attribute;
						if (!AllAttributes.TryGetValue(AttributeName, out Attribute))
						{
							AllAttributes[AttributeName] = new GraphAttribute(AllAttributes.Count, AttributeName, AttributeType);
						}
						else if (!Attribute.TypeName.Equals(AttributeType, StringComparison.Ordinal))
						{
							throw new InvalidOperationException("Multiple graph nodes with the same attribute name but different types encountered!");
						}
					}
				}

				using (XmlWriter Writer = XmlWriter.Create(Filename.FullName, Settings))
				{
					// NOTE: The GEXF XML format is defined here:  http://gexf.net/1.2draft/gexf-12draft-primer.pdf

					string GEXFNamespace = "http://www.gexf.net/1.2-draft";
					string SchemaNamespace = "http://www.w3.org/2001/XMLSchema-instance";
					string VizNamespace = "http://www.gexf.net/1.2draft/viz";

					Writer.WriteStartElement("gexf", GEXFNamespace);
					Writer.WriteAttributeString("xmlns", "xsi", null, SchemaNamespace);
					Writer.WriteAttributeString("schemaLocation", SchemaNamespace, "http://www.gexf.net/1.2draft http://www.gexf.net/1.2draft/gexf.xsd");
					Writer.WriteAttributeString("xmlns", "viz", null, VizNamespace);
					Writer.WriteAttributeString("version", "1.2");

					Writer.WriteStartElement("meta");
					{
						Writer.WriteAttributeString("creator", "UnrealBuildTool");
						Writer.WriteAttributeString("description", Description);
					}
					Writer.WriteEndElement();   // meta

					Dictionary<GraphNode, int> NodeToId = new Dictionary<GraphNode, int>();
					{
						Writer.WriteStartElement("graph");
						{
							Writer.WriteAttributeString("mode", "static");
							Writer.WriteAttributeString("defaultedgetype", "directed");

							if (AllAttributes.Count > 0)
							{
								Writer.WriteStartElement("attributes");
								{
									// @todo: Add support for edge attributes, not just node attributes
									Writer.WriteAttributeString("class", "node");   // Node attributes, not edges!

									foreach (GraphAttribute Attribute in AllAttributes.Values)
									{
										Writer.WriteStartElement("attribute");
										{
											Writer.WriteAttributeString("id", Attribute.Id.ToString());
											Writer.WriteAttributeString("title", Attribute.Name);
											Writer.WriteAttributeString("type", Attribute.TypeName);
										}
										Writer.WriteEndElement();   // attribute
									}

									// @todo: Add support for attribute type default values
								}
								Writer.WriteEndElement();   // attributes

							}

							Writer.WriteStartElement("nodes");
							{
								foreach (GraphNode GraphNode in GraphNodes)
								{
									Writer.WriteStartElement("node");
									{
										int Id = GetNodeId(GraphNode, NodeToId);
										Writer.WriteAttributeString("id", Id.ToString());
										Writer.WriteAttributeString("label", GraphNode.Label);

										Writer.WriteStartElement("color", VizNamespace);
										{
											Writer.WriteAttributeString("r", GraphNode.Color.R.ToString());
											Writer.WriteAttributeString("g", GraphNode.Color.G.ToString());
											Writer.WriteAttributeString("b", GraphNode.Color.B.ToString());
											Writer.WriteAttributeString("a", (GraphNode.Color.A / 255.0f).ToString());
										}
										Writer.WriteEndElement();   // viz:color

										Writer.WriteStartElement("size", VizNamespace);
										{
											Writer.WriteAttributeString("value", GraphNode.Size.ToString());
										}
										Writer.WriteEndElement();   // viz:size

										Writer.WriteStartElement("shape", VizNamespace);
										{
											// NOTE: Valid shapes are:  disc, square, triangle, diamond, image
											Writer.WriteAttributeString("value", "disc");
										}
										Writer.WriteEndElement();   // viz:shape

										if (GraphNode.Attributes.Count > 0)
										{
											Writer.WriteStartElement("attvalues");
											{
												foreach (KeyValuePair<string, object> AttributeHashEntry in GraphNode.Attributes)
												{
													string AttributeName = AttributeHashEntry.Key;
													object AttributeValue = AttributeHashEntry.Value;

													GraphAttribute Attribute = AllAttributes[AttributeName];

													Writer.WriteStartElement("attvalue");
													{
														Writer.WriteAttributeString("for", Attribute.Id.ToString());
														Writer.WriteAttributeString("value", AttributeValue.ToString());
													}
													Writer.WriteEndElement();
												}
											}
											Writer.WriteEndElement();   // attvalues
										}
									}
									Writer.WriteEndElement();   // node
								}
							}
							Writer.WriteEndElement();   // nodes

							Writer.WriteStartElement("edges");
							{
								for (int EdgeId = 0; EdgeId < GraphEdges.Count; EdgeId++)
								{
									GraphEdge GraphEdge = GraphEdges[EdgeId];
									Writer.WriteStartElement("edge");
									{
										Writer.WriteAttributeString("id", EdgeId.ToString());
										Writer.WriteAttributeString("source", GetNodeId(GraphEdge.Source, NodeToId).ToString());
										Writer.WriteAttributeString("target", GetNodeId(GraphEdge.Target, NodeToId).ToString());
										Writer.WriteAttributeString("weight", GraphEdge.Weight.ToString());

										Writer.WriteStartElement("color", VizNamespace);
										{
											Writer.WriteAttributeString("r", GraphEdge.Color.R.ToString());
											Writer.WriteAttributeString("g", GraphEdge.Color.G.ToString());
											Writer.WriteAttributeString("b", GraphEdge.Color.B.ToString());
											Writer.WriteAttributeString("a", (GraphEdge.Color.A / 255.0f).ToString());
										}
										Writer.WriteEndElement();   // viz:color

										Writer.WriteStartElement("thickness", VizNamespace);
										{
											Writer.WriteAttributeString("value", GraphEdge.Thickness.ToString());
										}
										Writer.WriteEndElement();   // viz:thickness

										Writer.WriteStartElement("shape", VizNamespace);
										{
											// NOTE: Valid shapes are:  solid, dotted, dashed, double
											Writer.WriteAttributeString("value", "solid");
										}
										Writer.WriteEndElement();   // viz:shape
									}
									Writer.WriteEndElement();   // edge
								}
							}
							Writer.WriteEndElement();   // nodes
						}

						Writer.WriteEndElement();   // graph
					}
					Writer.WriteEndElement();   // gexf

					Writer.Flush();
				}
			}
			finally
			{
				CultureInfo.CurrentCulture = OriginalCulture;
			}
		}

		private static int GetNodeId(GraphNode Node, Dictionary<GraphNode, int> NodeToId)
		{
			int Id;
			if (!NodeToId.TryGetValue(Node, out Id))
			{
				Id = NodeToId.Count;
				NodeToId.Add(Node, Id);
			}
			return Id;
		}

		private static string GetAttributeType(object Value)
		{
			string AttributeTypeName;
			if (Value.GetType() == typeof(int))
			{
				AttributeTypeName = "integer";
			}
			else if (Value.GetType() == typeof(float))
			{
				AttributeTypeName = "float";
			}
			else if (Value.GetType() == typeof(double))
			{
				AttributeTypeName = "double";
			}
			else if (Value.GetType() == typeof(string))
			{
				AttributeTypeName = "string";
			}
			else if (Value.GetType() == typeof(bool))
			{
				AttributeTypeName = "boolean";
			}
			else
			{
				throw new InvalidOperationException("Unsupported attribute data type encountered on graph node!");
			}
			return AttributeTypeName;
		}
	}
}
