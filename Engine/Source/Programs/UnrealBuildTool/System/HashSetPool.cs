// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using Microsoft.Extensions.ObjectPool;

namespace UnrealBuildTool;

// Usage:
//
// using var rented = HashSetPool<string>.Rent(estimate);
// var hashSet = rented.Value;
//
// use hashSet...

static class HashSetPool<T>
{
	public static readonly ObjectPool<HashSet<T>> Shared = new DefaultObjectPool<HashSet<T>>(new Policy(), maximumRetained: 256);

	sealed class Policy : PooledObjectPolicy<HashSet<T>>
	{
		public override HashSet<T> Create() => new(capacity: 256); // <- set a good default

		public override bool Return(HashSet<T> obj)
		{
			obj.Clear();
			return true;
		}
	}

	public static Pooled<HashSet<T>> Rent(int ensureCapacity = 0)
	{
		HashSet<T> set = Shared.Get();
		if (ensureCapacity > 0)
		{
			set.EnsureCapacity(ensureCapacity);
		}
		return new Pooled<HashSet<T>>(Shared, set);
	}
}

/// <summary>
/// Tiny disposable wrapper so you can `using` it
/// </summary>
public sealed class Pooled<T> : IDisposable where T : class
{
	readonly ObjectPool<T> _pool;

	/// <summary>
	/// </summary>
	public T Value { get; }

	/// <summary>
	/// </summary>
	public Pooled(ObjectPool<T> pool, T value)
	{
		_pool = pool;
		Value = value;
	}

	/// <summary>
	/// </summary>
	public void Dispose() => _pool.Return(Value);
}

/// <summary>
/// For ordinal compare
/// </summary>
sealed class StringSetPolicy : PooledObjectPolicy<HashSet<string>>
{
	public override HashSet<string> Create() => new(StringComparer.Ordinal);
	public override bool Return(HashSet<string> obj)
	{
		obj.Clear();
		return true;
	}
}