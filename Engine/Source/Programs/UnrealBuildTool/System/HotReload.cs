// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Text;
using System.Text.Json;
using System.Threading;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using OpenTracing.Util;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// The current hot reload mode
	/// </summary>
	enum HotReloadMode
	{
		Default,
		Disabled,
		FromIDE,
		FromEditor,
		LiveCoding,
		LiveCodingPassThrough, // Special mode for specific file compiles but live coding is currently active
	}

	/// <summary>
	/// Stores the current hot reload state, tracking temporary files created by previous invocations.
	/// </summary>
	class HotReloadState
	{
		/// <summary>
		/// The version number to write
		/// </summary>
		public const int CurrentVersion = 1;

		/// <summary>
		/// Suffix to use for the next hot reload invocation
		/// </summary>
		public int NextSuffix { get; set; } = 1;

		/// <summary>
		/// Map from original filename in the action graph to hot reload file
		/// </summary>
		public Dictionary<FileReference, FileReference> OriginalFileToHotReloadFile { get; set; } = [];

		/// <summary>
		/// Set of all temporary files created for hot reload
		/// </summary>
		public HashSet<FileReference> TemporaryFiles { get; set; } = [];

		/// <summary>
		/// Public constructor
		/// </summary>
		public HotReloadState()
		{
		}

		/// <summary>
		/// Adds all the actions into the hot reload state, so we can restore the action graph on next iteration
		/// </summary>
		/// <param name="ActionsToExecute">The actions being executed</param>
		/// <param name="OldLocationToNewLocation">Mapping from file from their original location (either a previously hot-reloaded file, or an originally compiled file)</param>
		public void CaptureActions(IEnumerable<LinkedAction> ActionsToExecute, Dictionary<FileReference, FileReference> OldLocationToNewLocation)
		{
			// Build a mapping of all file items to their original location
			Dictionary<FileReference, FileReference> HotReloadFileToOriginalFile = new Dictionary<FileReference, FileReference>();
			foreach (KeyValuePair<FileReference, FileReference> Pair in OriginalFileToHotReloadFile)
			{
				HotReloadFileToOriginalFile[Pair.Value] = Pair.Key;
			}
			foreach (KeyValuePair<FileReference, FileReference> Pair in OldLocationToNewLocation)
			{
				FileReference? OriginalLocation;
				if (!HotReloadFileToOriginalFile.TryGetValue(Pair.Key, out OriginalLocation))
				{
					OriginalLocation = Pair.Key;
				}
				HotReloadFileToOriginalFile[Pair.Value] = OriginalLocation;
			}

			// Now filter out all the hot reload files and update the state
			foreach (LinkedAction Action in ActionsToExecute)
			{
				// Metadata doesn't change name but their content might change
				// Example:
				// UnrealEditor.modules will be patched by HotReload to include the updated module name with suffix
				// When going back to regular build we need to restore the previous module name.
				if (Action.ActionType == ActionType.WriteMetadata)
				{
					foreach (FileItem FileItem in Action.ProducedItems)
					{
						TemporaryFiles.Add(FileItem.Location);
					}
				}
				else
				{
					foreach (FileItem ProducedItem in Action.ProducedItems)
					{
						FileReference? OriginalLocation;
						if (HotReloadFileToOriginalFile.TryGetValue(ProducedItem.Location, out OriginalLocation))
						{
							OriginalFileToHotReloadFile[OriginalLocation] = ProducedItem.Location;
							TemporaryFiles.Add(ProducedItem.Location);
						}
					}
				}
			}
		}

		/// <summary>
		/// Gets the location of the hot-reload state file for a particular target
		/// </summary>
		/// <param name="TargetDescriptor">Descriptor for the target</param>
		/// <returns>Location of the hot reload state file</returns>
		public static FileReference GetLocation(TargetDescriptor TargetDescriptor)
		{
			return GetLocation(TargetDescriptor.ProjectFile, TargetDescriptor.Name, TargetDescriptor.Platform, TargetDescriptor.Configuration, TargetDescriptor.Architectures);
		}

		/// <summary>
		/// Gets the location of the hot-reload state file for a particular target
		/// </summary>
		/// <param name="ProjectFile">Project containing the target</param>
		/// <param name="TargetName">Name of the target</param>
		/// <param name="Platform">Platform being built</param>
		/// <param name="Configuration">Configuration being built</param>
		/// <param name="Architectures">Architecture(s) being built</param>
		/// <returns>Location of the hot reload state file</returns>
		public static FileReference GetLocation(FileReference? ProjectFile, string TargetName, UnrealTargetPlatform Platform, UnrealTargetConfiguration Configuration, UnrealArchitectures Architectures)
		{
			DirectoryReference BaseDir = DirectoryReference.FromFile(ProjectFile) ?? Unreal.EngineDirectory;
			return FileReference.Combine(BaseDir, UEBuildTarget.GetPlatformIntermediateFolder(Platform, Architectures, false), TargetName, Configuration.ToString(), "HotReloadState.json");
		}

		/// <summary>
		/// Read the hot reload state from the given location
		/// </summary>
		/// <param name="Location">Location to read from</param>
		/// <returns>New hot reload state instance</returns>
		public static HotReloadState Load(FileReference Location)
		{
			using FileStream FileStream = FileReference.Open(Location, FileMode.Open, FileAccess.Read, FileShare.Read);
			HotReloadState? MaybeState = JsonSerializer.Deserialize<HotReloadState>(FileStream);
			return MaybeState ?? throw new BuildLogEventException("Hot reload state file '{FilePath}' was invalid", Location);
		}

		/// <summary>
		/// Writes the state to disk
		/// </summary>
		/// <param name="Location">Location to write to</param>
		public void Save(FileReference Location)
		{
			DirectoryReference.CreateDirectory(Location.Directory);
			using FileStream stream = new(Location.FullName, FileMode.Create, FileAccess.Write);
			JsonSerializer.Serialize(stream, this);
		}
	}

	/// <summary>
	/// Contents of the JSON version of the live coding modules file
	/// </summary>
	class LiveCodingModules
	{

		/// <summary>
		/// These modules have been loaded by a process and are enabled for patching
		/// </summary>
		public List<string> EnabledModules { get; set; } = new();

		/// <summary>
		/// These modules have been loaded by a process, but not explicitly enabled
		/// </summary>
		public List<string> LazyLoadModules { get; set; } = new();
	}

	static class HotReload
	{
		/// <summary>
		/// Getts the default hot reload mode for the given target
		/// </summary>
		/// <param name="TargetDescriptor">The target being built</param>
		/// <param name="Makefile">Makefile for the target</param>
		/// <param name="BuildConfiguration">Global build configuration</param>
		/// <param name="Logger">Logger for output</param>
		/// <returns>Default hotreload mode</returns>
		public static HotReloadMode GetDefaultMode(TargetDescriptor TargetDescriptor, TargetMakefile Makefile, BuildConfiguration BuildConfiguration, ILogger Logger)
		{
			if (Makefile.TargetType == TargetType.Program)
			{
				return HotReloadMode.Disabled;
			}
			else if (TargetDescriptor.HotReloadModuleNameToSuffix.Count > 0 && TargetDescriptor.ForeignPlugin == null)
			{
				return HotReloadMode.FromEditor;
			}
			else if (BuildConfiguration.bAllowHotReloadFromIDE && HotReload.ShouldDoHotReloadFromIDE(BuildConfiguration, TargetDescriptor, Logger))
			{
				return HotReloadMode.FromIDE;
			}
			else if (TargetDescriptor.SpecificFilesToCompile.Count > 0 && IsLiveCodingSessionActive(Makefile, Logger))
			{
				Logger.LogWarning("Live coding session active. Actions will be limited to compilation of specified files.  Output will be sent to a temporary location.");
				return HotReloadMode.LiveCodingPassThrough;
			}
			return HotReloadMode.Disabled;
		}

		/// <summary>
		/// Sets the appropriate hot reload mode for a target, and cleans up old state.
		/// </summary>
		/// <param name="TargetDescriptor">The target being built</param>
		/// <param name="Makefile">Makefile for the target</param>
		/// <param name="Actions">Actions for this target</param>
		/// <param name="BuildConfiguration">Global build configuration</param>
		/// <param name="Logger">Logger for output</param>
		public static Dictionary<FileReference, FileReference>? Setup(TargetDescriptor TargetDescriptor, TargetMakefile Makefile, List<LinkedAction> Actions, BuildConfiguration BuildConfiguration, ILogger Logger)
		{
			Dictionary<FileReference, FileReference>? PatchedOldLocationToNewLocation = null;

			// Get the hot-reload mode
			if (TargetDescriptor.HotReloadMode == HotReloadMode.LiveCoding || TargetDescriptor.HotReloadMode == HotReloadMode.LiveCodingPassThrough)
			{
				// In some instances such as packaged builds, we might not have hot reload modules names.
				// We don't want to lose the live coding setting in that case.
			}
			else if (Makefile.HotReloadModuleNames.Count == 0)
			{
				TargetDescriptor.HotReloadMode = HotReloadMode.Disabled;
			}
			else if (TargetDescriptor.HotReloadMode == HotReloadMode.Default)
			{
				TargetDescriptor.HotReloadMode = GetDefaultMode(TargetDescriptor, Makefile, BuildConfiguration, Logger);
			}

			// Apply the previous hot reload state
			if (TargetDescriptor.HotReloadMode == HotReloadMode.Disabled)
			{
				// Make sure we're not doing a partial build from the editor (eg. compiling a new plugin)
				if (TargetDescriptor.ForeignPlugin == null && TargetDescriptor.SpecificFilesToCompile.Count == 0)
				{
					// Delete the previous state file
					FileReference StateFile = HotReloadState.GetLocation(TargetDescriptor);
					HotReload.DeleteTemporaryFiles(StateFile, Logger);
				}
			}
			else
			{
				// Reapply the previous state
				FileReference StateFile = HotReloadState.GetLocation(TargetDescriptor);
				if (FileReference.Exists(StateFile))
				{
					// Read the previous state file and apply it to the action graph
					HotReloadState HotReloadState = HotReloadState.Load(StateFile);

					// Apply the old state to the makefile
					HotReload.ApplyState(HotReloadState, Makefile, Actions);
				}

				// If we want a specific suffix on any modules, apply that now. We'll track the outputs later, but the suffix has to be forced (and is always out of date if it doesn't exist).
				PatchedOldLocationToNewLocation = HotReload.PatchActionGraphWithNames(TargetDescriptor.HotReloadModuleNameToSuffix, Makefile, Actions);
			}
			return PatchedOldLocationToNewLocation;
		}

		public static void CheckForLiveCodingSessionActive(TargetDescriptor TargetDescriptor, TargetMakefile Makefile, BuildConfiguration BuildConfiguration, ILogger Logger)
		{
			// Guard against a live coding session for this target being active
			if (BuildConfiguration.bAllowHotReloadFromIDE && TargetDescriptor.ForeignPlugin == null &&
				TargetDescriptor.HotReloadMode != HotReloadMode.LiveCoding && TargetDescriptor.HotReloadMode != HotReloadMode.LiveCodingPassThrough &&
				HotReload.IsLiveCodingSessionActive(Makefile, Logger))
			{
				throw new BuildException("Unable to build while Live Coding is active. Exit the editor and game, or press Ctrl+Alt+F11 if iterating on code in the editor or game");
			}
		}

		/// <summary>
		/// Checks whether a live coding session is currently active for a target. If so, we don't want to allow modifying any object files before they're loaded.
		/// </summary>
		/// <param name="Makefile">Makefile for the target being built</param>
		/// <param name="Logger">Logger for output</param>
		/// <returns>True if a live coding session is active, false otherwise</returns>
		static bool IsLiveCodingSessionActive(TargetMakefile Makefile, ILogger Logger)
		{
			// Find the first output executable
			FileReference Executable = Makefile.ExecutableFile;
			if (Executable != null)
			{
				// Build the mutex name. This should match the name generated in LiveCodingModule.cpp.
				StringBuilder MutexName = new StringBuilder("Global\\LiveCoding_");
				for (int Idx = 0; Idx < Executable.FullName.Length; Idx++)
				{
					char Character = Executable.FullName[Idx];
					if (Character == '/' || Character == '\\' || Character == ':')
					{
						MutexName.Append('+');
					}
					else
					{
						MutexName.Append(Character);
					}
				}
				Logger.LogDebug("Checking for live coding mutex: {MutexName}", MutexName);

				// Try to open the mutex
				Mutex? Mutex;
				if (Mutex.TryOpenExisting(MutexName.ToString(), out Mutex))
				{
					Mutex.Dispose();
					return true;
				}
			}
			return false;
		}

		/// <summary>
		/// Checks if the editor is currently running and this is a hot-reload
		/// </summary>
		static bool ShouldDoHotReloadFromIDE(BuildConfiguration BuildConfiguration, TargetDescriptor TargetDesc, ILogger Logger)
		{
			// Check if Hot-reload is disabled globally for this project
			ConfigHierarchy Hierarchy = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, DirectoryReference.FromFile(TargetDesc.ProjectFile), TargetDesc.Platform);
			bool bAllowHotReloadFromIDE;
			if (Hierarchy.TryGetValue("BuildConfiguration", "bAllowHotReloadFromIDE", out bAllowHotReloadFromIDE) && !bAllowHotReloadFromIDE)
			{
				return false;
			}

			if (!BuildConfiguration.bAllowHotReloadFromIDE)
			{
				return false;
			}

			// Check if we're using LiveCode instead
			if (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Win64) // Temporary - new 5.0 projects will have live coding setting on for platforms that don't support it.
			{
				ConfigHierarchy EditorPerProjectHierarchy = ConfigCache.ReadHierarchy(ConfigHierarchyType.EditorPerProjectUserSettings, DirectoryReference.FromFile(TargetDesc.ProjectFile), TargetDesc.Platform);
				bool bEnableLiveCode;
				if (EditorPerProjectHierarchy.GetBool("/Script/LiveCoding.LiveCodingSettings", "bEnabled", out bEnableLiveCode) && bEnableLiveCode)
				{
					return false;
				}
			}

			bool bIsRunning = false;

			// @todo ubtmake: Kind of cheating here to figure out if an editor target.  At this point we don't have access to the actual target description, and
			// this code must be able to execute before we create or load module rules DLLs so that hot reload can work with bUseUBTMakefiles
			if (TargetDesc.Name.EndsWith("Editor", StringComparison.OrdinalIgnoreCase))
			{
				string EditorBaseFileName = "UnrealEditor";
				if (TargetDesc.Configuration != UnrealTargetConfiguration.Development)
				{
					EditorBaseFileName = String.Format("{0}-{1}-{2}", EditorBaseFileName, TargetDesc.Platform, TargetDesc.Configuration);
				}

				FileReference EditorLocation;
				if (TargetDesc.Platform == UnrealTargetPlatform.Win64)
				{
					EditorLocation = FileReference.Combine(Unreal.EngineDirectory, "Binaries", "Win64", String.Format("{0}.exe", EditorBaseFileName));
				}
				else if (TargetDesc.Platform == UnrealTargetPlatform.Mac)
				{
					EditorLocation = FileReference.Combine(Unreal.EngineDirectory, "Binaries", "Mac", String.Format("{0}.app/Contents/MacOS/{0}", EditorBaseFileName));
				}
				else if (TargetDesc.Platform == UnrealTargetPlatform.Linux)
				{
					EditorLocation = FileReference.Combine(Unreal.EngineDirectory, "Binaries", "Linux", EditorBaseFileName);
				}
				else
				{
					throw new BuildException("Unknown editor filename for this platform");
				}

				using (GlobalTracer.Instance.BuildSpan("Finding editor processes for hot-reload").StartActive())
				{
					DirectoryReference EditorRunsDir = DirectoryReference.Combine(Unreal.EngineDirectory, "Intermediate", "EditorRuns");
					if (!DirectoryReference.Exists(EditorRunsDir))
					{
						return false;
					}

					if (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Win64)
					{
						foreach (FileReference EditorInstanceFile in DirectoryReference.EnumerateFiles(EditorRunsDir))
						{
							int ProcessId;
							if (!Int32.TryParse(EditorInstanceFile.GetFileName(), out ProcessId))
							{
								FileReference.Delete(EditorInstanceFile);
								continue;
							}

							Process? RunningProcess;
							try
							{
								RunningProcess = Process.GetProcessById(ProcessId);
							}
							catch
							{
								RunningProcess = null;
							}

							bool bFileShouldBeDeleted = false;

							if (RunningProcess == null)
							{
								bFileShouldBeDeleted = true;
							}
							else
							{
								try
								{
									if (RunningProcess.HasExited)
									{
										bFileShouldBeDeleted = true;
									}
								}
								catch
								{
									// if the PID represents an editor that has exited, and is now reused as the pid of a system process,
									// RunningProcess.HasExited may fail with "Access is denied."
									// If we can't determine if the process has exited, let's assume that the file should be deleted.
									bFileShouldBeDeleted = true;
								}
							}

							// bugfix - the editor sometimes doesn't delete its editorrun file due to
							// crash or debugger stop or whatever. ~eventually~ this should get caught
							// by the above check where the PID no longer exists, however windows actually
							// keeps the process table entry around for a ~long~ time (days, across hibernations).
							//
							// What ends up happening is we successfully get the Process object, but we throw
							// an exception trying to retrieve the module handle for the filename, and then
							// don't delete it.
							//
							// On my machine this was ~750 ms _per orphaned file_, and I spoke to someone
							// with 10 of these in his Engine/Intermediate/EditorRun directory.
							// 
							FileReference? MainModuleFile;
							try
							{
								MainModuleFile = new FileReference(RunningProcess!.MainModule!.FileName!);
							}
							catch
							{
								MainModuleFile = null;
								bFileShouldBeDeleted = true;
							}

							if (bFileShouldBeDeleted)
							{
								try
								{
									FileReference.Delete(EditorInstanceFile);
								}
								catch
								{
									Logger.LogDebug("Failed to delete EditorRun file for exited process: {Process}", EditorInstanceFile.GetFileName());
								}
								continue;
							}

							if (!bIsRunning && EditorLocation == MainModuleFile)
							{
								bIsRunning = true;
							}
						}
					}
					else
					{
						FileInfo[] EditorRunsFiles = new DirectoryInfo(EditorRunsDir.FullName).GetFiles();
						BuildHostPlatform.ProcessInfo[] Processes = BuildHostPlatform.Current.GetProcesses();

						foreach (FileInfo File in EditorRunsFiles)
						{
							int PID;
							BuildHostPlatform.ProcessInfo? Proc = null;
							if (!Int32.TryParse(File.Name, out PID) || (Proc = Processes.FirstOrDefault(P => P.PID == PID)) == default(BuildHostPlatform.ProcessInfo))
							{
								// Delete stale files (it may happen if editor crashes).
								File.Delete();
								continue;
							}

							// Don't break here to allow clean-up of other stale files.
							if (!bIsRunning)
							{
								// Otherwise check if the path matches.
								bIsRunning = new FileReference(Proc.Filename) == EditorLocation;
							}
						}
					}
				}
			}
			return bIsRunning;
		}

		/// <summary>
		/// Delete all temporary files created by previous hot reload invocations
		/// </summary>
		/// <param name="HotReloadStateFile">Location of the state file</param>
		/// <param name="Logger">Logger for output</param>
		public static void DeleteTemporaryFiles(FileReference HotReloadStateFile, ILogger Logger)
		{
			if (FileReference.Exists(HotReloadStateFile))
			{
				// Try to load the state file. If it fails, we'll just warn and continue.
				HotReloadState? State = null;
				try
				{
					State = HotReloadState.Load(HotReloadStateFile);
				}
				catch (Exception Ex)
				{
					Logger.LogWarning("Unable to read hot reload state file: {HotReloadStateFile}", HotReloadStateFile);
					Log.WriteException(Ex, null);
					return;
				}

				// Delete all the output files
				foreach (FileReference Location in State.TemporaryFiles.OrderBy(x => x.FullName, StringComparer.OrdinalIgnoreCase))
				{
					if (FileReference.Exists(Location))
					{
						try
						{
							FileReference.Delete(Location);
						}
						catch (Exception Ex)
						{
							throw new BuildException(Ex, "Unable to delete hot-reload file: {0}", Location);
						}
						Logger.LogInformation("Deleted hot-reload file: {Location}", Location);
					}
				}

				// Delete the state file itself
				try
				{
					FileReference.Delete(HotReloadStateFile);
				}
				catch (Exception Ex)
				{
					throw new BuildException(Ex, "Unable to delete hot-reload state file: {0}", HotReloadStateFile);
				}
			}
		}

		/// <summary>
		/// Apply a saved hot reload state to a makefile
		/// </summary>
		/// <param name="HotReloadState">The hot-reload state</param>
		/// <param name="Makefile">Makefile to apply the state</param>
		/// <param name="Actions">Actions for this makefile</param>
		static void ApplyState(HotReloadState HotReloadState, TargetMakefile Makefile, List<LinkedAction> Actions)
		{
			// Update the action graph to produce these new files
			HotReload.PatchActionGraph(Actions, HotReloadState.OriginalFileToHotReloadFile);

			// Update the module to output file mapping
			foreach (string HotReloadModuleName in Makefile.HotReloadModuleNames)
			{
				FileItem[] ModuleOutputItems = Makefile.ModuleNameToOutputItems[HotReloadModuleName];
				for (int Idx = 0; Idx < ModuleOutputItems.Length; Idx++)
				{
					FileReference? NewLocation;
					if (HotReloadState.OriginalFileToHotReloadFile.TryGetValue(ModuleOutputItems[Idx].Location, out NewLocation))
					{
						ModuleOutputItems[Idx] = FileItem.GetItemByFileReference(NewLocation);
					}
				}
			}
		}

		/// <summary>
		/// Given a collection of strings which are file paths, create a hash set from the file name and extension.
		/// Empty strings are eliminated.
		/// </summary>
		/// <param name="Collection">Source collection</param>
		/// <returns>Trimmed and unique collection</returns>
		private static HashSet<string> CreateHashSetFromFileList(IEnumerable<string> Collection)
		{
			// Parse it out into a set of filenames
			HashSet<string> Out = new HashSet<string>(FileReference.Comparer);
			foreach (string Line in Collection)
			{
				string TrimLine = Line.Trim();
				if (TrimLine.Length > 0)
				{
					Out.Add(Path.GetFileName(TrimLine));
				}
			}
			return Out;
		}

		/// <summary>
		/// Determine what needs to be built for a target
		/// </summary>
		/// <param name="BuildConfiguration">The build configuration</param>
		/// <param name="TargetDescriptor">Target being built</param>
		/// <param name="Makefile">Makefile generated for this target</param>
		/// <param name="PrerequisiteActions">The actions to execute</param>
		/// <param name="TargetActionsToExecute">Actions to execute for this target</param>
		/// <param name="InitialPatchedOldLocationToNewLocation">Collection of all the renamed as part of module reload requests.  Can be null</param>
		/// <param name="Logger">Logger for output</param>
		/// <returns>Set of actions to execute</returns>
		public static List<LinkedAction> PatchActionsForTarget(BuildConfiguration BuildConfiguration, TargetDescriptor TargetDescriptor, TargetMakefile Makefile, List<LinkedAction> PrerequisiteActions, List<LinkedAction> TargetActionsToExecute, Dictionary<FileReference, FileReference>? InitialPatchedOldLocationToNewLocation, ILogger Logger)
		{
			// Get the dependency history
			CppDependencyCache CppDependencies = new CppDependencyCache();
			CppDependencies.Mount(TargetDescriptor, Makefile.TargetType, Logger);

			ActionHistory History = new ActionHistory();
			if (TargetDescriptor.ProjectFile != null)
			{
				History.Mount(TargetDescriptor.ProjectFile.Directory);
			}

			if (TargetDescriptor.HotReloadMode == HotReloadMode.LiveCoding || TargetDescriptor.HotReloadMode == HotReloadMode.LiveCodingPassThrough)
			{
				CompilationResult Result = CompilationResult.Succeeded;

				// Make sure we're not overwriting any lazy-loaded modules
				if (TargetDescriptor.LiveCodingModules != null)
				{

					// In the old style module list, which was just a text file, we allow only modules found in the known list of enabled modules.
					//		All other modules are assumed to be lazy loaded.
					// In the new style module list, which is a json file, we disallow modules found in list of lazy loaded modules and allow
					//		all other modules. The enabled module list is not used in the new format, but is there for diagnostics or future expansion.
					HashSet<string>? AllowedOutputFileNames = null;
					HashSet<string>? DisallowedOutputFileNames = null;
					if (TargetDescriptor.LiveCodingModules.GetExtension() == ".json")
					{
						LiveCodingModules? Modules = JsonSerializer.Deserialize<LiveCodingModules>(File.OpenRead(TargetDescriptor.LiveCodingModules.FullName));
						if (Modules == null)
						{
							throw new BuildException("Unable to load live coding modules file '{0}'", TargetDescriptor.LiveCodingModules.FullName);
						}
						DisallowedOutputFileNames = CreateHashSetFromFileList(Modules.LazyLoadModules);
					}
					else
					{
						// Read the list of modules that we're allowed to build
						string[] Lines = FileReference.ReadAllLines(TargetDescriptor.LiveCodingModules);
						AllowedOutputFileNames = CreateHashSetFromFileList(Lines);
					}

					// Find all the binaries that we're actually going to build
					HashSet<FileReference> OutputFiles = new HashSet<FileReference>();
					foreach (LinkedAction Action in TargetActionsToExecute)
					{
						if (Action.ActionType == ActionType.Link)
						{
							OutputFiles.UnionWith(Action.ProducedItems.Where(x => x.HasExtension(".exe") || x.HasExtension(".dll")).Select(x => x.Location));
						}
					}

					// Find all the files that will be built that aren't allowed
					List<FileReference> ProtectedOutputFiles = OutputFiles.Where(x =>
							(AllowedOutputFileNames != null && !AllowedOutputFileNames.Contains(x.GetFileName())) ||
							(DisallowedOutputFileNames != null && DisallowedOutputFileNames.Contains(x.GetFileName()))
						).ToList();

					// Generate the error messages
					if (ProtectedOutputFiles.Count > 0)
					{
						FileReference.WriteAllLines(new FileReference(TargetDescriptor.LiveCodingModules.FullName + ".out"), ProtectedOutputFiles.Select(x => x.ToString()));
						foreach (FileReference ProtectedOutputFile in ProtectedOutputFiles)
						{
							Logger.LogInformation("Module {ProtectedOutputFile} is not currently enabled for Live Coding", ProtectedOutputFile);
						}

						// Note the issue but continue processing to allow the limit to generate an error if hit.
						Result = CompilationResult.Canceled;
					}
				}

				// Filter the prerequisite actions down to just the compile actions, then recompute all the actions to execute
				PrerequisiteActions = new List<LinkedAction>(TargetActionsToExecute.Where(x => IsLiveCodingAction(x)));
				TargetActionsToExecute = ActionGraph.GetActionsToExecute(PrerequisiteActions, CppDependencies, History, BuildConfiguration.bIgnoreOutdatedImportLibraries, Logger);

				// Update the action graph with these new paths
				Dictionary<FileReference, FileReference> OriginalFileToPatchedFile = new Dictionary<FileReference, FileReference>();
				HotReload.PatchActionGraphForLiveCoding(PrerequisiteActions, OriginalFileToPatchedFile, TargetDescriptor.HotReloadMode, Logger);

				// Get a new list of actions to execute now that the graph has been modified
				TargetActionsToExecute = ActionGraph.GetActionsToExecute(PrerequisiteActions, CppDependencies, History, BuildConfiguration.bIgnoreOutdatedImportLibraries, Logger);

				// Check to see if we exceed the limit for live coding actions
				if (TargetDescriptor.LiveCodingLimit > 0 && TargetDescriptor.LiveCodingLimit < TargetActionsToExecute.Count)
				{
					Logger.LogInformation("The live coding request of {TargetActionsToExecuteCount} actions exceeds the number of allowed actions of {TargetDescriptorLiveCodingLimit}", TargetActionsToExecute.Count, TargetDescriptor.LiveCodingLimit);
					Logger.LogInformation("This limit helps to prevent the situation where seemingly simple changes result in large scale rebuilds.");
					Logger.LogInformation("It can also help to detect when the engine needs to be rebuilt outside of Live Coding due to compiler changes.");
					Result = CompilationResult.LiveCodingLimitError;
				}

				// Throw an exception if there is an issue
				if (Result != CompilationResult.Succeeded)
				{
					throw new CompilationResultException(Result);
				}

				// Output the Live Coding manifest
				if (TargetDescriptor.LiveCodingManifest != null)
				{
					HotReload.WriteLiveCodingManifest(TargetDescriptor.LiveCodingManifest, Makefile.Actions, OriginalFileToPatchedFile);
				}
			}
			else if (TargetDescriptor.HotReloadMode == HotReloadMode.FromEditor || TargetDescriptor.HotReloadMode == HotReloadMode.FromIDE)
			{
				// Get the path to the state file
				FileReference HotReloadStateFile = global::UnrealBuildTool.HotReloadState.GetLocation(TargetDescriptor);

				// Read the previous state file and apply it to the action graph
				HotReloadState HotReloadState;
				if (FileReference.Exists(HotReloadStateFile))
				{
					HotReloadState = HotReloadState.Load(HotReloadStateFile);
				}
				else
				{
					HotReloadState = new HotReloadState();
				}

				// Patch action history for hot reload when running in assembler mode.  In assembler mode, the suffix on the output file will be
				// the same for every invocation on that makefile, but we need a new suffix each time.

				// For all the hot-reloadable modules that may need a unique suffix appended, build a mapping from output item to all the output items in that module. We can't 
				// apply a suffix to one without applying a suffix to all of them.
				Dictionary<FileItem, FileItem[]> HotReloadItemToDependentItems = new Dictionary<FileItem, FileItem[]>();
				foreach (string HotReloadModuleName in Makefile.HotReloadModuleNames)
				{
					int ModuleSuffix;
					if (!TargetDescriptor.HotReloadModuleNameToSuffix.TryGetValue(HotReloadModuleName, out ModuleSuffix) || ModuleSuffix == -1)
					{
						FileItem[]? ModuleOutputItems;
						if (Makefile.ModuleNameToOutputItems.TryGetValue(HotReloadModuleName, out ModuleOutputItems))
						{
							foreach (FileItem ModuleOutputItem in ModuleOutputItems)
							{
								HotReloadItemToDependentItems[ModuleOutputItem] = ModuleOutputItems;
							}
						}
					}
				}

				// Expand the list of actions to execute to include everything that references any files with a new suffix. Unlike a regular build, we can't ignore
				// dependencies on import libraries under the assumption that a header would change if the API changes, because the dependency will be on a different DLL.
				HashSet<FileItem> FilesRequiringSuffix = new HashSet<FileItem>(TargetActionsToExecute.SelectMany(x => x.ProducedItems).Where(x => HotReloadItemToDependentItems.ContainsKey(x)));
				for (int LastNumFilesWithNewSuffix = 0; FilesRequiringSuffix.Count > LastNumFilesWithNewSuffix;)
				{
					LastNumFilesWithNewSuffix = FilesRequiringSuffix.Count;
					foreach (LinkedAction PrerequisiteAction in PrerequisiteActions.Where(x => !TargetActionsToExecute.Contains(x) && x.PrerequisiteItems.Intersect(FilesRequiringSuffix).Any()))
					{
						foreach (FileItem ProducedItem in PrerequisiteAction.ProducedItems)
						{
							FileItem[]? DependentItems;
							if (HotReloadItemToDependentItems.TryGetValue(ProducedItem, out DependentItems))
							{
								TargetActionsToExecute.Add(PrerequisiteAction);
								FilesRequiringSuffix.UnionWith(DependentItems);
							}
						}
					}
				}

				// Build a list of file mappings
				Dictionary<FileReference, FileReference> OldLocationToNewLocation = new Dictionary<FileReference, FileReference>();
				foreach (FileItem FileRequiringSuffix in FilesRequiringSuffix)
				{
					FileReference OldLocation = FileRequiringSuffix.Location;
					FileReference NewLocation = HotReload.ReplaceSuffix(OldLocation, HotReloadState.NextSuffix);
					OldLocationToNewLocation[OldLocation] = NewLocation;
				}

				// Update the action graph with these new paths
				Dictionary<FileReference, FileReference> PatchedOldLocationToNewLocation = HotReload.PatchActionGraph(PrerequisiteActions, OldLocationToNewLocation);

				// Get a new list of actions to execute now that the graph has been modified
				TargetActionsToExecute = ActionGraph.GetActionsToExecute(PrerequisiteActions, CppDependencies, History, BuildConfiguration.bIgnoreOutdatedImportLibraries, Logger);

				// Record all of the updated locations directly associated with actions.
				if (InitialPatchedOldLocationToNewLocation != null)
				{
					HotReloadState.CaptureActions(TargetActionsToExecute, InitialPatchedOldLocationToNewLocation);
				}
				HotReloadState.CaptureActions(TargetActionsToExecute, PatchedOldLocationToNewLocation);

				// Increment the suffix for the next iteration
				if (TargetActionsToExecute.Count > 0)
				{
					HotReloadState.NextSuffix++;
				}

				// Save the new state
				HotReloadState.Save(HotReloadStateFile);

				// Prevent this target from deploying
				Makefile.bDeployAfterCompile = false;
			}

			return TargetActionsToExecute;
		}

		/// <summary>
		/// Replaces a hot reload suffix in a filename.
		/// </summary>
		public static FileReference ReplaceSuffix(FileReference File, int Suffix)
		{
			string FileName = File.GetFileName();

			// Find the end of the target and module name
			int HyphenIdx = FileName.IndexOf('-', StringComparison.Ordinal);
			if (HyphenIdx == -1)
			{
				throw new BuildException("Hot-reloadable files are expected to contain a hyphen, eg. UnrealEditor-Core ({0})", FileName);
			}

			int NameEndIdx = HyphenIdx + 1;
			while (NameEndIdx < FileName.Length && FileName[NameEndIdx] != '.' && FileName[NameEndIdx] != '-')
			{
				NameEndIdx++;
			}

			// Strip any existing suffix
			if (NameEndIdx + 1 < FileName.Length && Char.IsDigit(FileName[NameEndIdx + 1]))
			{
				int SuffixEndIdx = NameEndIdx + 2;
				while (SuffixEndIdx < FileName.Length && Char.IsDigit(FileName[SuffixEndIdx]))
				{
					SuffixEndIdx++;
				}
				if (SuffixEndIdx == FileName.Length || FileName[SuffixEndIdx] == '-' || FileName[SuffixEndIdx] == '.')
				{
					FileName = FileName.Substring(0, NameEndIdx) + FileName.Substring(SuffixEndIdx);
				}
			}

			// NOTE: Formatting of this string must match the code in ModuleManager.cpp, MakeUniqueModuleFilename
			string NewFileName = String.Format("{0}-{1:D4}{2}", FileName.Substring(0, NameEndIdx), Suffix, FileName.Substring(NameEndIdx));

			return FileReference.Combine(File.Directory, NewFileName);
		}

		/// <summary>
		/// Replaces a base filename within a string. Ensures that the filename is not a substring of another longer string (eg. replacing "Foo" will match "Foo.Bar" but not "FooBar" or "BarFooBar"). 
		/// </summary>
		/// <param name="Text">Text to replace within</param>
		/// <param name="OldFileName">Old filename</param>
		/// <param name="NewFileName">New filename</param>
		/// <returns>Text with file names replaced</returns>
		static string ReplaceBaseFileName(string Text, string OldFileName, string NewFileName)
		{
			int StartIdx = 0;
			for (; ; )
			{
				int Idx = Text.IndexOf(OldFileName, StartIdx, StringComparison.OrdinalIgnoreCase);
				if (Idx == -1)
				{
					break;
				}
				else if ((Idx == 0 || !IsBaseFileNameCharacter(Text[Idx - 1])) && (Idx + OldFileName.Length == Text.Length || !IsBaseFileNameCharacter(Text[Idx + OldFileName.Length])))
				{
					Text = Text.Substring(0, Idx) + NewFileName + Text.Substring(Idx + OldFileName.Length);
					StartIdx = Idx + NewFileName.Length;
				}
				else
				{
					StartIdx = Idx + 1;
				}
			}
			return Text;
		}

		/// <summary>
		/// Determines if a character should be treated as part of a base filename, when updating strings for hot reload
		/// </summary>
		/// <param name="Character">The character to check</param>
		/// <returns>True if the character is part of a base filename, false otherwise</returns>
		static bool IsBaseFileNameCharacter(char Character)
		{
			return Char.IsLetterOrDigit(Character) || Character == '_';
		}

		/// <summary>
		/// Test to see if the action is an action live coding supports.  All other actions will be filtered
		/// </summary>
		/// <param name="Action">Action in question</param>
		/// <returns>True if the action is a compile action for the compiler.  This filters out RC compiles.</returns>
		static bool IsLiveCodingAction(LinkedAction Action)
		{
			return Action.ActionType == ActionType.Compile &&
				(Action.CommandPath.GetFileName().Equals("cl-filter.exe", StringComparison.OrdinalIgnoreCase)
					|| Action.CommandPath.GetFileName().Equals("cl.exe", StringComparison.OrdinalIgnoreCase)
					|| Action.CommandPath.GetFileName().Equals("clang-cl.exe", StringComparison.OrdinalIgnoreCase)
					|| Action.CommandPath.GetFileName().Equals("verse-clang-cl.exe", StringComparison.OrdinalIgnoreCase)
				);
		}

		/// <summary>
		/// Patches a set of actions for use with live coding. The new action list will output object files to a different location.
		/// </summary>
		/// <param name="Actions">Set of actions</param>
		/// <param name="OriginalFileToPatchedFile">Dictionary that receives a map of original object file to patched object file</param>
		/// <param name="hotReloadMode">Requested hot reload mode</param>
		/// <param name="Logger"></param>
		public static void PatchActionGraphForLiveCoding(IEnumerable<LinkedAction> Actions, Dictionary<FileReference, FileReference> OriginalFileToPatchedFile, HotReloadMode hotReloadMode, ILogger Logger)
		{
			string extraExt = hotReloadMode == HotReloadMode.LiveCoding ? ".lc" : ".lcpt";
			string dependencyFileExtension = hotReloadMode == HotReloadMode.LiveCoding ? ".lc.response" : ".lcpt.response";
			string objectFileExtension = hotReloadMode == HotReloadMode.LiveCoding ? ".lc.obj" : ".lcpt.obj";

			foreach (LinkedAction Action in Actions)
			{
				if (Action.ActionType == ActionType.Compile)
				{
					if (!Action.CommandPath.GetFileName().Equals("cl-filter.exe", StringComparison.OrdinalIgnoreCase)
						&& !Action.CommandPath.GetFileName().Equals("cl.exe", StringComparison.OrdinalIgnoreCase)
						&& !Action.CommandPath.GetFileName().Equals("clang-cl.exe", StringComparison.OrdinalIgnoreCase)
						&& !Action.CommandPath.GetFileName().Equals("verse-clang-cl.exe", StringComparison.OrdinalIgnoreCase)
						)
					{
						throw new BuildException("Unable to patch action graph - unexpected executable in compile action ({0})", Action.CommandPath);
					}

					List<string> Arguments = Utils.ParseArgumentList(Action.CommandArguments);

					Action NewAction = new Action(Action.Inner);
					Action.Inner = NewAction;

					int DelimiterIdx = -1;
					if (Action.CommandPath.GetFileName().Equals("cl-filter.exe", StringComparison.OrdinalIgnoreCase))
					{
						// Find the index of the cl-filter argument delimiter
						DelimiterIdx = Arguments.IndexOf("--");
						if (DelimiterIdx == -1)
						{
							throw new BuildException("Unable to patch action graph - missing '--' delimiter to cl-filter");
						}

						// Fix the dependencies path
						const string DependenciesPrefix = "-dependencies=";

						int DependenciesIdx = 0;
						for (; ; DependenciesIdx++)
						{
							if (DependenciesIdx == DelimiterIdx)
							{
								throw new BuildException("Unable to patch action graph - missing '{0}' argument to cl-filter", DependenciesPrefix);
							}
							else if (Arguments[DependenciesIdx].StartsWith(DependenciesPrefix, StringComparison.OrdinalIgnoreCase))
							{
								break;
							}
						}

						FileReference OldDependenciesFile = new FileReference(Arguments[DependenciesIdx].Substring(DependenciesPrefix.Length));
						FileItem OldDependenciesFileItem = Action.ProducedItems.First(x => x.Location == OldDependenciesFile);
						NewAction.ProducedItems.Remove(OldDependenciesFileItem);

						FileReference NewDependenciesFile = OldDependenciesFile.ChangeExtension(dependencyFileExtension);
						FileItem NewDependenciesFileItem = FileItem.GetItemByFileReference(NewDependenciesFile);
						NewAction.ProducedItems.Add(NewDependenciesFileItem);
						NewAction.DependencyListFile = NewDependenciesFileItem;
						Arguments[DependenciesIdx] = DependenciesPrefix + NewDependenciesFile.FullName;
					}

					// Fix the response file
					int ResponseFileIdx = DelimiterIdx + 1;
					for (; ; ResponseFileIdx++)
					{
						if (ResponseFileIdx == Arguments.Count)
						{
							throw new BuildException($"Unable to patch action graph - missing response file argument to {Action.CommandPath.GetFileName()}");
						}
						else if (Arguments[ResponseFileIdx].StartsWith("@", StringComparison.Ordinal))
						{
							break;
						}
					}

					FileReference OldResponseFile = Action.RootPaths.GetLocalPath(new FileReference(Arguments[ResponseFileIdx].Substring(1).Trim('\"')));
					FileReference NewResponseFile = OldResponseFile.ChangeExtension(".lc.rsp");

					NewAction.PrerequisiteItems.Remove(FileItem.GetItemByFileReference(OldResponseFile));
					NewAction.PrerequisiteItems.Add(FileItem.GetItemByFileReference(NewResponseFile));

					const string OutputFilePrefix = "/Fo";

					string[] ResponseLines = FileReference.ReadAllLines(OldResponseFile);
					for (int Idx = 0; Idx < ResponseLines.Length; Idx++)
					{
						string ResponseLine = ResponseLines[Idx];
						if (ResponseLine.StartsWith(OutputFilePrefix, StringComparison.Ordinal))
						{
							FileReference OldOutputFile = Action.RootPaths.GetLocalPath(new FileReference(ResponseLine.Substring(OutputFilePrefix.Length).Trim('\"')));
							FileItem OldOutputFileItem = Action.ProducedItems.First(x => x.Location == OldOutputFile);
							NewAction.ProducedItems.Remove(OldOutputFileItem);

							FileReference NewOutputFile = OldOutputFile.ChangeExtension(objectFileExtension);
							FileItem NewOutputFileItem = FileItem.GetItemByFileReference(NewOutputFile);
							NewAction.ProducedItems.Add(NewOutputFileItem);

							OriginalFileToPatchedFile[OldOutputFile] = NewOutputFile;

							string NewOutputPath = NewOutputFile.FullName;
							if (Action.RootPaths.GetVfsOverlayPath(NewOutputFile, out string? virtualPath))
							{
								NewOutputPath = virtualPath;
							}

							ResponseLines[Idx] = OutputFilePrefix + "\"" + NewOutputPath + "\"";
							break;
						}
					}

					// Update dependency file path for cl or clang-cl which is in the response file
					if (Action.CommandPath.GetFileName().Equals("cl.exe", StringComparison.OrdinalIgnoreCase) 
					    || Action.CommandPath.GetFileName().Equals("clang-cl.exe", StringComparison.OrdinalIgnoreCase)
						|| Action.CommandPath.GetFileName().Equals("verse-clang-cl.exe", StringComparison.OrdinalIgnoreCase))
					{
						string SourceDependencyPrefix = Action.CommandPath.GetFileName().Equals("cl.exe", StringComparison.OrdinalIgnoreCase) ? "/sourceDependencies" : "/clang:-MD /clang:-MF";
						string extension = Action.CommandPath.GetFileName().Equals("cl.exe", StringComparison.OrdinalIgnoreCase) ? ".dep.json" : ".d";

						for (int Idx = 0; Idx < ResponseLines.Length; Idx++)
						{
							string ResponseLine = ResponseLines[Idx];
							if (ResponseLine.StartsWith(SourceDependencyPrefix, StringComparison.Ordinal))
							{
								FileReference OldSourceDependencyFile = Action.RootPaths.GetLocalPath(new FileReference(ResponseLine.Substring(SourceDependencyPrefix.Length).Trim().Trim('\"')));
								FileItem OldSourceDependencyFileItem = Action.ProducedItems.First(x => x.Location == OldSourceDependencyFile);
								NewAction.ProducedItems.Remove(OldSourceDependencyFileItem);

								FileReference NewSourceDependencyFile = new(OldSourceDependencyFile.FullName.Substring(0, OldSourceDependencyFile.FullName.Length - extension.Length) + extraExt + extension);
								FileItem NewSourceDependencyFileItem = FileItem.GetItemByFileReference(NewSourceDependencyFile);
								NewAction.ProducedItems.Add(NewSourceDependencyFileItem);
								NewAction.DependencyListFile = NewSourceDependencyFileItem;

								string NewSourceDependencyPath = NewSourceDependencyFile.FullName;
								if (Action.RootPaths.GetVfsOverlayPath(NewSourceDependencyFile, out string? VirtualPath))
								{
									NewSourceDependencyPath = VirtualPath;
								}

								ResponseLines[Idx] = SourceDependencyPrefix + "\"" + NewSourceDependencyPath + "\"";
								break;
							}
						}
					}

					Utils.WriteFileIfChanged(NewResponseFile, ResponseLines, Logger);

					Arguments[ResponseFileIdx] = "@" + NewResponseFile.FullName;

					// Update the final arguments
					NewAction.CommandArguments = Utils.FormatCommandLine(Arguments);
				}
			}
		}

		/// <summary>
		/// Patch the action graph for hot reloading, mapping files according to the given dictionary.
		/// </summary>
		public static Dictionary<FileReference, FileReference> PatchActionGraph(IEnumerable<LinkedAction> Actions, Dictionary<FileReference, FileReference> OriginalFileToHotReloadFile)
		{
			// Gather all of the files for link action to be patched.  We're going to need to patch 'em up after we figure out new
			// names for all of the output files and import libraries
			Dictionary<FileReference, FileReference> OriginalToNewFilePaths = [];

			// Keep a map of the original file names and their new file names, so we can fix up response files after
			Dictionary<string, string> OriginalFileNameAndNewFileNameList_NoExtensions = new Dictionary<string, string>();

			// Finally, we'll keep track of any file items that we had to create counterparts for change file names, so we can fix those up too
			Dictionary<FileItem, FileItem> AffectedOriginalFileItemAndNewFileItemMap = new Dictionary<FileItem, FileItem>();

			string ResponseFileExtension = UEToolChain.ResponseExt;
			string ScriptExtension = (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Win64) ? ".bat" : ".sh";

			foreach (LinkedAction Action in Actions.Where((Action) => Action.ActionType == ActionType.Link))
			{
				// Find the first produced item that needs to be renamed
				FileItem? ProducedItem = Action.ProducedItems.FirstOrDefault(x => OriginalFileToHotReloadFile.ContainsKey(x.Location));
				if (ProducedItem == null)
				{
					continue;
				}

				FileReference? HotReloadFile;
				if (!OriginalFileToHotReloadFile.TryGetValue(ProducedItem.Location, out HotReloadFile))
				{
					continue;
				}

				string OriginalFileNameWithoutExtension = Utils.GetFilenameWithoutAnyExtensions(ProducedItem.AbsolutePath);
				string NewFileNameWithoutExtension = Utils.GetFilenameWithoutAnyExtensions(HotReloadFile.FullName);

				// Duplicate the action
				Action NewAction = new Action(Action);
				Action.Inner = NewAction;

				// Update this action's list of prerequisite items too
				List<FileItem> UpdatePrerequisiteItems = new List<FileItem>(NewAction.PrerequisiteItems);
				for (int ItemIndex = 0; ItemIndex < UpdatePrerequisiteItems.Count; ++ItemIndex)
				{
					FileItem OriginalPrerequisiteItem = UpdatePrerequisiteItems[ItemIndex];
					string NewPrerequisiteItemFilePath = ReplaceBaseFileName(OriginalPrerequisiteItem.AbsolutePath, OriginalFileNameWithoutExtension, NewFileNameWithoutExtension);

					if (OriginalPrerequisiteItem.AbsolutePath != NewPrerequisiteItemFilePath)
					{
						// OK, the prerequisite item's file name changed so we'll update it to point to our new file
						FileItem NewPrerequisiteItem = FileItem.GetItemByPath(NewPrerequisiteItemFilePath);
						UpdatePrerequisiteItems[ItemIndex] = NewPrerequisiteItem;

						// Keep track of it so we can fix up dependencies in a second pass afterwards
						AffectedOriginalFileItemAndNewFileItemMap.Add(OriginalPrerequisiteItem, NewPrerequisiteItem);

						int ResponseExtensionIndex = OriginalPrerequisiteItem.AbsolutePath.IndexOf(ResponseFileExtension, StringComparison.InvariantCultureIgnoreCase);
						if (ResponseExtensionIndex != -1)
						{
							string OriginalResponseFilePathWithoutExtension = OriginalPrerequisiteItem.AbsolutePath.Substring(0, ResponseExtensionIndex);
							string OriginalResponseFilePath = OriginalResponseFilePathWithoutExtension + ResponseFileExtension;

							string NewResponseFilePath = ReplaceBaseFileName(OriginalResponseFilePath, OriginalFileNameWithoutExtension, NewFileNameWithoutExtension);

							OriginalToNewFilePaths.Add(new FileReference(OriginalResponseFilePath), new FileReference(NewResponseFilePath));
						}

						int ScriptExtensionIndex = OriginalPrerequisiteItem.AbsolutePath.IndexOf(ScriptExtension, StringComparison.InvariantCultureIgnoreCase);
						if (ScriptExtensionIndex != -1)
						{
							string OriginalScriptFilePathWithoutExtension = OriginalPrerequisiteItem.AbsolutePath.Substring(0, ScriptExtensionIndex);
							string OriginalScriptFilePath = OriginalScriptFilePathWithoutExtension + ScriptExtension;

							string NewScriptFilePath = ReplaceBaseFileName(OriginalScriptFilePath, OriginalFileNameWithoutExtension, NewFileNameWithoutExtension);

							OriginalToNewFilePaths.Add(new FileReference(OriginalScriptFilePath), new FileReference(NewScriptFilePath));
						}
					}
				}
				NewAction.PrerequisiteItems = new (UpdatePrerequisiteItems, FileItem.SortByPath);

				// Update this action's list of produced items too
				List<FileItem> UpdateProducedItems = new List<FileItem>(NewAction.ProducedItems);
				for (int ItemIndex = 0; ItemIndex < UpdateProducedItems.Count; ++ItemIndex)
				{
					FileItem OriginalProducedItem = UpdateProducedItems[ItemIndex];

					string NewProducedItemFilePath = ReplaceBaseFileName(OriginalProducedItem.AbsolutePath, OriginalFileNameWithoutExtension, NewFileNameWithoutExtension);
					if (OriginalProducedItem.AbsolutePath != NewProducedItemFilePath)
					{
						// OK, the produced item's file name changed so we'll update it to point to our new file
						FileItem NewProducedItem = FileItem.GetItemByPath(NewProducedItemFilePath);
						UpdateProducedItems[ItemIndex] = NewProducedItem;

						// Keep track of it so we can fix up dependencies in a second pass afterwards
						AffectedOriginalFileItemAndNewFileItemMap.Add(OriginalProducedItem, NewProducedItem);
					}
				}
				NewAction.ProducedItems = new (UpdateProducedItems, FileItem.SortByPath);

				// Fix up the list of items to delete too
				List<FileItem> UpdateDeleteItems = new List<FileItem>(NewAction.DeleteItems);
				for (int Idx = 0; Idx < UpdateDeleteItems.Count; Idx++)
				{
					FileItem? NewItem;
					if (AffectedOriginalFileItemAndNewFileItemMap.TryGetValue(UpdateDeleteItems[Idx], out NewItem))
					{
						UpdateDeleteItems[Idx] = NewItem;
					}
				}
				NewAction.DeleteItems = new (UpdateDeleteItems, FileItem.SortByPath);

				// The status description of the item has the file name, so we'll update it too
				NewAction.StatusDescription = ReplaceBaseFileName(Action.StatusDescription, OriginalFileNameWithoutExtension, NewFileNameWithoutExtension);

				// Keep track of the file names, so we can fix up response files afterwards.
				if (!OriginalFileNameAndNewFileNameList_NoExtensions.ContainsKey(OriginalFileNameWithoutExtension))
				{
					OriginalFileNameAndNewFileNameList_NoExtensions[OriginalFileNameWithoutExtension] = NewFileNameWithoutExtension;
				}
				else if (OriginalFileNameAndNewFileNameList_NoExtensions[OriginalFileNameWithoutExtension] != NewFileNameWithoutExtension)
				{
					throw new BuildException("Unexpected conflict in renaming files; {0} maps to {1} and {2}", OriginalFileNameWithoutExtension, OriginalFileNameAndNewFileNameList_NoExtensions[OriginalFileNameWithoutExtension], NewFileNameWithoutExtension);
				}
			}

			// Do another pass and update any actions that depended on the original file names that we changed
			foreach (LinkedAction Action in Actions)
			{
				Action NewAction = new Action(Action.Inner);
				List<FileItem> UpdatePrerequisiteItems = new List<FileItem>(NewAction.PrerequisiteItems);
				for (int ItemIndex = 0; ItemIndex < UpdatePrerequisiteItems.Count; ++ItemIndex)
				{
					FileItem OriginalFileItem = UpdatePrerequisiteItems[ItemIndex];

					FileItem? NewFileItem;
					if (AffectedOriginalFileItemAndNewFileItemMap.TryGetValue(OriginalFileItem, out NewFileItem))
					{
						// OK, looks like we need to replace this file item because we've renamed the file
						UpdatePrerequisiteItems[ItemIndex] = NewFileItem;
					}
				}
				NewAction.PrerequisiteItems = new (UpdatePrerequisiteItems, FileItem.SortByPath);
				Action.Inner = NewAction;
			}

			if (OriginalFileNameAndNewFileNameList_NoExtensions.Count > 0)
			{
				// Update all the paths in link actions
				foreach (LinkedAction Action in Actions.Where((Action) => Action.ActionType == ActionType.Link))
				{
					foreach (KeyValuePair<string, string> FileNameTuple in OriginalFileNameAndNewFileNameList_NoExtensions)
					{
						string OriginalFileNameWithoutExtension = FileNameTuple.Key;
						string NewFileNameWithoutExtension = FileNameTuple.Value;

						Action NewAction = new Action(Action.Inner);
						NewAction.CommandArguments = ReplaceBaseFileName(Action.CommandArguments, OriginalFileNameWithoutExtension, NewFileNameWithoutExtension);
						Action.Inner = NewAction;
					}
				}

				foreach (KeyValuePair<FileReference, FileReference> Item in OriginalToNewFilePaths)
				{
					// Load the file up
					string FileContents = FileReference.ReadAllText(Item.Key);

					// Replace all of the old file names with new ones
					foreach (KeyValuePair<string, string> FileNameTuple in OriginalFileNameAndNewFileNameList_NoExtensions)
					{
						string OriginalFileNameWithoutExtension = FileNameTuple.Key;
						string NewFileNameWithoutExtension = FileNameTuple.Value;

						FileContents = ReplaceBaseFileName(FileContents, OriginalFileNameWithoutExtension, NewFileNameWithoutExtension);
					}

					// Write out the updated file location
					FileReference.WriteAllText(Item.Value, FileContents, new System.Text.UTF8Encoding(false));
				}
			}

			// Update the action that writes out the module manifests
			foreach (LinkedAction Action in Actions)
			{
				if (Action.ActionType == ActionType.WriteMetadata)
				{
					string Arguments = Action.CommandArguments;

					// Find the argument for the metadata file
					const string InputArgument = "-Input=";

					int InputIdx = Arguments.IndexOf(InputArgument, StringComparison.Ordinal);
					if (InputIdx == -1)
					{
						throw new Exception("Missing -Input= argument to WriteMetadata command when patching action graph.");
					}

					int FileNameIdx = InputIdx + InputArgument.Length;
					if (Arguments[FileNameIdx] == '\"')
					{
						FileNameIdx++;
					}

					int FileNameEndIdx = FileNameIdx;
					while (FileNameEndIdx < Arguments.Length && (Arguments[FileNameEndIdx] != ' ' || Arguments[FileNameIdx - 1] == '\"') && Arguments[FileNameEndIdx] != '\"')
					{
						FileNameEndIdx++;
					}

					// Read the metadata file
					FileReference TargetInfoFile = new FileReference(Arguments.Substring(FileNameIdx, FileNameEndIdx - FileNameIdx));
					if (!FileReference.Exists(TargetInfoFile))
					{
						throw new Exception(String.Format("Unable to find metadata file to patch action graph ({0})", TargetInfoFile));
					}
					WriteMetadataTargetInfo TargetInfo;
					using (FileStream FileStream = FileReference.Open(TargetInfoFile, FileMode.Open, FileAccess.Read, FileShare.Read))
					{
						WriteMetadataTargetInfo? MaybeTargetInfo = JsonSerializer.Deserialize<WriteMetadataTargetInfo>(FileStream, WriteMetadataMode.SerializerOptions);
						TargetInfo = MaybeTargetInfo ?? throw new BuildLogEventException("Target info file '{FilePath}' was invalid", TargetInfoFile);
					}

					// Update the module names
					bool bHasUpdatedModuleNames = false;
					foreach (KeyValuePair<FileReference, ModuleManifest> FileNameToVersionManifest in TargetInfo.FileToManifest)
					{
						KeyValuePair<string, string>[] ManifestEntries = FileNameToVersionManifest.Value.ModuleNameToFileName.ToArray();
						foreach (KeyValuePair<string, string> Manifest in ManifestEntries)
						{
							FileReference OriginalFile = FileReference.Combine(FileNameToVersionManifest.Key.Directory, Manifest.Value);

							FileReference? HotReloadFile;
							if (OriginalFileToHotReloadFile.TryGetValue(OriginalFile, out HotReloadFile))
							{
								FileNameToVersionManifest.Value.ModuleNameToFileName[Manifest.Key] = HotReloadFile.GetFileName();
								bHasUpdatedModuleNames = true;
							}
						}
					}

					// Write the hot-reload metadata file and update the argument list
					if (bHasUpdatedModuleNames)
					{
						FileReference HotReloadTargetInfoFile = FileReference.Combine(TargetInfoFile.Directory, "Metadata-HotReload.json");
						byte[] InfoBytes = JsonSerializer.SerializeToUtf8Bytes(TargetInfo, WriteMetadataMode.SerializerOptions);
						DirectoryReference.CreateDirectory(HotReloadTargetInfoFile.Directory);
						FileReference.WriteAllBytesIfDifferent(HotReloadTargetInfoFile, InfoBytes);

						Action NewAction = new Action(Action.Inner);

						NewAction.PrerequisiteItems.RemoveWhere(x => x.Location == TargetInfoFile);
						NewAction.PrerequisiteItems.Add(FileItem.GetItemByFileReference(HotReloadTargetInfoFile));

						NewAction.CommandArguments = Arguments.Substring(0, FileNameIdx) + HotReloadTargetInfoFile + Arguments.Substring(FileNameEndIdx);

						Action.Inner = NewAction;
					}
				}
			}

			Dictionary<FileReference, FileReference> PatchedOldLocationToNewLocation = new Dictionary<FileReference, FileReference>();
			foreach (KeyValuePair<FileItem, FileItem> Item in AffectedOriginalFileItemAndNewFileItemMap)
			{
				PatchedOldLocationToNewLocation.Add(Item.Key.Location, Item.Value.Location);
			}
			return PatchedOldLocationToNewLocation;
		}

		/// <summary>
		/// Patches a set of actions to use a specific list of suffixes for each module name
		/// </summary>
		/// <param name="ModuleNameToSuffix">Map of module name to suffix</param>
		/// <param name="Makefile">Makefile for the target being built</param>
		/// <param name="Actions">Actions to be executed for this makefile</param>
		/// <returns>Collection of file names patched.  Can be null.</returns>
		public static Dictionary<FileReference, FileReference>? PatchActionGraphWithNames(Dictionary<string, int> ModuleNameToSuffix, TargetMakefile Makefile, List<LinkedAction> Actions)
		{
			Dictionary<FileReference, FileReference>? PatchedOldLocationToNewLocation = null;
			if (ModuleNameToSuffix.Count > 0)
			{
				Dictionary<FileReference, FileReference> OldLocationToNewLocation = new Dictionary<FileReference, FileReference>();
				foreach (string HotReloadModuleName in Makefile.HotReloadModuleNames)
				{
					int ModuleSuffix;
					if (ModuleNameToSuffix.TryGetValue(HotReloadModuleName, out ModuleSuffix))
					{
						FileItem[] ModuleOutputItems = Makefile.ModuleNameToOutputItems[HotReloadModuleName];
						foreach (FileItem ModuleOutputItem in ModuleOutputItems)
						{
							FileReference OldLocation = ModuleOutputItem.Location;
							FileReference NewLocation = HotReload.ReplaceSuffix(OldLocation, ModuleSuffix);
							OldLocationToNewLocation[OldLocation] = NewLocation;
						}
					}
				}
				PatchedOldLocationToNewLocation = HotReload.PatchActionGraph(Actions, OldLocationToNewLocation);
			}
			return PatchedOldLocationToNewLocation;
		}

		/// <summary>
		/// Writes a manifest containing all the information needed to create a live coding patch
		/// </summary>
		/// <param name="ManifestFile">File to write to</param>
		/// <param name="Actions">List of actions that are part of the graph</param>
		/// <param name="OriginalFileToPatchedFile">Map of original object files to patched object files</param>
		public static void WriteLiveCodingManifest(FileReference ManifestFile, List<IExternalAction> Actions, Dictionary<FileReference, FileReference> OriginalFileToPatchedFile)
		{
			// Find all the output object files
			HashSet<FileItem> ObjectFiles = new HashSet<FileItem>();
			foreach (IExternalAction Action in Actions)
			{
				if (Action.ActionType == ActionType.Compile)
				{
					ObjectFiles.UnionWith(Action.ProducedItems.Where(x => x.HasExtension(".obj")));
				}
			}

			// Write the output manifest
			using (JsonWriter Writer = new JsonWriter(ManifestFile))
			{
				Writer.WriteObjectStart();

				IExternalAction? LinkAction = Actions.FirstOrDefault(x => x.ActionType == ActionType.Link && x.ProducedItems.Any(y => y.HasExtension(".exe") || y.HasExtension(".dll")));
				if (LinkAction != null)
				{
					Writer.WriteValue("LinkerPath", LinkAction.CommandPath.FullName);
				}

				Writer.WriteObjectStart("LinkerEnvironment");
				foreach (Nullable<System.Collections.DictionaryEntry> Entry in Environment.GetEnvironmentVariables())
				{
					if (Entry.HasValue)
					{
						Writer.WriteValue(Entry.Value.Key.ToString()!, Entry.Value.Value!.ToString());
					}
				}
				Writer.WriteObjectEnd();

				HashSet<DirectoryReference> vfsPathsAdded = new();
				List<(DirectoryReference virtualPath, DirectoryReference localPath)> vfsPaths = new();

				Writer.WriteArrayStart("Modules");
				foreach (IExternalAction Action in Actions)
				{
					if (Action.ActionType == ActionType.Link)
					{
						string GetPath(FileReference fileRef)
						{
							if (Action.RootPaths.GetVfsOverlayPath(fileRef, out string? VirtualPath))
							{
								return VirtualPath.Replace('/', '\\');
							}
							else
							{
								return fileRef.FullName;
							}
						}

						FileItem? OutputFile = Action.ProducedItems.FirstOrDefault(x => (x.HasExtension(".exe") || x.HasExtension(".dll")) && !x.HasExtension(".alt.exe") && !x.HasExtension(".alt.dll"));
						if (OutputFile != null && Action.PrerequisiteItems.Any(x => OriginalFileToPatchedFile.ContainsKey(x.Location)))
						{
							Writer.WriteObjectStart();
							Writer.WriteValue("Output", GetPath(OutputFile.Location));

							Writer.WriteArrayStart("Inputs");
							foreach (FileItem InputFile in Action.PrerequisiteItems)
							{
								FileReference? PatchedFile;
								if (OriginalFileToPatchedFile.TryGetValue(InputFile.Location, out PatchedFile))
								{
									Writer.WriteValue(GetPath(PatchedFile));
								}
							}
							Writer.WriteArrayEnd();

							Writer.WriteArrayStart("Libraries");
							foreach (FileItem InputFile in Action.PrerequisiteItems)
							{
								if (InputFile.HasExtension(".lib"))
								{
									Writer.WriteValue(GetPath(InputFile.Location));
								}
							}
							Writer.WriteArrayEnd();

							Writer.WriteObjectEnd();

							if (Action.RootPaths.bUseVfs)
							{
								foreach ((uint id, DirectoryReference vfs, DirectoryReference local) RootPath in Action.RootPaths)
								{
									if (vfsPathsAdded.Add(RootPath.vfs))
									{
										vfsPaths.Add((RootPath.vfs, RootPath.local));
									}
								}
							}
						}
					}
				}
				Writer.WriteArrayEnd();

				Writer.WriteArrayStart("Vfs");
				foreach ((DirectoryReference virtualPath, DirectoryReference localPath) kv in vfsPaths)
				{
					Writer.WriteObjectStart();
					Writer.WriteValue("Virtual", kv.virtualPath.FullName);
					Writer.WriteValue("Local", kv.localPath.FullName);
					Writer.WriteObjectEnd();
				}
				Writer.WriteArrayEnd();

				Writer.WriteObjectEnd();
			}
		}
	}
}
