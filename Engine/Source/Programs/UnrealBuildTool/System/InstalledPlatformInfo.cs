// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using EpicGames.Core;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// The project type that support is required for
	/// </summary>
	public enum EProjectType
	{
		/// <summary>
		/// 
		/// </summary>
		Unknown,

		/// <summary>
		/// 
		/// </summary>
		Any,

		/// <summary>
		/// Support for code projects
		/// </summary>
		Code,

		/// <summary>
		/// Support for deploying content projects
		/// </summary>
		Content,
	};

	/// <summary>
	/// The state of a downloaded platform
	/// </summary>
	public enum InstalledPlatformState
	{
		/// <summary>
		/// Query whether the platform is supported
		/// </summary>
		Supported,

		/// <summary>
		/// Query whether the platform has been downloaded
		/// </summary>
		Downloaded,
	}

	/// <summary>
	/// Contains methods to allow querying the available installed platforms
	/// </summary>
	public static class InstalledPlatformInfo
	{
#pragma warning disable CA1034 // Nested types should not be visible - preserve API of APIs exposed to UAT
		/// <summary>
		/// Information about a single installed platform configuration
		/// </summary>
		public struct InstalledPlatformConfiguration
		{
			/// <summary>
			/// Build Configuration of this combination
			/// </summary>
			public UnrealTargetConfiguration Configuration;

			/// <summary>
			/// Platform for this combination
			/// </summary>
			public UnrealTargetPlatform Platform;

			/// <summary>
			/// Type of Platform for this combination
			/// </summary>
			public TargetType PlatformType;

			/// <summary>
			/// Architecture for this combination
			/// </summary>
			public UnrealArch Architecture;

			/// <summary>
			/// Location of a file that must exist for this combination to be valid (optional)
			/// </summary>
			public string RequiredFile;

			/// <summary>
			/// Type of project this configuration can be used for
			/// </summary>
			public EProjectType ProjectType;

			/// <summary>
			/// Whether to display this platform as an option even if it is not valid
			/// </summary>
			public bool bCanBeDisplayed;

			/// <summary>
			/// Constructor
			/// </summary>
			/// <param name="InConfiguration"></param>
			/// <param name="InPlatform"></param>
			/// <param name="InPlatformType"></param>
			/// <param name="InArchitecture"></param>
			/// <param name="InRequiredFile"></param>
			/// <param name="InProjectType"></param>
			/// <param name="bInCanBeDisplayed"></param>
			public InstalledPlatformConfiguration(UnrealTargetConfiguration InConfiguration, UnrealTargetPlatform InPlatform, TargetType InPlatformType, UnrealArch InArchitecture, string InRequiredFile, EProjectType InProjectType, bool bInCanBeDisplayed)
			{
				Configuration = InConfiguration;
				Platform = InPlatform;
				PlatformType = InPlatformType;
				Architecture = InArchitecture;
				RequiredFile = InRequiredFile;
				ProjectType = InProjectType;
				bCanBeDisplayed = bInCanBeDisplayed;
			}
		}
#pragma warning restore CA1034

		private static readonly List<InstalledPlatformConfiguration>? InstalledPlatformConfigurations;

#pragma warning disable CA1810 // Initialize reference type static fields inline
		static InstalledPlatformInfo()
		{
			List<string>? InstalledPlatforms;
			ConfigHierarchy Ini = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, (DirectoryReference?)null, BuildHostPlatform.Current.Platform);

			bool bHasInstalledPlatformInfo;
			if (Ini.TryGetValue("InstalledPlatforms", "HasInstalledPlatformInfo", out bHasInstalledPlatformInfo) && bHasInstalledPlatformInfo)
			{
				InstalledPlatformConfigurations = new List<InstalledPlatformConfiguration>();
				if (Ini.GetArray("InstalledPlatforms", "InstalledPlatformConfigurations", out InstalledPlatforms))
				{
					foreach (string InstalledPlatform in InstalledPlatforms)
					{
						ParsePlatformConfiguration(InstalledPlatform);
					}
				}
			}
		}
#pragma warning restore CA1810

		/// <summary>
		/// Initializes the InstalledPlatformInfo. While this is not necessary to be called, it allows timing the static constructor call.
		/// </summary>
		internal static void Initialize()
		{
			// Unused, but allows timing call of the static constructor
		}

		private static void ParsePlatformConfiguration(string PlatformConfiguration)//, ILogger Logger)
		{
			// Trim whitespace at the beginning.
			PlatformConfiguration = PlatformConfiguration.Trim();
			// Remove brackets.
			PlatformConfiguration = PlatformConfiguration.TrimStart('(');
			PlatformConfiguration = PlatformConfiguration.TrimEnd(')');

			bool bCanCreateEntry = true;

			string ConfigurationName;
			UnrealTargetConfiguration Configuration = UnrealTargetConfiguration.Unknown;
			if (ParseSubValue(PlatformConfiguration, "Configuration=", out ConfigurationName))
			{
				_ = Enum.TryParse(ConfigurationName, out Configuration);
			}
			if (Configuration == UnrealTargetConfiguration.Unknown)
			{
				//				Logger.LogWarning("Unable to read configuration from {PlatformConfiguration}", PlatformConfiguration);
				bCanCreateEntry = false;
			}

			string PlatformName;
			if (ParseSubValue(PlatformConfiguration, "PlatformName=", out PlatformName))
			{
				if (!UnrealTargetPlatform.IsValidName(PlatformName))
				{
					//					Logger.LogWarning("Unable to read platform from {PlatformConfiguration}", PlatformConfiguration);
					bCanCreateEntry = false;
				}
			}

			string PlatformTypeName;
			TargetType PlatformType = TargetType.Game;
			if (ParseSubValue(PlatformConfiguration, "PlatformType=", out PlatformTypeName))
			{
				if (!Enum.TryParse(PlatformTypeName, out PlatformType))
				{
					//					Logger.LogWarning("Unable to read Platform Type from {PlatformConfiguration}, defaulting to Game", PlatformConfiguration);
					PlatformType = TargetType.Game;
				}
			}
			if (PlatformType == TargetType.Program)
			{
				//				Logger.LogWarning("Program is not a valid PlatformType for an Installed Platform, defaulting to Game");
				PlatformType = TargetType.Game;
			}

			string ArchitectureString;
			ParseSubValue(PlatformConfiguration, "Architecture=", out ArchitectureString);
			UnrealArch Architecture = UnrealArch.Parse(ArchitectureString);

			string RequiredFile;
			if (ParseSubValue(PlatformConfiguration, "RequiredFile=", out RequiredFile))
			{
				RequiredFile = FileReference.Combine(Unreal.RootDirectory, RequiredFile).ToNormalizedPath();
			}

			string ProjectTypeName;
			EProjectType ProjectType = EProjectType.Any;
			if (ParseSubValue(PlatformConfiguration, "ProjectType=", out ProjectTypeName))
			{
				_ = Enum.TryParse(ProjectTypeName, out ProjectType);
			}
			if (ProjectType == EProjectType.Unknown)
			{
				//				Logger.LogWarning("Unable to read project type from {PlatformConfiguration}", PlatformConfiguration);
				bCanCreateEntry = false;
			}

			string CanBeDisplayedString;
			bool bCanBeDisplayed = false;
			if (ParseSubValue(PlatformConfiguration, "bCanBeDisplayed=", out CanBeDisplayedString))
			{
				bCanBeDisplayed = Convert.ToBoolean(CanBeDisplayedString);
			}

			if (bCanCreateEntry)
			{
				InstalledPlatformConfigurations!.Add(new InstalledPlatformConfiguration(Configuration, UnrealTargetPlatform.Parse(PlatformName), PlatformType, Architecture, RequiredFile, ProjectType, bCanBeDisplayed));
			}
		}

		private static bool ParseSubValue(string TrimmedLine, string Match, out string Result)
		{
			Result = String.Empty;
			int MatchIndex = TrimmedLine.IndexOf(Match, StringComparison.Ordinal);
			if (MatchIndex < 0)
			{
				return false;
			}
			// Get the remainder of the string after the match
			MatchIndex += Match.Length;
			TrimmedLine = TrimmedLine.Substring(MatchIndex);
			if (String.IsNullOrEmpty(TrimmedLine))
			{
				return false;
			}
			// get everything up to the first comma and trim any new whitespace
			Result = TrimmedLine.Split(',')[0];
			Result = Result.Trim();
			if (Result.StartsWith('\"'))
			{
				// Remove quotes
				int QuoteEnd = Result.LastIndexOf('\"');
				if (QuoteEnd > 0)
				{
					Result = Result.Substring(1, QuoteEnd - 1);
				}
			}
			return true;
		}

		/// <summary>
		/// Determine if the given configuration is available for any platform
		/// </summary>
		/// <param name="Configuration">Configuration type to check</param>
		/// <param name="ProjectType">The type of project</param>
		/// <returns>True if supported</returns>
		public static bool IsValidConfiguration(UnrealTargetConfiguration Configuration, EProjectType ProjectType = EProjectType.Any)
		{
			return ContainsValidConfiguration(
				(InstalledPlatformConfiguration CurConfig) =>
				{
					return CurConfig.Configuration == Configuration
						&& (ProjectType == EProjectType.Any || CurConfig.ProjectType == EProjectType.Any
						|| CurConfig.ProjectType == ProjectType);
				}
			);
		}

		/// <summary>
		/// Determine if the given platform is available
		/// </summary>
		/// <param name="Platform">Platform to check</param>
		/// <param name="ProjectType">The type of project</param>
		/// <returns>True if supported</returns>
		public static bool IsValidPlatform(UnrealTargetPlatform Platform, EProjectType ProjectType = EProjectType.Any)
		{
			// HACK: For installed builds, we always need to treat Mac as a valid platform for generating project files. When remote building from PC, we won't have all the libraries to do this, so we need to fake it.
			if (Platform == UnrealTargetPlatform.Mac && ProjectType == EProjectType.Any && BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Mac && Unreal.IsEngineInstalled())
			{
				return true;
			}

			return ContainsValidConfiguration(
				(InstalledPlatformConfiguration CurConfig) =>
				{
					return CurConfig.Platform == Platform
						&& (ProjectType == EProjectType.Any || CurConfig.ProjectType == EProjectType.Any
						|| CurConfig.ProjectType == ProjectType);
				}
			);
		}

		/// <summary>
		/// Determine whether the given platform/configuration/project type combination is supported
		/// </summary>
		/// <param name="Configuration">Configuration for the project</param>
		/// <param name="Platform">Platform for the project</param>
		/// <param name="ProjectType">Type of the project</param>
		/// <returns>True if the combination is supported</returns>
		public static bool IsValidPlatformAndConfiguration(UnrealTargetConfiguration Configuration, UnrealTargetPlatform Platform, EProjectType ProjectType = EProjectType.Any)
		{
			return ContainsValidConfiguration(
				(InstalledPlatformConfiguration CurConfig) =>
				{
					return CurConfig.Configuration == Configuration && CurConfig.Platform == Platform
						&& (ProjectType == EProjectType.Any || CurConfig.ProjectType == EProjectType.Any
						|| CurConfig.ProjectType == ProjectType);
				}
			);
		}

		/// <summary>
		/// Determines whether the given target type is supported
		/// </summary>
		/// <param name="TargetType">The target type being built</param>
		/// <param name="Platform">The platform being built</param>
		/// <param name="Configuration">The configuration being built</param>
		/// <param name="ProjectType">The project type required</param>
		/// <param name="State">State of the given platform support</param>
		/// <returns>True if the target can be built</returns>
		public static bool IsValid(TargetType? TargetType, UnrealTargetPlatform? Platform, UnrealTargetConfiguration? Configuration, EProjectType ProjectType, InstalledPlatformState State)
		{
			if (!Unreal.IsEngineInstalled() || InstalledPlatformConfigurations == null)
			{
				return true;
			}

			foreach (InstalledPlatformConfiguration Config in InstalledPlatformConfigurations)
			{
				// Check whether this configuration matches all the criteria
				if (TargetType.HasValue && Config.PlatformType != TargetType.Value)
				{
					continue;
				}
				if (Platform.HasValue && Config.Platform != Platform.Value)
				{
					continue;
				}
				if (Configuration.HasValue && Config.Configuration != Configuration.Value)
				{
					continue;
				}
				if (ProjectType != EProjectType.Any && Config.ProjectType != EProjectType.Any && Config.ProjectType != ProjectType)
				{
					continue;
				}
				if (State == InstalledPlatformState.Downloaded && !String.IsNullOrEmpty(Config.RequiredFile) && !File.Exists(Config.RequiredFile))
				{
					continue;
				}

				// Success!
				return true;
			}

			return false;
		}

		private static bool ContainsValidConfiguration(Predicate<InstalledPlatformConfiguration> ConfigFilter)
		{
			if (Unreal.IsEngineInstalled() && InstalledPlatformConfigurations != null)
			{
				foreach (InstalledPlatformConfiguration PlatformConfiguration in InstalledPlatformConfigurations)
				{
					// Check whether filter accepts this configuration and it has required file
					if (ConfigFilter(PlatformConfiguration)
					&& (String.IsNullOrEmpty(PlatformConfiguration.RequiredFile)
					|| File.Exists(PlatformConfiguration.RequiredFile)))
					{
						return true;
					}
				}

				return false;
			}
			return true;
		}

		/// <summary>
		/// 
		/// </summary>
		/// <param name="Configs"></param>
		/// <param name="OutEntries"></param>
		public static void WriteConfigFileEntries(List<InstalledPlatformConfiguration> Configs, List<string> OutEntries)
		{
			// Write config section header
			OutEntries.Add("[InstalledPlatforms]");
			OutEntries.Add("HasInstalledPlatformInfo=true");

			foreach (InstalledPlatformConfiguration Config in Configs)
			{
				WriteConfigFileEntry(Config, OutEntries);
			}
		}

		private static void WriteConfigFileEntry(InstalledPlatformConfiguration Config, List<string> OutEntries)
		{
			string ConfigDescription = "+InstalledPlatformConfigurations=(";
			ConfigDescription += String.Format("PlatformName=\"{0}\", ", Config.Platform.ToString());
			if (Config.Configuration != UnrealTargetConfiguration.Unknown)
			{
				ConfigDescription += String.Format("Configuration=\"{0}\", ", Config.Configuration.ToString());
			}
			if (Config.PlatformType != TargetType.Program)
			{
				ConfigDescription += String.Format("PlatformType=\"{0}\", ", Config.PlatformType.ToString());
			}
			ConfigDescription += String.Format("Architecture=\"{0}\", ", Config.Architecture);
			if (!String.IsNullOrEmpty(Config.RequiredFile))
			{
				ConfigDescription += String.Format("RequiredFile=\"{0}\", ", Config.RequiredFile);
			}
			if (Config.ProjectType != EProjectType.Unknown)
			{
				ConfigDescription += String.Format("ProjectType=\"{0}\", ", Config.ProjectType.ToString());
			}
			ConfigDescription += String.Format("bCanBeDisplayed={0})", Config.bCanBeDisplayed.ToString());

			OutEntries.Add(ConfigDescription);
		}
	}
}
