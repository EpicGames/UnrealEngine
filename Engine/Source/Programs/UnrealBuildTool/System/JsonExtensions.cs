// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics.CodeAnalysis;
using EpicGames.Core;
using UnrealBuildBase;

namespace JsonExtensions
{
	/// <summary>
	/// Extension methods for JsonObject to provide some deprecated field helpers
	/// </summary>
	public static class DeprecatedFieldHelpers
	{
		/// <summary>
		/// Tries to read a string array field by the given name from the object; if that's not found, checks for an older deprecated name
		/// </summary>
		/// <param name="Obj">JSON object to check</param>
		/// <param name="FieldName">Name of the field to get</param>
		/// <param name="DeprecatedFieldName">Backup field name to check</param>
		/// <param name="Result">On success, receives the field value</param>
		/// <returns>True if the field could be read, false otherwise</returns>
		public static bool TryGetStringArrayFieldWithDeprecatedFallback(this JsonObject Obj, string FieldName, string DeprecatedFieldName, [NotNullWhen(true)] out string[]? Result)
		{
			if (Obj.TryGetStringArrayField(FieldName, out Result))
			{
				return true;
			}
			else if (Obj.TryGetStringArrayField(DeprecatedFieldName, out Result))
			{
				//@TODO: Warn here?
				return true;
			}
			else
			{
				return false;
			}
		}

		/// <summary>
		/// Reads an enum array field by the given name from the object; if that's not found, checks for an older deprecated name.
		/// Throws if any invalid values are found.
		/// </summary>
		/// <param name="Obj">JSON object to check</param>
		/// <param name="FieldName">Name of the field to get</param>
		/// <param name="DeprecatedFieldName">Backup field name to check</param>
		/// <returns>A parsed T[] if one exists, or null if the field is absent</returns>
		/// <exception cref="BuildLogEventException">Thrown if invalid values are found.</exception>
		public static T[]? GetEnumArrayFieldWithDeprecatedFallback<T>(this JsonObject Obj, string FieldName, string DeprecatedFieldName) where T : struct
		{
			T[]? Result;
			string? Errors;

			if (!Obj.ContainsField(FieldName) && !Obj.ContainsField(DeprecatedFieldName))
			{
				return null;
			}

			if (Obj.ContainsField(FieldName) && Obj.ContainsField(DeprecatedFieldName))
			{
				throw new BuildLogEventException("Field '{DeprecatedFieldName}' must be removed, as '{FieldName}' is already defined", DeprecatedFieldName, FieldName);
			}

			if (Obj.ContainsField(FieldName))
			{
				if (Obj.TryGetEnumArrayField(FieldName, out Result, out Errors))
				{
					return Result;
				}

				throw new BuildLogEventException("{JsonErrors}", Errors);
			}

			if (Obj.TryGetEnumArrayField(DeprecatedFieldName, out Result, out Errors))
			{
				//@TODO: Warn here?
				return Result;
			}

			throw new BuildLogEventException("{JsonErrors}", Errors);
		}
	}
}
