// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	internal static class JunkDeleter
	{
		/// <summary>
		/// Loads JunkManifest.txt file and removes all junk files/folders defined in it.
		/// </summary>
		/// <remarks>
		/// JunkManifest.txt is a manually edited file, changed when new things should be deleted.
		/// </remarks>
		public static void DeleteJunk(ILogger logger)
		{
			List<string> junkManifest = LoadJunkManifest();
			DeleteAllJunk(junkManifest, logger);
		}

		/// <summary>
		/// Loads JunkManifest.txt file.
		/// </summary>
		/// <returns>Junk manifest file contents.</returns>
		private static List<string> LoadJunkManifest()
		{
			string manifestPath = Path.Combine(Unreal.EngineDirectory.FullName, "Build", "JunkManifest.txt");
			if (!File.Exists(manifestPath) || new FileInfo(manifestPath).Length == 0)
			{
				return [];
			}

			List<string> junkManifest = [];
			string machineName = Unreal.MachineName;
			using (StreamReader reader = new StreamReader(manifestPath))
			{
				string? lineRead;
				// Only three ranges, as we support 'Machine=', 'Platform=' and the path.
				Span<Range> tokenRanges = stackalloc Range[3];
				Span<Range> innerTokenRanges = stackalloc Range[2];

				while ((lineRead = reader.ReadLine()) != null)
				{
					ReadOnlySpan<char> junkEntry = lineRead.AsSpan().Trim();
					if (junkEntry.Length > 0)
					{
						int numSplits = junkEntry.Split(tokenRanges, ':');
						if (numSplits > 3)
						{
							continue;
						}

						bool isValidJunkLine = true;

						foreach (Range range in tokenRanges[..numSplits])
						{
							ReadOnlySpan<char> token = junkEntry[range];
							if (token.StartsWith("Machine=", StringComparison.OrdinalIgnoreCase))
							{
								int numInnerSplits = token.Split(innerTokenRanges, '=');
								// check if the machine name on the line matches the current machine name, if not, we don't apply this junk
								if (numInnerSplits == 2 && !machineName.AsSpan().StartsWith(token[innerTokenRanges[1]]))
								{
									// Not meant for this machine
									isValidJunkLine = false;
									break;
								}
							}
							else if (token.StartsWith("Platform=", StringComparison.OrdinalIgnoreCase))
							{
								int numInnerSplits = token.Split(innerTokenRanges, '=');
								// if the platform is valid, then we want to keep the files, which means that we don't want to apply the junk line
								if (numInnerSplits == 2
									&& UnrealTargetPlatform.TryParse(token[innerTokenRanges[1]].ToString(), out UnrealTargetPlatform parsedPlatform)
									&& UEBuildPlatform.TryGetBuildPlatform(parsedPlatform, out _))
								{
									// this is a good platform, so don't delete any files!
									isValidJunkLine = false;
									break;
								}
							}
						}

						if (isValidJunkLine)
						{
							ReadOnlySpan<char> path = junkEntry[tokenRanges[numSplits - 1]];
							string pathString = path.Length == lineRead.Length ? lineRead : path.ToString();
							// the entry is always the last element in the token array (after the final :)
							string fixedPath = Path.Combine(Unreal.RootDirectory.FullName, pathString);
							fixedPath = fixedPath.Replace('\\', Path.DirectorySeparatorChar);
							junkManifest.Add(fixedPath);
						}
					}
				}
			}
			return junkManifest;
		}

		/// <summary>
		/// Goes through each entry from the junk manifest and deletes it.
		/// </summary>
		/// <param name="junkManifest">JunkManifest.txt entries.</param>
		/// <param name="logger">Logger for output</param>
		private static void DeleteAllJunk(List<string> junkManifest, ILogger logger)
		{
			foreach (string junk in junkManifest)
			{
				if (IsFile(junk))
				{
					string fileName = Path.GetFileName(junk);
					if (fileName.Contains('*', StringComparison.Ordinal))
					{
						// Wildcard search and delete
						string directoryToLookIn = Path.GetDirectoryName(junk)!;
						if (Directory.Exists(directoryToLookIn))
						{
							// Delete all files within the specified folder
							string[] filesToDelete = Directory.GetFiles(directoryToLookIn, fileName, SearchOption.TopDirectoryOnly);
							foreach (string junkFile in filesToDelete)
							{
								DeleteFile(junkFile, logger);
							}

							// Delete all subdirectories with the specified folder
							string[] directoriesToDelete = Directory.GetDirectories(directoryToLookIn, fileName, SearchOption.TopDirectoryOnly);
							foreach (string junkFolder in directoriesToDelete)
							{
								DeleteDirectory(junkFolder, logger);
							}
						}
					}
					else
					{
						DeleteFile(junk, logger);
					}
				}
				else if (Directory.Exists(junk))
				{
					DeleteDirectory(junk, logger);
				}
			}
		}

		private static bool IsFile(string pathToCheck)
		{
			string fileName = Path.GetFileName(pathToCheck);
			if (String.IsNullOrEmpty(fileName) == false)
			{
				if (fileName.Contains('*', StringComparison.Ordinal))
				{
					// Assume wildcards are file because the path will be searched for files and directories anyway.
					return true;
				}
				else
				{
					return File.Exists(pathToCheck);
				}
			}
			else
			{
				return false;
			}
		}

		/// <summary>
		/// Deletes a directory recursively gracefully handling all exceptions.
		/// </summary>
		/// <param name="directoryPath">Path.</param>
		/// <param name="logger">Logger for output</param>
		private static void DeleteDirectory(string directoryPath, ILogger logger)
		{
			try
			{
				logger.LogInformation("Deleting junk directory: \"{Dir}\".", directoryPath);
				Directory.Delete(directoryPath, true);
			}
			catch (Exception ex)
			{
				logger.LogInformation("Unable to delete junk directory: \"{Dir}\". Error: {Ex}", directoryPath, ex.Message.TrimEnd());
			}
		}

		/// <summary>
		/// Deletes a file gracefully handling all exceptions.
		/// </summary>
		/// <param name="filename">Filename.</param>
		/// <param name="logger">Logger for output</param>
		private static void DeleteFile(string filename, ILogger logger)
		{
			try
			{
				logger.LogInformation("Deleting junk file: \"{File}\".", filename);
				File.Delete(filename);
			}
			catch (Exception ex)
			{
				logger.LogInformation("Unable to delete junk file: \"{File}\". Error: {Ex}", filename, ex.Message.TrimEnd());
			}
		}
	}
}
