// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using EpicGames.Core;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// Encapsulates the environment that is used to link object files.
	/// </summary>
	class LinkEnvironment
	{
		/// <summary>
		/// The platform to be compiled/linked for.
		/// </summary>
		public readonly UnrealTargetPlatform Platform;

		/// <summary>
		/// The configuration to be compiled/linked for.
		/// </summary>
		public readonly CppConfiguration Configuration;

		/// <summary>
		/// The architecture that is being compiled/linked (empty string by default)
		/// </summary>
		public UnrealArchitectures Architectures;

		/// <summary>
		/// Gets the Architecture in the normal case where there is a single Architecture in Architectures
		/// </summary>
		public UnrealArch Architecture => Architectures.SingleArchitecture;

		/// <summary>
		/// Root paths for this environment. Used for Clang VFS and UBA
		/// </summary>
		public readonly CppRootPaths RootPaths;

		/// <summary>
		/// On Mac, indicates the path to the target's application bundle
		/// </summary>
		public DirectoryReference? BundleDirectory;

		/// <summary>
		/// The directory to put the non-executable files in (PDBs, import library, etc)
		/// </summary>
		public DirectoryReference? OutputDirectory;

		/// <summary>
		/// Intermediate file directory
		/// </summary>
		public DirectoryReference? IntermediateDirectory;

		/// <summary>
		/// The directory to shadow source files in for syncing to remote compile servers
		/// </summary>
		public DirectoryReference? LocalShadowDirectory = null;

		/// <summary>
		/// The file path for the executable file that is output by the linker.
		/// </summary>
		public List<FileReference> OutputFilePaths = [];

		/// <summary>
		/// Returns the OutputFilePath is there is only one entry in OutputFilePaths
		/// </summary>
		public FileReference OutputFilePath =>
			OutputFilePaths.Count != 1
				? throw new BuildException("Attempted to use LinkEnvironmentConfiguration.OutputFilePath property, but there are multiple (or no) OutputFilePaths. You need to handle multiple in the code that called this (size = {0})", OutputFilePaths.Count)
				: OutputFilePaths[0];

		/// <summary>
		/// List of libraries to link in
		/// </summary>
		public List<FileReference> Libraries = [];

		/// <summary>
		/// A list of additional libraries to link in (using the list of library search paths)
		/// </summary>
		public List<string> SystemLibraries = [];

		/// <summary>
		/// A list of the paths used to find libraries.
		/// </summary>
		public List<DirectoryReference> SystemLibraryPaths = [];

		/// <summary>
		/// A list of libraries to exclude from linking.
		/// </summary>
		public List<string> ExcludedLibraries = [];

		/// <summary>
		/// Paths to add as search paths for runtime libraries
		/// </summary>
		public List<string> RuntimeLibraryPaths = [];

		/// <summary>
		/// A list of additional frameworks to link in.
		/// </summary>
		public List<UEBuildFramework> AdditionalFrameworks = [];

		/// <summary>
		/// The iOS/Mac frameworks to link in
		/// </summary>
		public List<string> Frameworks = [];
		public List<string> WeakFrameworks = [];

		/// <summary>
		/// iOS/Mac resources that should be copied to the app bundle
		/// </summary>
		public List<UEBuildBundleResource> AdditionalBundleResources = [];

		/// <summary>
		/// A list of the dynamically linked libraries that shouldn't be loaded until they are first called
		/// into.
		/// </summary>
		public List<string> DelayLoadDLLs = [];

		/// <summary>
		/// Per-architecture lists of dependencies for linking to ignore (useful when building for multiple architectures, and a lib only is needed for one architecture), it's up to the Toolchain to use this
		/// </summary>
		public Dictionary<string, HashSet<UnrealArch>> DependenciesToSkipPerArchitecture = [];

		/// <summary>
		/// Additional arguments to pass to the linker.
		/// </summary>
		public string AdditionalArguments = "";

		/// <summary>
		/// True if debug info should be created.
		/// </summary>
		public bool bCreateDebugInfo = true;

		/// <summary>
		/// True if runtime symbols files should be generated as a post build step for some platforms.
		/// These files are used by the engine to resolve symbol names of callstack backtraces in logs.
		/// </summary>
		public bool bGenerateRuntimeSymbolFiles = true;

		/// <summary>
		/// True if debug symbols that are cached for some platforms should not be created.
		/// </summary>
		public bool bDisableSymbolCache = false;

		/// <summary>
		/// True if we're compiling .cpp files that will go into a library (.lib file)
		/// </summary>
		public bool bIsBuildingLibrary = false;

		/// <summary>
		/// True if we're compiling a DLL
		/// </summary>
		public bool bIsBuildingDLL = false;

		/// <summary>
		/// The method of linking the target uses
		/// </summary>
		public TargetLinkType LinkType;

		/// <summary>
		/// Whether we should compile using the statically-linked CRT. This is not widely supported for the whole engine, but is required for programs that need to run without dependencies.
		/// </summary>
		public bool bUseStaticCRT = false;

		/// <summary>
		/// Whether to use the debug CRT in debug configurations
		/// </summary>
		public bool bUseDebugCRT = false;

		/// <summary>
		/// True if this is a console application that's being build
		/// </summary>
		public bool bIsBuildingConsoleApplication = false;

		/// <summary>
		/// If set, overrides the program entry function on Windows platform.  This is used by the base Unreal
		/// program so we can link in either command-line mode or windowed mode without having to recompile the Launch module.
		/// </summary>
		public string WindowsEntryPointOverride = String.Empty;

		/// <summary>
		/// True if we're building a EXE/DLL target with an import library, and that library is needed by a dependency that
		/// we're directly dependent on.
		/// </summary>
		public bool bIsCrossReferenced = false;

		/// <summary>
		/// True if the application we're linking has any exports, and we should be expecting the linker to
		/// generate a .lib and/or .exp file along with the target output file
		/// </summary>
		public bool bHasExports = true;

		/// <summary>
		/// True if we're building a .NET assembly (e.g. C# project)
		/// </summary>
		public bool bIsBuildingDotNetAssembly = false;

		/// <summary>
		/// The default stack memory size allocation
		/// </summary>
		public int DefaultStackSize = 5000000;

		/// <summary>
		/// The amount of the default stack size to commit initially. Set to 0 to allow the OS to decide.
		/// </summary>
		public int DefaultStackSizeCommit = 0;

		/// <summary>
		/// Wether to link code coverage / tracing libs
		/// </summary>
		public bool bCodeCoverage = false;

		/// <summary>
		/// Whether to omit frame pointers or not. Disabling is useful for e.g. memory profiling on the PC
		/// </summary>
		public bool bOmitFramePointers = true;

		/// <summary>
		/// Whether to support edit and continue.  Only works on Microsoft compilers in 32-bit compiles.
		/// </summary>
		public bool bSupportEditAndContinue;

		/// <summary>
		/// Whether to use incremental linking or not.
		/// </summary>
		public bool bUseIncrementalLinking;

		/// <summary>
		/// Whether to allow the use of LTCG (link time code generation) 
		/// </summary>
		public bool bAllowLTCG;

		/// <summary>
		/// Whether to enable Profile Guided Optimization (PGO) instrumentation in this build.
		/// </summary>
		public bool bPGOProfile;

		/// <summary>
		/// Whether to optimize this build with Profile Guided Optimization (PGO).
		/// </summary>
		public bool bPGOOptimize;

		/// <summary>
		/// Platform specific directory where PGO profiling data is stored.
		/// </summary>
		public string? PGODirectory;

		/// <summary>
		/// Platform specific filename where PGO profiling data is saved.
		/// </summary>
		public string? PGOFilenamePrefix;

		/// <summary>
		/// Platform specific filename where pre-merged PGO profiling data is saved.
		/// </summary>
		public string? PGOMergedFilenamePrefix;

		/// <summary>
		/// Whether to request the linker create a map file as part of the build
		/// </summary>
		public bool bCreateMapFile;

		/// <summary>
		/// Whether PDB files should be used for Visual C++ builds.
		/// </summary>
		public bool bUsePDBFiles;

		/// <summary>
		/// Use Position Independent Executable (PIE). Has an overhead cost
		/// </summary>
		public bool bUsePIE = false;

		/// <summary>
		/// Whether to ignore dangling (i.e. unresolved external) symbols in modules
		/// </summary>
		public bool bIgnoreUnresolvedSymbols;

		/// <summary>
		/// Set flags for determinstic compiles.
		/// </summary>
		public bool bDeterministic;

		/// <summary>
		/// Whether to log detailed timing information
		/// </summary>
		public bool bPrintTimingInfo;

		/// <summary>
		/// Package full path (directory + filename) where to store input files used at link time 
		/// Normally used to debug a linker crash for platforms that support it
		/// </summary>
		public string? PackagePath;

		/// <summary>
		/// Directory where to put crash report files for platforms that support it
		/// </summary>
		public string? CrashDiagnosticDirectory;

		/// <summary>
		/// Directory where to put the ThinLTO cache for platforms that support it
		/// </summary>
		public string? ThinLTOCacheDirectory;

		/// <summary>
		/// Arguments that will be applied to prune the ThinLTO cache for platforms that support it.
		/// The arguments will only be applied if ThinLTOCacheDirectory is set.
		/// </summary>
		public string? ThinLTOCachePruningArguments;

		/// <summary>
		/// Bundle version for Mac apps
		/// </summary>
		public string? BundleVersion;

		/// <summary>
		/// When building a dynamic library on Apple platforms, specifies the installed name for other binaries that link against it.
		/// </summary>
		public string? InstallName;

		/// <summary>
		/// A list of the object files to be linked.
		/// </summary>
		public List<FileItem> InputFiles = [];

		/// <summary>
		/// The default resource file to link in to every binary if a custom one is not provided
		/// </summary>
		public List<FileItem> DefaultResourceFiles = [];

		/// <summary>
		/// The default resource file to link in to every dynamic if a custom one is not provided (converted to obj)
		/// </summary>
		public List<FileItem> DefaultConvertedResourceFiles = [];

		/// <summary>
		/// List of functions that should be exported from this module
		/// </summary>
		public List<string> IncludeFunctions = [];

		/// <summary>
		/// Debugger visualizer files to build into debug info for this binary. 
		/// </summary>
		public List<FileItem> DebuggerVisualizerFiles = [];

		/// <summary>
		/// Provides a Module Definition File (.def) to the linker to describe various attributes of a DLL.
		/// Necessary when exporting functions by ordinal values instead of by name.
		/// </summary>
		public string? ModuleDefinitionFile;

		/// <summary>
		/// All the additional properties from the modules linked into this binary
		/// </summary>
		public List<ReceiptProperty> AdditionalProperties = [];

		/// <summary>
		/// List of all of the runtime dependencies.
		/// </summary>
		public List<ModuleRules.RuntimeDependency> RuntimeDependencies = [];

		/// <summary>
		/// Default constructor.
		/// </summary>
		public LinkEnvironment(UnrealTargetPlatform Platform, CppConfiguration Configuration, UnrealArchitectures Architectures)
		{
			this.Platform = Platform;
			this.Configuration = Configuration;
			this.Architectures = new(Architectures);
			RootPaths = new();
		}

		/// <summary>
		/// Copy constructor.
		/// </summary>
		public LinkEnvironment(LinkEnvironment Other)
		{
			Platform = Other.Platform;
			Configuration = Other.Configuration;
			Architectures = Other.Architectures;
			LinkType = Other.LinkType;
			RootPaths = new(Other.RootPaths);
			BundleDirectory = Other.BundleDirectory;
			OutputDirectory = Other.OutputDirectory;
			IntermediateDirectory = Other.IntermediateDirectory;
			LocalShadowDirectory = Other.LocalShadowDirectory;
			OutputFilePaths = Other.OutputFilePaths.ToList();
			RuntimeDependencies.AddRange(Other.RuntimeDependencies);
			Libraries.AddRange(Other.Libraries);
			SystemLibraries.AddRange(Other.SystemLibraries);
			SystemLibraryPaths.AddRange(Other.SystemLibraryPaths);
			ExcludedLibraries.AddRange(Other.ExcludedLibraries);
			RuntimeLibraryPaths.AddRange(Other.RuntimeLibraryPaths);
			RuntimeDependencies.AddRange(Other.RuntimeDependencies);
			Frameworks.AddRange(Other.Frameworks);
			AdditionalFrameworks.AddRange(Other.AdditionalFrameworks);
			WeakFrameworks.AddRange(Other.WeakFrameworks);
			AdditionalBundleResources.AddRange(Other.AdditionalBundleResources);
			DelayLoadDLLs.AddRange(Other.DelayLoadDLLs);
			AdditionalArguments = Other.AdditionalArguments;
			bCreateDebugInfo = Other.bCreateDebugInfo;
			bGenerateRuntimeSymbolFiles = Other.bGenerateRuntimeSymbolFiles;
			bIsBuildingLibrary = Other.bIsBuildingLibrary;
			bDisableSymbolCache = Other.bDisableSymbolCache;
			bIsBuildingDLL = Other.bIsBuildingDLL;
			LinkType = Other.LinkType;
			bUseStaticCRT = Other.bUseStaticCRT;
			bUseDebugCRT = Other.bUseDebugCRT;
			bIsBuildingConsoleApplication = Other.bIsBuildingConsoleApplication;
			WindowsEntryPointOverride = Other.WindowsEntryPointOverride;
			bIsCrossReferenced = Other.bIsCrossReferenced;
			bHasExports = Other.bHasExports;
			bIsBuildingDotNetAssembly = Other.bIsBuildingDotNetAssembly;
			DefaultStackSize = Other.DefaultStackSize;
			DefaultStackSizeCommit = Other.DefaultStackSizeCommit;
			bCodeCoverage = Other.bCodeCoverage;
			bOmitFramePointers = Other.bOmitFramePointers;
			bSupportEditAndContinue = Other.bSupportEditAndContinue;
			bUseIncrementalLinking = Other.bUseIncrementalLinking;
			bAllowLTCG = Other.bAllowLTCG;
			bPGOOptimize = Other.bPGOOptimize;
			bPGOProfile = Other.bPGOProfile;
			PGODirectory = Other.PGODirectory;
			PGOFilenamePrefix = Other.PGOFilenamePrefix;
			PGOMergedFilenamePrefix = Other.PGOMergedFilenamePrefix;
			bCreateMapFile = Other.bCreateMapFile;
			bUsePDBFiles = Other.bUsePDBFiles;
			bUsePIE = Other.bUsePIE;
			bIgnoreUnresolvedSymbols = Other.bIgnoreUnresolvedSymbols;
			bDeterministic = Other.bDeterministic;
			bPrintTimingInfo = Other.bPrintTimingInfo;
			PackagePath = Other.PackagePath;
			CrashDiagnosticDirectory = Other.CrashDiagnosticDirectory;
			ThinLTOCacheDirectory = Other.ThinLTOCacheDirectory;
			ThinLTOCachePruningArguments = Other.ThinLTOCachePruningArguments;
			BundleVersion = Other.BundleVersion;
			InstallName = Other.InstallName;
			InputFiles.AddRange(Other.InputFiles);
			DefaultResourceFiles.AddRange(Other.DefaultResourceFiles);
			DefaultConvertedResourceFiles.AddRange(Other.DefaultConvertedResourceFiles);
			IncludeFunctions.AddRange(Other.IncludeFunctions);
			DebuggerVisualizerFiles.AddRange(Other.DebuggerVisualizerFiles);
			ModuleDefinitionFile = Other.ModuleDefinitionFile;
			AdditionalProperties.AddRange(Other.AdditionalProperties);

			foreach (KeyValuePair<string, HashSet<UnrealArch>> Pair in Other.DependenciesToSkipPerArchitecture)
			{
				DependenciesToSkipPerArchitecture[Pair.Key] = new HashSet<UnrealArch>(Pair.Value);
			}
		}

		/// <summary>
		/// Construct a LinkEnvironment from another, and filter it down to only files/dependencies that apply to the given architecture
		/// </summary>
		/// <param name="Other">Parent LinkEnvironment to start with that may have been created from multiple architectures (arch1+arch2)</param>
		/// <param name="OverrideArchitecture">The single architecture to filter down to</param>
		public LinkEnvironment(LinkEnvironment Other, UnrealArch OverrideArchitecture)
			: this(Other)
		{
			Architectures = new UnrealArchitectures(OverrideArchitecture);

			// filter the input files 
			UnrealArchitectureConfig ArchConfig = UnrealArchitectureConfig.ForPlatform(Platform);
			string IntermediateDirPart = ArchConfig.GetFolderNameForArchitectures(Architectures);
			InputFiles = Other.InputFiles.Where(x => x.Location.ContainsName(IntermediateDirPart, 0)).ToList();

			IntermediateDirectory = new DirectoryReference(Other.IntermediateDirectory!.FullName.Replace(
				ArchConfig.GetFolderNameForArchitectures(Other.Architectures),
				ArchConfig.GetFolderNameForArchitecture(OverrideArchitecture),
				StringComparison.Ordinal));

			if (DependenciesToSkipPerArchitecture.Count > 0)
			{
				// add more arrays here?
				Libraries = Libraries.Where(x => !DependenciesToSkipPerArchitecture.ContainsKey(x.FullName) || !DependenciesToSkipPerArchitecture[x.FullName].Contains(Architecture)).ToList();
				SystemLibraries = SystemLibraries.Where(x => !DependenciesToSkipPerArchitecture.ContainsKey(x) || !DependenciesToSkipPerArchitecture[x].Contains(Architecture)).ToList();
				Frameworks = Frameworks.Where(x => !DependenciesToSkipPerArchitecture.ContainsKey(x) || !DependenciesToSkipPerArchitecture[x].Contains(Architecture)).ToList();
				AdditionalFrameworks = AdditionalFrameworks.Where(x => !DependenciesToSkipPerArchitecture.ContainsKey(x.Name) || !DependenciesToSkipPerArchitecture[x.Name].Contains(Architecture)).ToList();
				WeakFrameworks = WeakFrameworks.Where(x => !DependenciesToSkipPerArchitecture.ContainsKey(x) || !DependenciesToSkipPerArchitecture[x].Contains(Architecture)).ToList();
				AdditionalBundleResources = AdditionalBundleResources.Where(x => x.ResourcePath == null || !DependenciesToSkipPerArchitecture.ContainsKey(x.ResourcePath) || !DependenciesToSkipPerArchitecture[x.ResourcePath].Contains(Architecture)).ToList();
				DelayLoadDLLs = DelayLoadDLLs.Where(x => !DependenciesToSkipPerArchitecture.ContainsKey(x) || !DependenciesToSkipPerArchitecture[x].Contains(Architecture)).ToList();
			}
		}
	}
}
