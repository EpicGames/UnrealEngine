// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using EpicGames.Core;

namespace UnrealBuildTool
{
	/// <summary>
	/// Manifest file (.loadorder) describing the order in which all dynamic modules should be loaded.
	/// For this reason, it's only relevant to modular builds on some platforms.
	/// 
	/// This file exists because on some non-Windows platforms dynamic modules (such as DLLs) don't support
	/// dependencies and it's our responsibility to load them in the right order (Windows DLLs know which other
	/// DLLs they rely on and these dependencies are guaranteed to be loaded first).
	/// If we fail respect this order, we're likely to get crashes e.g. when global statics are being initialized
	/// because some dereferenced globals may not be available yet (null).
	/// </summary>
	[Serializable]
	class LoadOrderManifest
	{
		/// <summary>
		/// List of all project's dynamic modules in the order in which they should be loaded.
		/// All paths are relative to the main binary's directory.
		/// </summary>
		public List<string> Libraries = [];

		/// <summary>
		/// Gets a filename for a load order manifest for a build with the given platform/configuration.
		/// </summary>
		/// <param name="AppName">The modular app name being built</param>
		/// <param name="Configuration">The target configuration</param>
		/// <param name="Platform">The target platform</param>
		/// <param name="BuildArchitectures">The architecture of the target platform</param>
		/// <returns>Filename for the app receipt</returns>
		public static string GetStandardFileName(string AppName, UnrealTargetPlatform Platform, UnrealTargetConfiguration Configuration, UnrealArchitectures BuildArchitectures)
		{
			string BaseName = AppName;
			if (Configuration != UnrealTargetConfiguration.Development)
			{
				BaseName += String.Format("-{0}-{1}", Platform.ToString(), Configuration.ToString());
			}
			if (UnrealArchitectureConfig.ForPlatform(Platform).RequiresArchitectureFilenames(BuildArchitectures))
			{
				BaseName += BuildArchitectures.ToString();
			}
			return String.Format("{0}.loadorder", BaseName);
		}

		/// <summary>
		/// Read the load order manifest from a file.
		/// </summary>
		/// <param name="FileName">Filename to read from</param>
		/// <returns>The receipt that was read</returns>
		public static LoadOrderManifest Read(FileReference FileName)
		{
			using (StreamReader Reader = new StreamReader(FileName.FullName))
			{
				LoadOrderManifest Manifest = new LoadOrderManifest();

				while (!Reader.EndOfStream)
				{
					string? Library = Reader.ReadLine();

					if (Library != null)
					{
						Manifest.Libraries.Add(Library.Trim());
					}
				}

				return Manifest;
			}
		}

		/// <summary>
		/// Write the load order manifest to a file.
		/// </summary>
		/// <param name="FileName">Filename to write to</param>
		public void Write(FileReference FileName)
		{
			DirectoryReference.CreateDirectory(FileName.Directory);
			using (StreamWriter Writer = new StreamWriter(FileName.FullName))
			{
				Write(Writer);
			}
		}

		/// <summary>
		/// Write the load order manifest to a text writer.
		/// </summary>
		/// <param name="Writer">The writer to output to</param>
		public void Write(TextWriter Writer)
		{
			foreach (string Library in Libraries)
			{
				Writer.WriteLine(Library);
			}
		}
	}
}
