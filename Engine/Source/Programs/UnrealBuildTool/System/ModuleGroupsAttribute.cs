// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace UnrealBuildTool
{
	/// <summary>
	/// Attribute which can be applied to a ModuleRules-dervied class to indicate which module groups it belongs to
	/// </summary>
	[AttributeUsage(AttributeTargets.Class, AllowMultiple = true)]
	public sealed class ModuleGroupsAttribute : Attribute
	{
		/// <summary>
		/// Array of module group names
		/// </summary>
		public string[] ModuleGroups { get; }

		/// <summary>
		/// Initialize the attribute with a list of module groups
		/// </summary>
		/// <param name="moduleGroups">Variable-length array of module group arguments</param>
		public ModuleGroupsAttribute(params string[] moduleGroups)
		{
			ModuleGroups = moduleGroups;
		}
	}
}
