// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using System.Linq;
using System.Text.Json.Serialization;
using EpicGames.Core;

namespace UnrealBuildTool
{
	/// <summary>
	/// Stores the version (or a unique build ID) for the modules for a target in a certain folder.
	/// 
	/// This allows the runtime to identify which modules are used for which files, and which version they're at. This prevents stale binaries from being
	/// loaded by the runtime when making a local unversioned build, and allows faster incremental builds than compiling the build changelist into every 
	/// module when making versioned builds.
	/// </summary>
	[Serializable]
	public class ModuleManifest
	{
		/// <summary>
		/// Unique build id for the modules described in the manifest.
		/// </summary>
		public required string BuildId;

		/// <summary>
		/// Map of module name to file name.
		/// </summary>
		public Dictionary<string, string> ModuleNameToFileName = new Dictionary<string, string>();

		/// <summary>
		/// Map of dependencies for each merged library.
		/// </summary>
		public Dictionary<string, HashSet<string>> LibraryDependencies = new Dictionary<string, HashSet<string>>();

		/// <summary>
		/// Constructs the module map with the given changelist
		/// </summary>
		/// <param name="InBuildId">The unique build id</param>
		[SetsRequiredMembers]
		public ModuleManifest(string InBuildId)
		{
			BuildId = InBuildId;
		}

		[JsonConstructor]
		private ModuleManifest()
		{
		}

		/// <summary>
		/// Merge another manifest into this one
		/// </summary>
		/// <param name="Other">The manifest to merge in</param>
		public void Include(ModuleManifest Other)
		{
			foreach (KeyValuePair<string, string> Pair in Other.ModuleNameToFileName)
			{
				if (!ModuleNameToFileName.ContainsKey(Pair.Key))
				{
					ModuleNameToFileName.Add(Pair.Key, Pair.Value);
				}
			}
		}

		/// <summary>
		/// Gets the standard path for an manifest
		/// </summary>
		/// <param name="AppName">The modular app name being built</param>
		/// <param name="Configuration">The target configuration</param>
		/// <param name="Platform">The target platform</param>
		/// <param name="BuildArchitectures">The architecture of the target platform</param>
		/// <param name="bIsGameDirectory"></param>
		/// <returns>Filename for the app receipt</returns>
		public static string GetStandardFileName(string AppName, UnrealTargetPlatform Platform, UnrealTargetConfiguration Configuration, UnrealArchitectures BuildArchitectures, bool bIsGameDirectory)
		{
			string BaseName = AppName;
			if (Configuration != UnrealTargetConfiguration.Development && !(Configuration == UnrealTargetConfiguration.DebugGame && !bIsGameDirectory))
			{
				BaseName += String.Format("-{0}-{1}", Platform.ToString(), Configuration.ToString());
			}
			if (UnrealArchitectureConfig.ForPlatform(Platform).RequiresArchitectureFilenames(BuildArchitectures))
			{
				BaseName += BuildArchitectures.ToString();
			}
			return String.Format("{0}.modules", BaseName);
		}

		/// <summary>
		/// Read an app receipt from disk
		/// </summary>
		/// <param name="FileName">Filename to read from</param>
		/// <returns>The receipt that was read</returns>
		public static ModuleManifest Read(FileReference FileName)
		{
			JsonObject Object = JsonObject.Read(FileName);

			ModuleManifest Receipt = new ModuleManifest(Object.GetStringField("BuildId"));

			JsonObject Modules = Object.GetObjectField("Modules");
			foreach (string ModuleName in Modules.KeyNames)
			{
				Receipt.ModuleNameToFileName.Add(ModuleName, Modules.GetStringField(ModuleName));
			}
			return Receipt;
		}

		/// <summary>
		/// Tries to read a receipt from disk.
		/// </summary>
		/// <param name="FileName">The filename that was read</param>
		/// <param name="Result">If successful, the receipt that was read. Null otherwise.</param>
		/// <returns>True if the file was read succesfully.</returns>
		public static bool TryRead(FileReference FileName, [NotNullWhen(true)] out ModuleManifest? Result)
		{
			if (!FileReference.Exists(FileName))
			{
				Result = null;
				return false;
			}
			try
			{
				Result = Read(FileName);
				return true;
			}
			catch (Exception)
			{
				Result = null;
				return false;
			}
		}

		/// <summary>
		/// Write the receipt to disk.
		/// </summary>
		/// <param name="FileName">The file to write to</param>
		public void Write(FileReference FileName)
		{
			DirectoryReference.CreateDirectory(FileName.Directory);
			using (StreamWriter Writer = new StreamWriter(FileName.FullName))
			{
				Write(Writer);
			}
		}

		/// <summary>
		/// Write the receipt to disk.
		/// </summary>
		/// <param name="Writer">The writer to output to</param>
		public void Write(TextWriter Writer)
		{
			using (JsonWriter OutputWriter = new JsonWriter(Writer, true))
			{
				OutputWriter.WriteObjectStart();
				OutputWriter.WriteValue("BuildId", BuildId);

				OutputWriter.WriteObjectStart("Modules");
				foreach (KeyValuePair<string, string> ModuleNameToFileNamePair in ModuleNameToFileName.OrderBy(x => x.Key))
				{
					OutputWriter.WriteValue(ModuleNameToFileNamePair.Key, ModuleNameToFileNamePair.Value);
				}
				OutputWriter.WriteObjectEnd();

				if (LibraryDependencies.Any())
				{
					OutputWriter.WriteObjectStart("LibraryDependencies");
					foreach (KeyValuePair<string, HashSet<string>> LibraryNameToDependenciesPair in LibraryDependencies.Where(x => x.Value.Any()).OrderBy(x => x.Key))
					{
						OutputWriter.WriteArrayStart(LibraryNameToDependenciesPair.Key);
						foreach (string DependencyName in LibraryNameToDependenciesPair.Value)
						{
							OutputWriter.WriteValue(DependencyName);
						}
						OutputWriter.WriteArrayEnd();
					}
					OutputWriter.WriteObjectEnd();
				}

				OutputWriter.WriteObjectEnd();
			}
		}
	}
}
