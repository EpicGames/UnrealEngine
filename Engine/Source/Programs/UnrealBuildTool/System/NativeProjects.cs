// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using System.Linq;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// Utility functions for querying native projects (ie. those found via a .uprojectdirs query)
	/// </summary>
	public class NativeProjects : NativeProjectsBase
	{
		/// <summary>
		/// Cached map of target names to the project file they belong to
		/// </summary>
		static Dictionary<string, FileReference>? CachedTargetNameToProjectFile;

		/// <summary>
		/// Clear our cached properties. Generally only needed if your script has modified local files...
		/// </summary>
		public static void ClearCache()
		{
			ClearCacheBase();
			CachedTargetNameToProjectFile = null;
		}

		/// <summary>
		/// Get the project folder for the given target name
		/// </summary>
		/// <param name="InTargetName">Name of the target of interest</param>
		/// <param name="Logger">Logger for output</param>
		/// <param name="OutProjectFileName">The project filename</param>
		/// <returns>True if the target was found</returns>
		public static bool TryGetProjectForTarget(string InTargetName, ILogger Logger, [NotNullWhen(true)] out FileReference? OutProjectFileName)
		{
			if (CachedTargetNameToProjectFile == null)
			{
				lock (LockObject)
				{
#pragma warning disable CA1508 // Avoid dead conditional code - required to check for null again after locking
					if (CachedTargetNameToProjectFile == null)
#pragma warning restore CA1508
					{
						using ITimelineEvent _ = Timeline.ScopeEvent("Find target files");
						Dictionary<string, FileReference> TargetNameToProjectFile = [];

						IEnumerable<DirectoryReference> AllPotentialTargetDirectories = EnumerateProjectFiles(Logger)
							.SelectMany(ProjectFile => Unreal.GetExtensionDirs(ProjectFile.Directory))
							.SelectMany(ExpandSource);
						Rules.Populate(AllPotentialTargetDirectories);

						foreach (FileReference ProjectFile in EnumerateProjectFiles(Logger))
						{
							IEnumerable<DirectoryReference> ProjectTargetDirectories = Unreal.GetExtensionDirs(ProjectFile.Directory).SelectMany(ExpandSource);
							List<FileReference> TargetFiles = Rules.Get(ProjectTargetDirectories, Rules.RulesFileType.Target);
							foreach (FileReference TargetFile in TargetFiles)
							{
								string TargetName = TargetFile.GetFileNameWithoutAnyExtensions();
								TargetNameToProjectFile[TargetName] = ProjectFile;
							}

							// Programs are a special case where the .uproject files are separated from the main project source code- in this case,
							// we guarantee that a project under the Programs dir will always have an associated target file with the same name.
							if (!TargetNameToProjectFile.ContainsKey(ProjectFile.GetFileNameWithoutAnyExtensions()) && ProjectFile.ContainsName("Programs", 0))
							{
								TargetNameToProjectFile.Add(ProjectFile.GetFileNameWithoutAnyExtensions(), ProjectFile);
							}
						}
						CachedTargetNameToProjectFile = TargetNameToProjectFile;
					}
				}
			}
			return CachedTargetNameToProjectFile.TryGetValue(InTargetName, out OutProjectFileName);
		}

		#region Hybrid content-only/code-based project

		/// <summary>
		/// Returns true if this project is a Hybrid content only project that requires it to be built as code
		/// </summary>
		/// <param name="UProjectFile"></param>
		/// <param name="Reason"></param>
		/// <param name="Logger"></param>
		/// <returns></returns>
		public static bool IsHybridContentOnlyProject(FileReference UProjectFile, [NotNullWhen(true)] out string? Reason, ILogger Logger)
		{
			return RequiresTempTarget(
				UProjectFile,
				new List<UnrealTargetConfiguration>() { UnrealTargetConfiguration.Development, UnrealTargetConfiguration.Shipping },
				out Reason,
				Logger);
		}

		/// <summary>
		/// Creates temporary target files, if needed, for a hybrid content only project
		/// </summary>
		/// <param name="UProjectFile"></param>
		/// <param name="Logger"></param>
		/// <returns>True if the project is hybrid</returns>
		public static bool ConditionalMakeTempTargetForHybridProject(FileReference UProjectFile, ILogger Logger)
		{
			string? Reason;
			bool bIsHybrid = IsHybridContentOnlyProject(UProjectFile, out Reason, Logger);

			DirectoryReference TempDir = DirectoryReference.Combine(UProjectFile.Directory, "Intermediate", "Source");

			// Get the project name for use in temporary files
			string ProjectName = UProjectFile.GetFileNameWithoutExtension();
			Dictionary<TargetType, FileReference> TargetFiles = new()
			{
				{ TargetType.Editor, FileReference.Combine(TempDir, $"{ProjectName}Editor.Target.cs") },
				{ TargetType.Game, FileReference.Combine(TempDir, $"{ProjectName}.Target.cs") },
				{ TargetType.Client, FileReference.Combine(TempDir, $"{ProjectName}Client.Target.cs") },
				{ TargetType.Server, FileReference.Combine(TempDir, $"{ProjectName}Server.Target.cs") },
			};
			FileReference ModuleLocation = FileReference.Combine(TempDir, ProjectName + ".Build.cs");
			FileReference SourceFileLocation = FileReference.Combine(TempDir, ProjectName + ".cpp");

			// if all files exist, early out
			bool bWasHybrid = TargetFiles.Values.All(x => FileReference.Exists(x)) && FileReference.Exists(ModuleLocation) && FileReference.Exists(SourceFileLocation);
			if (!bIsHybrid)
			{
				// clean up if needed
				if (bWasHybrid)
				{
					Logger.LogWarning("Cleaning old temporary Target files for {Project} because it no longer being treated as a code-based project for any enabled platform.", ProjectName);
					DirectoryReference.Delete(TempDir, bRecursive: true);
				}
				return false;
			}

			// if the files existed, just leave them be
			if (bIsHybrid && bWasHybrid)
			{
				return true;
			}

			Logger.LogInformation("{Reason} Creating temporary .Target.cs files.", Reason);

			// make sure directory exists
			DirectoryReference.CreateDirectory(TempDir);

			// Create a target.cs file
			foreach (TargetType TargetType in TargetFiles.Keys)
			{
				string TargetTypeString = TargetType == TargetType.Game ? "" : TargetType.ToString();

				MemoryStream TargetStream = new MemoryStream();
				using (StreamWriter Writer = new StreamWriter(TargetStream))
				{
					Writer.WriteLine("using UnrealBuildTool;");
					Writer.WriteLine();
					Writer.WriteLine("public class {0}{1}Target : TargetRules", ProjectName, TargetTypeString);
					Writer.WriteLine("{");
					Writer.WriteLine("\tpublic {0}{1}Target(TargetInfo Target) : base(Target)", ProjectName, TargetTypeString);
					Writer.WriteLine("\t{");
					Writer.WriteLine("\t\tDefaultBuildSettings = BuildSettingsVersion.Latest;");
					Writer.WriteLine("\t\tIncludeOrderVersion = EngineIncludeOrderVersion.Latest;");
					Writer.WriteLine("\t\tType = TargetType.{0};", TargetType);
					Writer.WriteLine("\t\tExtraModuleNames.Add(\"{0}\");", ProjectName);
					Writer.WriteLine("\t}");
					Writer.WriteLine("}");
				}
				FileReference.WriteAllBytesIfDifferent(TargetFiles[TargetType], TargetStream.ToArray());
			}

			// Create a build.cs file
			MemoryStream ModuleStream = new MemoryStream();
			using (StreamWriter Writer = new StreamWriter(ModuleStream))
			{
				Writer.WriteLine("using UnrealBuildTool;");
				Writer.WriteLine();
				Writer.WriteLine("public class {0} : ModuleRules", ProjectName);
				Writer.WriteLine("{");
				Writer.WriteLine("\tpublic {0}(ReadOnlyTargetRules Target) : base(Target)", ProjectName);
				Writer.WriteLine("\t{");
				Writer.WriteLine("\t\tPCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;");
				Writer.WriteLine();
				Writer.WriteLine("\t\tPrivateDependencyModuleNames.Add(\"Core\");");
				Writer.WriteLine("\t\tPrivateDependencyModuleNames.Add(\"Core\");");
				Writer.WriteLine("\t}");
				Writer.WriteLine("}");
			}
			FileReference.WriteAllBytesIfDifferent(ModuleLocation, ModuleStream.ToArray());

			// Create a main module cpp file
			MemoryStream SourceFileStream = new MemoryStream();
			using (StreamWriter Writer = new StreamWriter(SourceFileStream))
			{
				Writer.WriteLine("#include \"CoreTypes.h\"");
				Writer.WriteLine("#include \"Modules/ModuleManager.h\"");
				Writer.WriteLine();
				Writer.WriteLine("IMPLEMENT_PRIMARY_GAME_MODULE(FDefaultModuleImpl, {0}, \"{0}\");", ProjectName);
			}
			FileReference.WriteAllBytesIfDifferent(SourceFileLocation, SourceFileStream.ToArray());

			// need to clear out some caches now that we've added a Target
			ClearCache();
			Rules.InvalidateRulesFileCache(UProjectFile.Directory.FullName);
			Rules.InvalidateRulesFileCache(TempDir.FullName);
			DirectoryItem.ResetCachedInfo(UProjectFile.Directory.FullName);
			DirectoryItem.ResetCachedInfo(TempDir.FullName);

			return true;
		}

		/// <summary>
		/// Determines if a project (given a .uproject file) has source code. This is determined by finding at least one .Target.cs file
		/// </summary>
		/// <param name="UProjectFile">Path to a .uproject file</param>
		/// <param name="bCheckForTempTargets">If true, search Intermediate/Source for target files</param>
		/// <returns>True if this is a source-based project</returns>
		public static bool ProjectHasCode(FileReference UProjectFile, bool bCheckForTempTargets)
		{
			DirectoryReference SourceDir = DirectoryReference.Combine(UProjectFile.Directory, "Source");
			DirectoryReference TempSourceDir = DirectoryReference.Combine(UProjectFile.Directory, "Intermediate", "Source");

			// check to see if we have a Target.cs file in Source or Intermediate/Source
			if (DirectoryReference.Exists(SourceDir) && DirectoryReference.EnumerateFiles(SourceDir, "*.Target.cs", SearchOption.TopDirectoryOnly).Any())
			{
				return true;
			}
			if (bCheckForTempTargets && DirectoryReference.Exists(TempSourceDir) && DirectoryReference.EnumerateFiles(TempSourceDir, "*.Target.cs", SearchOption.TopDirectoryOnly).Any())
			{
				return true;
			}
			return false;
		}

		private static IEnumerable<DirectoryReference> ExpandSource(DirectoryReference Item)
		{
			yield return DirectoryReference.Combine(Item, "Source");
			yield return DirectoryReference.Combine(Item, "Intermediate", "Source");
		}

		private static bool RequiresTempTarget(FileReference UProjectFile, List<UnrealTargetConfiguration> Configurations, [NotNullWhen(true)] out string? Reason, ILogger Logger)
		{
			// no reason by default
			Reason = null;

			// never create temp targets for the Template projects
			if (UProjectFile.ContainsName("Templates", 0))
			{
				return false;
			}

			List<UnrealTargetPlatform> Platforms = DataDrivenPlatformInfo.GetAllPlatformInfos()
				.Where(x => x.Value.bIsEnabled)
				.Select(x => UnrealTargetPlatform.Parse(x.Key.Replace("Windows", "Win64", StringComparison.Ordinal)))
				.ToList();

			bool bHasCode = ProjectHasCode(UProjectFile, bCheckForTempTargets: false);
			foreach (UnrealTargetPlatform Platform in Platforms)
			{
				foreach (UnrealTargetConfiguration Configuration in Configurations)
				{
					string? InnerReason;
					if (RequiresTempTarget(UProjectFile, bHasCode, Platform, Configuration, TargetType.Game, out InnerReason, Logger))
					{
						string PlatformNames = String.Join(", ", Platforms.Select(x => x.ToString()));
						Reason = $"{UProjectFile.GetFileName()} is has no code, but is being treated as a code-based project for platforms {PlatformNames} because: {InnerReason}.";
						return true;
					}
				}
			}
			return false;
		}

		/// <summary>
		/// NOTE: This function must mirror the functionality of TargetPlatformBase::RequiresTempTarget
		/// </summary>
		private static bool RequiresTempTarget(FileReference RawProjectPath, bool bProjectHasCode, UnrealTargetPlatform Platform, UnrealTargetConfiguration Configuration, TargetType TargetType, out string? OutReason, ILogger Logger)
		{
			// check to see if we already have a Target.cs file
			if (bProjectHasCode)
			{
				OutReason = null;
				return false;
			}

			// Check if encryption or signing is enabled
			EncryptionAndSigning.CryptoSettings Settings = EncryptionAndSigning.ParseCryptoSettings(RawProjectPath.Directory, Platform, Log.Logger);
			if (Settings.IsAnyEncryptionEnabled() || Settings.IsPakSigningEnabled())
			{
				OutReason = "encryption/signing is enabled";
				return true;
			}

			// check the target platforms for any differences in build settings or additional plugins
			if (!Unreal.IsEngineInstalled() && !PlatformExports.HasDefaultBuildConfig(RawProjectPath, Platform))
			{
				OutReason = "project has non-default build configuration";
				return true;
			}
			if (PlatformExports.RequiresBuild(RawProjectPath, Platform))
			{
				OutReason = "overriden by target platform";
				return true;
			}

			// Read the project descriptor, and find all the plugins available to this project
			ProjectDescriptor Project = ProjectDescriptor.FromFile(RawProjectPath);

			// Enumerate all the available plugins
			List<PluginInfo> EnabledPlugins = Plugins.ReadAvailablePlugins(Unreal.EngineDirectory, DirectoryReference.FromFile(RawProjectPath), null);

			// instead of using .ToDictionary, we do this in a loop so that the non-engine plugins replace engine plugins with the same name
			Dictionary<string, PluginInfo> AllPlugins = new(StringComparer.OrdinalIgnoreCase);
			foreach (PluginInfo Plugin in EnabledPlugins)
			{
				// if we don't already have it, or we do but this one is in the game directory, use it
				if (!AllPlugins.ContainsKey(Plugin.Name) || Plugin.File.IsUnderDirectory(DirectoryReference.FromFile(RawProjectPath)))
				{
					AllPlugins[Plugin.Name] = Plugin;
				}
			}

			// find if there are any plugins enabled or disabled which differ from the default
			string? Reason;
			if (RequiresTempTargetForCodePlugin(Project, Platform, Configuration, TargetType, AllPlugins, out Reason, Logger))
			{
				OutReason = Reason;
				return true;
			}

			OutReason = null;
			return false;
		}

		/// <summary>
		/// NOTE: This function must mirror FPluginManager::RequiresTempTargetForCodePlugin
		/// </summary>
		static bool RequiresTempTargetForCodePlugin(ProjectDescriptor ProjectDescriptor, UnrealTargetPlatform Platform, UnrealTargetConfiguration Configuration, TargetType TargetType, Dictionary<string, PluginInfo> AllPlugins, out string? OutReason, ILogger Logger)
		{
			PluginReferenceDescriptor? MissingPlugin;

			HashSet<string> ProjectCodePlugins = new HashSet<string>(StringComparer.OrdinalIgnoreCase);
			if (!GetCodePluginsForTarget(ProjectDescriptor, Platform, Configuration, TargetType, ProjectCodePlugins, AllPlugins, out MissingPlugin, Logger))
			{
				OutReason = String.Format("{0} plugin is referenced by target but not found", MissingPlugin!.Name);
				return true;
			}

			HashSet<string> DefaultCodePlugins = new HashSet<string>(StringComparer.OrdinalIgnoreCase);
			if (!GetCodePluginsForTarget(null, Platform, Configuration, TargetType, DefaultCodePlugins, AllPlugins, out MissingPlugin, Logger))
			{
				OutReason = String.Format("{0} plugin is referenced by the default target but not found", MissingPlugin!.Name);
				return true;
			}

			foreach (string ProjectCodePlugin in ProjectCodePlugins)
			{
				if (!DefaultCodePlugins.Contains(ProjectCodePlugin))
				{
					OutReason = String.Format("{0} plugin is enabled", ProjectCodePlugin);
					return true;
				}
			}

			foreach (string DefaultCodePlugin in DefaultCodePlugins)
			{
				if (!ProjectCodePlugins.Contains(DefaultCodePlugin))
				{
					OutReason = String.Format("{0} plugin is disabled", DefaultCodePlugin);
					return true;
				}
			}

			OutReason = null;
			return false;
		}

		/// <summary>
		/// NOTE: This function must mirror FPluginManager::GetCodePluginsForTarget
		/// </summary>
		static bool GetCodePluginsForTarget(ProjectDescriptor? ProjectDescriptor, UnrealTargetPlatform Platform, UnrealTargetConfiguration Configuration, TargetType TargetType, HashSet<string> CodePluginNames, Dictionary<string, PluginInfo> AllPlugins, out PluginReferenceDescriptor? OutMissingPlugin, ILogger Logger)
		{
			bool bLoadPluginsForTargetPlatforms = (TargetType == TargetType.Editor);

			// Map of all enabled plugins
			Dictionary<string, PluginInfo> EnabledPlugins = new Dictionary<string, PluginInfo>(StringComparer.OrdinalIgnoreCase);

			// Keep a set of all the plugin names that have been configured. We read configuration data from different places, but only configure a plugin from the first place that it's referenced.
			HashSet<string> ConfiguredPluginNames = new HashSet<string>(StringComparer.OrdinalIgnoreCase);

			bool bAllowEnginePluginsEnabledByDefault = true;

			// Find all the plugin references in the project file
			if (ProjectDescriptor != null)
			{
				bAllowEnginePluginsEnabledByDefault = !ProjectDescriptor.DisableEnginePluginsByDefault;
				if (ProjectDescriptor.Plugins != null)
				{
					// Copy the plugin references, since we may modify the project if any plugins are missing
					foreach (PluginReferenceDescriptor PluginReference in ProjectDescriptor.Plugins)
					{
						if (!ConfiguredPluginNames.Contains(PluginReference.Name))
						{
							PluginReferenceDescriptor? MissingPlugin;
							if (!ConfigureEnabledPluginForTarget(PluginReference, null, Platform, Configuration, TargetType, bLoadPluginsForTargetPlatforms, AllPlugins, EnabledPlugins, out MissingPlugin, Logger))
							{
								OutMissingPlugin = MissingPlugin;
								return false;
							}
							ConfiguredPluginNames.Add(PluginReference.Name);
						}
					}
				}
			}

			// Add the plugins which are enabled by default
			foreach (KeyValuePair<string, PluginInfo> PluginPair in AllPlugins)
			{
				if (PluginPair.Value.IsEnabledByDefault(bAllowEnginePluginsEnabledByDefault) && !ConfiguredPluginNames.Contains(PluginPair.Key))
				{
					PluginReferenceDescriptor? MissingPlugin;
					if (!ConfigureEnabledPluginForTarget(new PluginReferenceDescriptor(PluginPair.Key, null, true), null, Platform, Configuration, TargetType, bLoadPluginsForTargetPlatforms, AllPlugins, EnabledPlugins, out MissingPlugin, Logger))
					{
						OutMissingPlugin = MissingPlugin;
						return false;
					}
					ConfiguredPluginNames.Add(PluginPair.Key);
				}
			}

			// Figure out which plugins have code 
			bool bBuildDeveloperTools = (TargetType == TargetType.Editor || TargetType == TargetType.Program || (Configuration != UnrealTargetConfiguration.Test && Configuration != UnrealTargetConfiguration.Shipping));
			bool bRequiresCookedData = (TargetType != TargetType.Editor);
			foreach (KeyValuePair<string, PluginInfo> Pair in EnabledPlugins)
			{
				if (Pair.Value.Descriptor.Modules != null)
				{
					foreach (ModuleDescriptor Module in Pair.Value.Descriptor.Modules)
					{
						if (Module.IsCompiledInConfiguration(Platform, Configuration, "", TargetType, bBuildDeveloperTools, bRequiresCookedData))
						{
							CodePluginNames.Add(Pair.Key);
							break;
						}
					}
				}
			}

			OutMissingPlugin = null;
			return true;
		}

		/// <summary>
		/// NOTE: This function should mirror FPluginManager::ConfigureEnabledPluginForTarget
		/// </summary>
		static bool ConfigureEnabledPluginForTarget(PluginReferenceDescriptor FirstReference, string? TargetName, UnrealTargetPlatform Platform, UnrealTargetConfiguration Configuration, TargetType TargetType, bool bLoadPluginsForTargetPlatforms, Dictionary<string, PluginInfo> AllPlugins, Dictionary<string, PluginInfo> EnabledPlugins, out PluginReferenceDescriptor? OutMissingPlugin, ILogger Logger)
		{
			if (!EnabledPlugins.ContainsKey(FirstReference.Name))
			{
				// Set of plugin names we've added to the queue for processing
				HashSet<string> NewPluginNames = new HashSet<string>(StringComparer.OrdinalIgnoreCase);
				NewPluginNames.Add(FirstReference.Name);

				// Queue of plugin references to consider
				List<PluginReferenceDescriptor> NewPluginReferences = new List<PluginReferenceDescriptor>();
				NewPluginReferences.Add(FirstReference);

				// Loop through the queue of plugin references that need to be enabled, queuing more items as we go
				for (int Idx = 0; Idx < NewPluginReferences.Count; Idx++)
				{
					PluginReferenceDescriptor Reference = NewPluginReferences[Idx];

					// Check if the plugin is required for this platform
					if (!Reference.IsEnabledForPlatform(Platform) || !Reference.IsEnabledForTargetConfiguration(Configuration) || !Reference.IsEnabledForTarget(TargetType))
					{
						Logger.LogDebug("Ignoring plugin '{Arg0}' for platform/configuration", Reference.Name);
						continue;
					}

					// Check if the plugin is required for this platform
					if (!bLoadPluginsForTargetPlatforms && !Reference.IsSupportedTargetPlatform(Platform))
					{
						Logger.LogDebug("Ignoring plugin '{Arg0}' due to unsupported platform", Reference.Name);
						continue;
					}

					// Find the plugin being enabled
					PluginInfo? Plugin;
					if (!AllPlugins.TryGetValue(Reference.Name, out Plugin))
					{
						// Ignore any optional plugins
						if (Reference.bOptional)
						{
							Logger.LogDebug("Ignored optional reference to '{Plugin}' plugin; plugin was not found.", Reference.Name);
							continue;
						}

						// Add it to the missing list
						OutMissingPlugin = Reference;
						return false;
					}

					// Check the plugin supports this platform
					if (!bLoadPluginsForTargetPlatforms && !Plugin.Descriptor.SupportsTargetPlatform(Platform))
					{
						Logger.LogDebug("Ignoring plugin '{Arg0}' due to unsupported platform in plugin descriptor", Reference.Name);
						continue;
					}

					// Check that this plugin supports the current program
					if (TargetType == TargetType.Program && !Plugin.Descriptor.SupportedPrograms!.Contains(TargetName))
					{
						Logger.LogDebug("Ignoring plugin '{Arg0}' due to absence from the supported programs list", Reference.Name);
						continue;
					}

					// Add references to all its dependencies
					if (Plugin.Descriptor.Plugins != null)
					{
						foreach (PluginReferenceDescriptor NextReference in Plugin.Descriptor.Plugins)
						{
							if (!EnabledPlugins.ContainsKey(NextReference.Name) && !NewPluginNames.Contains(NextReference.Name))
							{
								NewPluginNames.Add(NextReference.Name);
								NewPluginReferences.Add(NextReference);
							}
						}
					}

					// Add the plugin
					EnabledPlugins.Add(Plugin.Name, Plugin);
				}
			}

			OutMissingPlugin = null;
			return true;
		}
		#endregion
	}
}
