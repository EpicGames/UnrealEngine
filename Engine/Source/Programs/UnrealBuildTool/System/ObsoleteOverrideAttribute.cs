// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace UnrealBuildTool
{
	/// <summary>
	/// Attribute used to denote that a method should no longer be overriden. Used by RulesCompiler.
	/// </summary>
	[AttributeUsage(AttributeTargets.Method)]
	public sealed class ObsoleteOverrideAttribute : Attribute
	{
		/// <summary>
		/// Message to display to the user if the method is overridden.
		/// </summary>
		public string Message { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="message">Message to display to the user if the method is overridden</param>
		public ObsoleteOverrideAttribute(string message)
		{
			Message = message;
		}
	}
}
