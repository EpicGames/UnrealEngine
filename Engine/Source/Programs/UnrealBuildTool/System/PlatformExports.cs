// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace UnrealBuildTool
{
	/// <summary>
	/// General platform support functions, exported for UAT
	/// </summary>
	public static class PlatformExports
	{
		/// <summary>
		/// Checks whether the given platform is available
		/// </summary>
		/// <param name="Platform">Platform to check</param>
		/// <returns>True if the platform is available, false otherwise</returns>
		public static bool IsPlatformAvailable(UnrealTargetPlatform Platform)
		{
			return UEBuildPlatform.IsPlatformAvailable(Platform);
		}

		/// <summary>
		/// Gets a list of the registered platforms
		/// </summary>
		/// <returns>List of platforms</returns>
		public static IEnumerable<UnrealTargetPlatform> GetRegisteredPlatforms()
		{
			return UEBuildPlatform.GetRegisteredPlatforms();
		}

		/// <summary>
		/// Checks whether the given project has a default build configuration
		/// </summary>
		/// <param name="ProjectFile">The project file</param>
		/// <param name="Platform">Platform to check settings for</param>
		/// <returns>True if the project uses the default build configuration</returns>
		public static bool HasDefaultBuildConfig(FileReference ProjectFile, UnrealTargetPlatform Platform)
		{
			UEBuildPlatform.TryGetBuildPlatform(Platform, out UEBuildPlatform? BuildPlat);
			return (BuildPlat == null) || BuildPlat.HasDefaultBuildConfig(Platform, ProjectFile.Directory);
		}

		/// <summary>
		/// Checks whether the given project requires a build because of platform needs
		/// </summary>
		/// <param name="ProjectFile">The project file</param>
		/// <param name="Platform">Platform to check settings for</param>
		/// <returns>True if the project requires a build for the platform</returns>
		public static bool RequiresBuild(FileReference ProjectFile, UnrealTargetPlatform Platform)
		{
			UEBuildPlatform.TryGetBuildPlatform(Platform, out UEBuildPlatform? BuildPlat);
			return BuildPlat?.RequiresBuild(Platform, ProjectFile.Directory) == true;
		}

		/// <summary>
		/// Returns an array of all platform folder names
		/// </summary>
		/// <returns>All platform folder names</returns>
		public static string[] GetPlatformFolderNames()
		{
			return UEBuildPlatform.GetPlatformFolderNames();
		}

		/// <summary>
		/// Returns an array of all platform folder names
		/// </summary>
		/// <param name="Platform">The platform to get the included folder names for</param>
		/// <returns>All platform folder names</returns>
		public static string[] GetIncludedFolderNames(UnrealTargetPlatform Platform)
		{
			UEBuildPlatform BuildPlatform = UEBuildPlatform.GetBuildPlatform(Platform);
			return BuildPlatform.GetIncludedFolderNames().ToArray();
		}

		/// <summary>
		/// Returns all the excluded folder names for a given platform
		/// </summary>
		/// <param name="Platform">The platform to get the excluded folder names for</param>
		/// <returns>Array of folder names</returns>
		public static string[] GetExcludedFolderNames(UnrealTargetPlatform Platform)
		{
			UEBuildPlatform BuildPlatform = UEBuildPlatform.GetBuildPlatform(Platform);
			return BuildPlatform.GetExcludedFolderNames().ToArray();
		}

		/// <summary>
		/// Check whether the given platform supports XGE
		/// </summary>
		/// <param name="Platform">Platform to check</param>
		/// <param name="Logger">Logger for output</param>
		/// <returns>True if the platform supports XGE</returns>
		public static bool CanUseXGE(UnrealTargetPlatform Platform, ILogger Logger)
		{
			return UEBuildPlatform.IsPlatformAvailable(Platform) && UEBuildPlatform.GetBuildPlatform(Platform).CanUseXGE() && XGE.IsAvailable(Logger);
		}

		/// <summary>
		/// Check whether the given platform supports the parallel executor in UAT
		/// </summary>
		/// <param name="Platform">Platform to check</param>
		/// <param name="Logger">Logger for output</param>
		/// <returns>True if the platform supports the parallel executor in UAT</returns>
		[Obsolete("Deprecated in UE5.8. All platforms support the parallel executor")]
		public static bool CanUseParallelExecutor(UnrealTargetPlatform Platform, ILogger Logger)
		{
			return true;
		}

		/// <summary>
		/// Gets the path for the XGE console executable
		/// </summary>
		/// <param name="OutXgConsoleExe">On success, receives the path to the XGE console executable</param>
		/// <returns>True if the path was found, false otherwise</returns>
		public static bool TryGetXgConsoleExecutable(out string? OutXgConsoleExe)
		{
			if (!OperatingSystem.IsWindows())
			{
				OutXgConsoleExe = null;
				return false;
			}

			return XGE.TryGetXgConsoleExecutable(out OutXgConsoleExe);
		}

		/// <summary>
		///
		/// </summary>
		public static void PreventAutoSDKSwitching()
		{
			UEBuildPlatformSDK.bAllowAutoSDKSwitching = false;
		}

		/// <summary>
		/// 
		/// </summary>
		/// <param name="Path"></param>
		public static void SetRemoteIniPath(string Path)
		{
			UnrealBuildTool.SetRemoteIniPath(Path);
		}

		/// <summary>
		/// Initialize UBT in the context of another host process (presumably UAT)
		/// </summary>
		/// <param name="CommandLineArgs">Command Line arguments that UBT may need access to for initializing platforms</param>
		/// <param name="Logger">Logger for output</param>
		/// <returns>True if initialization was successful</returns>
		public static bool Initialize(string[] CommandLineArgs, ILogger Logger)
		{
			// Read the XML configuration files
			XmlConfig.ReadConfigFiles(null, null, Logger);

			// Register all the platform classes
			UEBuildPlatform.RegisterPlatforms(false, false, CommandLineArgs, Logger);
			return true;
		}
	}
}
