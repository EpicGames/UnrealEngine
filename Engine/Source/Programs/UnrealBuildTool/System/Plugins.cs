// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// Where a plugin was loaded from
	/// </summary>
	public enum PluginLoadedFrom
	{
		/// <summary>
		/// Plugin is built-in to the engine
		/// </summary>
		Engine,

		/// <summary>
		/// Project-specific plugin, stored within a game project directory
		/// </summary>
		Project
	}

	/// <summary>
	/// Where a plugin was loaded from. The order of this enum is important; in the case of name collisions, larger-valued types will take precedence. Plugins of the same type may not be duplicated.
	/// </summary>
	public enum PluginType
	{
		/// <summary>
		/// Plugin is built-in to the engine
		/// </summary>
		Engine,

		/// <summary>
		/// Project-specific plugin, stored within a game project directory
		/// </summary>
		Project,

		/// <summary>
		/// Plugin found in an external directory (found in an AdditionalPluginDirectory listed in the project file, or referenced on the command line)
		/// </summary>
		External,

		/// <summary>
		/// Project-specific mod plugin
		/// </summary>
		Mod,
	}

	/// <summary>
	/// Information about a single plugin
	/// </summary>
	[DebuggerDisplay("\\{{File}\\}")]
	public class PluginInfo
	{
		/// <summary>
		/// Plugin name
		/// </summary>
		public readonly string Name;

		/// <summary>
		/// Path to the plugin
		/// </summary>
		public readonly FileReference File;

		/// <summary>
		/// Path to the plugin's root directory
		/// </summary>
		public readonly DirectoryReference Directory;

		/// <summary>
		/// Children plugin files that can be added to this plugin (platform extensions)
		/// </summary>
		public readonly List<FileReference> ChildFiles = new List<FileReference>();

		/// <summary>
		/// The plugin descriptor
		/// </summary>
		public PluginDescriptor Descriptor;

		/// <summary>
		/// The type of this plugin
		/// </summary>
		public PluginType Type;

		/// <summary>
		/// Used to indicate whether a plugin is being explicitly packaged via the -plugin command line
		/// </summary>
		public bool bExplicitPluginTarget = false;

		/// <summary>
		/// Constructs a PluginInfo object
		/// </summary>
		/// <param name="InFile">Path to the plugin descriptor</param>
		/// <param name="InType">The type of this plugin</param>
		public PluginInfo(FileReference InFile, PluginType InType)
		{
			Name = Path.GetFileNameWithoutExtension(InFile.FullName);
			File = InFile;
			Directory = File.Directory;
			Descriptor = PluginDescriptor.FromFile(File);
			Type = InType;
		}

		/// <summary>
		/// Determines whether the plugin should be enabled by default
		/// </summary>
		public bool IsEnabledByDefault(bool bAllowEnginePluginsEnabledByDefault)
		{
			if (Descriptor.bEnabledByDefault.HasValue)
			{
				if (Descriptor.bEnabledByDefault.Value)
				{
					return (LoadedFrom == PluginLoadedFrom.Project || bAllowEnginePluginsEnabledByDefault);
				}
				else
				{
					return false;
				}
			}
			else
			{
				return (LoadedFrom == PluginLoadedFrom.Project);
			}
		}

		/// <summary>
		/// Determines where the plugin was loaded from
		/// </summary>
		public PluginLoadedFrom LoadedFrom
		{
			get
			{
				if (Type == PluginType.Engine)
				{
					return PluginLoadedFrom.Engine;
				}
				else
				{
					return PluginLoadedFrom.Project;
				}
			}
		}
	}

	/// <summary>
	/// Represents a group of plugins all sharing the same name -- notionally all different versions of the same plugin.
	/// Since UE can only manage a single plugin of a given name (to avoid module conflicts, etc.), this object 
	/// prioritizes and bubbles up a single "choice" plugin from the set -- see `ChoiceVersion`.
	/// </summary>
	public class PluginSet
	{
		/// <summary>
		/// Shared name that all the plugins in the set go by.
		/// </summary>
		public readonly string Name;

		/// <summary>
		/// Unordered list of all the discovered plugins matching the above name.
		/// </summary>
		public readonly List<PluginInfo> KnownVersions = new List<PluginInfo>();

		/// <summary>
		/// Constructor which takes the specified PluginInfo and adds it to the set.
		/// </summary>
		/// <param name="FirstPlugin">The fist plugin discovered for this set -- will automatically be prioritized as the "choice" plugin.</param>
		public PluginSet(PluginInfo FirstPlugin)
		{
			Name = FirstPlugin.Name;
			Add(FirstPlugin, /*bPromoteToChoiceVersion =*/true);
		}

		/// <summary>
		/// Pushes the specified PluginInfo into the set, and optionally promotes it to the new "choice" version.
		/// </summary>
		/// <param name="Plugin">The new plugin to add to this set (its name should match this set's name)</param>
		/// <param name="bPromoteToChoiceVersion">Whether or not to make this the new prioritized "choice" plugin in the set.</param>
		public void Add(PluginInfo Plugin, bool bPromoteToChoiceVersion = true)
		{
			if (bPromoteToChoiceVersion || KnownVersions.Count == 0)
			{
				IndexOfChoiceVersion = KnownVersions.Count;
			}
			KnownVersions.Add(Plugin);
		}

		/// <summary>
		/// The single plugin, out of the entire set, that we prioritize to be built and utilized by UE.
		/// </summary>
		public PluginInfo? ChoiceVersion
		{
			get
			{
				if (IndexOfChoiceVersion >= 0 && IndexOfChoiceVersion < KnownVersions.Count)
				{
					return KnownVersions[IndexOfChoiceVersion];
				}
				return null;
			}

			set
			{
				if (value == null)
				{
					throw new BuildException("Cannot manually set a PluginSet's ChoiceVersion to be null.");
				}
				else
				{
					IndexOfChoiceVersion = KnownVersions.FindIndex(x => x.File == value.File);
					if (IndexOfChoiceVersion == -1)
					{
						Add(value, /*bPromoteToChoiceVersion =*/ true);
					}
				}
			}
		}

		/// <summary>
		/// An index that we use to identify which plugin in the list is the prioritized "choice" version.
		/// </summary>
		private int IndexOfChoiceVersion = -1;
	}

	/// <summary>
	/// Class for enumerating plugin metadata
	/// </summary>
	public static class Plugins
	{
		/// <summary>
		/// Cache of plugins
		/// </summary>
		static ConcurrentDictionary<string, PluginInfo> PluginInfoCache = [];

		/// <summary>
		/// Cache of plugins under each directory
		/// </summary>
		static ConcurrentDictionary<DirectoryReference, Lazy<List<PluginInfo>>> PluginDirectoryCache = [];

#pragma warning disable CA1707 // Identifiers should not contain underscores - public facing API
		/// <summary>
		/// Invalidate cached plugin data so that we can pickup new things
		/// Warning: Will make subsequent plugin lookups and directory scans slow until the caches are repopulated
		/// </summary>
		public static void InvalidateCaches_SLOW()
		{
			PluginInfoCache = [];
			PluginDirectoryCache = [];
			PluginsBase.InvalidateCache_SLOW();
			DirectoryItem.ResetAllCachedInfo_SLOW();
		}
#pragma warning restore CA1707

		/// <summary>
		/// Add a means to retrieve a plugin to determine if it is present.
		/// </summary>
		/// <param name="PluginName">The name of the plugin.</param>
		/// <returns>The cached plugin info, or null if not found.</returns>
		public static PluginInfo? GetPlugin(string PluginName)
		{
			if (PluginInfoCache.TryGetValue(PluginName, out PluginInfo? plugin))
			{
				return plugin;
			}
			return null;
		}

		/// <summary>
		/// Returns a filtered list of plugins as a name:plugin dictionary to ensure that any game plugins override engine plugins with the same
		/// name, and otherwise that no two plugins with the same name exist. 
		/// </summary>
		/// <param name="Plugins">List of plugins to filter</param>
		/// <returns>Filtered Dictionary of plugins</returns>
		public static Dictionary<string, PluginSet> ToFilteredDictionary(IEnumerable<PluginInfo> Plugins)
		{
			Dictionary<string, PluginSet> NameToPluginInfos = new Dictionary<string, PluginSet>(StringComparer.OrdinalIgnoreCase);
			foreach (PluginInfo Plugin in Plugins)
			{
				PluginSet? ExistingPluginSet;
				if (!NameToPluginInfos.TryGetValue(Plugin.Name, out ExistingPluginSet))
				{
					NameToPluginInfos.Add(Plugin.Name, new PluginSet(Plugin));
				}
				else if (Plugin.Type > ExistingPluginSet.ChoiceVersion?.Type)
				{
					ExistingPluginSet.Add(Plugin, /*bPromoteToChoiceVersion =*/true);
				}
				else
				{
					bool bPromoteToChoiceVersion = Plugin.Descriptor.Version > ExistingPluginSet.ChoiceVersion?.Descriptor.Version;
					ExistingPluginSet.Add(Plugin, bPromoteToChoiceVersion);
				}
			}
			return NameToPluginInfos;
		}

		/// <summary>
		/// Filters the list of plugins to ensure that any game plugins override engine plugins with the same name, and otherwise that no two
		/// plugins with the same name exist. 
		/// </summary>
		/// <param name="Plugins">List of plugins to filter</param>
		/// <returns>Filtered list of plugins in the original order</returns>
		public static IEnumerable<PluginSet> FilterPlugins(IEnumerable<PluginInfo> Plugins)
		{
			Dictionary<string, PluginSet> NameToPluginInfos = ToFilteredDictionary(Plugins);
			return NameToPluginInfos.Values.AsEnumerable();
		}

		/// <summary>
		/// Read all the plugins available to a given project
		/// </summary>
		/// <param name="EngineDir">Path to the engine directory</param>
		/// <param name="ProjectDir">Path to the project directory (or null)</param>
		/// <param name="AdditionalDirectories">List of additional directories to scan for available plugins</param>
		/// <returns>Sequence of PluginInfo objects, one for each discovered plugin</returns>
		public static List<PluginInfo> ReadAvailablePlugins(DirectoryReference EngineDir, DirectoryReference? ProjectDir, List<DirectoryReference>? AdditionalDirectories)
		{
			List<PluginInfo> Plugins = new List<PluginInfo>();

			// Read all the engine plugins
			Plugins.AddRange(ReadEnginePlugins(EngineDir));

			// Read all the project plugins
			if (ProjectDir != null)
			{
				Plugins.AddRange(ReadProjectPlugins(ProjectDir));
			}

			// Scan for shared plugins in project specified additional directories
			if (AdditionalDirectories != null)
			{
				foreach (DirectoryReference AdditionalDirectory in AdditionalDirectories)
				{
					Plugins.AddRange(ReadPluginsFromDirectory(AdditionalDirectory, "", PluginType.External));
				}
			}

			return Plugins;
		}

		/// <summary>
		/// Read all the plugin descriptors under the given engine directory
		/// </summary>
		/// <param name="EngineDirectory">The parent directory to look in.</param>
		/// <returns>Sequence of the found PluginInfo object.</returns>
		public static IReadOnlyList<PluginInfo> ReadEnginePlugins(DirectoryReference EngineDirectory)
		{
			return ReadPluginsFromDirectory(EngineDirectory, "Plugins", PluginType.Engine);
		}

		/// <summary>
		/// Read all the plugin descriptors under the given project directory
		/// </summary>
		/// <param name="ProjectDirectory">The parent directory to look in.</param>
		/// <returns>Sequence of the found PluginInfo object.</returns>
		public static IReadOnlyList<PluginInfo> ReadProjectPlugins(DirectoryReference ProjectDirectory)
		{
			List<PluginInfo> Plugins = new List<PluginInfo>();
			Plugins.AddRange(ReadPluginsFromDirectory(ProjectDirectory, "Plugins", PluginType.Project));
			Plugins.AddRange(ReadPluginsFromDirectory(ProjectDirectory, "Mods", PluginType.Mod));
			return Plugins.AsReadOnly();
		}

		/// <summary>
		/// Read all of the plugins found in the project specified additional plugin directories
		/// </summary>
		/// <param name="AdditionalDirectory">The additional directory to scan</param>
		/// <param name="Logger">Logger for output</param>
		/// <returns>List of the found PluginInfo objects</returns>
		public static IReadOnlyList<PluginInfo> ReadAdditionalPlugins(DirectoryReference AdditionalDirectory, ILogger Logger)
		{
			DirectoryReference FullPath = DirectoryReference.Combine(AdditionalDirectory, "");
			if (!DirectoryReference.Exists(FullPath))
			{
				Logger.LogWarning("AdditionalPluginDirectory {FullPath} not found. Path should be relative to the project", FullPath);
			}
			return ReadPluginsFromDirectory(AdditionalDirectory, "", PluginType.External);
		}

		/// <summary>
		/// Determines whether the given suffix is valid for a child plugin
		/// </summary>
		/// <param name="Suffix"></param>
		/// <returns>Whether the suffix is appopriate</returns>
		private static bool IsValidChildPluginSuffix(string Suffix)
		{
			foreach (UnrealPlatformGroup Group in UnrealPlatformGroup.GetValidGroups())
			{
				if (Group.ToString().Equals(Suffix, StringComparison.OrdinalIgnoreCase))
				{
					return true;
				}
			}

			foreach (UnrealTargetPlatform Platform in UnrealTargetPlatform.GetValidPlatforms())
			{
				if (Platform.ToString().Equals(Suffix, StringComparison.OrdinalIgnoreCase))
				{
					return true;
				}
			}

			return false;
		}

		/// <summary>
		///  Attempt to merge a child plugin up into a parent plugin (via file naming scheme). Very little merging happens
		///  but it does allow for platform extensions to extend a plugin with module files
		/// </summary>
		/// <param name="Plugins"></param>
		/// <param name="Child">Child plugin that needs to merge to a main, parent plugin</param>
		/// <param name="Filename">Child plugin's filename, used to determine the parent's name</param>
		private static void TryMergeWithParent(List<PluginInfo> Plugins, PluginInfo Child, FileReference Filename)
		{
			// find the parent
			PluginInfo? Parent = null;

			string FilenameWithoutExtension = Filename.GetFileNameWithoutAnyExtensions();
			int LastUnderscoreIndex = FilenameWithoutExtension.LastIndexOf('_');

			string ParentPluginName;
			if (LastUnderscoreIndex > 0 && LastUnderscoreIndex < FilenameWithoutExtension.Length - 1)
			{
				ParentPluginName = FilenameWithoutExtension.Substring(0, LastUnderscoreIndex);
				Parent = Plugins.FirstOrDefault(x => x.Name.Equals(ParentPluginName, StringComparison.OrdinalIgnoreCase) && x.LoadedFrom == Child.LoadedFrom);
			}
			else
			{
				throw new BuildException("Platform extension plugin {0} was named improperly. It must be in the form <ParentPlugin>_<Platform>.uplugin", Filename);
			}

			// did we find a parent plugin?
			if (Parent == null)
			{
				throw new BuildException("Unable to find parent plugin {0} for platform extension plugin {1}. Make sure {0}.uplugin exists.", ParentPluginName, Filename);
			}

			// validate child plugin file name
			string PlatformName = FilenameWithoutExtension.Substring(LastUnderscoreIndex + 1);
			if (!IsValidChildPluginSuffix(PlatformName))
			{
				Log.TraceWarningTask(Filename, $"Ignoring child plugin: {Child.File.GetFileName()} - Unknown suffix \"{PlatformName}\": expected valid platform or group.");
				return;
			}

			// The reason we need a lock here is because there could be multiple targets using the same plugins
			// and ReadPluginsFromDirectory can be called in parallel from multiple targets causing ChildPlugins
			// to be populated from different threads.. not the same ChildPlugins but some from each thread.
			lock (Parent)
			{
				MergeWithParent(Parent, Child);
			}
		}
		private static void MergeWithParent(PluginInfo Parent, PluginInfo Child)
		{
			// add our uplugin file to the existing plugin to be used to search for modules later
			Parent.ChildFiles.Add(Child.File);

			// this should cause an error if it's invalid platform name
			//UnrealTargetPlatform Platform = UnrealTargetPlatform.Parse(PlatformName);

			// merge the supported platforms
			if (Child.Descriptor.SupportedTargetPlatforms != null)
			{
				if (Parent.Descriptor.SupportedTargetPlatforms == null)
				{
					Parent.Descriptor.SupportedTargetPlatforms = Child.Descriptor.SupportedTargetPlatforms;
				}
				else
				{
					Parent.Descriptor.SupportedTargetPlatforms = Parent.Descriptor.SupportedTargetPlatforms.Union(Child.Descriptor.SupportedTargetPlatforms).ToList();
				}
			}

			// make sure we are allowed for any modules we list
			if (Child.Descriptor.Modules != null)
			{
				if (Parent.Descriptor.Modules == null)
				{
					Parent.Descriptor.Modules = Child.Descriptor.Modules;
				}
				else
				{
					foreach (ModuleDescriptor ChildModule in Child.Descriptor.Modules)
					{
						ModuleDescriptor? ParentModule = Parent.Descriptor.Modules.FirstOrDefault(x => x.Name.Equals(ChildModule.Name, StringComparison.Ordinal) && x.Type == ChildModule.Type);
						if (ParentModule != null)
						{
							// merge allow/deny lists (if the parent had a list, and child didn't specify a list, just add the child platform to the parent list - for allow/deny lists!)
							if (ChildModule.PlatformAllowList != null)
							{
								if (ParentModule.PlatformAllowList == null)
								{
									ParentModule.PlatformAllowList = ChildModule.PlatformAllowList;
								}
								else
								{
									ParentModule.PlatformAllowList = ParentModule.PlatformAllowList.Union(ChildModule.PlatformAllowList).ToList();
								}
							}
							if (ChildModule.PlatformDenyList != null)
							{
								if (ParentModule.PlatformDenyList == null)
								{
									ParentModule.PlatformDenyList = ChildModule.PlatformDenyList;
								}
								else
								{
									ParentModule.PlatformDenyList = ParentModule.PlatformDenyList.Union(ChildModule.PlatformDenyList).ToList();
								}
							}

							// merge platform architecture allow/deny lists
							static void MergePlatformArchitectureLists( Dictionary<UnrealTargetPlatform,List<UnrealArch>> Parent, Dictionary<UnrealTargetPlatform,List<UnrealArch>> Child)
							{
								foreach (KeyValuePair<UnrealTargetPlatform,List<UnrealArch>> Pair in Child)
								{
									if (!Parent.TryGetValue(Pair.Key, out List<UnrealArch>? ArchList))
									{
										Parent[Pair.Key] = Pair.Value;
									}
									else
									{
										foreach (UnrealArch Arch in Pair.Value)
										{
											if (!ArchList.Contains(Arch))
											{
												ArchList.Add(Arch);
											}
										}
									}
								}
							}

							if (ChildModule.PlatformArchitectureAllowList != null)
							{
								if (ParentModule.PlatformArchitectureAllowList == null)
								{
									ParentModule.PlatformArchitectureAllowList = ChildModule.PlatformArchitectureAllowList;
								}
								else
								{
									MergePlatformArchitectureLists(ParentModule.PlatformArchitectureAllowList, ChildModule.PlatformArchitectureAllowList);
								}
							}

							if (ChildModule.PlatformArchitectureDenyList != null)
							{
								if (ParentModule.PlatformArchitectureDenyList == null)
								{
									ParentModule.PlatformArchitectureDenyList = ChildModule.PlatformArchitectureDenyList;
								}
								else
								{
									MergePlatformArchitectureLists(ParentModule.PlatformArchitectureDenyList, ChildModule.PlatformArchitectureDenyList);
								}
							}
						}
						else
						{
							Parent.Descriptor.Modules.Add(ChildModule);
						}
					}
				}
			}

			// make sure we are allowed for any plugins we list
			if (Child.Descriptor.Plugins != null)
			{
				if (Parent.Descriptor.Plugins == null)
				{
					Parent.Descriptor.Plugins = Child.Descriptor.Plugins;
				}
				else
				{
					foreach (PluginReferenceDescriptor ChildPluginReference in Child.Descriptor.Plugins)
					{
						PluginReferenceDescriptor? ParentPluginReference = Parent.Descriptor.Plugins.FirstOrDefault(x => x.Name.Equals(ChildPluginReference.Name, StringComparison.Ordinal));
						if (ParentPluginReference != null)
						{
							// we only need to explicitly list the platform in an allow list if the parent also had an allow list (otherwise, we could mistakenly remove all other platforms)
							if (ParentPluginReference.PlatformAllowList != null)
							{
								if (ChildPluginReference.PlatformAllowList != null)
								{
									ParentPluginReference.PlatformAllowList = ParentPluginReference.PlatformAllowList.Union(ChildPluginReference.PlatformAllowList).ToArray();
								}
							}

							// if we want to deny a platform, add it even if the parent didn't have a deny list. this won't cause problems with other platforms
							if (ChildPluginReference.PlatformDenyList != null)
							{
								if (ParentPluginReference.PlatformDenyList == null)
								{
									ParentPluginReference.PlatformDenyList = ChildPluginReference.PlatformDenyList;
								}
								else
								{
									ParentPluginReference.PlatformDenyList = ParentPluginReference.PlatformDenyList.Union(ChildPluginReference.PlatformDenyList).ToArray();
								}
							}
						}
						else
						{
							Parent.Descriptor.Plugins.Add(ChildPluginReference);
						}
					}
				}
			}
			// @todo platplug: what else do we want to support merging?!?
		}

		/// <summary>
		/// Read all the plugin descriptors under the given directory
		/// </summary>
		/// <param name="RootDirectory">The directory to look in.</param>
		/// <param name="Subdirectory">A subdirectory to look in in RootDirectory and any other Platform directories under Root</param>
		/// <param name="Type">The plugin type</param>
		/// <returns>Sequence of the found PluginInfo object.</returns>
		public static IReadOnlyList<PluginInfo> ReadPluginsFromDirectory(DirectoryReference RootDirectory, string Subdirectory, PluginType Type)
		{
			DirectoryReference pluginDir = DirectoryReference.Combine(RootDirectory, Subdirectory);

			return PluginDirectoryCache.GetOrAdd(pluginDir, new Lazy<List<PluginInfo>>(() =>
			{
				// look for directories in RootDirectory and and extension directories under RootDirectory
				List<DirectoryReference> RootDirectories = Unreal.GetExtensionDirs(RootDirectory, Subdirectory);

				Dictionary<PluginInfo, FileReference> ChildPlugins = new Dictionary<PluginInfo, FileReference>();
				List<PluginInfo> AllParentPlugins = new List<PluginInfo>();

				// Create all in parallel
				ConcurrentDictionary<FileReference, PluginInfo> CreatedInfos = [];
				Parallel2.ForEach(RootDirectories.SelectMany(Dir => PluginsBase.EnumeratePlugins(Dir)), (PluginFileName) =>
				{
					CreatedInfos.TryAdd(PluginFileName, new PluginInfo(PluginFileName, Type));
				});

				// Then add them properly sorted
				foreach (DirectoryReference Dir in RootDirectories)
				{
					foreach (FileReference PluginFileName in PluginsBase.EnumeratePlugins(Dir))
					{
						PluginInfo Plugin = CreatedInfos[PluginFileName];

						// is there a parent to merge up into?
						if (Plugin.Descriptor.bIsPluginExtension)
						{
							ChildPlugins.Add(Plugin, PluginFileName);
						}
						else
						{
							AllParentPlugins.Add(Plugin);
						}
					}
				}

				// now that all parent plugins are read in, we can let the children look up the parents
				foreach (KeyValuePair<PluginInfo, FileReference> Pair in ChildPlugins)
				{
					TryMergeWithParent(AllParentPlugins, Pair.Key, Pair.Value);
				}

				// Add to cache. Note that we TryAdd because there is a chance the plugin is already added through another RootDirectory
				// (VersePrototypes.uproject has AdditionalPluginDirectory set to a subdir of a plugin folder so PluginDirectoryCache will not catch that)
				foreach (PluginInfo plugin in AllParentPlugins)
				{
					PluginInfoCache.TryAdd(plugin.Name, plugin);
				}
				return AllParentPlugins;
			})).Value;
		}

		/// <summary>
		/// Determine if a plugin is enabled for a given project
		/// </summary>
		/// <param name="Project">The project to check. May be null.</param>
		/// <param name="Plugin">Information about the plugin</param>
		/// <param name="Platform">The target platform</param>
		/// <param name="Configuration">The target configuration</param>
		/// <param name="TargetType">The type of target being built</param>
		/// <returns>True if the plugin should be enabled for this project</returns>
		public static bool IsPluginEnabledForTarget(PluginInfo Plugin, ProjectDescriptor? Project, UnrealTargetPlatform Platform, UnrealTargetConfiguration Configuration, TargetType TargetType)
		{
			if (!Plugin.Descriptor.SupportsTargetPlatform(Platform))
			{
				return false;
			}

			bool bAllowEnginePluginsEnabledByDefault = (Project == null || !Project.DisableEnginePluginsByDefault);
			bool bEnabled = Plugin.IsEnabledByDefault(bAllowEnginePluginsEnabledByDefault);
			if (Project != null && Project.Plugins != null)
			{
				foreach (PluginReferenceDescriptor PluginReference in Project.Plugins)
				{
					if (String.Equals(PluginReference.Name, Plugin.Name, StringComparison.OrdinalIgnoreCase) && !PluginReference.bOptional)
					{
						bEnabled = PluginReference.IsEnabledForPlatform(Platform) && PluginReference.IsEnabledForTargetConfiguration(Configuration) && PluginReference.IsEnabledForTarget(TargetType);
					}
				}
			}
			return bEnabled;
		}

		/// <summary>
		/// Determine if a plugin is enabled for a given project
		/// </summary>
		/// <param name="Project">The project to check. May be null.</param>
		/// <param name="Plugin">Information about the plugin</param>
		/// <param name="Platform">The target platform</param>
		/// <param name="Configuration">The target configuration</param>
		/// <param name="TargetType">The type of target being built</param>
		/// <param name="bRequiresCookedData">Whether the target requires cooked data</param>
		/// <param name="Architectures">The architectures being built</param>
		/// <returns>True if the plugin should be enabled for this project</returns>
		public static bool IsPluginCompiledForTarget(PluginInfo Plugin, ProjectDescriptor Project, UnrealTargetPlatform Platform, UnrealTargetConfiguration Configuration, TargetType TargetType, bool bRequiresCookedData, UnrealArchitectures? Architectures = null)
		{
			bool bCompiledForTarget = false;
			if (IsPluginEnabledForTarget(Plugin, Project, Platform, Configuration, TargetType) && Plugin.Descriptor.Modules != null)
			{
				bool bBuildDeveloperTools = (TargetType == TargetType.Editor || TargetType == TargetType.Program || (Configuration != UnrealTargetConfiguration.Test && Configuration != UnrealTargetConfiguration.Shipping));
				foreach (ModuleDescriptor Module in Plugin.Descriptor.Modules)
				{
					if (Module.IsCompiledInConfiguration(Platform, Configuration, "", TargetType, bBuildDeveloperTools, bRequiresCookedData, Architectures))
					{
						bCompiledForTarget = true;
						break;
					}
				}
			}
			return bCompiledForTarget;
		}
	}
}
