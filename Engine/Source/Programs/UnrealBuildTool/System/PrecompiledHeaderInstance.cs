// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// Information about a PCH instance
	/// </summary>
	/// <param name="HeaderFile">The file to include to use this shared PCH</param>
	/// <param name="DefinitionsFile">The definitions file</param>
	/// <param name="CompileEnvironment">The compile environment for this shared PCH</param>
	/// <param name="Output">The output files for the shared PCH</param>
	/// <param name="ImmutableDefinitions">These are definitions that are immutable and should never be #undef. There are a few exceptions and we make sure those are not ending up in this list</param>
	/// <param name="ParentPCHInstance">Parent PCH instance used in PCH chaining</param>
	record class PrecompiledHeaderInstance(FileItem HeaderFile, FileItem DefinitionsFile, CppCompileEnvironment CompileEnvironment, CPPOutput Output, IReadOnlySet<string> ImmutableDefinitions, PrecompiledHeaderInstance? ParentPCHInstance = null)
	{
		/// <summary>
		/// Modules using this PCH instance
		/// </summary>
		public HashSet<UEBuildModuleCPP> Modules { get; } = [];

		/// <inheritdoc/>
		public override string ToString() => HeaderFile.Location.GetFileName();
	}
}
