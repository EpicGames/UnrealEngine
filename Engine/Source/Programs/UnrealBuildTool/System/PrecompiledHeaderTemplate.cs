// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using EpicGames.Core;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// A template for creating a shared PCH. Instances of it are created depending on the configurations required.
	/// </summary>
	/// <param name="Module">The module providing valid shared PCH</param>
	/// <param name="BaseCompileEnvironment">The base compile environment, including all the public compile environment that all consuming modules inherit</param>
	/// <param name="HeaderFile">The header file to generate a PCH from</param>
	/// <param name="OutputDir">Output directory for instances of this PCH</param>
	/// <param name="ModuleDependencies">All the module dependencies this template has</param>
	record class PrecompiledHeaderTemplate(UEBuildModuleCPP Module, CppCompileEnvironment BaseCompileEnvironment, FileItem HeaderFile, DirectoryReference OutputDir, IReadOnlySet<UEBuildModule> ModuleDependencies)
	{
		/// <summary>
		/// Instances of this PCH
		/// </summary>
		public List<PrecompiledHeaderInstance> Instances { get; } = [];

		/// <summary>
		/// Checks whether this template is valid for the given compile environment
		/// </summary>
		/// <param name="compileEnvironment">Compile environment to check with</param>
		/// <returns>True if the template is compatible with the given compile environment</returns>
		public bool IsValidFor(CppCompileEnvironment compileEnvironment) =>
			compileEnvironment.bIsBuildingDLL == BaseCompileEnvironment.bIsBuildingDLL &&
			compileEnvironment.bIsBuildingLibrary == BaseCompileEnvironment.bIsBuildingLibrary;

		/// <inheritdoc/>
		public override string ToString() => HeaderFile.AbsolutePath;
	}
}
