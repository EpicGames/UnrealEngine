// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using EpicGames.Core;

namespace UnrealBuildTool
{
	/// <summary>
	/// Stores information about a compiled binary or module, including the build products and intermediate folders.
	/// </summary>
	class PrecompiledManifest
	{
		/// <summary>
		/// List of files produced by compiling the module. These are within the module output directory.
		/// </summary>
		public List<FileReference> OutputFiles = new List<FileReference>();

		/// <summary>
		/// The MSVC toolchain version used to compile the precompiled objects.
		/// Null for non-MSVC or manifests produced before this field was added.
		/// </summary>
		public VersionNumber? ToolChainVersion = null;

		/// <summary>
		/// Default constructor
		/// </summary>
		public PrecompiledManifest()
		{
		}

		/// <summary>
		/// Read a receipt from disk.
		/// </summary>
		/// <param name="Location">Filename to read from</param>
		public static PrecompiledManifest Read(FileReference Location)
		{
			DirectoryReference BaseDir = Location.Directory;
			PrecompiledManifest Manifest = new PrecompiledManifest();

			JsonObject RawObject = JsonObject.Read(Location);

			string[] OutputFiles = RawObject.GetStringArrayField("OutputFiles");
			foreach (string OutputFile in OutputFiles)
			{
				Manifest.OutputFiles.Add(FileReference.Combine(BaseDir, OutputFile));
			}

			if (RawObject.TryGetStringField("ToolChainVersion", out string? ToolChainVersionStr))
			{
				_ = VersionNumber.TryParse(ToolChainVersionStr, out Manifest.ToolChainVersion);
			}

			return Manifest;
		}

		/// <summary>
		/// Try to read a manifest from disk, failing gracefully if it can't be read.
		/// </summary>
		/// <param name="Location">Filename to read from</param>
		/// <param name="Manifest">If successful, the manifest that was read</param>
		/// <returns>True if successful</returns>
		public static bool TryRead(FileReference Location, [NotNullWhen(true)] out PrecompiledManifest? Manifest)
		{
			if (!FileReference.Exists(Location))
			{
				Manifest = null;
				return false;
			}

			try
			{
				Manifest = Read(Location);
				return true;
			}
			catch (Exception)
			{
				Manifest = null;
				return false;
			}
		}

		/// <summary>
		/// Write the receipt to disk.
		/// </summary>
		/// <param name="Location">Output filename</param>
		public void WriteIfModified(FileReference Location)
		{
			DirectoryReference BaseDir = Location.Directory;

			MemoryStream MemoryStream = new MemoryStream();
			using (JsonWriter Writer = new JsonWriter(new StreamWriter(MemoryStream)))
			{
				Writer.WriteObjectStart();

				string[] OutputFileStrings = new string[OutputFiles.Count];
				for (int Idx = 0; Idx < OutputFiles.Count; Idx++)
				{
					OutputFileStrings[Idx] = OutputFiles[Idx].MakeRelativeTo(BaseDir).Replace('\\', '/');
				}
				Writer.WriteStringArrayField("OutputFiles", OutputFileStrings);

				if (ToolChainVersion != null)
				{
					Writer.WriteValue("ToolChainVersion", ToolChainVersion.ToString());
				}

				Writer.WriteObjectEnd();
			}

			FileReference.WriteAllBytesIfDifferent(Location, MemoryStream.ToArray());
		}
	}
}
