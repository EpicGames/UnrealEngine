// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Threading;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace UnrealBuildTool
{
	/// <summary>
	/// Class to display an incrementing progress percentage. Handles progress markup and direct console output.
	/// </summary>
	public sealed class ProgressWriter : IDisposable
	{
		/// <summary>
		/// Global setting controlling whether to output markup
		/// </summary>
		public static bool bWriteMarkup = false;

		/// <summary>
		/// Logger for output
		/// </summary>
		readonly ILogger Logger;

		/// <summary>
		/// The name to include with the status message
		/// </summary>
		readonly string Message;

		/// <summary>
		/// The inner scope object
		/// </summary>
		LogStatusScope? Status;

		/// <summary>
		/// The current progress message
		/// </summary>
		int CurrentProgressValue = -1;

		private readonly Lock LockObject = new();

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="InMessage">The message to display before the progress percentage</param>
		/// <param name="bInUpdateStatus">Whether to write messages to the console</param>
		/// <param name="InLogger">Logger for output</param>
		public ProgressWriter(string InMessage, bool bInUpdateStatus, ILogger InLogger)
		{
			Message = InMessage;
			Logger = InLogger;
			if (bInUpdateStatus)
			{
				Status = new LogStatusScope(InMessage);
			}
			Write(0, 100);
		}

		/// <summary>
		/// Write the terminating newline
		/// </summary>
		public void Dispose()
		{
			if (Status != null)
			{
				Status.Dispose();
				Status = null;
			}
		}

		/// <summary>
		/// Writes the current progress
		/// </summary>
		/// <param name="Numerator">Numerator for the progress fraction</param>
		/// <param name="Denominator">Denominator for the progress fraction</param>
		public void Write(int Numerator, int Denominator)
		{
			float ProgressValueFloat = Denominator > 0 ? ((float)Numerator / (float)Denominator) : 1.0f;
			int ProgressValue = (int)Math.Round(ProgressValueFloat * 100.0f);

			if (ProgressValue != CurrentProgressValue)
			{
				lock (LockObject)
				{
					if (ProgressValue <= CurrentProgressValue)
					{
						return;
					}
					CurrentProgressValue = ProgressValue;

					string ProgressString = $"{ProgressValue}%";

					if (bWriteMarkup)
					{
						Logger.LogInformation("@progress '{Message}' {ProgressString}", Message, ProgressString);
					}
					Status?.SetProgress(ProgressString);
				}
			}
		}
	}
}
