// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using EpicGames.Core;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// Result from protoc action
	/// </summary>
	public class ProtocOutput
	{
		/// <summary>
		/// C++ files generated by a ProtocAction
		/// </summary>
		public List<FileItem> CppFiles = [];

		/// <summary>
		/// Header files generated by a ProtocAction
		/// </summary>
		public List<FileItem> HeaderFiles = [];
	}

	class ProtocAction : Action
	{
		public ProtocAction(List<FileItem> ProtoFiles, DirectoryReference ProtoModuleDir, DirectoryReference OutputDir, ProtocOutputType OutputType) : base(ActionType.Compile)
		{
			DirectoryReference ProtobufIncludeDir = DirectoryReference.Combine(ProtoExecution.GetProtobufDirectory(), "include");

			PrerequisiteItems.UnionWith(ProtoFiles);
			WorkingDirectory = ProtoModuleDir;
			CommandPath = ProtoExecution.GetProtocPath();
			CommandArguments = $" -I {ProtobufIncludeDir}";
			CommandArguments += $" -I {ProtoModuleDir}";
			CommandArguments += $" {OutputType.GetArgs(OutputDir)}";

			foreach (FileItem ProtoFile in ProtoFiles)
			{
				CommandArguments += $" {ProtoFile.AbsolutePath}";

				string ProducedName = Path.GetRelativePath(ProtoModuleDir.ToString(), ProtoFile.FullName);
				ProducedItems.Add(ProtoExecution.GetProducedFileItem(OutputDir, ProducedName, OutputType, ".h"));
				ProducedItems.Add(ProtoExecution.GetProducedFileItem(OutputDir, ProducedName, OutputType, ".cc"));
			}
		}

		public ProtocAction(BinaryArchiveReader Reader) : base(ActionType.Compile)
		{
			PrerequisiteItems = Reader.ReadSortedSet(Reader.ReadFileItem)!;
			ProducedItems = Reader.ReadSortedSet(Reader.ReadFileItem)!;
			CommandPath = Reader.ReadFileReference()!;
			CommandArguments = Reader.ReadString()!;
		}

		/// <inheritdoc/>
		public new void Write(BinaryArchiveWriter Writer)
		{
			Writer.WriteSortedSet(PrerequisiteItems, Writer.WriteFileItem);
			Writer.WriteSortedSet(ProducedItems, Writer.WriteFileItem);
			Writer.WriteFileReference(CommandPath);
			Writer.WriteString(CommandArguments);
		}
	}

	/// <summary>
	/// Serializer for <see cref="ProtocAction"/> instances
	/// </summary>
	class ProtocActionSerializer : ActionSerializerBase<ProtocAction>
	{
		/// <inheritdoc/>
		public override ProtocAction Read(BinaryArchiveReader reader)
		{
			return new(reader);
		}

		/// <inheritdoc/>
		public override void Write(BinaryArchiveWriter writer, ProtocAction action)
		{
			action.Write(writer);
		}
	}

	enum ProtocOutputType
	{
		Cpp,
		GrpcCpp,
	}

	static class ProtocOutputExtension
	{
		public static string GetArgs(this ProtocOutputType Output, DirectoryReference OutputLocation)
		{
			switch (Output)
			{
				case ProtocOutputType.Cpp:
					return $"--cpp_out={OutputLocation}";
				case ProtocOutputType.GrpcCpp:
					return $"--plugin=\"protoc-gen-grpc={ProtoExecution.GetGrpcCppPluginPath()}\" --grpc_out={OutputLocation}";
			}

			return "";
		}

		public static string GetExtensionPrefix(this ProtocOutputType Output)
		{
			switch (Output)
			{
				case ProtocOutputType.Cpp:
					return "pb";
				case ProtocOutputType.GrpcCpp:
					return "grpc.pb";
			}

			return "";
		}
	}

	class ProtoExecution
	{
		public const string ProtobufVersion = "30.0";
		public const string GrpcVersion = "1.72.1";

		public static ProtocOutput CompileProtoFiles(IEnumerable<FileItem> ProtoFiles, DirectoryReference ProtoModuleDir, DirectoryReference OutputDir, IActionGraphBuilder Graph)
		{
			ProtocOutput Result = new ProtocOutput();

			List<FileItem> ProtoCppFiles = new List<FileItem>();
			List<FileItem> ProtoGrpcFiles = new List<FileItem>();

			foreach (FileItem ProtoFile in ProtoFiles)
			{
				ProtoCppFiles.Add(ProtoFile);

				string ProducedName = Path.GetRelativePath(ProtoModuleDir.ToString(), ProtoFile.FullName);
				Result.HeaderFiles.Add(GetProducedFileItem(OutputDir, ProducedName, ProtocOutputType.Cpp, ".h"));
				Result.CppFiles.Add(GetProducedFileItem(OutputDir, ProducedName, ProtocOutputType.Cpp, ".cc"));

				if (FileReference.ReadAllText(ProtoFile.Location).Contains("service", StringComparison.Ordinal))
				{
					ProtoGrpcFiles.Add(ProtoFile);

					Result.HeaderFiles.Add(GetProducedFileItem(OutputDir, ProducedName, ProtocOutputType.GrpcCpp, ".h"));
					Result.CppFiles.Add(GetProducedFileItem(OutputDir, ProducedName, ProtocOutputType.GrpcCpp, ".cc"));
				}
			}

			if (ProtoCppFiles.Count > 0)
			{
				Graph.AddAction(new ProtocAction(ProtoCppFiles, ProtoModuleDir, OutputDir, ProtocOutputType.Cpp));
			}

			if (ProtoGrpcFiles.Count > 0)
			{
				Graph.AddAction(new ProtocAction(ProtoGrpcFiles, ProtoModuleDir, OutputDir, ProtocOutputType.GrpcCpp));
			}

			return Result;
		}

		public static DirectoryReference GetProtobufDirectory()
		{
			return DirectoryReference.Combine(Unreal.EngineDirectory, "Source", "ThirdParty", "Protobuf", ProtobufVersion);
		}

		public static DirectoryReference GetGrpcDirectory()
		{
			return DirectoryReference.Combine(Unreal.EngineDirectory, "Source", "ThirdParty", "gRPC", GrpcVersion);
		}

		public static FileReference GetProtocPath()
		{
			string ExecutableName = "protoc" + BuildHostPlatform.Current.BinarySuffix;

			if (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Win64)
			{
				return FileReference.Combine(GetProtobufDirectory(), "bin", "Win64", UnrealArch.Host.Value.WindowsName, "Release", ExecutableName);
			}
			else if (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Mac)
			{
				return FileReference.Combine(GetProtobufDirectory(), "bin", "Mac", "Release", ExecutableName);
			}
			else if (BuildHostPlatform.Current.Platform.IsInGroup(UnrealPlatformGroup.Unix))
			{
				return FileReference.Combine(GetProtobufDirectory(), "bin", "Unix", UnrealArch.Host.Value.LinuxName, "Release", ExecutableName);
			}

			throw new BuildException($"Unsupported platform {BuildHostPlatform.Current.Platform} for Protobuf compilation");

		}

		public static FileReference GetGrpcCppPluginPath()
		{
			string ExecutableName = "grpc_cpp_plugin" + BuildHostPlatform.Current.BinarySuffix;

			if (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Win64)
			{
				return FileReference.Combine(GetGrpcDirectory(), "bin", "Win64", UnrealArch.Host.Value.WindowsName, "Release", ExecutableName);
			}
			else if (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Mac)
			{
				return FileReference.Combine(GetGrpcDirectory(), "bin", "Mac", "Release", ExecutableName);
			}
			else if (BuildHostPlatform.Current.Platform.IsInGroup(UnrealPlatformGroup.Unix))
			{
				return FileReference.Combine(GetGrpcDirectory(), "bin", "Unix", UnrealArch.Host.Value.LinuxName, "Release", ExecutableName);
			}

			throw new BuildException($"Unsupported platform {BuildHostPlatform.Current.Platform} for Protobuf compilation");
		}

		internal static FileItem GetProducedFileItem(DirectoryReference OutputDir, string BaseName, ProtocOutputType OutputType, string extension)
		{
			return FileItem.GetItemByFileReference(FileReference.Combine(OutputDir, Path.ChangeExtension(BaseName, OutputType.GetExtensionPrefix() + extension)));
		}
	}
}
