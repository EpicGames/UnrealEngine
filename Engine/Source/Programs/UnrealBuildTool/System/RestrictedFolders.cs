// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using EpicGames.Core;

namespace UnrealBuildTool
{
	/// <summary>
	/// Names of restricted folders. Note: The name of each entry is used to search for/create folders
	/// </summary>
	public partial struct RestrictedFolder : IEquatable<RestrictedFolder>
	{
		/// <summary>
		/// Unique Id for this folder
		/// </summary>
		private readonly int Id;

		/// <summary>
		/// Mapping for unique ids
		/// </summary>
		private static readonly UniqueStringRegistry StringRegistry = new UniqueStringRegistry();

		/// <summary>
		/// Array of all restricted folder names
		/// </summary>
		private static string[]? Names;

		/// <summary>
		/// Array of all restricted folders
		/// </summary>
		private static RestrictedFolder[]? Values;

		/// <summary>
		/// Set of permitted references for each restricted folder. Determined via data-driven platform info.
		/// </summary>
		private static Dictionary<RestrictedFolder, RestrictedFolder[]>? PermittedReferences;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="Id">Id of the string</param>
		private RestrictedFolder(int Id)
		{
			this.Id = Id;
		}

		/// <summary>
		/// Creates a restricted folder instance from a string
		/// </summary>
		/// <param name="Name">Name of the folder</param>
		/// <returns>New restricted folder instance</returns>
		private static RestrictedFolder FindOrAddByName(string Name)
		{
			return new RestrictedFolder(StringRegistry.FindOrAddByName(Name));
		}

		/// <summary>
		/// Tests for equality between two restricted folder instances
		/// </summary>
		/// <param name="A">First instance</param>
		/// <param name="B">Second instance</param>
		/// <returns>True if the two instances are equal</returns>
		public static bool operator ==(RestrictedFolder A, RestrictedFolder B)
		{
			return A.Id == B.Id;
		}

		/// <summary>
		/// Tests for inequality between two restricted folder instances
		/// </summary>
		/// <param name="A">First instance</param>
		/// <param name="B">Second instance</param>
		/// <returns>True if the two instances are not equal</returns>
		public static bool operator !=(RestrictedFolder A, RestrictedFolder B)
		{
			return A.Id != B.Id;
		}

		/// <summary>
		/// Tests whether two restricted folder instances are equal
		/// </summary>
		/// <param name="Other">The restricted folder to compare against</param>
		/// <returns>True if the restricted folder is equal to the other instance</returns>
		public override bool Equals(object? Other)
		{
			return Other is RestrictedFolder folder && Id == folder.Id;
		}

		/// <inheritdoc/>
		public readonly bool Equals(RestrictedFolder other) => this == other;

		/// <summary>
		/// Gets a hash code for this object
		/// </summary>
		/// <returns>Hash code for the object</returns>
		public override int GetHashCode()
		{
			return Id;
		}

		/// <summary>
		/// Returns an array of folders which are allowed to be referenced from this restricted folder
		/// </summary>
		/// <returns>Collection of restricted folders</returns>
		public IEnumerable<RestrictedFolder> GetPermittedReferences()
		{
			RestrictedFolder[]? References;
			if (PermittedReferences != null && PermittedReferences.TryGetValue(this, out References))
			{
				foreach (RestrictedFolder Reference in References)
				{
					yield return Reference;
				}
			}
		}

		/// <summary>
		/// Creates entries for all the confidential platforms. Should be called before returning any list of all folder values.
		/// </summary>
		private static void AddConfidentialPlatforms()
		{
			if (PermittedReferences == null)
			{
				Dictionary<RestrictedFolder, RestrictedFolder[]> NewPermittedReferences = new Dictionary<RestrictedFolder, RestrictedFolder[]>();
				foreach (KeyValuePair<string, DataDrivenPlatformInfo.ConfigDataDrivenPlatformInfo> Pair in DataDrivenPlatformInfo.GetAllPlatformInfos())
				{
					if (Pair.Value.bIsConfidential)
					{
						RestrictedFolder Folder = FindOrAddByName(Pair.Key);
						if (Pair.Value.AdditionalRestrictedFolders != null && Pair.Value.AdditionalRestrictedFolders.Length > 0)
						{
							RestrictedFolder[] References = Array.ConvertAll(Pair.Value.AdditionalRestrictedFolders, x => FindOrAddByName(x));
							NewPermittedReferences[Folder] = References;
						}
					}
				}
				PermittedReferences = NewPermittedReferences;
			}
		}

		/// <summary>
		/// Gets an array of all the restricted folder names
		/// </summary>
		/// <returns></returns>
		public static string[] GetNames()
		{
			if (Names == null)
			{
				AddConfidentialPlatforms();
				Names = StringRegistry.GetStringNames();
			}
			return Names;
		}

		/// <summary>
		/// Ensures that we've added all the restricted folders, and return an array of them
		/// </summary>
		/// <returns>Array of restricted folder values</returns>
		public static RestrictedFolder[] GetValues()
		{
			if (Values == null)
			{
				AddConfidentialPlatforms();
				Values = Array.ConvertAll(StringRegistry.GetStringIds(), x => new RestrictedFolder(x));
			}
			return Values;
		}

		/// <summary>
		/// Return the string representation
		/// </summary>
		/// <returns></returns>
		public override string ToString()
		{
			return StringRegistry.GetStringForId(Id);
		}
	}

	/// <summary>
	/// Values for RestrictedFolder
	/// </summary>
	public partial struct RestrictedFolder
	{
		/// <summary>
		/// Legacy. Should not be used any more.
		/// </summary>
		public static RestrictedFolder EpicInternal = FindOrAddByName("EpicInternal");

		/// <summary>
		/// Can be used by UE but not required
		/// </summary>
		public static RestrictedFolder CarefullyRedist = FindOrAddByName("CarefullyRedist");

		/// <summary>
		/// Epic Employees and Contractors as well as some explicitly permitted external groups
		/// </summary>
		public static RestrictedFolder LimitedAccess = FindOrAddByName("LimitedAccess");

		/// <summary>
		/// Epic Employees and Contractors
		/// </summary>
		public static RestrictedFolder NotForLicensees = FindOrAddByName("NotForLicensees");

		/// <summary>
		/// Epic Employees only
		/// </summary>
		public static RestrictedFolder NoRedist = FindOrAddByName("NoRedist");
	}

	/// <summary>
	/// Utility functions for getting restricted folder
	/// </summary>
	public static class RestrictedFolders
	{
		/// <summary>
		/// Finds all the restricted folder names relative to a base directory
		/// </summary>
		/// <param name="BaseDir">The base directory to check against</param>
		/// <param name="OtherDir">The file or directory to check</param>
		/// <returns>Array of restricted folder names</returns>
		public static List<RestrictedFolder> FindRestrictedFolders(DirectoryReference BaseDir, DirectoryReference OtherDir)
		{
			List<RestrictedFolder> Folders = new List<RestrictedFolder>();
			if (OtherDir.IsUnderDirectory(BaseDir))
			{
				foreach (RestrictedFolder Value in RestrictedFolder.GetValues())
				{
					string Name = Value.ToString();
					if (OtherDir.ContainsName(Name, BaseDir.FullName.Length))
					{
						Folders.Add(Value);
					}
				}
			}
			return Folders;
		}

		/// <summary>
		/// Finds all the permitted restricted folder references for a given path
		/// </summary>
		/// <param name="BaseDir">The base directory to check against</param>
		/// <param name="OtherDir">The file or directory to check</param>
		/// <returns>Array of restricted folder names</returns>
		public static List<RestrictedFolder> FindPermittedRestrictedFolderReferences(DirectoryReference BaseDir, DirectoryReference OtherDir)
		{
			List<RestrictedFolder> Folders = FindRestrictedFolders(BaseDir, OtherDir);
			for (int Idx = 0; Idx < Folders.Count; Idx++)
			{
				foreach (RestrictedFolder Folder in Folders[Idx].GetPermittedReferences())
				{
					if (!Folders.Contains(Folder))
					{
						Folders.Add(Folder);
					}
				}
			}
			return Folders;
		}
	}
}
