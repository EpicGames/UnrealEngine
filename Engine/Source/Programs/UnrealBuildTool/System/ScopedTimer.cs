// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Text;
using System.Threading;
using Microsoft.Extensions.Logging;

namespace UnrealBuildTool
{
	/// <summary>
	/// Scoped timer, start is in the constructor, end in Dispose. Best used with using(ScopedTimer Timer = new ScopedTimer()). Suports nesting.
	/// </summary>
	public sealed class ScopedTimer : IDisposable
	{
		readonly DateTime StartTime;
		readonly string Name;
		readonly ILogger Logger;
		readonly LogLevel Level;
		readonly bool bIncreaseIndent;
		// TODO: This isn't true nesting support, as it assumes that all new ScopedTimers are within the previous scope.
		// This doesn't account for totally unrelated timers, so this should probably use some context-local storage to deal with this.
		static int Indent = 0;
		static readonly Lock IndentLock = new();

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="Name">Name of the block being measured</param>
		/// <param name="Logger">Logger for output</param>
		/// <param name="InLevel">Verbosity for output messages</param>
		/// <param name="bIncreaseIndent">Whether global indent should be increased or not; set to false when running a scope in parallel. Message will still be printed indented relative to parent scope.</param>
		public ScopedTimer(string Name, ILogger Logger, LogLevel InLevel = LogLevel.Debug, bool bIncreaseIndent = true)
		{
			this.Name = Name;
			this.Logger = Logger;
			if (bIncreaseIndent)
			{
				lock (IndentLock)
				{
					Indent++;
				}
			}
			Level = InLevel;
			StartTime = DateTime.UtcNow;
			this.bIncreaseIndent = bIncreaseIndent;
		}

		/// <summary>
		/// Prints out the timing message
		/// </summary>
		public void Dispose()
		{
			double TotalSeconds = (DateTime.UtcNow - StartTime).TotalSeconds;
			int LogIndent = Indent;
			if (bIncreaseIndent)
			{
				lock (IndentLock)
				{
					LogIndent = --Indent;
				}
			}
			StringBuilder IndentText = new StringBuilder(LogIndent * 2);
			IndentText.Append(' ', LogIndent * 2);

			Logger.Log(Level, "{Indent}{Name} took {TimeSeconds:0.000}s", IndentText.ToString(), Name, TotalSeconds);
		}
	}
}
