// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Text.RegularExpressions;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using OpenTracing.Util;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// The header unit type markup is used for multiple things:
	/// 1. To figure out if a header can be compiled by itself or not. Many includes are included in the middle of other includes and those can never be compiled by themselves
	///    This markup is used by both msvc header unit feature and IWYU toolchain
	/// 2. Tell if the header even supports being part of a header unit. If it does not support it will also prevent all includes including it from producing a header unit
	/// This is how to markup in headers to provide above info:
	///    // HEADER_UNIT_SKIP - Here you can write why this file can't be compiled standalone
	///    // HEADER_UNIT_UNSUPPORTED - Here you can write why this file can't be part of header units.
	/// </summary>
	enum HeaderUnitType
	{
		Valid = 0,
		Unsupported = 1,
		Skip = 2,
		Ignore = 3
	}

	/// <summary>
	/// Caches information about C++ source files; whether they contain reflection markup, what the first included header is, and so on.
	/// </summary>
	class SourceFileMetadataCache
	{
		/// <summary>
		/// Source(cpp/c) file info
		/// </summary>
		class SourceFileInfo
		{
			/// <summary>
			/// Last write time of the file when the data was cached
			/// </summary>
			public long LastWriteTimeUtc;

			/// <summary>
			/// Contents of the include directive
			/// </summary>
			public string? IncludeText;

			/// <summary>
			/// List of files this particular file is inlining
			/// </summary>
			public List<string> InlinedFileNames = new List<string>();
		}

		/// <summary>
		/// Header file info
		/// </summary>
		class HeaderFileInfo
		{
			/// <summary>
			/// Last write time of the file when the data was cached
			/// </summary>
			public long LastWriteTimeUtc;

			/// <summary>
			/// Whether or not the file contains reflection markup
			/// </summary>
			public bool bContainsMarkup;

			/// <summary>
			/// Whether or not the file has types that use DLL export/import defines
			/// </summary>
			public bool bUsesAPIDefine;

			/// <summary>
			/// List of includes that header contains
			/// </summary>
			public List<string>? Includes;

			public HeaderUnitType UnitType;
		}

		/// <summary>
		/// The current file version
		/// </summary>
		public const int CurrentVersion = 8;

		/// <summary>
		/// Location of this dependency cache
		/// </summary>
		readonly FileReference Location;

		/// <summary>
		/// Directory for files to cache dependencies for.
		/// </summary>
		readonly DirectoryReference BaseDirectory;

		/// <summary>
		/// The parent cache.
		/// </summary>
		readonly SourceFileMetadataCache? Parent;

		/// <summary>
		/// Map from file item to source file info
		/// </summary>
		readonly ConcurrentDictionary<FileItem, SourceFileInfo> FileToSourceFileInfo = new ConcurrentDictionary<FileItem, SourceFileInfo>();

		/// <summary>
		/// Map from file item to header file info
		/// </summary>
		readonly ConcurrentDictionary<FileItem, HeaderFileInfo> FileToHeaderFileInfo = new ConcurrentDictionary<FileItem, HeaderFileInfo>();

		/// <summary>
		/// Map from file item to source file
		/// </summary>
		readonly ConcurrentDictionary<FileItem, SourceFile> FileToSourceFile = new ConcurrentDictionary<FileItem, SourceFile>();

		/// <summary>
		/// Whether the cache has been modified and needs to be saved
		/// </summary>
		bool bModified;

		/// <summary>
		/// Logger for output
		/// </summary>
		readonly ILogger Logger;

		/// <summary>
		/// Regex that matches C++ code with UObject declarations which we will need to generated code for.
		/// </summary>
		static readonly Regex ReflectionMarkupRegex = new Regex("^\\s*(UCLASS|USTRUCT|UENUM|UINTERFACE|UDELEGATE|UPARTIAL|VMODULE)\\b", RegexOptions.Compiled);

		/// <summary>
		/// Regex that matches #include UE_INLINE_GENERATED_CPP_BY_NAME(****) statements.
		/// </summary>
		static readonly Regex InlineReflectionMarkupRegex = new Regex("^\\s*#\\s*include\\s*UE_INLINE_GENERATED_CPP_BY_NAME\\((.+)\\)", RegexOptions.Compiled | RegexOptions.Multiline);

		/// <summary>
		/// Regex that matches #include statements.
		/// </summary>
		static readonly Regex IncludeRegex = new Regex("^[ \t]*#[ \t]*include[ \t]*[<\"](?<HeaderFile>[^\">]*)[\">]", RegexOptions.Compiled | RegexOptions.Singleline | RegexOptions.ExplicitCapture);

		/// <summary>
		/// Regex that matches #import directives in mm files
		/// </summary>
		static readonly Regex ImportRegex = new Regex("^[ \t]*#[ \t]*import[ \t]*[<\"](?<HeaderFile>[^\">]*)[\">]", RegexOptions.Compiled | RegexOptions.Singleline | RegexOptions.ExplicitCapture);

		/// <summary>
		/// Static cache of all constructed dependency caches
		/// </summary>
		static readonly ConcurrentDictionary<FileReference, Lazy<SourceFileMetadataCache>> Caches = new();

		/// <summary>
		/// Constructs a dependency cache. This method is private; call CppDependencyCache.Create() to create a cache hierarchy for a given project.
		/// </summary>
		/// <param name="Location">File to store the cache</param>
		/// <param name="BaseDir">Base directory for files that this cache should store data for</param>
		/// <param name="Parent">The parent cache to use</param>
		/// <param name="Logger">Logger for output</param>
		private SourceFileMetadataCache(FileReference Location, DirectoryReference BaseDir, SourceFileMetadataCache? Parent, ILogger Logger)
		{
			this.Location = Location;
			BaseDirectory = BaseDir;
			this.Parent = Parent;
			this.Logger = Logger;

			if (FileReference.Exists(Location))
			{
				using (GlobalTracer.Instance.BuildSpan("Reading source file metadata cache").StartActive())
				{
					Read();
				}
			}
		}

		/// <summary>
		/// Returns a SourceFileInfo struct for a file (and parse the file if not already cached)
		/// </summary>
		/// <param name="SourceFile">The file to parse</param>
		/// <returns>SourceFileInfo for file</returns>
		SourceFileInfo GetSourceFileInfo(FileItem SourceFile)
		{
			if (Parent != null && !SourceFile.Location.IsUnderDirectory(BaseDirectory))
			{
				return Parent.GetSourceFileInfo(SourceFile);
			}
			else
			{
				SourceFileInfo UpdateSourceFileInfo(FileItem SourceFile)
				{
					SourceFileInfo SourceFileInfo = new SourceFileInfo();
					string FileText = FileReference.ReadAllText(SourceFile.Location);
					string[] FileTextLines = FileText.Split('\n');

					SourceFileInfo.LastWriteTimeUtc = SourceFile.LastWriteTimeUtc.Ticks;

					// Inline reflection data
					MatchCollection FileMatches = InlineReflectionMarkupRegex.Matches(FileText);
					foreach (Match Match in FileMatches)
					{
						SourceFileInfo.InlinedFileNames.Add(Match.Groups[1].Value);
					}

					SourceFileInfo.IncludeText = ParseFirstInclude(SourceFile, FileTextLines);

					bModified = true;
					return SourceFileInfo;
				}

				return FileToSourceFileInfo.AddOrUpdate(SourceFile, _ =>
				{
					return UpdateSourceFileInfo(SourceFile);
				},
				(k, v) =>
				{
					if (SourceFile.LastWriteTimeUtc.Ticks > v.LastWriteTimeUtc)
					{
						return UpdateSourceFileInfo(SourceFile);
					}
					return v;
				}
				);
			}
		}

		/// <summary>
		/// Parse the first include directive from a source file
		/// </summary>
		/// <param name="SourceFile">The source file to parse</param>
		/// <param name="FileToSourceFileFileText">The source file contents</param>
		/// <returns>The first include directive</returns>
		static string? ParseFirstInclude(FileItem SourceFile, string[] FileToSourceFileFileText)
		{
			bool bMatchImport = SourceFile.HasExtension(".m") || SourceFile.HasExtension(".mm");
			foreach (string Line in FileToSourceFileFileText)
			{
				if (Line == null)
				{
					return null;
				}

				Match IncludeMatch = IncludeRegex.Match(Line);
				if (IncludeMatch.Success)
				{
					return IncludeMatch.Groups[1].Value;
				}

				if (bMatchImport)
				{
					Match ImportMatch = ImportRegex.Match(Line);
					if (ImportMatch.Success)
					{
						return ImportMatch.Groups[1].Value;
					}
				}
			}
			return null;
		}

		HeaderFileInfo GetHeaderFileInfo(FileItem HeaderFile)
		{
			if (Parent != null && !HeaderFile.Location.IsUnderDirectory(BaseDirectory))
			{
				return Parent.GetHeaderFileInfo(HeaderFile);
			}
			else
			{
				Func<FileItem, HeaderFileInfo> UpdateHeaderFileInfo = (FileItem HeaderFile) =>
				{
					HeaderFileInfo HeaderFileInfo = new HeaderFileInfo();
					string FileText = FileReference.ReadAllText(HeaderFile.Location);
					string[] FileTextLines = FileText.Split('\n');

					HeaderFileInfo.LastWriteTimeUtc = HeaderFile.LastWriteTimeUtc.Ticks;

					ParseHeader(HeaderFileInfo, FileTextLines);

					bModified = true;
					return HeaderFileInfo;
				};

				return FileToHeaderFileInfo.AddOrUpdate(HeaderFile, _ =>
				{
					return UpdateHeaderFileInfo(HeaderFile);
				},
				(k, v) =>
				{
					if (HeaderFile.LastWriteTimeUtc.Ticks > v.LastWriteTimeUtc)
					{
						return UpdateHeaderFileInfo(HeaderFile);
					}
					return v;
				}
				);
			}
		}

		/// <summary>
		/// Read entire header file to find markup and includes
		/// </summary>
		/// <returns>A HeaderInfo struct containing information about header</returns>
		private static void ParseHeader(HeaderFileInfo HeaderFileInfo, string[] FileText)
		{
			bool bContainsMarkup = false;
			bool bUsesAPIDefine = false;
			int InsideDeprecationScope = 0;

			SortedSet<string> Includes = new();
			foreach (string Line in FileText)
			{
				if (!bUsesAPIDefine)
				{
					bUsesAPIDefine = Line.Contains("_API", StringComparison.Ordinal);
				}

				if (!bContainsMarkup)
				{
					bContainsMarkup = ReflectionMarkupRegex.IsMatch(Line);
				}

				ReadOnlySpan<char> LineSpan = Line.AsSpan().TrimStart();
				if (LineSpan.StartsWith("#include") && InsideDeprecationScope == 0)
				{
					ReadOnlySpan<char> IncludeSpan = LineSpan.Slice("#include".Length).TrimStart();
					if (IncludeSpan.IsEmpty)
					{
						continue;
					}
					char EndChar;
					bool TrimQuotation = true;
					if (IncludeSpan[0] == '"')
					{
						EndChar = '"';
					}
					else if (IncludeSpan[0] == '<')
					{
						EndChar = '>';
					}
					else
					{
						EndChar = ')';
						TrimQuotation = false;
					}

					if (TrimQuotation)
					{
						IncludeSpan = IncludeSpan.Slice(1);
					}

					if (IncludeSpan.Contains("HEADER_UNIT_IGNORE", StringComparison.OrdinalIgnoreCase))
					{
						continue;
					}

					int EndIndex = IncludeSpan.IndexOf(EndChar);

					if (EndIndex == -1)
					{
						continue;
					}

					// This will include the ')' at the end
					if (!TrimQuotation)
					{
						EndIndex++;
					}

					IncludeSpan = IncludeSpan.Slice(0, EndIndex);
					Includes.Add(IncludeSpan.ToString());
				}
				else if (InsideDeprecationScope != 0)
				{
					if (LineSpan.StartsWith("#endif"))
					{
						--InsideDeprecationScope;
					}
					else if (LineSpan.StartsWith("#if"))
					{
						++InsideDeprecationScope;
					}
				}
				else if (LineSpan.StartsWith("#if"))
				{
					if (Line.Contains("UE_ENABLE_INCLUDE_ORDER_DEPRECATED_", StringComparison.CurrentCulture))
					{
						++InsideDeprecationScope;
					}
				}
				else if (LineSpan.StartsWith("UE_DEPRECATED_HEADER"))
				{
					HeaderFileInfo.UnitType = HeaderUnitType.Skip;
				}
				else
				{
					int HeaderUnitIndex = Line.IndexOf("HEADER_UNIT_", StringComparison.Ordinal);
					if (HeaderUnitIndex != -1)
					{
						ReadOnlySpan<char> Span = Line.AsSpan(HeaderUnitIndex + "HEADER_UNIT_".Length);
						if (Span.StartsWith("UNSUPPORTED"))
						{
							HeaderFileInfo.UnitType = HeaderUnitType.Unsupported;
						}
						else if (Span.StartsWith("SKIP"))
						{
							HeaderFileInfo.UnitType = HeaderUnitType.Skip;
						}
					}
				}
			}

			HeaderFileInfo.bContainsMarkup = bContainsMarkup;
			HeaderFileInfo.bUsesAPIDefine = bUsesAPIDefine;
			HeaderFileInfo.Includes = Includes.ToList();
		}

		/// <summary>
		/// Gets the first included file from a source file
		/// </summary>
		/// <param name="SourceFile">The source file to parse</param>
		/// <returns>Text from the first include directive. Null if the file did not contain any include directives.</returns>
		public string? GetFirstInclude(FileItem SourceFile)
		{
			return GetSourceFileInfo(SourceFile).IncludeText;
		}

		/// <summary>
		/// Finds or adds a SourceFile class for the given file
		/// </summary>
		/// <param name="File">File to fetch the source file data for</param>
		/// <returns>SourceFile instance corresponding to the given source file</returns>
		public SourceFile GetSourceFile(FileItem File)
		{
			if (Parent != null && !File.Location.IsUnderDirectory(BaseDirectory))
			{
				return Parent.GetSourceFile(File);
			}
			else
			{
				return FileToSourceFile.AddOrUpdate(File, _ =>
				{
					return new SourceFile(File);
				},
				(k, v) =>
				{
					if (File.LastWriteTimeUtc.Ticks > v.LastWriteTimeUtc)
					{
						return new SourceFile(File);
					}
					return v;
				});
			}
		}

		/// <summary>
		/// Returns a list of inlined generated cpps that this source file contains.
		/// </summary>
		/// <param name="SourceFile">The source file to parse</param>
		/// <returns>List of marked files this source file contains</returns>
		public IList<string> GetListOfInlinedGeneratedCppFiles(FileItem SourceFile)
		{
			return GetSourceFileInfo(SourceFile).InlinedFileNames;
		}

		/// <summary>
		/// Determines whether the given file contains reflection markup
		/// </summary>
		/// <param name="HeaderFile">The source file to parse</param>
		/// <returns>True if the file contains reflection markup</returns>
		public bool ContainsReflectionMarkup(FileItem HeaderFile)
		{
			return GetHeaderFileInfo(HeaderFile).bContainsMarkup;
		}

		/// <summary>
		/// Determines whether the given file uses the *_API define
		/// </summary>
		/// <param name="HeaderFile">The source file to parse</param>
		/// <returns>True if the file uses the *_API define</returns>
		public bool UsesAPIDefine(FileItem HeaderFile)
		{
			return GetHeaderFileInfo(HeaderFile).bUsesAPIDefine;
		}

		/// <summary>
		/// Returns header unit type for a header file (and parse the file if not already cached)
		/// </summary>
		/// <param name="HeaderFile">The header file to parse</param>
		/// <returns>Header unit type</returns>
		public HeaderUnitType GetHeaderUnitType(FileItem HeaderFile)
		{
			return GetHeaderFileInfo(HeaderFile).UnitType;
		}

		/// <summary>
		/// Returns all #includes existing inside a header file (and parse the file if not already cached)
		/// </summary>
		/// <param name="HeaderFile">The header file to parse</param>
		/// <returns>List of includes</returns>
		public List<string> GetHeaderIncludes(FileItem HeaderFile)
		{
			return GetHeaderFileInfo(HeaderFile).Includes!;
		}

		/// <summary>
		/// Creates a cache hierarchy for a particular target
		/// </summary>
		/// <param name="ProjectFile">Project file for the target being built</param>
		/// <param name="Logger">Logger for output</param>
		/// <returns>Dependency cache hierarchy for the given project</returns>
		public static SourceFileMetadataCache CreateHierarchy(FileReference? ProjectFile, ILogger Logger)
		{
			SourceFileMetadataCache? Cache = null;

			if (ProjectFile == null || !Unreal.IsEngineInstalled())
			{
				FileReference EngineCacheLocation = FileReference.Combine(Unreal.EngineDirectory, "Intermediate", "Build", "SourceFileCache.bin");
				Cache = FindOrAddCache(EngineCacheLocation, Unreal.EngineDirectory, Cache, Logger);
			}

			if (ProjectFile != null)
			{
				FileReference ProjectCacheLocation = FileReference.Combine(ProjectFile.Directory, "Intermediate", "Build", "SourceFileCache.bin");
				Cache = FindOrAddCache(ProjectCacheLocation, ProjectFile.Directory, Cache, Logger);
			}

			return Cache!;
		}

		/// <summary>
		/// Enumerates all the locations of metadata caches for the given target
		/// </summary>
		/// <param name="ProjectFile">Project file for the target being built</param>
		/// <returns>Dependency cache hierarchy for the given project</returns>
		public static IEnumerable<FileReference> GetFilesToClean(FileReference? ProjectFile)
		{
			if (ProjectFile == null || !Unreal.IsEngineInstalled())
			{
				yield return FileReference.Combine(Unreal.EngineDirectory, "Intermediate", "Build", "SourceFileCache.bin");
			}
			if (ProjectFile != null)
			{
				yield return FileReference.Combine(ProjectFile.Directory, "Intermediate", "Build", "SourceFileCache.bin");
			}
		}

		/// <summary>
		/// Reads a cache from the given location, or creates it with the given settings
		/// </summary>
		/// <param name="Location">File to store the cache</param>
		/// <param name="BaseDirectory">Base directory for files that this cache should store data for</param>
		/// <param name="Parent">The parent cache to use</param>
		/// <param name="Logger"></param>
		/// <returns>Reference to a dependency cache with the given settings</returns>
		static SourceFileMetadataCache FindOrAddCache(FileReference Location, DirectoryReference BaseDirectory, SourceFileMetadataCache? Parent, ILogger Logger)
		{
			SourceFileMetadataCache Cache = Caches.GetOrAdd(Location, new Lazy<SourceFileMetadataCache>(() =>
			{
				return new SourceFileMetadataCache(Location, BaseDirectory, Parent, Logger);
				;
			})).Value;

			Debug.Assert(Cache.BaseDirectory == BaseDirectory);
			Debug.Assert(Cache.Parent == Parent);

			return Cache;
		}

		/// <summary>
		/// Save all the caches that have been modified
		/// </summary>
		public static void SaveAll()
		{
			Parallel.ForEach(Caches.Values, CacheLazy =>
			{
				SourceFileMetadataCache Cache = CacheLazy.Value;
				if (Cache.bModified)
				{
					Cache.Write();
				}
			});
		}

		/// <summary>
		/// Reads data for this dependency cache from disk
		/// </summary>
		private void Read()
		{
			try
			{
				using (BinaryArchiveReader Reader = new BinaryArchiveReader(Location))
				{
					int Version = Reader.ReadInt();
					if (Version != CurrentVersion)
					{
						Logger.LogDebug("Unable to read dependency cache from {File}; version {Version} vs current {CurrentVersion}", Location, Version, CurrentVersion);
						return;
					}

					int FileToMarkupFlagCount = Reader.ReadInt();
					for (int Idx = 0; Idx < FileToMarkupFlagCount; Idx++)
					{
						FileItem File = Reader.ReadCompactFileItem();

						HeaderFileInfo HeaderFileInfo = new HeaderFileInfo();
						HeaderFileInfo.LastWriteTimeUtc = Reader.ReadLong();
						HeaderFileInfo.bContainsMarkup = Reader.ReadBool();
						HeaderFileInfo.bUsesAPIDefine = Reader.ReadBool();
						HeaderFileInfo.UnitType = (HeaderUnitType)Reader.ReadByte();
						HeaderFileInfo.Includes = Reader.ReadList(() => Reader.ReadString())!;

						FileToHeaderFileInfo[File] = HeaderFileInfo;
					}

					int FileToInlineMarkupFlagCount = Reader.ReadInt();
					for (int Idx = 0; Idx < FileToInlineMarkupFlagCount; Idx++)
					{
						FileItem File = Reader.ReadCompactFileItem();

						SourceFileInfo SourceFileInfo = new SourceFileInfo();
						SourceFileInfo.LastWriteTimeUtc = Reader.ReadLong();
						SourceFileInfo.IncludeText = Reader.ReadString();
						SourceFileInfo.InlinedFileNames = Reader.ReadList(() => Reader.ReadString())!;

						FileToSourceFileInfo[File] = SourceFileInfo;
					}
				}
			}
			catch (Exception Ex)
			{
				Logger.LogWarning("Unable to read {Location}. See log for additional information.", Location);
				Logger.LogDebug(Ex, "{Ex}", ExceptionUtils.FormatExceptionDetails(Ex));
			}
		}

		/// <summary>
		/// Writes data for this dependency cache to disk
		/// </summary>
		private void Write()
		{
			DirectoryReference.CreateDirectory(Location.Directory);
			using (FileStream Stream = File.Open(Location.FullName, FileMode.Create, FileAccess.Write, FileShare.Read))
			{
				using (BinaryArchiveWriter Writer = new BinaryArchiveWriter(Stream))
				{
					Writer.WriteInt(CurrentVersion);

					Writer.WriteInt(FileToHeaderFileInfo.Count);
					foreach (KeyValuePair<FileItem, HeaderFileInfo> Pair in FileToHeaderFileInfo)
					{
						Writer.WriteCompactFileItem(Pair.Key);
						Writer.WriteLong(Pair.Value.LastWriteTimeUtc);
						Writer.WriteBool(Pair.Value.bContainsMarkup);
						Writer.WriteBool(Pair.Value.bUsesAPIDefine);
						Writer.WriteByte((byte)Pair.Value.UnitType);
						Writer.WriteList(Pair.Value.Includes, Item => Writer.WriteString(Item));
					}

					Writer.WriteInt(FileToSourceFileInfo.Count);
					foreach (KeyValuePair<FileItem, SourceFileInfo> Pair in FileToSourceFileInfo)
					{
						Writer.WriteCompactFileItem(Pair.Key);
						Writer.WriteLong(Pair.Value.LastWriteTimeUtc);
						Writer.WriteString(Pair.Value.IncludeText);
						Writer.WriteList(Pair.Value.InlinedFileNames, Item => Writer.WriteString(Item));
					}
				}
			}
			bModified = false;
		}
	}
}
