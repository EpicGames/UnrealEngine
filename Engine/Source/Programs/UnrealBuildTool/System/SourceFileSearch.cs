// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using EpicGames.Core;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	static class SourceFileSearch
	{
		// Certain file types should never be added to project files. These extensions must all be lowercase.
		static readonly string[] DefaultExcludedFileSuffixes = new string[]
		{
			".vcxproj",				// Visual Studio project files
			".vcxproj.filters",		// Visual Studio filter file
			".vcxproj.user",		// Visual Studio project user file
			".vcxproj.vspscc",		// Visual Studio source control state
			".sln",					// Visual Studio solution file
			".sdf",					// Visual Studio embedded symbol database file
			".suo",					// Visual Studio solution option files
			".sln.docstates.suo",	// Visual Studio temp file
			".tmp",					// Temp files used by P4 diffing (e.g. t6884t87698.tmp)
			".csproj",				// Visual Studio C# project files
			".userprefs",			// MonoDevelop project settings
			".ds_store",			// Mac Desktop Services Store hidden files
			".bin",					// Binary files
			Path.DirectorySeparatorChar + "do_not_delete.txt",		// Perforce placeholder file
		};

		// Default directory names to exclude. Must be lowercase.
		static readonly string[] DefaultExcludedDirectorySuffixes = new string[]
		{
			Path.DirectorySeparatorChar + "intermediate",
			Path.DirectorySeparatorChar + "source" + Path.DirectorySeparatorChar + "thirdparty",
			Path.DirectorySeparatorChar + "third_party",
		};

		/// Finds mouse source files
		public static List<FileReference> FindModuleSourceFiles(FileReference ModuleRulesFile, bool SearchSubdirectories = true, HashSet<DirectoryReference>? SearchedDirectories = null)
		{
			// The module's "base directory" is simply the directory where its xxx.Build.cs file is stored.  We'll always
			// harvest source files for this module in this base directory directory and all of its sub-directories.
			return SourceFileSearch.FindFiles(ModuleRulesFile.Directory, SubdirectoryNamesToExclude: null, SearchSubdirectories: SearchSubdirectories, SearchedDirectories: SearchedDirectories);
		}

		/// <summary>
		/// Fills in a project file with source files discovered in the specified list of paths
		/// </summary>
		/// <param name="DirectoryToSearch">Directory to search</param>
		/// <param name="SubdirectoryNamesToExclude">Directory base names to ignore when searching subdirectories.  Can be null.</param>
		/// <param name="SearchSubdirectories">True to include subdirectories, otherwise we only search the list of base directories</param>
		/// <param name="SearchedDirectories">If non-null, is updated with a list of the directories searched</param>
		public static List<FileReference> FindFiles(DirectoryReference DirectoryToSearch, List<string>? SubdirectoryNamesToExclude = null, bool SearchSubdirectories = true, HashSet<DirectoryReference>? SearchedDirectories = null)
		{
			return FindFiles(DirectoryItem.GetItemByDirectoryReference(DirectoryToSearch), SubdirectoryNamesToExclude, SearchSubdirectories, SearchedDirectories);
		}
		public static List<FileReference> FindFiles2(IEnumerable<DirectoryReference> DirectoriesToSearch, List<string>? SubdirectoryNamesToExclude = null, bool SearchSubdirectories = true, HashSet<DirectoryReference>? SearchedDirectories = null)
		{
			// Build a list of directory names that we don't want to search under. We always ignore intermediate directories.
			string[] ExcludedDirectorySuffixes;
			if (SubdirectoryNamesToExclude == null)
			{
				ExcludedDirectorySuffixes = DefaultExcludedDirectorySuffixes;
			}
			else
			{
				ExcludedDirectorySuffixes = SubdirectoryNamesToExclude.Select(x => $"{Path.DirectorySeparatorChar}{x}").Union(DefaultExcludedDirectorySuffixes).ToArray();
			}

			// Find all the files
			List<FileReference> FoundFiles = new List<FileReference>();

			ParallelQueue queue = new();
			foreach (DirectoryReference dir in DirectoriesToSearch)
			{
				queue.Enqueue(() =>
				{
					DirectoryItem dirItem = DirectoryItem.GetItemByDirectoryReference(dir);
					if (SearchSubdirectories)
					{
						FindFilesInternalRecursive(dirItem, ExcludedDirectorySuffixes, FoundFiles, SearchedDirectories, queue);
					}
					else
					{
						FindFilesInternal(dirItem, FoundFiles, SearchedDirectories, queue);
					}
				});
			}
			return FoundFiles;
		}

		public static List<FileReference> FindFiles(DirectoryItem DirectoryToSearch, List<string>? SubdirectoryNamesToExclude = null, bool SearchSubdirectories = true, HashSet<DirectoryReference>? SearchedDirectories = null)
		{
			// Build a list of directory names that we don't want to search under. We always ignore intermediate directories.
			string[] ExcludedDirectorySuffixes;
			if (SubdirectoryNamesToExclude == null)
			{
				ExcludedDirectorySuffixes = DefaultExcludedDirectorySuffixes;
			}
			else
			{
				ExcludedDirectorySuffixes = SubdirectoryNamesToExclude.Select(x => $"{Path.DirectorySeparatorChar}{x}").Union(DefaultExcludedDirectorySuffixes).ToArray();
			}

			// Find all the files
			List<FileReference> FoundFiles = new List<FileReference>();
			if (SearchSubdirectories)
			{
				FindFilesInternalRecursive(DirectoryToSearch, ExcludedDirectorySuffixes, FoundFiles, SearchedDirectories, null);
			}
			else
			{
				FindFilesInternal(DirectoryToSearch, FoundFiles, SearchedDirectories, null);
			}
			return FoundFiles;
		}

		static bool FindFilesInternal(DirectoryItem Directory, List<FileReference> FoundFiles, HashSet<DirectoryReference>? SearchedDirectories, ParallelQueue? queue)
		{
			if (queue != null && SearchedDirectories != null)
			{
				lock (SearchedDirectories)
				{
					SearchedDirectories.Add(Directory.Location);
				}
			}
			else
			{
				SearchedDirectories?.Add(Directory.Location);
			}

			List<FileReference> originalFoundFiles = FoundFiles;
			if (queue != null)
			{
				FoundFiles = [];
			}

			foreach (FileItem File in Directory.EnumerateFiles())
			{
				// Ignore Mac resource fork files on non-HFS partitions
				if (File.Name.StartsWith("._", StringComparison.Ordinal))
				{
					continue;
				}

				if (IsExcluded(DefaultExcludedFileSuffixes, File.Name))
				{
					continue;
				}

				if (File.Name == ".ubtignore")
				{
					return false;
				}

				FoundFiles.Add(File.Location);
			}

			if (queue != null)
			{
				lock (originalFoundFiles)
				{
					originalFoundFiles.AddRange(FoundFiles);
				}
			}

			return true;
		}

		static void FindFilesInternalRecursive(DirectoryItem Directory, string[] ExcludedDirectorySuffixes, List<FileReference> FoundFiles, HashSet<DirectoryReference>? SearchedDirectories, ParallelQueue? queue)
		{
			if (!FindFilesInternal(Directory, FoundFiles, SearchedDirectories, queue))
			{
				return;
			}

			foreach (DirectoryItem SubDirectory in Directory.EnumerateDirectories())
			{
				if (IsExcluded(ExcludedDirectorySuffixes, SubDirectory.Location.FullName))
				{
					continue;
				}

				if (SubDirectory.Name.StartsWith("__External", StringComparison.Ordinal))
				{
					continue;
				}

				if (queue != null)
				{
					queue.Enqueue(() => FindFilesInternalRecursive(SubDirectory, ExcludedDirectorySuffixes, FoundFiles, SearchedDirectories, queue));
				}
				else
				{
					FindFilesInternalRecursive(SubDirectory, ExcludedDirectorySuffixes, FoundFiles, SearchedDirectories, queue);
				}
			}
		}

		static bool IsExcluded(string[] excludeSuffixes, string testString)
		{
			foreach (string excludeSuffix in excludeSuffixes)
			{
				if (testString.EndsWith(excludeSuffix, FileSystemReference.Comparison))
				{
					return true;
				}
			}
			return false;
		}
	}
}
