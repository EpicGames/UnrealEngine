// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using System.Linq;
using System.Threading;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// Defines an interface which allows querying the working set. Files which are in the working set are excluded from unity builds, to improve iterative compile times.
	/// </summary>
	interface ISourceFileWorkingSet : IDisposable
	{
		/// <summary>
		/// Checks if the given file is part of the working set
		/// </summary>
		/// <param name="File">File to check</param>
		/// <returns>True if the file is part of the working set, false otherwise</returns>
		abstract bool Contains(FileItem File);
	}

	/// <summary>
	/// Implementation of ISourceFileWorkingSet which does not contain any files
	/// </summary>
	class EmptySourceFileWorkingSet : ISourceFileWorkingSet
	{
		/// <summary>
		/// Dispose of the current instance.
		/// </summary>
		public void Dispose()
		{
		}

		/// <summary>
		/// Checks if the given file is part of the working set
		/// </summary>
		/// <param name="File">File to check</param>
		/// <returns>True if the file is part of the working set, false otherwise</returns>
		public bool Contains(FileItem File)
		{
			return false;
		}
	}

	/// <summary>
	/// Queries the working set for files tracked by Perforce.
	/// </summary>
	class PerforceSourceFileWorkingSet : ISourceFileWorkingSet
	{
		/// <summary>
		/// Dispose of the current instance.
		/// </summary>
		public void Dispose()
		{
		}

		/// <summary>
		/// Checks if the given file is part of the working set
		/// </summary>
		/// <param name="File">File to check</param>
		/// <returns>True if the file is part of the working set, false otherwise</returns>
		public bool Contains(FileItem File)
		{
			// Generated .cpp files should never be treated as part of the working set
			if (File.HasExtension(".gen.cpp"))
			{
				return false;
			}

			// Check if the file is read-only
			try
			{
				return !File.Attributes.HasFlag(FileAttributes.ReadOnly);
			}
			catch (FileNotFoundException)
			{
				return false;
			}
		}
	}

	/// <summary>
	/// Queries the working set for files tracked by Git.
	/// </summary>
	class GitSourceFileWorkingSet : ISourceFileWorkingSet
	{
		readonly DirectoryReference RootDir;
		Process? BackgroundProcess;
		readonly Lock BackgroundProcessDisposeLock = new();
		readonly Lazy<bool> BackgroundProcessDone;
		readonly HashSet<FileReference> Files;
		readonly List<DirectoryReference> Directories;
		readonly List<string> ErrorOutput;
		readonly GitSourceFileWorkingSet? Inner;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="GitPath">Path to the Git executable</param>
		/// <param name="RootDir">Root directory to run queries from (typically the directory containing the .git folder, to ensure all subfolders can be searched)</param>
		/// <param name="Inner">An inner working set. This allows supporting multiple Git repositories (one containing the engine, another containing the project, for example)</param>
		/// <param name="Logger">Logger for output</param>
		public GitSourceFileWorkingSet(string GitPath, DirectoryReference RootDir, GitSourceFileWorkingSet? Inner, ILogger Logger)
		{
			this.RootDir = RootDir;
			Files = [];
			Directories = [];
			ErrorOutput = [];
			this.Inner = Inner;

			Logger.LogInformation("Using 'git status' to determine working set for adaptive non-unity build ({RootDir}).", RootDir);

			string Arguments = "--no-optional-locks status --porcelain";
			BackgroundProcess = new()
			{
				StartInfo = new()
				{
					FileName = GitPath,
					Arguments = Arguments,
					WorkingDirectory = RootDir.FullName,
					RedirectStandardOutput = true,
					RedirectStandardError = true,
					UseShellExecute = false
				}
			};
			try
			{
				BackgroundProcess.ErrorDataReceived += ErrorDataReceived;
				BackgroundProcess.OutputDataReceived += OutputDataReceived;
				BackgroundProcess.Start();
				BackgroundProcess.BeginErrorReadLine();
				BackgroundProcess.BeginOutputReadLine();
			}
			catch
			{
				Logger.LogInformation("Failed to start git process using command line {GitPath} {Arguments}", GitPath, Arguments);
				BackgroundProcess.Dispose();
				BackgroundProcess = null;
			}

			BackgroundProcessDone = new Lazy<bool>(() =>
			{
				if (BackgroundProcess == null)
				{
					return true;
				}

				using ITimelineEvent _ = Timeline.ScopeEvent("GitWait");

				if (BackgroundProcess.WaitForExit(500) == false)
				{
					Logger.LogInformation("Waiting for 'git status' command to complete");
				}
				if (BackgroundProcess.WaitForExit(15000) == false)
				{
					Logger.LogInformation("Terminating git child process due to timeout");
					try
					{
						BackgroundProcess.Kill();
					}
					catch
					{
					}
				}
				BackgroundProcess.WaitForExit();
				lock (BackgroundProcessDisposeLock)
				{
					BackgroundProcess.Dispose();
					BackgroundProcess = null;
				}
				return true;
			});
		}

		/// <summary>
		/// Dispose of this object
		/// </summary>
		public void Dispose()
		{
			lock (BackgroundProcessDisposeLock)
			{
				if (BackgroundProcess != null && !BackgroundProcess.HasExited)
				{
					try
					{
						BackgroundProcess.Kill();
						BackgroundProcess.Dispose();
						BackgroundProcess = null;
					}
					catch
					{
					}
				}
			}
			bool _ = BackgroundProcessDone.Value;

			Inner?.Dispose();
		}

		/// <summary>
		/// Checks if the given file is part of the working set
		/// </summary>
		/// <param name="File">File to check</param>
		/// <returns>True if the file is part of the working set, false otherwise</returns>
		public bool Contains(FileItem File)
		{
			// Generated .cpp files should never be treated as part of the working set
			if (File.HasExtension(".gen.cpp"))
			{
				return false;
			}

			bool _ = BackgroundProcessDone.Value;

			if (Files.Contains(File.Location) || Directories.Any(x => File.Location.IsUnderDirectory(x)))
			{
				return true;
			}
			if (Inner != null && Inner.Contains(File))
			{
				return true;
			}
			return false;
		}

		/// <summary>
		/// Parse output text from Git
		/// </summary>
		void OutputDataReceived(object Sender, DataReceivedEventArgs Args)
		{
			if (Args.Data != null && Args.Data.Length > 3 && Args.Data[2] == ' ')
			{
				int MinIdx = 3;
				int MaxIdx = Args.Data.Length;

				while (MinIdx < MaxIdx && Char.IsWhiteSpace(Args.Data[MinIdx]))
				{
					MinIdx++;
				}

				while (MinIdx < MaxIdx && Char.IsWhiteSpace(Args.Data[MaxIdx - 1]))
				{
					MaxIdx--;
				}

				int ArrowIdx = Args.Data.IndexOf(" -> ", MinIdx, MaxIdx - MinIdx, StringComparison.Ordinal);
				if (ArrowIdx == -1)
				{
					AddPath(Args.Data.Substring(MinIdx, MaxIdx - MinIdx));
				}
				else
				{
					AddPath(Args.Data.Substring(MinIdx, ArrowIdx - MinIdx));
					int ArrowEndIdx = ArrowIdx + 4;
					AddPath(Args.Data.Substring(ArrowEndIdx, MaxIdx - ArrowEndIdx));
				}
			}
		}

		/// <summary>
		/// Handle error output text from Git
		/// </summary>
		void ErrorDataReceived(object Sender, DataReceivedEventArgs Args)
		{
			if (Args.Data != null)
			{
				ErrorOutput.Add(Args.Data);
			}
		}

		/// <summary>
		/// Add a path to the working set
		/// </summary>
		/// <param name="Path">Path to be added</param>
		void AddPath(string Path)
		{
			if (Path.EndsWith('/'))
			{
				Directories.Add(DirectoryReference.Combine(RootDir, Path));
			}
			else
			{
				Files.Add(FileReference.Combine(RootDir, Path));
			}
		}
	}

	/// <summary>
	/// Utility class for ISourceFileWorkingSet
	/// </summary>
	static class SourceFileWorkingSet
	{
		enum ProviderType
		{
			None,
			Default,
			Perforce,
			Git
		};

#pragma warning disable IDE0044 // Make field readonly - these private static fields are set by XML configuration.
		/// <summary>
		/// Sets the provider to use for determining the working set.
		/// </summary>
		[XmlConfigFile]
		static ProviderType Provider = ProviderType.Default;
#pragma warning restore IDE0044

		/// <summary>
		/// Sets the path to use for the repository. Interpreted relative to the Unreal Engine root directory (the folder above the Engine folder) -- if relative.
		/// </summary>
		[XmlConfigFile]
		public static string? RepositoryPath = null;

		/// <summary>
		/// Sets the path to use for the Git executable. Defaults to "git" (assuming it is in the PATH).
		/// </summary>
		[XmlConfigFile]
		public static string GitPath = "git";

		/// <summary>
		/// Create an ISourceFileWorkingSet instance suitable for the given project or root directory
		/// </summary>
		/// <param name="RootDir">The root directory</param>
		/// <param name="ProjectDirs">The project directories</param>
		/// <param name="Logger">Logger for output</param>
		/// <returns>Working set instance for the given directory</returns>
		public static ISourceFileWorkingSet Create(DirectoryReference RootDir, IEnumerable<DirectoryReference> ProjectDirs, ILogger Logger)
		{
			if (Provider == ProviderType.None || ProjectFileGenerator.bGenerateProjectFiles)
			{
				return new EmptySourceFileWorkingSet();
			}
			else if (Provider == ProviderType.Git)
			{
				ISourceFileWorkingSet? WorkingSet;
				if (!String.IsNullOrEmpty(RepositoryPath))
				{
					WorkingSet = new GitSourceFileWorkingSet(GitPath, DirectoryReference.Combine(RootDir, RepositoryPath), null, Logger);
				}
				else if (!TryCreateGitWorkingSet(RootDir, ProjectDirs, Logger, out WorkingSet))
				{
					WorkingSet = new GitSourceFileWorkingSet(GitPath, RootDir, null, Logger);
				}
				return WorkingSet;
			}
			else if (Provider == ProviderType.Perforce)
			{
				return new PerforceSourceFileWorkingSet();
			}
			else
			{
				ISourceFileWorkingSet? WorkingSet;
				if (TryCreateGitWorkingSet(RootDir, ProjectDirs, Logger, out WorkingSet))
				{
					return WorkingSet;
				}
				else if (TryCreatePerforceWorkingSet(RootDir, out WorkingSet))
				{
					return WorkingSet;
				}
			}
			return new EmptySourceFileWorkingSet();
		}

		static bool TryCreateGitWorkingSet(DirectoryReference RootDir, IEnumerable<DirectoryReference> ProjectDirs, ILogger Logger, [NotNullWhen(true)] out ISourceFileWorkingSet? OutWorkingSet)
		{
			GitSourceFileWorkingSet? WorkingSet = null;

			// Create the working set for the engine directory
			if (DirectoryReference.Exists(DirectoryReference.Combine(RootDir, ".git")) || FileReference.Exists(FileReference.Combine(RootDir, ".git")))
			{
				WorkingSet = new GitSourceFileWorkingSet(GitPath, RootDir, WorkingSet, Logger);
			}

			// Try to create a working set for the project directory
			foreach (DirectoryReference ProjectDir in ProjectDirs)
			{
				if (WorkingSet == null || !ProjectDir.IsUnderDirectory(RootDir))
				{
					if (DirectoryReference.Exists(DirectoryReference.Combine(ProjectDir, ".git")) || FileReference.Exists(FileReference.Combine(ProjectDir, ".git")))
					{
						WorkingSet = new GitSourceFileWorkingSet(GitPath, ProjectDir, WorkingSet, Logger);
					}
					else if (DirectoryReference.Exists(DirectoryReference.Combine(ProjectDir.ParentDirectory!, ".git")) || FileReference.Exists(FileReference.Combine(ProjectDir.ParentDirectory!, ".git")))
					{
						WorkingSet = new GitSourceFileWorkingSet(GitPath, ProjectDir.ParentDirectory!, WorkingSet, Logger);
					}
				}
			}

			// Set the output value
			OutWorkingSet = WorkingSet;
			return OutWorkingSet != null;
		}

		static bool TryCreatePerforceWorkingSet(DirectoryReference RootDir, [NotNullWhen(true)] out ISourceFileWorkingSet? OutWorkingSet)
		{
			PerforceSourceFileWorkingSet? WorkingSet = null;
			// If an installed engine, or the root directory contains any read-only files assume this is a perforce working set
			if (Unreal.IsEngineInstalled() || DirectoryReference.EnumerateFiles(RootDir).Any(x => x.ToFileInfo().Attributes.HasFlag(FileAttributes.ReadOnly)))
			{
				WorkingSet = new PerforceSourceFileWorkingSet();
			}

			// Set the output value
			OutWorkingSet = WorkingSet;
			return OutWorkingSet != null;
		}
	}
}
