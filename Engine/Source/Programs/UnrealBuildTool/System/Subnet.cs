// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Net;

namespace UnrealBuildTool
{
	/// <summary>
	/// Represents an IP subnet
	/// </summary>
	class Subnet
	{
		/// <summary>
		/// The prefix address
		/// </summary>
		public IPAddress Prefix
		{
			get;
			private set;
		}

		/// <summary>
		/// Number of bits that need to match in an IP address for this subnet
		/// </summary>
		public int MaskBits
		{
			get;
			private set;
		}

		/// <summary>
		/// Bytes corresponding to the prefix address
		/// </summary>
		readonly byte[] PrefixBytes;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="Prefix">The prefix IP address</param>
		/// <param name="MaskBits">Number of bits to match for this subnet</param>
		public Subnet(IPAddress Prefix, int MaskBits)
		{
			this.Prefix = Prefix;
			PrefixBytes = Prefix.GetAddressBytes();
			this.MaskBits = MaskBits;
		}

		/// <summary>
		/// Parses a subnet from a string
		/// </summary>
		/// <param name="Text">The string to parse</param>
		/// <returns>New subnet that was parsed</returns>
		public static Subnet Parse(string Text)
		{
			int SlashIdx = Text.IndexOf('/', StringComparison.Ordinal);
			IPAddress Address = IPAddress.Parse(Text.Substring(0, SlashIdx));
			return new Subnet(Address, Int32.Parse(Text.Substring(SlashIdx + 1)));
		}

		/// <summary>
		/// Checks if this subnet contains the given address
		/// </summary>
		/// <param name="Address">IP address to test</param>
		/// <returns>True if the subnet contains this address</returns>
		public bool Contains(IPAddress Address)
		{
			return Contains(Address.GetAddressBytes());
		}

		/// <summary>
		/// Checks if this subnet contains the given address bytes
		/// </summary>
		/// <param name="Bytes">Bytes of the IP address to text</param>
		/// <returns>True if the subnet contains this address</returns>
		public bool Contains(byte[] Bytes)
		{
			if (Bytes.Length != PrefixBytes.Length)
			{
				return false;
			}

			int Index = 0;
			int RemainingBits = MaskBits;

			// Check all the full bytes first
			for (; RemainingBits >= 8; RemainingBits -= 8)
			{
				if (Bytes[Index] != PrefixBytes[Index])
				{
					return false;
				}
				Index++;
			}

			// Check the remaining bits
			if (RemainingBits > 0)
			{
				int LastMaskByte = ((1 << RemainingBits) - 1) << (8 - RemainingBits);
				if ((Bytes[Index] & LastMaskByte) != (PrefixBytes[Index] & LastMaskByte))
				{
					return false;
				}
			}

			return true;
		}

		/// <summary>
		/// Formats this subnet as text
		/// </summary>
		/// <returns>String representation of the subnet</returns>
		public override string ToString()
		{
			return String.Format("{0}/{1}", Prefix, MaskBits);
		}
	}
}
