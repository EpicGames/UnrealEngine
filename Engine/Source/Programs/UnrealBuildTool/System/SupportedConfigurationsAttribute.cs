// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace UnrealBuildTool
{
	/// <summary>
	/// Attribute which can be applied to a TargetRules-dervied class to indicate which configurations it supports
	/// </summary>
	[AttributeUsage(AttributeTargets.Class, AllowMultiple = true)]
	public sealed class SupportedConfigurationsAttribute : Attribute
	{
		/// <summary>
		/// Array of supported platforms
		/// </summary>
		public UnrealTargetConfiguration[] Configurations { get; }

		/// <summary>
		/// Initialize the attribute with a list of configurations
		/// </summary>
		/// <param name="configurations">Variable-length array of configuration arguments</param>
		public SupportedConfigurationsAttribute(params UnrealTargetConfiguration[] configurations)
		{
			Configurations = configurations;
		}
	}
}
