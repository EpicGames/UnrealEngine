// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;

namespace UnrealBuildTool
{
	/// <summary>
	/// Attribute which can be applied to a TargetRules-derived or ModuleRules-derived class to indicate which platforms it supports
	/// </summary>
	[AttributeUsage(AttributeTargets.Class, AllowMultiple = true)]
	public sealed class SupportedPlatformGroupsAttribute : SupportedPlatformsAttribute
	{
		/// <summary>
		/// Initialize the attribute with a list of platform groups
		/// </summary>
		/// <param name="platformGroups">Variable-length array of platform group arguments</param>
		[System.Diagnostics.CodeAnalysis.SuppressMessage("Design", "CA1019:Define accessors for attribute arguments", Justification = "Unneeded")]
		public SupportedPlatformGroupsAttribute(params string[] platformGroups) : base(GetPlatformsForGroups(platformGroups))
		{
		}

		private static string[] GetPlatformsForGroups(params string[] platformGroups)
		{
			HashSet<UnrealTargetPlatform> supportedPlatforms = new();
			try
			{
				foreach (string name in platformGroups)
				{
					if (UnrealPlatformGroup.TryParse(name, out UnrealPlatformGroup group))
					{
						supportedPlatforms.UnionWith(UnrealTargetPlatform.GetValidPlatforms().Where(x => x.IsInGroup(group)));
						continue;
					}
					throw new BuildException(String.Format("The platform group name {0} is not a valid platform group name. Valid names are ({1})", name,
						String.Join(",", UnrealPlatformGroup.GetValidGroupNames())));
				}
			}
			catch (BuildException ex)
			{
				EpicGames.Core.ExceptionUtils.AddContext(ex, $"while parsing a SupportedPlatformGroups attribute '{String.Join(',', platformGroups)}'");
				throw;
			}

			return supportedPlatforms.Select(x => x.ToString()).ToArray();
		}
	}
}
