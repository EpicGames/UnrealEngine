// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace UnrealBuildTool
{
	/// <summary>
	/// Attribute which can be applied to a TargetRules-derived or ModuleRules-derived class to indicate which platforms it supports
	/// </summary>
	[AttributeUsage(AttributeTargets.Class, AllowMultiple = true)]
	[System.Diagnostics.CodeAnalysis.SuppressMessage("Performance", "CA1813:Avoid unsealed attributes")]
	public class SupportedPlatformsAttribute : Attribute
	{
		/// <summary>
		/// Array of supported platforms
		/// </summary>
		public UnrealTargetPlatform[] Platforms { get; }

		/// <summary>
		/// Initialize the attribute with a list of platforms
		/// </summary>
		/// <param name="platforms">Variable-length array of platform arguments</param>
		[System.Diagnostics.CodeAnalysis.SuppressMessage("Design", "CA1019:Define accessors for attribute arguments", Justification = "Unneeded")]
		public SupportedPlatformsAttribute(params string[] platforms)
		{
			try
			{
				Platforms = Array.ConvertAll(platforms, x => UnrealTargetPlatform.Parse(x));
			}
			catch (BuildException ex)
			{
				EpicGames.Core.ExceptionUtils.AddContext(ex, "while parsing a SupportedPlatforms attribute");
				throw;
			}
		}

		/// <summary>
		/// Initialize the attribute with all the platforms in a given category
		/// </summary>
		/// <param name="category">Category of platforms to add</param>
		[System.Diagnostics.CodeAnalysis.SuppressMessage("Design", "CA1019:Define accessors for attribute arguments", Justification = "Unneeded")]
		public SupportedPlatformsAttribute(UnrealPlatformClass category)
		{
			Platforms = Utils.GetPlatformsInClass(category);
		}
	}
}
