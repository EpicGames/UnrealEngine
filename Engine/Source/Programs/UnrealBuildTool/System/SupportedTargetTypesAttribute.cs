// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;

namespace UnrealBuildTool
{
	/// <summary>
	/// Attribute which can be applied to a TargetRules-derived class to indicate which target types it supports
	/// </summary>
	[AttributeUsage(AttributeTargets.Class, AllowMultiple = true)]
	public sealed class SupportedTargetTypesAttribute : Attribute
	{
		/// <summary>
		/// Enumerable of supported target types
		/// </summary>
		public IEnumerable<TargetType> TargetTypes => _targetTypes;

		/// <summary>
		/// HashSet of supported target types
		/// </summary>
		private readonly HashSet<TargetType> _targetTypes;

		/// <summary>
		/// Initialize the attribute with a list of target types
		/// </summary>
		/// <param name="targetTypes">Variable-length array of target type arguments</param>
		public SupportedTargetTypesAttribute(params TargetType[] targetTypes)
		{
			_targetTypes = [.. targetTypes];

			// Client or Server imply Game is supported
			if (_targetTypes.Contains(TargetType.Client) || _targetTypes.Contains(TargetType.Server))
			{
				_targetTypes.Add(TargetType.Game);
			}
		}
	}
}
