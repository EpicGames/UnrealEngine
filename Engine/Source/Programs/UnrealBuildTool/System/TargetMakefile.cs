// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Reflection;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.UHT.Utils;
using Microsoft.Extensions.Logging;
using OpenTracing.Util;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	class TargetMakefileSourceFileInfo
	{
		public FileItem? SourceFileItem;

		/// <summary>
		/// Hash of all the gen.cpp files being inlined in this source file
		/// </summary>
		public int InlineGenCppHash = 0;

		public TargetMakefileSourceFileInfo(FileItem FileItem)
		{
			SourceFileItem = FileItem;
		}

		public TargetMakefileSourceFileInfo(BinaryArchiveReader Reader)
		{
			SourceFileItem = Reader.ReadFileItem();
			InlineGenCppHash = Reader.ReadInt();
		}

		public void Write(BinaryArchiveWriter Writer)
		{
			Writer.WriteFileItem(SourceFileItem);
			Writer.WriteInt(InlineGenCppHash);
		}

		/// <summary>
		/// Helper function to calculate a hash for the passed in inline gen.cpp file names
		/// </summary>
		/// <param name="InlinedGenCppNames">Gen.cpp file names</param>
		/// <returns>Hash of file names</returns>
		public static int CalculateInlineGenCppHash(IEnumerable<string> InlinedGenCppNames)
		{
			int InlineHash = 0;
			foreach (string FileName in InlinedGenCppNames)
			{
				int Result = 0;
				if (FileName != null)
				{
					for (int Idx = 0; Idx < FileName.Length; Idx++)
					{
						Result = (Result * 13) + FileName[Idx];
					}
				}
				InlineHash += Result;
			}
			return InlineHash;
		}
	}

	class TargetMakefileBuilder : IActionGraphBuilder
	{
		private readonly ILogger Logger;
		public TargetMakefile Makefile { get; }

		public FileHashCache FileHashCache { get; }

		private readonly ConcurrentBag<Task> WriteTasks = new();

		public TargetMakefileBuilder(TargetMakefile InMakefile, FileHashCache fileHashCache, ILogger InLogger)
		{
			Logger = InLogger;
			FileHashCache = fileHashCache;
			Makefile = InMakefile;
		}

		/// <inheritdoc/>
		public void AddAction(IExternalAction Action)
		{
			lock (Makefile.Actions)
			{
				Makefile.Actions.Add(Action);
			}
		}

		/// <inheritdoc/>
		public void CreateIntermediateTextFile(FileItem FileItem, string Contents, bool AllowAsync = true)
		{
			AddInternalDependency(FileItem);

			Task? task = FileHashCache.WriteFileIfChanged(FileItem, Contents, Logger, AllowAsync);
			if (task != null)
			{
				WriteTasks.Add(task);
			}
		}

		/// <inheritdoc/>
		public void CreateIntermediateTextFile(FileItem FileItem, IEnumerable<string> ContentLines, bool AllowAsync = true)
		{
			AddInternalDependency(FileItem);

			Task? task = FileHashCache.WriteFileIfChanged(FileItem, ContentLines, Logger, AllowAsync);
			if (task != null)
			{
				WriteTasks.Add(task);
			}
		}

		/// <inheritdoc/>
		public void AddSourceDir(DirectoryItem SourceDir)
		{
			lock (Makefile.SourceDirectories)
			{
				Makefile.SourceDirectories.Add(SourceDir);
			}
		}

		/// <inheritdoc/>
		public void AddSourceFiles(DirectoryItem SourceDir, FileItem[] SourceFiles)
		{
			lock (Makefile.DirectoryToSourceFiles)
			{
				Makefile.DirectoryToSourceFiles[SourceDir] = SourceFiles.Select(fi => new TargetMakefileSourceFileInfo(fi)).ToArray();
			}
		}

		/// <inheritdoc/>
		public virtual void AddHeaderFiles(FileItem[] HeaderFiles)
		{
			lock (Makefile.HeaderFiles)
			{
				Makefile.HeaderFiles.UnionWith(HeaderFiles);
			}
		}

		/// <inheritdoc/>
		public void AddDiagnostic(string Message)
		{
			lock (Makefile.Diagnostics)
			{
				if (!Makefile.Diagnostics.Contains(Message))
				{
					Makefile.Diagnostics.Add(Message);
				}
			}
		}

		/// <inheritdoc/>
		public void AddFileToWorkingSet(FileItem File)
		{
			lock (Makefile.WorkingSet)
			{
				Makefile.WorkingSet.Add(File);
			}
		}

		/// <inheritdoc/>
		public void AddCandidateForWorkingSet(FileItem File)
		{
			lock (Makefile.CandidatesForWorkingSet)
			{
				Makefile.CandidatesForWorkingSet.Add(File);
			}
		}

		/// <inheritdoc/>
		public void SetOutputItemsForModule(string ModuleName, FileItem[] OutputItems)
		{
			lock (Makefile.ModuleNameToOutputItems)
			{
				Makefile.ModuleNameToOutputItems[ModuleName] = OutputItems;
			}
		}

		public void AddOutputFiles(IEnumerable<FileItem> OutputFiles)
		{
			lock (Makefile.OutputItems)
			{
				Makefile.OutputItems.AddRange(OutputFiles);
			}
		}

		public void AddInternalDependency(FileItem Dependency)
		{
			lock (Makefile.InternalDependencies)
			{
				Makefile.InternalDependencies.Add(Dependency);
			}
		}

		internal void WaitOnWriteTasks()
		{
			Task.WaitAll(WriteTasks.ToArray());
		}
	}

	/// <summary>
	/// Cached list of actions that need to be executed to build a target, along with the information needed to determine whether they are valid.
	/// </summary>
	class TargetMakefile
	{
		/// <summary>
		/// The version number to write
		/// </summary>
		public const int CurrentVersion = 37;

		/// <summary>
		/// The time at which the makefile was created
		/// </summary>
		public DateTime CreateTimeUtc;

		/// <summary>
		/// The time at which the makefile was modified
		/// </summary>
		public DateTime ModifiedTimeUtc;

		/// <summary>
		/// Additional diagnostic output to print before building this target (toolchain version, etc...)
		/// </summary>
		public List<string> Diagnostics;

		/// <summary>
		/// Any additional information about the build environment which the platform can use to invalidate the makefile
		/// </summary>
		public string ExternalMetadata;

		/// <summary>
		/// The main executable output by this target
		/// </summary>
		public FileReference ExecutableFile;

		/// <summary>
		/// Path to the receipt file for this target
		/// </summary>
		public FileReference ReceiptFile;

		/// <summary>
		/// The project intermediate directory
		/// </summary>
		public DirectoryReference ProjectIntermediateDirectory;

		/// <summary>
		/// The project intermediate directory without architecture info
		/// </summary>
		public DirectoryReference ProjectIntermediateDirectoryNoArch;

		/// <summary>
		/// Type of the target
		/// </summary>
		public TargetType TargetType;

		/// <summary>
		/// Is test target?
		/// </summary>
		public bool IsTestTarget;

		/// <summary>
		/// Map of config file keys to values. Makefile will be invalidated if any of these change.
		/// </summary>
		public ConfigValueTracker ConfigValueTracker;

		/// <summary>
		/// Whether the target should be deployed after being built
		/// </summary>
		public bool bDeployAfterCompile;

		/// <summary>
		/// Collection of all located UBT C# plugins regardless of if they are in an enabled plugin (project file)
		/// </summary>
		public FileReference[]? UbtPlugins;

		/// <summary>
		/// Collection of UBT C# plugins contained in enabled plugins (project file)
		/// </summary>
		public FileReference[]? EnabledUbtPlugins;

		/// <summary>
		/// Collection of UHT C# plugins contained in enabled plugins (target assembly file)
		/// </summary>
		public FileReference[]? EnabledUhtPlugins;

		/// <summary>
		/// The array of command-line arguments. The makefile will be invalidated whenever these change.
		/// </summary>
		public string[]? AdditionalArguments;

		/// <summary>
		/// The array of command-line arguments passed to UnrealHeaderTool.
		/// </summary>
		public string[]? UHTAdditionalArguments;

		/// <summary>
		/// Scripts which should be run before building anything
		/// </summary>
		public FileReference[] PreBuildScripts = Array.Empty<FileReference>();

		/// <summary>
		/// Targets which should be build before building anything
		/// </summary>
		public TargetInfo[] PreBuildTargets = Array.Empty<TargetInfo>();

		/// <summary>
		/// LogEvents captured during makefile creation
		/// </summary>
		public LogEvent[] LogEvents;

		/// <summary>
		/// Every action in the action graph
		/// </summary>
		public List<IExternalAction> Actions;

		/// <summary>
		/// Environment variables that we'll need in order to invoke the platform's compiler and linker
		/// </summary>
		// @todo ubtmake: Really we want to allow a different set of environment variables for every Action.  This would allow for targets on multiple platforms to be built in a single assembling phase.  We'd only have unique variables for each platform that has actions, so we'd want to make sure we only store the minimum set.
		public readonly List<Tuple<string, string>> EnvironmentVariables = new List<Tuple<string, string>>();

		/// <summary>
		/// The final output items for all target
		/// </summary>
		public List<FileItem> OutputItems;

		/// <summary>
		/// Maps module names to output items
		/// </summary>
		public Dictionary<string, FileItem[]> ModuleNameToOutputItems;

		/// <summary>
		/// List of game module names, for hot-reload
		/// </summary>
		public HashSet<string> HotReloadModuleNames;

		/// <summary>
		/// List of all source directories
		/// </summary>
		public List<DirectoryItem> SourceDirectories;

		/// <summary>
		/// Set of all source directories. Any files being added or removed from these directories will invalidate the makefile.
		/// </summary>
		public Dictionary<DirectoryItem, TargetMakefileSourceFileInfo[]> DirectoryToSourceFiles;

		/// <summary>
		/// Set of all known header files, changes will not invalidate the makefile but are used to determine if a header is included in the target.
		/// </summary>
		public HashSet<FileItem> HeaderFiles;

		/// <summary>
		/// The set of source files that UnrealBuildTool determined to be part of the programmer's "working set". Used for adaptive non-unity builds.
		/// </summary>
		public HashSet<FileItem> WorkingSet = new HashSet<FileItem>();

		/// <summary>
		/// Set of files which are currently not part of the working set, but could be.
		/// </summary>
		public HashSet<FileItem> CandidatesForWorkingSet = new HashSet<FileItem>();

		/// <summary>
		/// Maps each target to a list of UObject module info structures
		/// </summary>
		public List<UHTModuleInfo> UObjectModules;

		/// <summary>
		/// Used to map names of modules to their .Build.cs filename
		/// </summary>
		public List<UHTModuleHeaderInfo> UObjectModuleHeaders = new List<UHTModuleHeaderInfo>();

		/// <summary>
		/// Target-wide settings for UHT that should not be in UHTModuleInfo
		/// </summary>
		public UHTTargetSettings UHTTargetSettings = new();

#if __VPROJECT_AVAILABLE__
		/// <summary>
		/// All build modules containing Verse code
		/// </summary>
		public List<VNIModuleInfo> VNIModules;

		/// <summary>
		/// If Verse should use the BPVM backend
		/// </summary>
		public bool bUseVerseBPVM = true;
#endif

		/// <summary>
		/// List of all plugin names. The makefile will be considered invalid if any of these changes, or new plugins are added.
		/// </summary>
		public HashSet<FileItem> PluginFiles;

		/// <summary>
		/// Set of external (ie. user-owned) files which will cause the makefile to be invalidated if modified after
		/// </summary>
		public HashSet<FileItem> ExternalDependencies;

		/// <summary>
		/// Set of internal (eg. response files, unity files) which will cause the makefile to be invalidated if modified.
		/// </summary>
		public HashSet<FileItem> InternalDependencies;

		/// <summary>
		/// Collection of available Platform SDK info which will cause the makefile to be invalidated if modified.
		/// </summary>
		public Dictionary<string, string> PlatformSDKInfo;

		/// <summary>
		/// Set of paths that are disallowed to be transferred with UBA to remote machines
		/// </summary>
		public HashSet<DirectoryItem> UbaDisallowedPaths;

		/// <summary>
		/// Mapping of binary paths for cross architecture host binaries, to allow for using UBA helpers of a different architecture
		/// </summary>
		public Dictionary<DirectoryItem, DirectoryItem> UbaCrossArchitecturePaths;

		/// <summary>
		/// Set of path hashes that are registered with UBA, to prevent needing to individually hash each file
		/// </summary>
		public Dictionary<DirectoryItem, string> UbaPathHashes;

		/// <summary>
		/// TargetRules-set memory estimate per action. Used to control the number of parallel actions that are spawned.
		/// </summary>
		public double MemoryPerActionGB = 0.0;

		/// <summary>
		/// Enumerable of all source and header files tracked by this makefile.
		/// </summary>
		public IEnumerable<FileItem> SourceAndHeaderFiles
		{
			get
			{
				foreach (KeyValuePair<DirectoryItem, TargetMakefileSourceFileInfo[]> Item in DirectoryToSourceFiles)
				{
					foreach (TargetMakefileSourceFileInfo File in Item.Value.Where(x => x.SourceFileItem != null))
					{
						yield return File.SourceFileItem!;
					}
				}

				foreach (FileItem File in HeaderFiles)
				{
					yield return File;
				}
			}
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="ExternalMetadata">External build metadata from the platform</param>
		/// <param name="ExecutableFile">Path to the executable or primary output binary for this target</param>
		/// <param name="ReceiptFile">Path to the receipt file</param>
		/// <param name="ProjectIntermediateDirectory">Path to the project intermediate directory</param>
		/// <param name="ProjectIntermediateDirectoryNoArch">Path to the project intermediate directory</param>
		/// <param name="TargetType">The type of target</param>
		/// <param name="ConfigValueTracker">Set of dependencies on config files</param>
		/// <param name="bDeployAfterCompile">Whether to deploy the target after compiling</param>
		/// <param name="UbtPlugins">Collection of UBT plugins</param>
		/// <param name="EnabledUbtPlugins">Collection of UBT plugins</param>
		/// <param name="EnabledUhtPlugins">Collection of UBT plugins for UHT</param>
		public TargetMakefile(string ExternalMetadata, FileReference ExecutableFile, FileReference ReceiptFile,
			DirectoryReference ProjectIntermediateDirectory, DirectoryReference ProjectIntermediateDirectoryNoArch,
			TargetType TargetType, ConfigValueTracker ConfigValueTracker, bool bDeployAfterCompile,
			FileReference[]? UbtPlugins, FileReference[]? EnabledUbtPlugins, FileReference[]? EnabledUhtPlugins)
		{
			CreateTimeUtc = UnrealBuildTool.StartTimeUtc;
			ModifiedTimeUtc = CreateTimeUtc;
			Diagnostics = new List<string>();
			this.ExternalMetadata = ExternalMetadata;
			this.ExecutableFile = ExecutableFile;
			this.ReceiptFile = ReceiptFile;
			this.ProjectIntermediateDirectory = ProjectIntermediateDirectory;
			this.ProjectIntermediateDirectoryNoArch = ProjectIntermediateDirectoryNoArch;
			this.TargetType = TargetType;
			this.ConfigValueTracker = ConfigValueTracker;
			this.bDeployAfterCompile = bDeployAfterCompile;
			this.UbtPlugins = UbtPlugins;
			this.EnabledUbtPlugins = EnabledUbtPlugins;
			this.EnabledUhtPlugins = EnabledUhtPlugins;
			LogEvents = Array.Empty<LogEvent>();
			Actions = new List<IExternalAction>();
			OutputItems = new List<FileItem>();
			ModuleNameToOutputItems = new Dictionary<string, FileItem[]>(StringComparer.OrdinalIgnoreCase);
			HotReloadModuleNames = new HashSet<string>(StringComparer.OrdinalIgnoreCase);
			SourceDirectories = new List<DirectoryItem>();
			DirectoryToSourceFiles = new();
			HeaderFiles = new();
			WorkingSet = new HashSet<FileItem>();
			CandidatesForWorkingSet = new HashSet<FileItem>();
			UObjectModules = new List<UHTModuleInfo>();
			UObjectModuleHeaders = new List<UHTModuleHeaderInfo>();
#if __VPROJECT_AVAILABLE__
			VNIModules = new List<VNIModuleInfo>();
#endif
			PluginFiles = new HashSet<FileItem>();
			ExternalDependencies = new HashSet<FileItem>();
			InternalDependencies = new HashSet<FileItem>();
			PlatformSDKInfo = new Dictionary<string, string>();

			UbaDisallowedPaths = [.. EpicGames.UBA.Utils.DisallowedPaths.Select(DirectoryItem.GetItemByDirectoryReference)];
			UbaCrossArchitecturePaths = new(EpicGames.UBA.Utils.CrossArchitecturePaths.Select(x => new KeyValuePair<DirectoryItem, DirectoryItem>(DirectoryItem.GetItemByDirectoryReference(x.Key), DirectoryItem.GetItemByDirectoryReference(x.Value))));
			UbaPathHashes = new(EpicGames.UBA.Utils.PathHashes.Select(x => new KeyValuePair<DirectoryItem, string>(DirectoryItem.GetItemByDirectoryReference(x.Key), x.Value)));

			RegisterUBAHandlers();
		}

		/// <summary>
		/// Constructor. Reads a makefile from disk.
		/// </summary>
		/// <param name="Reader">The archive to read from</param>
		/// <param name="LastWriteTimeUtc">Last modified time for the makefile</param>
		public TargetMakefile(BinaryArchiveReader Reader, DateTime LastWriteTimeUtc)
		{
			CreateTimeUtc = new DateTime(Reader.ReadLong(), DateTimeKind.Utc);
			ModifiedTimeUtc = LastWriteTimeUtc;
			Diagnostics = Reader.ReadList(() => Reader.ReadString())!;
			ExternalMetadata = Reader.ReadString()!;
			ExecutableFile = Reader.ReadFileReference();
			ReceiptFile = Reader.ReadFileReference();
			ProjectIntermediateDirectory = Reader.ReadDirectoryReferenceNotNull();
			ProjectIntermediateDirectoryNoArch = Reader.ReadDirectoryReferenceNotNull();
			TargetType = (TargetType)Reader.ReadInt();
			IsTestTarget = Reader.ReadBool();
			ConfigValueTracker = new ConfigValueTracker(Reader);
			bDeployAfterCompile = Reader.ReadBool();
			UbtPlugins = Reader.ReadArray(() => Reader.ReadFileReference())!;
			EnabledUbtPlugins = Reader.ReadArray(() => Reader.ReadFileReference())!;
			EnabledUhtPlugins = Reader.ReadArray(() => Reader.ReadFileReference())!;
			AdditionalArguments = Reader.ReadArray(() => Reader.ReadString())!;
			UHTAdditionalArguments = Reader.ReadArray(() => Reader.ReadString())!;
			PreBuildScripts = Reader.ReadArray(() => Reader.ReadFileReference())!;
			PreBuildTargets = Reader.ReadArray(() => new TargetInfo(Reader))!;
			LogEvents = Reader.ReadArray(() => Reader.ReadLogEvent())!;
			Actions = Reader.ReadList(() => Reader.ReadAction())!;
			EnvironmentVariables = Reader.ReadList(() => Tuple.Create(Reader.ReadString(), Reader.ReadString()))!;
			OutputItems = Reader.ReadList(() => Reader.ReadFileItem())!;
			ModuleNameToOutputItems = Reader.ReadDictionary(() => Reader.ReadString()!, () => Reader.ReadArray(() => Reader.ReadFileItem()), StringComparer.OrdinalIgnoreCase)!;
			HotReloadModuleNames = Reader.ReadHashSet(() => Reader.ReadString(), StringComparer.OrdinalIgnoreCase)!;
			SourceDirectories = Reader.ReadList(() => Reader.ReadDirectoryItem())!;
			DirectoryToSourceFiles = Reader.ReadDictionary(() => Reader.ReadDirectoryItem()!, () => Reader.ReadArray(() => new TargetMakefileSourceFileInfo(Reader)))!;
			HeaderFiles = Reader.ReadHashSet(() => Reader.ReadFileItem())!;
			WorkingSet = Reader.ReadHashSet(() => Reader.ReadFileItem())!;
			CandidatesForWorkingSet = Reader.ReadHashSet(() => Reader.ReadFileItem())!;
			UObjectModules = Reader.ReadList(() => new UHTModuleInfo(Reader))!;
			UObjectModuleHeaders = Reader.ReadList(() => new UHTModuleHeaderInfo(Reader))!;
			UHTTargetSettings = new UHTTargetSettings(Reader);
#if __VPROJECT_AVAILABLE__
			VNIModules = Reader.ReadList(() => new VNIModuleInfo(Reader))!;
			bUseVerseBPVM = Reader.ReadBool();
#endif
			PluginFiles = Reader.ReadHashSet(() => Reader.ReadFileItem())!;
			ExternalDependencies = Reader.ReadHashSet(() => Reader.ReadFileItem())!;
			InternalDependencies = Reader.ReadHashSet(() => Reader.ReadFileItem())!;
			PlatformSDKInfo = Reader.ReadDictionary(() => Reader.ReadString()!, () => Reader.ReadString()!)!;

			UbaDisallowedPaths = Reader.ReadHashSet(() => Reader.ReadDirectoryItem())!;
			UbaCrossArchitecturePaths = Reader.ReadDictionary(() => Reader.ReadDirectoryItem()!, () => Reader.ReadDirectoryItem())!;
			UbaPathHashes = Reader.ReadDictionary(() => Reader.ReadDirectoryItem()!, () => Reader.ReadString())!;
			MemoryPerActionGB = Reader.ReadDouble();

			EpicGames.UBA.Utils.RegisterDisallowedPaths([.. UbaDisallowedPaths.Where(x => x.Exists).Select(x => x.Location)]);
			foreach (KeyValuePair<DirectoryItem, DirectoryItem> item in UbaCrossArchitecturePaths.Where(x => x.Key.Exists && x.Value.Exists))
			{
				EpicGames.UBA.Utils.RegisterCrossArchitecturePath(item.Key.Location, item.Value.Location);
			}
			foreach (KeyValuePair<DirectoryItem, string> item in UbaPathHashes.Where(x => x.Key.Exists))
			{
				EpicGames.UBA.Utils.RegisterPathHash(item.Key.Location, item.Value);
			}

			RegisterUBAHandlers();
		}

		void RegisterUBAHandlers()
		{
			EpicGames.UBA.Utils.DisallowedPathRegistered += (sender, e) =>
			{
				lock (UbaDisallowedPaths)
				{
					UbaDisallowedPaths.UnionWith(e.Paths.Select(DirectoryItem.GetItemByDirectoryReference));
				}
			};

			EpicGames.UBA.Utils.CrossArchitecturePathRegistered += (sender, e) =>
			{
				lock (UbaCrossArchitecturePaths)
				{
					UbaCrossArchitecturePaths.TryAdd(DirectoryItem.GetItemByDirectoryReference(e.Path), DirectoryItem.GetItemByDirectoryReference(e.CrossPath));
				}
			};

			EpicGames.UBA.Utils.PathHashRegistered += (sender, e) =>
			{
				lock (UbaPathHashes)
				{
					UbaPathHashes.TryAdd(DirectoryItem.GetItemByDirectoryReference(e.Path), e.Hash);
				}
			};
		}

		/// <summary>
		/// Write the makefile to the given archive
		/// </summary>
		/// <param name="Writer">The archive to write to</param>
		public void Write(BinaryArchiveWriter Writer)
		{
			Writer.WriteLong(CreateTimeUtc.Ticks);
			Writer.WriteList(Diagnostics, x => Writer.WriteString(x));
			Writer.WriteString(ExternalMetadata);
			Writer.WriteFileReference(ExecutableFile);
			Writer.WriteFileReference(ReceiptFile);
			Writer.WriteDirectoryReference(ProjectIntermediateDirectory);
			Writer.WriteDirectoryReference(ProjectIntermediateDirectoryNoArch);
			Writer.WriteInt((int)TargetType);
			Writer.WriteBool(IsTestTarget);
			ConfigValueTracker.Write(Writer);
			Writer.WriteBool(bDeployAfterCompile);
			Writer.WriteArray(UbtPlugins, Item => Writer.WriteFileReference(Item));
			Writer.WriteArray(EnabledUbtPlugins, Item => Writer.WriteFileReference(Item));
			Writer.WriteArray(EnabledUhtPlugins, Item => Writer.WriteFileReference(Item));
			Writer.WriteArray(AdditionalArguments, Item => Writer.WriteString(Item));
			Writer.WriteArray(UHTAdditionalArguments, Item => Writer.WriteString(Item));
			Writer.WriteArray(PreBuildScripts, Item => Writer.WriteFileReference(Item));
			Writer.WriteArray(PreBuildTargets, Item => Item.Write(Writer));
			Writer.WriteArray(LogEvents, Item => Writer.WriteLogEvent(Item));
			Writer.WriteList(Actions, x => Writer.WriteAction(x));
			Writer.WriteList(EnvironmentVariables, x =>
			{
				Writer.WriteString(x.Item1);
				Writer.WriteString(x.Item2);
			});
			Writer.WriteList(OutputItems, Item => Writer.WriteFileItem(Item));
			Writer.WriteDictionary(ModuleNameToOutputItems, k => Writer.WriteString(k), v => Writer.WriteArray(v, e => Writer.WriteFileItem(e)));
			Writer.WriteHashSet(HotReloadModuleNames, x => Writer.WriteString(x));
			Writer.WriteList(SourceDirectories, x => Writer.WriteDirectoryItem(x));
			Writer.WriteDictionary(DirectoryToSourceFiles, k => Writer.WriteDirectoryItem(k), v => Writer.WriteArray(v, e => e.Write(Writer)));
			Writer.WriteHashSet(HeaderFiles, x => Writer.WriteFileItem(x));
			Writer.WriteHashSet(WorkingSet, x => Writer.WriteFileItem(x));
			Writer.WriteHashSet(CandidatesForWorkingSet, x => Writer.WriteFileItem(x));
			Writer.WriteList(UObjectModules, e => e.Write(Writer));
			Writer.WriteList(UObjectModuleHeaders, x => x.Write(Writer));
			UHTTargetSettings.Write(Writer);
#if __VPROJECT_AVAILABLE__
			Writer.WriteList(VNIModules, e => e.Write(Writer));
			Writer.WriteBool(bUseVerseBPVM);
#endif
			Writer.WriteHashSet(PluginFiles, x => Writer.WriteFileItem(x));
			Writer.WriteHashSet(ExternalDependencies, x => Writer.WriteFileItem(x));
			Writer.WriteHashSet(InternalDependencies, x => Writer.WriteFileItem(x));
			Writer.WriteDictionary(PlatformSDKInfo, k => Writer.WriteString(k), v => Writer.WriteString(v));

			Writer.WriteHashSet(UbaDisallowedPaths, Writer.WriteDirectoryItem);
			Writer.WriteDictionary(UbaCrossArchitecturePaths, Writer.WriteDirectoryItem, Writer.WriteDirectoryItem);
			Writer.WriteDictionary(UbaPathHashes, Writer.WriteDirectoryItem, Writer.WriteString);
			Writer.WriteDouble(MemoryPerActionGB);
		}

		/// <summary>
		/// Saves a makefile to disk
		/// </summary>
		/// <param name="Location">Path to save the makefile to</param>
		public void Save(FileReference Location)
		{
			DirectoryReference.CreateDirectory(Location.Directory);
			using (BinaryArchiveWriter Writer = new BinaryArchiveWriter(Location))
			{
				Writer.WriteInt(CurrentVersion);
#if __VPROJECT_AVAILABLE__
				Writer.WriteBool(true);
#else
				Writer.WriteBool(false);
#endif
				Write(Writer);
			}
		}

		/// <summary>
		/// Loads a makefile  from disk
		/// </summary>
		/// <param name="MakefilePath">Path to the makefile to load</param>
		/// <param name="ProjectFile">Path to the project file</param>
		/// <param name="Platform">Platform for this makefile</param>
		/// <param name="Arguments">Command line arguments for this target</param>
		/// <param name="Logger">Logger for output</param>
		/// <param name="ReasonNotLoaded">If the function returns null, this string will contain the reason why</param>
		/// <returns>The loaded makefile, or null if it failed for some reason.  On failure, the 'ReasonNotLoaded' variable will contain information about why</returns>
		public static TargetMakefile? Load(FileReference MakefilePath, FileReference? ProjectFile, UnrealTargetPlatform Platform, string[] Arguments, ILogger Logger, out string? ReasonNotLoaded)
		{
			FileInfo MakefileInfo;
			using (GlobalTracer.Instance.BuildSpan("Checking dependent timestamps").StartActive())
			{
				// Check the directory timestamp on the project files directory.  If the user has generated project files more recently than the makefile, then we need to consider the file to be out of date
				MakefileInfo = new FileInfo(MakefilePath.FullName);
				if (!MakefileInfo.Exists)
				{
					// Makefile doesn't even exist, so we won't bother loading it
					ReasonNotLoaded = "no existing makefile";
					return null;
				}

				// Check the build version
				FileInfo BuildVersionFileInfo = new FileInfo(BuildVersion.GetDefaultFileName().FullName);
				if (BuildVersionFileInfo.Exists && MakefileInfo.LastWriteTime.CompareTo(BuildVersionFileInfo.LastWriteTime) < 0)
				{
					Logger.LogDebug("Existing makefile is older than Build.version, ignoring it");
					ReasonNotLoaded = "Build.version is newer";
					return null;
				}

				if (ProjectFile != null)
				{
					// Check the game project
					string ProjectFilename = ProjectFile.FullName;
					FileInfo ProjectFileInfo = new FileInfo(ProjectFilename);
					if (!ProjectFileInfo.Exists || MakefileInfo.LastWriteTime.CompareTo(ProjectFileInfo.LastWriteTime) < 0)
					{
						// .uproject file is newer than makefile
						Logger.LogDebug("Makefile is older than .uproject file, ignoring it");
						ReasonNotLoaded = ".uproject file is newer";
						return null;
					}
				}

				// Check to see if UnrealBuildTool was compiled more recently than the makefile
				DateTime UnrealBuildToolTimestamp = new FileInfo(Assembly.GetExecutingAssembly().Location).LastWriteTime;
				if (MakefileInfo.LastWriteTime.CompareTo(UnrealBuildToolTimestamp) < 0)
				{
					// UnrealBuildTool was compiled more recently than the makefile
					Logger.LogDebug("Makefile is older than UnrealBuildTool assembly, ignoring it");
					ReasonNotLoaded = "UnrealBuildTool assembly is newer";
					return null;
				}

				// Check to see if EpicGames.UHT was compiled more recently than the makefile
				DateTime UhtTimestamp = new FileInfo(typeof(UhtSession).Assembly.Location).LastWriteTime;
				if (MakefileInfo.LastWriteTime.CompareTo(UhtTimestamp) < 0)
				{
					// UnrealBuildTool was compiled more recently than the makefile
					Logger.LogDebug("Makefile is older than EpicGames.UHT assembly, ignoring it");
					ReasonNotLoaded = "EpicGames.UHT assembly is newer";
					return null;
				}

				// Check to see if any BuildConfiguration files have changed since the last build
				foreach (XmlConfig.InputFile InputFile in XmlConfig.InputFiles)
				{
					FileInfo InputFileInfo = new FileInfo(InputFile.Location.FullName);
					if (InputFileInfo.LastWriteTime > MakefileInfo.LastWriteTime)
					{
						Logger.LogDebug("Makefile is older than BuildConfiguration.xml, ignoring it");
						ReasonNotLoaded = "BuildConfiguration.xml is newer";
						return null;
					}
				}
			}

			// Run task that parse file system and find plugin files etc. (this can run in parallel with make file creation)
			Task<FileReference[]> EnumPluginsTask = Task.Run(() =>
			{
				return PluginsBase.EnumerateUbtPlugins(ProjectFile).ToArray();
			});

			TargetMakefile Makefile;
			using (GlobalTracer.Instance.BuildSpan("Loading makefile").StartActive())
			{
				using ITimelineEvent _ = Timeline.ScopeEvent("LoadMakefile");
				try
				{
					using (BinaryArchiveReader Reader = new BinaryArchiveReader(MakefilePath))
					{
						int Version = Reader.ReadInt();
						if (Version != CurrentVersion)
						{
							ReasonNotLoaded = "makefile version does not match";
							return null;
						}
						bool bVProjectAvailable = Reader.ReadBool();
#if __VPROJECT_AVAILABLE__
						if (!bVProjectAvailable)
#else
						if (bVProjectAvailable)
#endif
						{
							ReasonNotLoaded = "makefile VProject availability does not match";
							return null;
						}
						Makefile = new TargetMakefile(Reader, MakefileInfo.LastWriteTimeUtc);
					}
				}
				catch (Exception Ex)
				{
					Logger.LogInformation("Failed to read makefile {File}: {ExMessage}", MakefilePath.FullName, Ex.Message);
					Logger.LogDebug(Ex, "Exception: {Ex}", Ex.ToString());
					ReasonNotLoaded = "couldn't read existing makefile";
					return null;
				}
			}

			using (GlobalTracer.Instance.BuildSpan("Checking makefile validity").StartActive())
			{
				// Check if the arguments are different
				if (!Enumerable.SequenceEqual(Makefile.AdditionalArguments!, Arguments))
				{
					Logger.LogDebug("Old command line arguments:\n{Args}", String.Join(' ', Makefile.AdditionalArguments!));
					Logger.LogDebug("New command line arguments:\n{Args}", String.Join(' ', Arguments));
					ReasonNotLoaded = "command line arguments changed";
					return null;
				}

				// Check if any manifests are out of date
				foreach (FileReference Manifest in Makefile.AdditionalArguments!
					.Where(x => x.StartsWith("-Manifest=", StringComparison.OrdinalIgnoreCase))
					.Select(x => FileReference.FromString(x.Substring("-Manifest=".Length).Trim('"').Trim('\'')))
					.OfType<FileReference>())
				{
					if (!FileReference.Exists(Manifest))
					{
						ReasonNotLoaded = $"manifest '{Manifest}' not found";
						return null;
					}
					else if (FileReference.GetLastWriteTimeUtc(Manifest) < Makefile.CreateTimeUtc)
					{
						ReasonNotLoaded = $"manifest '{Manifest}' not found";
						return null;
					}
				}

				// Check if any config settings have changed. Ini files contain build settings too.
				if (!Makefile.ConfigValueTracker.IsValid())
				{
					ReasonNotLoaded = "config setting changed";
					return null;
				}

				// Get the current build metadata from the platform
				string CurrentExternalMetadata = UEBuildPlatform.GetBuildPlatform(Platform).GetExternalBuildMetadata(ProjectFile);
				if (!String.Equals(CurrentExternalMetadata, Makefile.ExternalMetadata, StringComparison.Ordinal))
				{
					Logger.LogDebug("Old metadata:\n{Metadata}", Makefile.ExternalMetadata);
					Logger.LogDebug("New metadata:\n{Metadata}", CurrentExternalMetadata);
					ReasonNotLoaded = "build metadata has changed";
					return null;
				}

				// Check to see if the number of plugins changed
				{
					using ITimelineEvent _ = Timeline.ScopeEvent("FindPlugins");

					FileReference[] Plugins = EnumPluginsTask.Result;

					if ((Plugins != null) != (Makefile.UbtPlugins != null) ||
						(Plugins != null && !Enumerable.SequenceEqual(Plugins, Makefile.UbtPlugins!)))
					{
						Logger.LogDebug("Available UBT plugins changed");
						ReasonNotLoaded = "Available UBT plugins changed";
						return null;
					}
				}

				// Check to make sure the enabled UBT plugins are up to date
				if (Makefile.EnabledUbtPlugins != null)
				{
					List<DirectoryReference> BaseDirectories = new List<DirectoryReference>();
					BaseDirectories.Add(Unreal.EngineDirectory);
					if (ProjectFile != null)
					{
						BaseDirectories.Add(ProjectFile.Directory);
					}
					HashSet<FileReference> EnabledPlugins = new HashSet<FileReference>(Makefile.EnabledUbtPlugins);
					if (!CompileScriptModule.AreScriptModulesUpToDate(Rules.RulesFileType.UbtPlugin, EnabledPlugins, BaseDirectories, Logger))
					{
						Logger.LogDebug("Enabled UBT plugins need to be recompiled");
						ReasonNotLoaded = "Enabled UBT plugins need to be recompiled";
						return null;
					}
				}
			}

			// The makefile is ok
			ReasonNotLoaded = null;
			return Makefile;
		}

		/// <summary>
		/// Checks if the makefile is valid for the current set of source files. This is done separately to the Load() method to allow pre-build steps to modify source files.
		/// </summary>
		/// <param name="Makefile">The makefile that has been loaded</param>
		/// <param name="ProjectFile">Path to the project file</param>
		/// <param name="Platform">The platform being built</param>
		/// <param name="WorkingSet">The current working set of source files</param>
		/// <param name="Logger">Logger for output diagnostics</param>
		/// <param name="ReasonNotLoaded">If the makefile is not valid, is set to a message describing why</param>
		/// <returns>True if the makefile is valid, false otherwise</returns>
		public static bool IsValidForSourceFiles(TargetMakefile Makefile, FileReference? ProjectFile, UnrealTargetPlatform Platform, ISourceFileWorkingSet WorkingSet, ILogger Logger, out string? ReasonNotLoaded)
		{
			using ITimelineEvent _ = Timeline.ScopeEvent("MakefileValid");

			using (GlobalTracer.Instance.BuildSpan("TargetMakefile.IsValidForSourceFiles()").StartActive())
			{
				// Get the list of excluded folder names for this platform
				IReadOnlySet<string> ExcludedFolderNames = UEBuildPlatform.GetBuildPlatform(Platform).GetExcludedFolderNames();

				string reasonNotLoaded1 = "";

				SourceFileMetadataCache? MetadataCache = null;

				Task<bool> checkTask = Task.Run(() =>
				{

					// Load the metadata cache
					MetadataCache = SourceFileMetadataCache.CreateHierarchy(ProjectFile, Logger);

					// Check if any source files have been added or removed
					foreach ((DirectoryItem InputDirectory, TargetMakefileSourceFileInfo[] SourceFileInfos) in Makefile.DirectoryToSourceFiles)
					{
						if (!InputDirectory.Exists || InputDirectory.LastWriteTimeUtc > Makefile.CreateTimeUtc)
						{
							FileItem[] SourceFiles = UEBuildModuleCPP.GetSourceFiles(InputDirectory, Logger);
							if (SourceFiles.Length < SourceFileInfos.Length)
							{
								reasonNotLoaded1 = "source file removed";
								return false;
							}
							else if (SourceFiles.Length > SourceFileInfos.Length)
							{
								reasonNotLoaded1 = "source file added";
								return false;
							}
							else if (SourceFiles.Intersect(SourceFileInfos.Select((sfi) => sfi.SourceFileItem)).Count() != SourceFiles.Length)
							{
								reasonNotLoaded1 = "source file modified";
								return false;
							}
							else if (!InputDirectory.Exists)
							{
								reasonNotLoaded1 = "source directory removed";
								return false;
							}

							foreach (DirectoryItem Directory in InputDirectory.EnumerateDirectories())
							{
								if (!Makefile.DirectoryToSourceFiles.ContainsKey(Directory) && ContainsSourceFilesOrHeaders(Directory, ExcludedFolderNames, Logger))
								{
									reasonNotLoaded1 = "source directory added";
									return false;
								}
							}
						}

						// Make sure the inlined gen.cpp files didn't change
						foreach (TargetMakefileSourceFileInfo SourceFileInfo in SourceFileInfos)
						{
							int InlineHash = TargetMakefileSourceFileInfo.CalculateInlineGenCppHash(MetadataCache.GetListOfInlinedGeneratedCppFiles(SourceFileInfo.SourceFileItem!));
							if (SourceFileInfo.InlineGenCppHash != InlineHash)
							{
								reasonNotLoaded1 = "inlined gen.cpp files changed";
								return false;
							}
						}
					}

					// Check if any external dependencies have changed. These comparisons are done against the makefile creation time.
					foreach (FileItem ExternalDependency in Makefile.ExternalDependencies)
					{
						if (!ExternalDependency.Exists)
						{
							Logger.LogDebug("{File} has been deleted since makefile was built.", ExternalDependency.Location);
							reasonNotLoaded1 = String.Format("{0} deleted", ExternalDependency.Location.GetFileName());
							return false;
						}
						if (ExternalDependency.LastWriteTimeUtc > Makefile.CreateTimeUtc)
						{
							Logger.LogDebug("{File} has been modified since makefile was built.", ExternalDependency.Location);
							reasonNotLoaded1 = String.Format("{0} modified", ExternalDependency.Location.GetFileName());
							return false;
						}
					}

					return true;
				});

				bool isValid2 = true;
				Lock isValidLock2 = new();
				string reasonNotLoaded2 = "";

				// First collect file timestamps by caching directories. This is much faster than letting each file do a GetFileAttribute
				List<FileItem> fileItems = new(Makefile.InternalDependencies);
				fileItems.Sort(static (a, b) => a.Location.CompareTo(b.Location));
				List<DirectoryItem> dirs = new();
				DirectoryItem? lastDir = null;
				foreach (FileItem item in fileItems)
				{
					DirectoryItem dir = item.Directory;
					if (dir != lastDir)
					{
						dirs.Add(dir);
						lastDir = dir;
					}
				}
				Parallel2.ForEach(dirs, dir =>
				{
					dir.CacheFiles();
				});

				// Check if any internal dependencies has changed. These comparisons are done against the makefile modified time.
				Parallel2.ForEach(Makefile.InternalDependencies, InternalDependency =>
				{
					if (!InternalDependency.Exists && isValid2)
					{
						lock (isValidLock2)
						{
							if (isValid2)
							{
								isValid2 = false;
								Logger.LogDebug("{File} has been deleted since makefile was written.", InternalDependency.Location);
								reasonNotLoaded2 = String.Format("{0} deleted", InternalDependency.Location.GetFileName());
							}
						}
					}
					if (InternalDependency.LastWriteTimeUtc > Makefile.ModifiedTimeUtc && isValid2)
					{
						lock (isValidLock2)
						{
							if (isValid2)
							{
								isValid2 = false;
								Logger.LogDebug("{File} has been modified since makefile was written.", InternalDependency.Location);
								reasonNotLoaded2 = String.Format("{0} modified", InternalDependency.Location.GetFileName());
							}
						}
					}
				});

				string reasonNotLoaded3 = "";

				bool Func3()
				{
					// Check if available Platform SDKs have changed
					foreach (KeyValuePair<string, string> Pair in Makefile.PlatformSDKInfo)
					{
						string value = UEBuildPlatformSDK.GetSDKForPlatform(Pair.Key)?.GetAllSDKInfo().ToString() ?? String.Empty;
						if (Pair.Value != value)
						{
							Logger.LogDebug("Platform SDK {Key} '{Value}' has been modified to '{NewValue}' since makefile was written.", Pair.Key, Pair.Value, !String.IsNullOrEmpty(value) ? value : "Missing");
							reasonNotLoaded3 = String.Format("Platform SDK {0} modified", Pair.Key);
							return false;
						}
					}

					IEnumerable<DirectoryItem> UbaPaths = [.. Makefile.UbaDisallowedPaths, .. Makefile.UbaPathHashes.Keys, .. Makefile.UbaCrossArchitecturePaths.Keys, .. Makefile.UbaCrossArchitecturePaths.Values];
					foreach (DirectoryItem item in UbaPaths.Distinct())
					{
						if (!item.Exists)
						{
							Logger.LogDebug("{Directory} has been deleted since makefile was written.", item.Location);
							reasonNotLoaded3 = String.Format("{0} deleted", item.Location);
							return false;
						}
					}

					// Check that no new plugins have been added
					foreach (FileReference PluginFile in PluginsBase.EnumeratePlugins(ProjectFile))
					{
						FileItem PluginFileItem = FileItem.GetItemByFileReference(PluginFile);
						if (!Makefile.PluginFiles.Contains(PluginFileItem))
						{
							Logger.LogDebug("{File} has been added", PluginFile.GetFileName());
							reasonNotLoaded3 = String.Format("{0} has been added", PluginFile.GetFileName());
							return false;
						}
					}
					return true;
				}

				bool isValid3 = Func3();

				bool isValid1 = checkTask.Result;
				if (!isValid1)
				{
					ReasonNotLoaded = reasonNotLoaded1;
					return false;
				}

				if (!isValid2)
				{
					ReasonNotLoaded = reasonNotLoaded2;
					return false;
				}

				if (!isValid3)
				{
					ReasonNotLoaded = reasonNotLoaded3;
					return false;
				}

				// Find the set of files that contain reflection markup
				ConcurrentBag<FileItem> NewFilesWithMarkupBag = new ConcurrentBag<FileItem>();
				ParallelQueue Queue = new();
				{
					foreach (DirectoryItem SourceDirectory in Makefile.SourceDirectories)
					{
						Queue.Enqueue(() => FindFilesWithMarkup(SourceDirectory, MetadataCache!, ExcludedFolderNames, NewFilesWithMarkupBag, Queue));
					}
					Queue.Drain();
				}

				// Check whether the list has changed
				List<FileItem> PrevFilesWithMarkup = Makefile.UObjectModuleHeaders.Where(x => !x.bUsePrecompiled).SelectMany(x => x.HeaderFiles).ToList();
				List<FileItem> NextFilesWithMarkup = NewFilesWithMarkupBag.ToList();
				if (NextFilesWithMarkup.Count != PrevFilesWithMarkup.Count || NextFilesWithMarkup.Intersect(PrevFilesWithMarkup).Count() != PrevFilesWithMarkup.Count)
				{
					ReasonNotLoaded = "UHT files changed";
					return false;
				}

				// If adaptive unity build is enabled, do a check to see if there are any source files that became part of the
				// working set since the Makefile was created (or, source files were removed from the working set.)  If anything
				// changed, then we'll force a new Makefile to be created so that we have fresh unity build blobs.  We always
				// want to make sure that source files in the working set are excluded from those unity blobs (for fastest possible
				// iteration times.)

				// Check if any source files in the working set no longer belong in it
				foreach (FileItem SourceFile in Makefile.WorkingSet)
				{
					if (!WorkingSet.Contains(SourceFile) && SourceFile.LastWriteTimeUtc > Makefile.CreateTimeUtc)
					{
						Logger.LogDebug("{File} was part of source working set and now is not; invalidating makefile", SourceFile.Location);
						ReasonNotLoaded = String.Format("working set of source files changed");
						return false;
					}
				}

				// Check if any source files that are eligible for being in the working set have been modified
				foreach (FileItem SourceFile in Makefile.CandidatesForWorkingSet)
				{
					if (WorkingSet.Contains(SourceFile) && SourceFile.LastWriteTimeUtc > Makefile.CreateTimeUtc)
					{
						Logger.LogDebug("{File} was not part of source working set and should be", SourceFile.Location);
						ReasonNotLoaded = String.Format("working set of source files changed");
						return false;
					}
				}
			}

			ReasonNotLoaded = null;
			return true;
		}

		/// <summary>
		/// Determines if a directory, or any subdirectory of it, contains new source files or headers
		/// </summary>
		/// <param name="Directory">Directory to search through</param>
		/// <param name="ExcludedFolderNames">Set of directory names to exclude</param>
		/// <param name="Logger">Logger for output diagnostics</param>
		/// <returns>True if the directory contains any source files</returns>
		static bool ContainsSourceFilesOrHeaders(DirectoryItem Directory, IReadOnlySet<string> ExcludedFolderNames, ILogger Logger)
		{
			// Check this directory isn't ignored
			if (!ExcludedFolderNames.Contains(Directory.Name))
			{
				// Check for any source files in this actual directory
				FileItem[] SourceFiles = UEBuildModuleCPP.GetSourceFilesAndHeaders(Directory, Logger);
				if (SourceFiles.Length > 0)
				{
					return true;
				}

				// Check for any source files in a subdirectory
				foreach (DirectoryItem SubDirectory in Directory.EnumerateDirectories())
				{
					if (ContainsSourceFilesOrHeaders(SubDirectory, ExcludedFolderNames, Logger))
					{
						return true;
					}
				}
			}
			return false;
		}

		/// <summary>
		/// Finds all the source files under a directory that contain reflection markup
		/// </summary>
		/// <param name="Directory">The directory to search</param>
		/// <param name="MetadataCache">Cache of source file metadata</param>
		/// <param name="ExcludedFolderNames">Set of folder names to ignore when recursing the directory tree</param>
		/// <param name="FilesWithMarkup">Receives the set of files which contain reflection markup</param>
		/// <param name="Queue">Queue to add sub-tasks to</param>
		static void FindFilesWithMarkup(DirectoryItem Directory, SourceFileMetadataCache MetadataCache, IReadOnlySet<string> ExcludedFolderNames, ConcurrentBag<FileItem> FilesWithMarkup, ParallelQueue Queue)
		{
			if (Directory.TryGetFile(".ubtignore", out FileItem? OutFile))
			{
				return;
			}

			// Check for all the headers in this folder
			foreach (FileItem File in Directory.EnumerateFiles().Where((fi) => fi.HasExtension(".h") && MetadataCache.ContainsReflectionMarkup(fi)))
			{
				FilesWithMarkup.Add(File);
			}

			// Search through all the subfolders
			foreach (DirectoryItem SubDirectory in Directory.EnumerateDirectories())
			{
				if (!ExcludedFolderNames.Contains(SubDirectory.Name))
				{
					Queue.Enqueue(() => FindFilesWithMarkup(SubDirectory, MetadataCache, ExcludedFolderNames, FilesWithMarkup, Queue));
				}
			}
		}

		/// <summary>
		/// Gets the location of the makefile for particular target
		/// </summary>
		/// <param name="ProjectFile">Project file for the build</param>
		/// <param name="TargetName">Name of the target being built</param>
		/// <param name="Platform">The platform that the target is being built for</param>
		/// <param name="Architectures">The architectures the target is being built for (can be blank to signify a default)</param>
		/// <param name="Configuration">The configuration being built</param>
		/// <param name="IntermediateEnvironment">Intermediate environment to use</param>
		/// <returns>Path to the makefile</returns>
		public static FileReference GetLocation(FileReference? ProjectFile, string TargetName, UnrealTargetPlatform Platform, UnrealArchitectures Architectures, UnrealTargetConfiguration Configuration, UnrealIntermediateEnvironment IntermediateEnvironment)
		{
			DirectoryReference BaseDirectory = Unreal.EngineDirectory;
			// Programs with .uprojects still want the Intermediate under Engine (see UEBuildTarget constructor for similar code) 
			if (ProjectFile != null && !ProjectFile.ContainsName("Programs", 0))
			{
				BaseDirectory = ProjectFile.Directory;
			}
			string IntermediateFolder = UEBuildTarget.GetPlatformIntermediateFolder(Platform, Architectures, false);
			string TargetFolderName = UEBuildTarget.GetTargetIntermediateFolderName(TargetName, IntermediateEnvironment);
			return FileReference.Combine(BaseDirectory, IntermediateFolder, TargetFolderName, Configuration.ToString(), "Makefile.bin");
		}
	}
}
