// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using System.Linq;
using System.Text.Json;
using System.Text.Json.Serialization;
using System.Xml.Serialization;
using EpicGames.Core;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// Type of a build product
	/// </summary>
	public enum BuildProductType
	{
		/// <summary>
		/// An executable file
		/// </summary>
		Executable,

		/// <summary>
		/// A dynamically loaded module.
		/// </summary>
		DynamicLibrary,

		/// <summary>
		/// A symbol file. Not required for the executable to run.
		/// </summary>
		SymbolFile,

		/// <summary>
		/// A map file. Not required for the executable to run.
		/// </summary>
		MapFile,

		/// <summary>
		/// A resource file which was generated by the build and is required for the executable to run.
		/// </summary>
		RequiredResource,

		/// <summary>
		/// A build resource which was generated by the build, but is not required for the executable to run.
		/// </summary>
		BuildResource,

		/// <summary>
		/// A package which can be deployed on device (eg. *.apk for Android, *.stub for iOS)
		/// </summary>
		Package
	}

	/// <summary>
	/// A file that was created as part of the build process
	/// </summary>
	[Serializable]
	public class BuildProduct
	{
		/// <summary>
		/// Path to the file.
		/// </summary>
		public FileReference Path;

		/// <summary>
		/// Type of the build product.
		/// </summary>
		public BuildProductType Type;

		/// <summary>
		/// Private constructor, for serialization.
		/// </summary>
		private BuildProduct()
		{
			Path = null!;
		}

		/// <summary>
		/// Constructor.
		/// </summary>
		/// <param name="Path">Path to the build product</param>
		/// <param name="Type">Type of the build product</param>
		[JsonConstructor]
		public BuildProduct(FileReference Path, BuildProductType Type)
		{
			this.Path = Path;
			this.Type = Type;
		}

		/// <summary>
		/// Copy constructor.
		/// </summary>
		/// <param name="Other">Build product to copy settings from</param>
		public BuildProduct(BuildProduct Other)
		{
			Path = Other.Path;
			Type = Other.Type;
		}

		/// <summary>
		/// Convert this object to a string, for debugging.
		/// </summary>
		/// <returns>Path to this build product</returns>
		public override string ToString()
		{
			return Path.ToString();
		}
	}

	/// <summary>
	/// How a file may be staged
	/// </summary>
	public enum StagedFileType
	{
		/// <summary>
		/// Only accessed through Unreal filesystem functions; may be included in a PAK file.
		/// </summary>
		UFS,

		/// <summary>
		/// Must be kept as part of the loose filesystem.
		/// </summary>
		NonUFS,

		/// <summary>
		/// Debug file which must be kept as part of the loose filesystem.
		/// </summary>
		DebugNonUFS,

		/// <summary>
		/// System file which must be kept as part of the loose filesystem. System files are not subject to being automatic remapping or renaming by the platform layer.
		/// </summary>
		SystemNonUFS,
	}

	/// <summary>
	/// Information about a file which is required by the target at runtime, and must be moved around with it.
	/// </summary>
	[Serializable]
	public class RuntimeDependency
	{
		/// <summary>
		/// The file that should be staged. Should use $(EngineDir) and $(ProjectDir) variables as a root, so that the target can be relocated to different machines.
		/// </summary>
		public FileReference Path;

		/// <summary>
		/// How to stage this file.
		/// </summary>
		public StagedFileType Type;

		/// <summary>
		/// Private constructor, for serialization.
		/// </summary>
		private RuntimeDependency()
		{
			Path = null!;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="Path">Path to the runtime dependency</param>
		/// <param name="Type">How to stage the given path</param>
		[JsonConstructor]
		public RuntimeDependency(FileReference Path, StagedFileType Type = StagedFileType.NonUFS)
		{
			this.Path = Path;
			this.Type = Type;
		}

		/// <summary>
		/// Copy constructor
		/// </summary>
		/// <param name="Other">Runtime dependency to copy settings from</param>
		public RuntimeDependency(RuntimeDependency Other)
		{
			Path = Other.Path;
			Type = Other.Type;
		}

		/// <summary>
		/// Convert this object to a string for debugging
		/// </summary>
		/// <returns>String representation of the object</returns>
		public override string ToString()
		{
			return Path.ToString();
		}
	}

	/// <summary>
	/// List of runtime dependencies, with convenience methods for adding new items
	/// </summary>
	[Serializable]
	public class RuntimeDependencyList : List<RuntimeDependency>
	{
		/// <summary>
		/// Default constructor
		/// </summary>
		public RuntimeDependencyList()
		{
		}

		/// <summary>
		/// Copy constructor
		/// </summary>
		/// <param name="Other">Sequence of runtime dependencies to initialize with</param>
		public RuntimeDependencyList(IEnumerable<RuntimeDependency> Other) : base(Other)
		{
		}

		/// <summary>
		/// Add a runtime dependency to the list
		/// </summary>
		/// <param name="InPath">Path to the runtime dependency. May include wildcards.</param>
		/// <param name="InType">How to stage this file</param>
		public void Add(FileReference InPath, StagedFileType InType)
		{
			Add(new RuntimeDependency(InPath, InType));
		}
	}

	/// <summary>
	/// Arbitrary property name/value which metadata from the build scripts can be passed on to downstream tasks
	/// </summary>
	[Serializable]
	public class ReceiptProperty
	{
		/// <summary>
		/// Property name
		/// </summary>
		[XmlAttribute]
		public string Name;

		/// <summary>
		/// Value of the property
		/// </summary>
		[XmlAttribute]
		public string Value;

		/// <summary>
		/// Construct a property with the given name and value
		/// </summary>
		/// <param name="Name">Name of the property</param>
		/// <param name="Value">Value of the property</param>
		public ReceiptProperty(string Name, string Value)
		{
			this.Name = Name;
			this.Value = Value;
		}
	}

	/// <summary>
	/// Stores a record of a built target, with all metadata that other tools may need to know about the build.
	/// </summary>
	[Serializable]
	public class TargetReceipt
	{
		/// <summary>
		/// Path to the project file for this target
		/// </summary>
		public FileReference? ProjectFile;

		/// <summary>
		/// The project directory
		/// </summary>
		public DirectoryReference? ProjectDir;

		/// <summary>
		/// The name of this target
		/// </summary>
		public required string TargetName;

		/// <summary>
		/// The build settings version used for this target.
		/// </summary>
		public BuildSettingsVersion BuildSettingsVersion;

		/// <summary>
		/// The build environment used for this target.
		/// </summary>
		public TargetBuildEnvironment TargetBuildEnvironment;

		/// <summary>
		/// Which platform the target is compiled for
		/// </summary>
		public UnrealTargetPlatform Platform;

		/// <summary>
		/// Which platform the target is compiled for
		/// </summary>
		public required UnrealArchitectures Architectures;

		/// <summary>
		/// Which configuration this target is compiled in
		/// </summary>
		public UnrealTargetConfiguration Configuration;

		/// <summary>
		/// The type of the target
		/// </summary>
		public TargetType TargetType;

		/// <summary>
		/// Whether it's a low level tests target
		/// </summary>
		public bool IsTestTarget;

		/// <summary>
		/// Version information for this target.
		/// </summary>
		public required BuildVersion Version;

		/// <summary>
		/// The exectuable to launch for this target
		/// </summary>
		public FileReference? Launch;

		/// <summary>
		/// The console subsystem/commmandlet exectuable to launch for this target
		/// </summary>
		public FileReference? LaunchCmd;

		/// <summary>
		/// The build products which are part of this target
		/// </summary>
		public List<BuildProduct> BuildProducts = new List<BuildProduct>();

		/// <summary>
		/// All the runtime dependencies that this target relies on
		/// </summary>
		public RuntimeDependencyList RuntimeDependencies = new RuntimeDependencyList();

		/// <summary>
		/// All plugins that were either enabled or disabled via the target rules.
		/// </summary>
		public Dictionary<string, bool> PluginNameToEnabledState = new Dictionary<string, bool>();

		/// <summary>
		/// All plugins that were built via the target rules.
		/// </summary>
		public List<string> BuildPlugins = new List<string>();

		/// <summary>
		/// Additional build properties passed through from the module rules
		/// </summary>
		public List<ReceiptProperty> AdditionalProperties = new List<ReceiptProperty>();

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="InProjectFile">Path to the project file for this target</param>
		/// <param name="InTargetName">The name of the target being compiled</param>
		/// <param name="InTargetType">The type of target</param>
		/// <param name="InPlatform">Platform for the target being compiled</param>
		/// <param name="InConfiguration">Configuration of the target being compiled</param>
		/// <param name="InVersion">Version information for the target</param>
		/// <param name="InArchitectures">Architecture information for the target</param>
		/// <param name="InIsTestTarget">Whether it's a target for low level tests</param>
		/// <param name="InBuildSettingsVersion">Build settings version of the target.</param>
		/// <param name="InTargetBuildEnvironment">Build environment used to produce the target.</param>
		[SetsRequiredMembers]
		public TargetReceipt(FileReference? InProjectFile, string InTargetName, TargetType InTargetType, UnrealTargetPlatform InPlatform, UnrealTargetConfiguration InConfiguration, BuildVersion InVersion, UnrealArchitectures InArchitectures, bool InIsTestTarget, BuildSettingsVersion InBuildSettingsVersion, TargetBuildEnvironment InTargetBuildEnvironment)
		{
			ProjectFile = InProjectFile;
			ProjectDir = DirectoryReference.FromFile(InProjectFile);
			TargetName = InTargetName;
			BuildSettingsVersion = InBuildSettingsVersion;
			TargetBuildEnvironment = InTargetBuildEnvironment;
			Platform = InPlatform;
			Configuration = InConfiguration;
			TargetType = InTargetType;
			Version = InVersion;
			Architectures = InArchitectures;
			IsTestTarget = InIsTestTarget;
		}

		/// <summary>
		/// Constructor required for JSON serialization.
		/// </summary>
		[JsonConstructor]
		private TargetReceipt()
		{
		}

		/// <summary>
		/// Adds a build product to the receipt. Does not check whether it already exists.
		/// </summary>
		/// <param name="Path">Path to the build product.</param>
		/// <param name="Type">Type of build product.</param>
		/// <returns>The BuildProduct object that was created</returns>
		public BuildProduct AddBuildProduct(FileReference Path, BuildProductType Type)
		{
			BuildProduct NewBuildProduct = new BuildProduct(Path, Type);
			BuildProducts.Add(NewBuildProduct);
			return NewBuildProduct;
		}

		/// <summary>
		/// Merges another receipt to this one.
		/// </summary>
		/// <param name="Other">Receipt which should be merged</param>
		public void Merge(TargetReceipt Other)
		{
			foreach (BuildProduct OtherBuildProduct in Other.BuildProducts)
			{
				BuildProducts.Add(OtherBuildProduct);
			}
			foreach (RuntimeDependency OtherRuntimeDependency in Other.RuntimeDependencies)
			{
				if (!RuntimeDependencies.Any(x => x.Path == OtherRuntimeDependency.Path))
				{
					RuntimeDependencies.Add(OtherRuntimeDependency);
				}
			}
			foreach (KeyValuePair<string, bool> Pair in Other.PluginNameToEnabledState)
			{
				if (!PluginNameToEnabledState.ContainsKey(Pair.Key))
				{
					PluginNameToEnabledState.Add(Pair.Key, Pair.Value);
				}
			}
			foreach (string PluginName in Other.BuildPlugins)
			{
				if (!BuildPlugins.Contains(PluginName))
				{
					BuildPlugins.Add(PluginName);
				}
			}
		}

		/// <summary>
		/// Inserts variables to make a file relative to $(EngineDir) or $(ProjectDir)
		/// </summary>
		/// <param name="File">The file to insert variables into.</param>
		/// <param name="EngineDir">Value of the $(EngineDir) variable.</param>
		/// <param name="ProjectDir">Value of the $(ProjectDir) variable.</param>
		/// <returns>Converted path for the file.</returns>
		static string InsertPathVariables(FileReference File, DirectoryReference EngineDir, DirectoryReference? ProjectDir)
		{
			if (File.IsUnderDirectory(EngineDir))
			{
				return "$(EngineDir)/" + File.MakeRelativeTo(EngineDir).Replace(Path.DirectorySeparatorChar, '/');
			}
			else if (ProjectDir != null && File.IsUnderDirectory(ProjectDir))
			{
				return "$(ProjectDir)/" + File.MakeRelativeTo(ProjectDir).Replace(Path.DirectorySeparatorChar, '/');
			}
			else
			{
				return File.FullName;
			}
		}

		/// <summary>
		/// Inserts variables to make a file relative to $(EngineDir) or $(ProjectDir)
		/// </summary>
		/// <param name="Path">The path to insert variables into.</param>
		/// <param name="EngineDir">Value of the $(EngineDir) variable.</param>
		/// <param name="ProjectDir">Value of the $(ProjectDir) variable.</param>
		/// <returns>Converted path for the file.</returns>
		static FileReference ExpandPathVariables(string Path, DirectoryReference EngineDir, DirectoryReference? ProjectDir)
		{
			const string EnginePrefix = "$(EngineDir)";
			if (Path.StartsWith(EnginePrefix, StringComparison.InvariantCultureIgnoreCase))
			{
				return new FileReference(EngineDir.FullName + Path.Substring(EnginePrefix.Length));
			}

			const string ProjectPrefix = "$(ProjectDir)";
			if (ProjectDir != null && Path.StartsWith(ProjectPrefix, StringComparison.InvariantCultureIgnoreCase))
			{
				return new FileReference(ProjectDir.FullName + Path.Substring(ProjectPrefix.Length));
			}

			return new FileReference(Path);
		}

		/// <summary>
		/// Returns the standard path to the build receipt for a given target
		/// </summary>
		/// <param name="BaseDir">Base directory for the target being built; either the project directory or engine directory.</param>
		/// <param name="TargetName">The target being built</param>
		/// <param name="Platform">The target platform</param>
		/// <param name="Configuration">The target configuration</param>
		/// <param name="BuildArchitectures">The architecture being built</param>
		/// <returns>Path to the receipt for this target</returns>
		public static FileReference GetDefaultPath(DirectoryReference BaseDir, string TargetName, UnrealTargetPlatform Platform, UnrealTargetConfiguration Configuration, UnrealArchitectures? BuildArchitectures)
		{
			// Get the architecture suffix. Platforms have the option of overriding whether to include this string in filenames.
			string ArchitectureSuffix = "";
			// @todo disallow null here?
			if (BuildArchitectures != null && UnrealArchitectureConfig.ForPlatform(Platform).RequiresArchitectureFilenames(BuildArchitectures))
			{
				ArchitectureSuffix = BuildArchitectures.ToString();
			}

			// Build the output filename
			if (String.IsNullOrEmpty(ArchitectureSuffix) && Configuration == UnrealTargetConfiguration.Development)
			{
				return FileReference.Combine(BaseDir, "Binaries", Platform.ToString(), String.Format("{0}.target", TargetName));
			}
			else
			{
				return FileReference.Combine(BaseDir, "Binaries", Platform.ToString(), String.Format("{0}-{1}-{2}{3}.target", TargetName, Platform.ToString(), Configuration.ToString(), ArchitectureSuffix));
			}
		}

		/// <summary>
		/// Determine whether a BuildProduct Type is NonUFS or DebugNonUFS
		/// </summary>
		/// <param name="BuildProduct">The build product to check</param>
		public static StagedFileType GetStageTypeFromBuildProductType(BuildProduct BuildProduct)
		{
			if (BuildProduct.Type == BuildProductType.SymbolFile || BuildProduct.Type == BuildProductType.MapFile)
			{
				return StagedFileType.DebugNonUFS;
			}
			return StagedFileType.NonUFS;
		}

		/// <summary>
		/// Checks the Additional properties for one with the given name that matches the given value
		/// </summary>
		/// <param name="PropertyName">Property name to search for</param>
		/// <param name="Value">Value to compare against (with StringComparison.OrdinalIgnoreCase)</param>
		/// <returns>True if any property with PropertyName has a value matching Value</returns>
		public bool HasValueForAdditionalProperty(string PropertyName, string Value)
		{
			// get all properties with the given name?
			IEnumerable<ReceiptProperty> Results = AdditionalProperties.Where(x => PropertyName.Equals(x.Name, StringComparison.Ordinal));
			foreach (ReceiptProperty Property in Results)
			{
				// does the property value match?
				if (Property.Value.Equals(Value, StringComparison.OrdinalIgnoreCase))
				{
					return true;
				}
			}

			return false;
		}

		/// <summary>
		/// Read a receipt from disk.
		/// </summary>
		/// <param name="Location">Filename to read from</param>
		public static TargetReceipt Read(FileReference Location)
		{
			return Read(Location, Unreal.EngineDirectory);
		}

		/// <summary>
		/// Read a receipt from disk.
		/// </summary>
		/// <param name="Location">Filename to read from</param>
		/// <param name="EngineDir">Engine directory for expanded variables</param>
		public static TargetReceipt Read(FileReference Location, DirectoryReference EngineDir)
		{
			JsonObject RawObject = JsonObject.Read(Location);

			// Read the initial fields
			string TargetName = RawObject.GetStringField("TargetName");
			TargetType TargetType = RawObject.GetEnumField<TargetType>("TargetType");
			UnrealTargetPlatform Platform = UnrealTargetPlatform.Parse(RawObject.GetStringField("Platform"));
			UnrealTargetConfiguration Configuration = RawObject.GetEnumField<UnrealTargetConfiguration>("Configuration");

			// Try to read the build version
			BuildVersion? Version;
			if (!BuildVersion.TryParse(RawObject.GetObjectField("Version"), out Version))
			{
				throw new JsonException("Invalid 'Version' field", Location.FullName, null, null);
			}

			// Read the project path
			FileReference? ProjectFile;

			string? RelativeProjectFile;
			if (RawObject.TryGetStringField("Project", out RelativeProjectFile))
			{
				ProjectFile = FileReference.Combine(Location.Directory, RelativeProjectFile);
			}
			else
			{
				ProjectFile = null;
			}

			// Read the launch executable
			string? Architecture;
			UnrealArchitectures Architectures;
			if (RawObject.TryGetStringField("Architecture", out Architecture))
			{
				Architectures = UnrealArchitectures.FromString(Architecture, Platform)!;
			}
			else
			{
				// @todo this doesn't necessarily match how it was compiled - should this be an error case?
				Architectures = UnrealArchitectureConfig.ForPlatform(Platform).ActiveArchitectures(ProjectFile, TargetName);
			}

			bool IsTestTarget;
			RawObject.TryGetBoolField("IsTestTarget", out IsTestTarget);

			BuildSettingsVersion BuildSettingsVersion;
			if(!RawObject.TryGetEnumField<BuildSettingsVersion>("BuildSettingsVersion", out BuildSettingsVersion))
			{
				// @todo this doesn't necessailry match how it was compiled - should this be an error case?
				BuildSettingsVersion = BuildSettingsVersion.Latest;
			}

			TargetBuildEnvironment TargetBuildEnvironment;
			if (!RawObject.TryGetEnumField<TargetBuildEnvironment>("TargetBuildEnvironment", out TargetBuildEnvironment))
			{
				// @todo this doesn't necessailry match how it was compiled - should this be an error case?
				TargetBuildEnvironment = TargetBuildEnvironment.Unique;
			}

			// Create the receipt
			TargetReceipt Receipt = new TargetReceipt(ProjectFile, TargetName, TargetType, Platform, Configuration, Version, Architectures, IsTestTarget, BuildSettingsVersion, TargetBuildEnvironment);

			// Get the project directory
			DirectoryReference? ProjectDir = Receipt.ProjectDir;

			// Read the launch executable
			string? Launch;
			if (RawObject.TryGetStringField("Launch", out Launch))
			{
				Receipt.Launch = ExpandPathVariables(Launch, EngineDir, ProjectDir);
			}
			string? LaunchCmd;
			if (RawObject.TryGetStringField("LaunchCmd", out LaunchCmd))
			{
				Receipt.LaunchCmd = ExpandPathVariables(LaunchCmd, EngineDir, ProjectDir);
			}

			// Read the build products
			JsonObject[]? BuildProductObjects;
			if (RawObject.TryGetObjectArrayField("BuildProducts", out BuildProductObjects))
			{
				foreach (JsonObject BuildProductObject in BuildProductObjects)
				{
					string? Path;
					BuildProductType Type;
					if (BuildProductObject.TryGetStringField("Path", out Path) && BuildProductObject.TryGetEnumField("Type", out Type))
					{
						FileReference File = ExpandPathVariables(Path, EngineDir, ProjectDir);

						Receipt.AddBuildProduct(File, Type);
					}
				}
			}

			// Read the runtime dependencies
			JsonObject[]? RuntimeDependencyObjects;
			if (RawObject.TryGetObjectArrayField("RuntimeDependencies", out RuntimeDependencyObjects))
			{
				foreach (JsonObject RuntimeDependencyObject in RuntimeDependencyObjects)
				{
					string? Path;
					if (RuntimeDependencyObject.TryGetStringField("Path", out Path))
					{
						FileReference File = ExpandPathVariables(Path, EngineDir, ProjectDir);

						StagedFileType Type;
						if (!RuntimeDependencyObject.TryGetEnumField("Type", out Type))
						{
							// Previous format included an optional IgnoreIfMissing flag, which was only used for debug files. We can explicitly reference them as DebugNonUFS files now.
							bool bIgnoreIfMissing;
							if (RuntimeDependencyObject.TryGetBoolField("IgnoreIfMissing", out bIgnoreIfMissing))
							{
								bIgnoreIfMissing = false;
							}
							Type = bIgnoreIfMissing ? StagedFileType.DebugNonUFS : StagedFileType.NonUFS;
						}

						Receipt.RuntimeDependencies.Add(File, Type);
					}
				}
			}

			// Read the enabled/disabled plugins
			JsonObject[]? PluginObjects;
			if (RawObject.TryGetObjectArrayField("Plugins", out PluginObjects))
			{
				foreach (JsonObject PluginObject in PluginObjects)
				{
					string? PluginName;
					if (PluginObject.TryGetStringField("Name", out PluginName))
					{
						bool PluginEnabled;
						if (PluginObject.TryGetBoolField("Enabled", out PluginEnabled))
						{
							Receipt.PluginNameToEnabledState.Add(PluginName, PluginEnabled);
						}
					}
				}
			}

			// Read the build plugins
			string[]? BuildPlugins;
			if (RawObject.TryGetStringArrayField("BuildPlugins", out BuildPlugins))
			{
				Receipt.BuildPlugins.AddAll(BuildPlugins);
			}

			// Read the additional properties
			JsonObject[]? AdditionalPropertyObjects;
			if (RawObject.TryGetObjectArrayField("AdditionalProperties", out AdditionalPropertyObjects))
			{
				foreach (JsonObject AdditionalPropertyObject in AdditionalPropertyObjects)
				{
					string? Name;
					if (AdditionalPropertyObject.TryGetStringField("Name", out Name))
					{
						string? Value;
						if (AdditionalPropertyObject.TryGetStringField("Value", out Value))
						{
							Receipt.AdditionalProperties.Add(new ReceiptProperty(Name, Value));
						}
					}
				}
			}

			return Receipt;
		}

		/// <summary>
		/// Try to read a receipt from disk, failing gracefully if it can't be read.
		/// </summary>
		/// <param name="Location">Filename to read from</param>
		/// <param name="Receipt">If successful, the receipt that was read</param>
		/// <returns>True if successful</returns>
		public static bool TryRead(FileReference Location, [NotNullWhen(true)] out TargetReceipt? Receipt)
		{
			return TryRead(Location, Unreal.EngineDirectory, out Receipt);
		}

		/// <summary>
		/// Try to read a receipt from disk, failing gracefully if it can't be read.
		/// </summary>
		/// <param name="Location">Filename to read from</param>
		/// <param name="EngineDir">Engine directory for expanded paths</param>
		/// <param name="Receipt">If successful, the receipt that was read</param>
		/// <returns>True if successful</returns>
		public static bool TryRead(FileReference Location, DirectoryReference EngineDir, [NotNullWhen(true)] out TargetReceipt? Receipt)
		{
			if (!FileReference.Exists(Location))
			{
				Receipt = null;
				return false;
			}

			try
			{
				Receipt = Read(Location, EngineDir);
				return true;
			}
			catch (Exception)
			{
				Receipt = null;
				return false;
			}
		}

		/// <summary>
		/// Write the receipt to disk.
		/// </summary>
		/// <param name="Location">Output filename</param>
		public void Write(FileReference Location)
		{
			Write(Location, Unreal.EngineDirectory);
		}

		/// <summary>
		/// Write the receipt to disk.
		/// </summary>
		/// <param name="Location">Output filename</param>
		/// <param name="EngineDir">Engine directory for expanded paths</param>
		public void Write(FileReference Location, DirectoryReference EngineDir)
		{
			using (JsonWriter Writer = new JsonWriter(Location.FullName))
			{
				Writer.WriteObjectStart();
				Writer.WriteValue("TargetName", TargetName);
				Writer.WriteValue("Platform", Platform.ToString());
				Writer.WriteValue("Configuration", Configuration.ToString());
				Writer.WriteValue("BuildSettingsVersion", BuildSettingsVersion.ToString());
				Writer.WriteValue("TargetBuildEnvironment", TargetBuildEnvironment.ToString());
				Writer.WriteValue("TargetType", TargetType.ToString());
				Writer.WriteValue("IsTestTarget", IsTestTarget);
				Writer.WriteValue("Architecture", Architectures.ToString());

				if (ProjectFile != null)
				{
					Writer.WriteValue("Project", ProjectFile.MakeRelativeTo(Location.Directory).Replace(Path.DirectorySeparatorChar, '/'));
				}

				if (Launch != null)
				{
					Writer.WriteValue("Launch", InsertPathVariables(Launch, EngineDir, ProjectDir));
				}

				if (LaunchCmd != null)
				{
					Writer.WriteValue("LaunchCmd", InsertPathVariables(LaunchCmd, EngineDir, ProjectDir));
				}

				Writer.WriteObjectStart("Version");
				Version.WriteProperties(Writer);
				Writer.WriteObjectEnd();

				Writer.WriteArrayStart("BuildProducts");
				foreach (BuildProduct BuildProduct in BuildProducts.OrderBy(x => x.Path.FullName))
				{
					Writer.WriteObjectStart();
					Writer.WriteValue("Path", InsertPathVariables(BuildProduct.Path, EngineDir, ProjectDir));
					Writer.WriteValue("Type", BuildProduct.Type.ToString());
					Writer.WriteObjectEnd();
				}
				Writer.WriteArrayEnd();

				Writer.WriteArrayStart("RuntimeDependencies");
				foreach (RuntimeDependency RuntimeDependency in RuntimeDependencies.OrderBy(x => x.Path.FullName))
				{
					Writer.WriteObjectStart();
					Writer.WriteValue("Path", InsertPathVariables(RuntimeDependency.Path, EngineDir, ProjectDir));
					Writer.WriteValue("Type", RuntimeDependency.Type.ToString());
					Writer.WriteObjectEnd();
				}
				Writer.WriteArrayEnd();

				if (PluginNameToEnabledState.Count > 0)
				{
					Writer.WriteArrayStart("Plugins");
					foreach (KeyValuePair<string, bool> PluginNameToEnabledStatePair in PluginNameToEnabledState.OrderBy(x => x.Key))
					{
						Writer.WriteObjectStart();
						Writer.WriteValue("Name", PluginNameToEnabledStatePair.Key);
						Writer.WriteValue("Enabled", PluginNameToEnabledStatePair.Value);
						Writer.WriteObjectEnd();
					}
					Writer.WriteArrayEnd();
				}

				if (BuildPlugins.Count > 0)
				{
					Writer.WriteStringArrayField("BuildPlugins", BuildPlugins.OrderBy(x => x));
				}

				if (AdditionalProperties.Count > 0)
				{
					Writer.WriteArrayStart("AdditionalProperties");
					foreach (ReceiptProperty AdditionalProperty in AdditionalProperties)
					{
						Writer.WriteObjectStart();
						Writer.WriteValue("Name", AdditionalProperty.Name);
						Writer.WriteValue("Value", AdditionalProperty.Value);
						Writer.WriteObjectEnd();
					}
					Writer.WriteArrayEnd();
				}

				Writer.WriteObjectEnd();
			}
		}
	}
}
