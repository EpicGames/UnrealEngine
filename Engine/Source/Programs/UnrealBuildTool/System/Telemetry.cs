// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Linq;
using System.Net.Http;
using System.Text.Json.Serialization;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Core.Telemetry;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	internal abstract class TelemetryEvent : DataRouterEvent
	{
		/// <inheritdoc/>
		public override string EventName => "Core.UBT";

		public abstract int SchemaVersion { get; }

		public string Mode { get; protected set; } = String.Empty;

		// If multiple targets are built, these will contain the first target found
		public string Target { get; protected set; } = String.Empty;
		public string Configuration { get; protected set; } = String.Empty;
		public string Platform { get; protected set; } = String.Empty;
		public string Architecture { get; protected set; } = String.Empty;

		public string Build_BranchName { get; protected set; } = String.Empty;
		public bool Config_IsBuildMachine { get; protected set; } = false;

		public string Horde_BatchID { get; protected set; } = String.Empty;
		public string Horde_JobID { get; protected set; } = String.Empty;
		public string Horde_JobURL { get; protected set; } = String.Empty;
		public string Horde_StepID { get; protected set; } = String.Empty;
		public string Horde_StepName { get; protected set; } = String.Empty;
		public string Horde_StepURL { get; protected set; } = String.Empty;
		public string Horde_TemplateID { get; protected set; } = String.Empty;
		public string Horde_TemplateName { get; protected set; } = String.Empty;

		protected TelemetryEvent(DateTime? timestamp = null) : base(timestamp)
		{
		}

		public void SetMetadata(string appId, TelemetryMetadata metadata)
		{
			AppId = appId;
			AppVersion = metadata.AppVersion;
			AppEnvironment = metadata.AppEnvironment;
			UploadType = metadata.UploadType;
			UserId = metadata.UserId;
			SessionId = metadata.SessionId;
			Mode = metadata.Mode;
			Target = metadata.Target;
			Configuration = metadata.Configuration;
			Platform = metadata.Platform;
			Architecture = metadata.Architecture;

			Build_BranchName = metadata.Build_BranchName;
			Config_IsBuildMachine = metadata.Config_IsBuildMachine;

			Horde_TemplateID = metadata.Horde_TemplateID;
			Horde_TemplateName = metadata.Horde_TemplateName;
			Horde_JobURL = metadata.Horde_JobURL;
			Horde_JobID = metadata.Horde_JobID;
			Horde_StepName = metadata.Horde_StepName;
			Horde_StepID = metadata.Horde_StepID;
			Horde_StepURL = metadata.Horde_StepURL;
			Horde_BatchID = metadata.Horde_BatchID;
		}

		public TelemetryEvent CloneWithMetadata(string appId, TelemetryMetadata metadata)
		{
			TelemetryEvent clone = (TelemetryEvent)MemberwiseClone();
			clone.SetMetadata(appId, metadata);
			return clone;
		}
	}

	internal class TelemetryCompletedEvent : TelemetryEvent
	{
		/// <inheritdoc/>
		public override string EventName => "Core.UBT.Completed";

		public override int SchemaVersion => 3;

		public string[] Arguments { get; }

		[JsonConverter(typeof(UnixTimestampDataRouterEventConverter))]
		public DateTime SessionStartUTC { get; }
		[JsonConverter(typeof(DurationDataRouterEventConverter))]
		public TimeSpan Duration { get; }
		public CompilationResult Result { get; }

		public TelemetryCompletedEvent(string[] arguments, DateTime sessionStartUTC, CompilationResult result, DateTime timestamp)
			: base(timestamp)
		{
			Arguments = arguments;
			SessionStartUTC = sessionStartUTC;
			Duration = timestamp - SessionStartUTC;
			Result = result;
		}
	}

	internal class TelemetryExecutorEvent : TelemetryEvent
	{
		/// <inheritdoc/>
		public override string EventName => "Core.UBT.Executor";

		public override int SchemaVersion => 3;

		public string Executor { get; }
		[JsonConverter(typeof(UnixTimestampDataRouterEventConverter))]
		public DateTime StartUTC { get; }
		[JsonConverter(typeof(DurationDataRouterEventConverter))]
		public TimeSpan Duration { get; }
		public ulong PeakProcessMemory { get; }
		public ulong PeakTotalMemory { get; }

		public bool Result { get; }
		public int TotalActions { get; }
		public int SucceededActions { get; }
		public int FailedActions { get; }
		public int SkippedActions => TotalActions - SucceededActions - FailedActions;
		public int CacheCheckActions => CacheHitActions + CacheMissActions;
		public int CacheHitActions { get; }
		public int CacheMissActions { get; }
		public double SuccessRate => SucceededActions / (double)TotalActions;
		public double FailureRate => FailedActions / (double)TotalActions;
		public double SkipRate => SkippedActions / (double)TotalActions;
		public double CacheRate => CacheHitActions + CacheMissActions > 0 ? CacheHitActions / (double)CacheCheckActions : 0.0;

		public TelemetryExecutorEvent(string executor, DateTime startUTC, bool result, int totalActions, int succeededActions, int failedActions, int cacheHitActions, int cacheMissActions, DateTime timestamp,
			ulong peakProcessMemory, ulong peakTotalMemory)
			: base(timestamp)
		{
			Executor = executor;
			StartUTC = startUTC;
			Duration = timestamp - StartUTC;
			Result = result;
			TotalActions = totalActions;
			SucceededActions = succeededActions;
			FailedActions = failedActions;
			CacheHitActions = cacheHitActions;
			CacheMissActions = cacheMissActions;
			PeakProcessMemory = peakProcessMemory;
			PeakTotalMemory = peakTotalMemory;
		}
	}

	internal class TelemetryExecutorUBAEvent : TelemetryExecutorEvent
	{
		public int LocalActions { get; }
		public int RemoteActions { get; }

		public int RetriedLocalActions { get; }
		public int RetriedDisabledActions { get; }

		public int TotalCoordinators { get; }
		public int SucceededCoordinators { get; }
		public int FailedCoordinators { get; }

		[JsonConverter(typeof(DurationDataRouterEventConverter))]
		public TimeSpan DurationWaitingForRemote { get; }

		public string ConnectionMode { get; }

		public double RetriedLocalRate => RetriedLocalActions / (double)TotalActions;
		public double RetriedDisabledRate => RetriedDisabledActions / (double)TotalActions;

		public double LocalUsage => LocalActions / (double)(TotalActions + RetriedLocalActions + RetriedDisabledActions - SkippedActions);
		public double RemoteUsage => RemoteActions / (double)(TotalActions + RetriedLocalActions + RetriedDisabledActions - SkippedActions);

		public TelemetryExecutorUBAEvent(string executor, DateTime startUTC, bool result, int totalActions, int succeededActions, int failedActions, int cacheHitActions, int cacheMissActions,
			int localActions, int remoteActions,
			int retriedLocalActions, int retriedDisabledActions,
			int totalCoordinators, int succeededCoordinators, int failedCoordinators,
			TimeSpan durationWaitingForRemote, string connectionMode,
			DateTime timestamp, ulong peakProcessMemory, ulong peakTotalMemory)
			: base(executor, startUTC, result, totalActions, succeededActions, failedActions, cacheHitActions, cacheMissActions, timestamp, peakProcessMemory, peakTotalMemory)
		{
			LocalActions = localActions;
			RemoteActions = remoteActions;
			RetriedLocalActions = retriedLocalActions;
			RetriedDisabledActions = retriedDisabledActions;
			TotalCoordinators = totalCoordinators;
			SucceededCoordinators = succeededCoordinators;
			FailedCoordinators = failedCoordinators;
			DurationWaitingForRemote = durationWaitingForRemote;
			ConnectionMode = connectionMode;
		}
	}

	internal sealed class TelemetryEndpoint : IDisposable
	{
		readonly DataRouterTelemetryService _service;
		readonly HttpClient _httpClient;
		readonly Uri _baseAddress;
		readonly string _appId;
		readonly Lazy<TelemetryMetadata> _metadata;
		bool _disposedValue;

		public TelemetryEndpoint(HttpClient httpClient, Uri baseAddress, string appId, Lazy<TelemetryMetadata> metadata)
		{
			_httpClient = httpClient;
			_baseAddress = baseAddress;
			_appId = appId;
			_metadata = metadata;
			_service = new(_httpClient, _baseAddress);
		}

		public async Task FlushEventsAsync() => await _service.FlushEventsAsync();

		public void RecordEvent(TelemetryEvent eventData) => _service.RecordEvent(eventData.CloneWithMetadata(_appId, _metadata.Value));

		public void Dispose()
		{
			if (!_disposedValue)
			{
				_service.Dispose();
				_disposedValue = true;
			}

			GC.SuppressFinalize(this);
		}
	}

	internal readonly struct TelemetryMetadata
	{
		public string AppVersion { get; }
		public string AppEnvironment { get; }
		public string UploadType { get; }
		public string UserId { get; }
		public string SessionId { get; }
		public string Mode { get; }
		public string Target { get; }
		public string Configuration { get; }
		public string Platform { get; }
		public string Architecture { get; }

		public string Build_BranchName { get; }
		public bool Config_IsBuildMachine { get; }

		public string Horde_BatchID { get; }
		public string Horde_JobID { get; }
		public string Horde_URL { get; }
		public string Horde_JobURL { get; }
		public string Horde_StepID { get; }
		public string Horde_StepName { get; }
		public string Horde_StepURL { get; }
		public string Horde_TemplateID { get; }
		public string Horde_TemplateName { get; }

		public TelemetryMetadata(string appVersion, string appEnvironment, string uploadType, string userID, string sessionId, string mode, TargetDescriptor? descriptor,
			string? buildBranchName, bool configIsBuildMachine,
			string? hordeBatchID, string? hordeJobID, string? hordeURL, string? hordeStepID, string? hordeStepName, string? hordeTemplateID, string? hordeTemplateName)
		{
			AppVersion = appVersion;
			AppEnvironment = appEnvironment;
			UploadType = uploadType;
			UserId = userID;
			SessionId = sessionId;
			Mode = mode;
			Target = descriptor?.Name ?? "Unknown";
			Configuration = descriptor?.Configuration.ToString() ?? "Unknown";
			Platform = descriptor?.Platform.ToString() ?? "Unknown";
			Architecture = descriptor?.Architectures.ToString() ?? "Unknown";

			Build_BranchName = buildBranchName ?? String.Empty;
			Config_IsBuildMachine = configIsBuildMachine;

			Horde_BatchID = hordeBatchID ?? String.Empty;
			Horde_JobID = hordeJobID ?? String.Empty;
			Horde_URL = hordeURL ?? String.Empty;
			Horde_JobURL = !String.IsNullOrEmpty(Horde_URL) && !String.IsNullOrEmpty(Horde_JobID) ? $"{Horde_URL}job/{Horde_JobID}" : String.Empty;
			Horde_StepID = hordeStepID ?? String.Empty;
			Horde_StepName = hordeStepName ?? String.Empty;
			Horde_StepURL = !String.IsNullOrEmpty(Horde_JobURL) && !String.IsNullOrEmpty(Horde_StepID) ? $"{Horde_JobURL}?step={Horde_StepID}" : String.Empty;
			Horde_TemplateID = hordeTemplateID ?? String.Empty;
			Horde_TemplateName = hordeTemplateName ?? String.Empty;

			if (Mode.EndsWith("Mode", StringComparison.Ordinal))
			{
				Mode = Mode[..^4];
			}

			if (descriptor?.bRebuild == true && Mode == "Build")
			{
				Mode = "Rebuild";
			}
		}
	}

	/// <summary>
	/// Telemetry service for UnrealBuildTool
	/// </summary>
	internal class TelemetryService : ITelemetryService<TelemetryEvent>
	{
#if DEBUG
		readonly bool _bAllowRecordingEventsDebug = false;
#endif

		const string ConfigCategoryPrefix = "StudioTelemetry.";
		static readonly string? _hordeUrl = Environment.GetEnvironmentVariable("UE_HORDE_URL");

		// Lazy to defer resolution until an event is first sent
		readonly Lazy<TelemetryMetadata> _metadata = new(() =>
		{
			BuildVersion.TryRead(BuildVersion.GetDefaultFileName(), out BuildVersion? version);
			string buildBranchName = version?.BranchName ?? String.Empty;
			if (Unreal.IsBuildMachine())
			{
				buildBranchName = Environment.GetEnvironmentVariable("uebp_BuildRoot_Escaped") ?? buildBranchName;
			}
#pragma warning disable CA1308 // Normalize strings to uppercase
			buildBranchName = buildBranchName.ToLowerInvariant();
#pragma warning restore CA1308 // Normalize strings to uppercase
			return new TelemetryMetadata(
				version != null ? $"{version.MajorVersion}.{version.MinorVersion}.{version.PatchVersion}-{version.Changelist}+{version.BranchName}" : String.Empty,
				"UnrealBuildTool",
				"eteventstream",
				Environment.UserName,
				UnrealBuildTool.SessionIdentifier,
				UnrealBuildTool.CurrentMode,
				Get()._descriptor,
				buildBranchName,
				Unreal.IsBuildMachine(),
				Environment.GetEnvironmentVariable("UE_HORDE_BATCHID"),
				Environment.GetEnvironmentVariable("UE_HORDE_JOBID"),
				Environment.GetEnvironmentVariable("UE_HORDE_URL"),
				Environment.GetEnvironmentVariable("UE_HORDE_STEPID"),
				Environment.GetEnvironmentVariable("UE_HORDE_STEPNAME"),
				Environment.GetEnvironmentVariable("UE_HORDE_TEMPLATEID"),
				Environment.GetEnvironmentVariable("UE_HORDE_TEMPLATENAME")
			);
		});

		readonly Lazy<HttpClient> _httpClient = new(HttpClientDefaults.GetClient);
		readonly ConcurrentDictionary<Tuple<Uri, string>, TelemetryEndpoint> _endpoints = [];
		readonly HashSet<string> _telemetryProviders = [];
		bool _disposed = false;

		TargetDescriptor? _descriptor = null;

		#region Singleton
		static readonly Lazy<TelemetryService> s_instance = new(() => new());
		public static TelemetryService Get() => s_instance.Value;

		private TelemetryService()
		{
		}
		#endregion

		public void SetPrimaryTargetDetails(TargetDescriptor? descriptor) => _descriptor ??= descriptor;

		public void AddTelemetryConfigProviders(IEnumerable<string> providers) => _telemetryProviders.UnionWith(providers);

		public void AddEndpointsFromConfig(DirectoryReference? projectDir, ILogger logger)
		{
			ConfigHierarchy engineIni = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, projectDir, BuildHostPlatform.Current.Platform);
			foreach (string provider in _telemetryProviders.Where(x => !String.IsNullOrEmpty(x)))
			{
				ConfigHierarchySection section = engineIni.FindSection(provider);
				// If we are running via horde and UE_HORDE_URL is set, skip any horde providers, which will be added below
				if (_hordeUrl != null && (!section.TryGetValue("APIKeyET", out string? apiKeyET) || apiKeyET.Contains("Horde", StringComparison.OrdinalIgnoreCase)))
				{
					continue;
				}
				AddEndpointFromConfigInternal(provider, section, logger);
			}

			// Add matching horde provider if found
			if (_hordeUrl != null)
			{
				AddMatchingHordeEndpointFromConfig(projectDir, logger);
			}
		}

		public void AddEndpointsFromConfig(ILogger logger) => AddEndpointsFromConfig(null, logger);

		private bool AddMatchingHordeEndpointFromConfig(DirectoryReference? projectDir, ILogger logger)
		{
			ConfigHierarchy engineIni = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, projectDir, BuildHostPlatform.Current.Platform);
			foreach (string provider in engineIni.SectionNames.Where(x => x.StartsWith(ConfigCategoryPrefix, StringComparison.Ordinal)))
			{
				ConfigHierarchySection section = engineIni.FindSection(provider);
				if (section.TryGetValue("APIServerET", out string? apiServerET) && apiServerET.Equals(_hordeUrl, StringComparison.OrdinalIgnoreCase))
				{
					return AddEndpointFromConfigInternal(provider, section, logger);
				}
			}
			return false;
		}

		private bool AddEndpointFromConfigInternal(string provider, ConfigHierarchySection section, ILogger logger)
		{
			if (section.TryGetValue("APIServerET", out string? apiServerET) && section.TryGetValue("APIKeyET", out string? apiKeyET) && section.TryGetValue("APIEndpointET", out string? apiEndpointET))
			{
				Uri baseAddress = new($"{apiServerET}{apiEndpointET}");
				Tuple<Uri, string> key = new(baseAddress, apiKeyET);
				if (_endpoints.TryAdd(key, new(_httpClient.Value, baseAddress, apiKeyET, _metadata)))
				{
					logger.LogDebug("Loaded telemetry endpoint Name={Name} Key={Key} Url={Url}", provider, apiKeyET, baseAddress);
					return true;
				}
			}

			return false;
		}

		#region ITelemetryService
		/// <inheritdoc/>
		public void RecordEvent(TelemetryEvent eventData)
		{
#if DEBUG
			if (_bAllowRecordingEventsDebug)
			{
				foreach (KeyValuePair<Tuple<Uri, string>, TelemetryEndpoint> endpoint in _endpoints)
				{
					endpoint.Value.RecordEvent(eventData);
				}
				return;
			}
#endif
			foreach (KeyValuePair<Tuple<Uri, string>, TelemetryEndpoint> endpoint in _endpoints)
			{
				endpoint.Value.RecordEvent(eventData);
			}
		}

		/// <inheritdoc/>
		public void FlushEvents()
		{
			IEnumerable<Task> flushTasks = _endpoints.Select(x => x.Value.FlushEventsAsync());
			Task.WaitAll([.. flushTasks]);
		}
		#endregion

		#region IDisposable
		/// <summary>
		/// Dispose
		/// </summary>
		/// <param name="disposing"></param>
		protected virtual void Dispose(bool disposing)
		{
			if (!_disposed)
			{
				FlushEvents();

				if (disposing)
				{
					foreach (TelemetryEndpoint endpoint in _endpoints.Values)
					{
						endpoint.Dispose();
					}
				}

				// free unmanaged resources (unmanaged objects) and override finalizer
				// set large fields to null

				_disposed = true;
			}
		}

		/// <inheritdoc/>
		public void Dispose()
		{
			Dispose(disposing: true);
			GC.SuppressFinalize(this);
		}
		#endregion
	}
}
