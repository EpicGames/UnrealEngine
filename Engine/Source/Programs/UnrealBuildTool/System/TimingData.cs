// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.IO;
using System.Linq;
using EpicGames.Core;

namespace UnrealBuildTool
{
	/// <summary>
	/// Types of timing data events.
	/// </summary>
	public enum TimingDataType
	{
		/// <summary>
		/// No type assigned.
		/// </summary>
		None,

		/// <summary>
		/// This event represents an aggregate of summary events.
		/// </summary>
		Aggregate,

		/// <summary>
		/// This event represents a summary of related include, class, and function events, usually
		/// summarizing a single compilation unit or part of a compilation unit.
		/// </summary>
		Summary,

		/// <summary>
		/// Timing related to compiling include files.
		/// </summary>
		Include,

		/// <summary>
		/// Timing related to compiling classes.
		/// </summary>
		Class,

		/// <summary>
		/// Timing related to compiling functions.
		/// </summary>
		Function,
	}

	/// <summary>
	/// Class used to capture timing data from events like those generated by MSVC when timing data
	/// is being generated.
	/// </summary>
	public class TimingData : IBinarySerializable
	{
		/// <summary>
		/// Constructs a new instance of <see cref="TimingData"/>.
		/// </summary>
		public TimingData(string Name, TimingDataType Type)
		{
			this.Name = Name;
			this.Type = Type;
		}

		/// <summary>
		/// Constructs a new instance of <see cref="TimingData"/> using data from a <see cref="BinaryReader"/>.
		/// </summary>
		/// <param name="Reader">The <see cref="BinaryReader"/> to get the data from.</param>
		public TimingData(BinaryReader Reader)
		{
			Name = Reader.ReadString();
			Type = (TimingDataType)Reader.ReadByte();
			Count = Reader.ReadInt32();
			ExclusiveDuration = Reader.ReadDouble();
			int ChildCount = Reader.ReadInt32();
			for (int i = 0; i < ChildCount; ++i)
			{
				TimingData NewChild = new TimingData(Reader);
				Children.Add(NewChild.Name, NewChild);
			}
		}

		/// <summary>
		/// The name of the event being represented, such as a file or category name.
		/// </summary>
		public string Name { get; set; }

		/// <summary>
		/// The type of timing data this event represents.
		/// </summary>
		public TimingDataType Type { get; set; }

		/// <summary>
		/// Gets the number of times this event was encountered.
		/// </summary>
		public int Count { get; set; } = 1;

		/// <summary>
		/// Gets the amount of time, in milliseconds, that the event took to complete disregarding its sub-events.
		/// </summary>
		public double ExclusiveDuration { get; set; } = 0.0;

		/// <summary>
		/// Gets the amount of time, in milliseconds, that the event took including its sub-events.
		/// </summary>
		public double InclusiveDuration => ExclusiveDuration + Children.Sum(c => c.Value.InclusiveDuration);

		/// <summary>
		/// If this event is a sub-event, the parent is the event it is the sub-event to.
		/// </summary>
		public TimingData? Parent { get; set; }

		/// <summary>
		/// Any sub-events of this event.
		/// </summary>
		public Dictionary<string, TimingData> Children { get; } = [];

		/// <summary>
		/// Adds a sub-event (child) to this event if it doesn't exist, or increase the duration by the
		/// duration of the provided event.
		/// </summary>
		/// <param name="ChildData">The sub-event to add to this event.</param>
		public void AddChild(TimingData ChildData)
		{
			TimingData? MatchingData;
			if (Children.TryGetValue(ChildData.Name, out MatchingData))
			{
				MatchingData.ExclusiveDuration += ChildData.ExclusiveDuration;
				foreach (TimingData ChildChildData in ChildData.Children.Values)
				{
					MatchingData.AddChild(ChildChildData);
				}
			}
			else
			{
				Children.Add(ChildData.Name, ChildData);
			}
		}

		/// <summary>
		/// Creates a deep clone of this event.
		/// </summary>
		/// <returns>A deep clone of this event.</returns>
		public TimingData Clone()
		{
			TimingData ClonedTimingData = new TimingData(Name, Type)
			{
				ExclusiveDuration = ExclusiveDuration,
			};

			foreach (KeyValuePair<string, TimingData> Child in Children)
			{
				ClonedTimingData.Children.Add(Child.Key, Child.Value.Clone());
			}

			return ClonedTimingData;
		}

		/// <summary>
		/// Writes a binary representation of this event and its sub-events.
		/// </summary>
		/// <param name="Writer">The <see cref="BinaryWriter"/> to write the binary representation to.</param>
		public void Write(BinaryWriter Writer)
		{
			Writer.Write(Name);
			Writer.Write((byte)Type);
			Writer.Write(Count);
			Writer.Write(ExclusiveDuration);
			Writer.Write(Children.Count);
			foreach (TimingData Child in Children.Values)
			{
				Writer.Write(Child);
			}
		}
	}
}
