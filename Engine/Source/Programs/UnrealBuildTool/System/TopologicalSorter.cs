// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using Microsoft.Extensions.Logging;

namespace UnrealBuildTool
{
	/// <summary>
	/// Helper class implementing the standard topological sorting algorithm for directed graphs.
	/// It generates a flattened sequence of nodes in the order that respects all the given dependencies
	/// between nodes (edges). So, if we have edges A -> B and B -> C, A will be always before B and C, and
	/// B will be before C.
	/// 
	/// In many cases there are more than one possible solutions. This helper just generates one of them.
	/// The algorithm only works for directed acyclic graphs, as otherwise it's impossible to order the nodes.
	/// In this implementation we can choose how cycles should be handled (CycleHandling member). By default
	/// sorting fails, but we may set the class to arbitrarily break cycles and complete the task.
	/// </summary>
	/// <typeparam name="T">Type of node objects</typeparam>
	class TopologicalSorter<T> where T : notnull
	{
		/// <summary>
		/// Enumeration indicating how graph cycles should be handled.
		/// </summary>
		public enum CycleMode
		{
			/// <summary>
			/// Fail sorting on the first cycle encountered (the algorithmically correct solution).
			/// </summary>
			Fail,

			/// <summary>
			/// Break cycles at the point they are detected (allows the algorithm to always complete but may skip some edges).
			/// </summary>
			Break,

			/// <summary>
			/// Break cycles at the point they are detected (same as Break but outputs warning logs with information about the cycles).
			/// </summary>
			BreakWithInfo
		}

		/// <summary>
		/// Determines what should be done when a cycle in the graph is encountered.
		/// </summary>
		public CycleMode CycleHandling = CycleMode.Fail;

		/// <summary>
		/// UBT's logger object. Necessary if we want to get diagnostics, especially when BreakWithInfo is used.
		/// </summary>
		public ILogger? Logger = null;

		/// <summary>
		/// Functor returning a node's name. Necessary for some logging diagnostics, especially when BreakWithInfo is used.
		/// </summary>
		public Func<T, string>? NodeToString = null;

		/// <summary>
		/// List of all graph nodes (each user-provided node T is wrapped in an internal GraphNode).
		/// </summary>
		private readonly List<GraphNode> Nodes = new List<GraphNode>();

		/// <summary>
		/// When traversing the graph, it represents all the nodes visited in the current sequence.
		/// For instance, if we go from node A to K and then X, Callstack will store A, K, X.
		/// It's only needed for diagnostic purposes i.e. when we find a cycle we use Callstack
		/// to output all the cycle's nodes.
		/// </summary>
		private readonly Stack<GraphNode> Callstack = new Stack<GraphNode>();

		/// <summary>
		/// The resulting sequence of nodes (T, provided by the user) in the topological order.
		/// </summary>
		private readonly List<T> Result = new List<T>();

		/// <summary>
		/// Initialize the sorter object and create the internal graph representation.
		/// The list of all the graph nodes doesn't need to be provided because it's internally generated
		/// from Edges (union of all edges' Item1 and Item2.
		/// </summary>
		/// <param name="Edges">List of connections between nodes</param>
		public TopologicalSorter(List<Tuple<T, T>> Edges)
		{
			CreateGraph(Edges);
		}

		/// <summary>
		/// Sort the graph to generate a flat sequence of nodes respecting all graph dependencies.
		/// </summary>
		/// <returns>True on success, false if failed (most likely a cycle encountered)</returns>
		public bool Sort()
		{
			Result.Clear();
			Callstack.Clear();

			ClearNodesStates();

			foreach (GraphNode Node in Nodes)
			{
				Callstack.Push(Node);

				if (!Visit(Node))
				{
					return false;
				}

				Callstack.Pop();
			}

			Result.Reverse();

			return true;
		}

		/// <summary>
		/// Get the list of sorted nodes (valid after successfully calling Sort).
		/// </summary>
		/// <returns></returns>
		public List<T> GetResult()
		{
			return Result;
		}

		/// <summary>
		/// Process a single node.
		/// </summary>
		/// <param name="Node"></param>
		/// <returns>True on success, false if failed (most likely a cycle encountered)</returns>
		private bool Visit(GraphNode Node)
		{
			if (Node.Mark == Mark.Done)
			{
				// This node has already been processed.
				return true;
			}

			if (Node.Mark == Mark.InProgress)
			{
				// This node is being processed i.e. we've detected a cycle.
				return HandleCycle();
			}

			Node.Mark = Mark.InProgress;

			foreach (GraphNode Next in Node.Links)
			{
				Callstack.Push(Next);

				if (!Visit(Next))
				{
					return false;
				}

				Callstack.Pop();
			}

			Node.Mark = Mark.Done;

			Result.Add(Node.Data);

			return true;
		}

		/// <summary>
		/// Create the internal graph structure for the provided list of edges.
		/// </summary>
		/// <param name="Edges">List of tuples representing connections between nodes</param>
		private void CreateGraph(List<Tuple<T, T>> Edges)
		{
			HashSet<T> AllNodes = new HashSet<T>();

			// Create a collection of all nodes based on the ones used in Edges.
			foreach (Tuple<T, T> Edge in Edges)
			{
				AllNodes.Add(Edge.Item1);
				AllNodes.Add(Edge.Item2);
			}

			Dictionary<T, GraphNode> NodeToGraphNode = new Dictionary<T, GraphNode>();

			// Create graph node objects.
			foreach (T Node in AllNodes)
			{
				GraphNode GraphNode = new GraphNode(Node);

				Nodes.Add(GraphNode);
				NodeToGraphNode.Add(Node, GraphNode);
			}

			// Add edges to the graph.
			foreach (Tuple<T, T> Edge in Edges)
			{
				GraphNode? NodeSrc;
				GraphNode? NodeDst;

				if (!NodeToGraphNode.TryGetValue(Edge.Item1, out NodeSrc))
				{
					throw new Exception($"TopologicalSorter: Failed to build graph (source node {Edge.Item1} from an edge unknown)!");
				}

				if (!NodeToGraphNode.TryGetValue(Edge.Item2, out NodeDst))
				{
					throw new Exception($"TopologicalSorter: Failed to build graph (source node {Edge.Item2} from an edge unknown)!");
				}

				NodeSrc.Links.Add(NodeDst);
			}
		}

		/// <summary>
		/// Clear the state (in-progress, done) of all graph nodes.
		/// </summary>
		private void ClearNodesStates()
		{
			foreach (GraphNode Node in Nodes)
			{
				Node.Mark = Mark.None;
			}
		}

		/// <summary>
		/// Member executed when a graph cycle is encountered.
		/// </summary>
		/// <returns>True if operation should continue, false if a cycle is considered an error</returns>
		private bool HandleCycle()
		{
			switch (CycleHandling)
			{
				case CycleMode.Fail:
					// We stop further processing and fail the whole sort.
					Logger?.LogError("TopologicalSorter: Cycle found in graph ({NodesInCycle})", GetNodesInCycleString());
					return false;

				case CycleMode.Break:
					// Treat the 'in-progress' node as if it's been 'done' i.e. break the cycle at this point.
					return true;

				case CycleMode.BreakWithInfo:
					// As in Break but output log information about the cycle.
					Logger?.LogWarning("TopologicalSorter: Cycle found in graph ({NodesInCycle})", GetNodesInCycleString());
					return true;
			}

			return false;
		}

		/// <summary>
		/// Returns all nodes being part of a cycle (call valid only if we've really detected a cycle).
		/// </summary>
		/// <returns></returns>
		private IEnumerable<GraphNode> GetNodesInCycle()
		{
			GraphNode[] CallstackNodes = Callstack.ToArray();

			// First, find where in the callstack the cycle starts.
			int CycleStart = 1;

			for (; CycleStart < CallstackNodes.Length; ++CycleStart)
			{
				if (CallstackNodes[CycleStart] == CallstackNodes[0])
				{
					break;
				}
			}

			return CallstackNodes.Take(CycleStart + 1).Reverse();
		}

		/// <summary>
		/// Returns a string with a list of names of nodes forming a graph cycle (call valid only if we've really detected a cycle).
		/// </summary>
		/// <returns></returns>
		private string GetNodesInCycleString()
		{
			if (NodeToString == null)
			{
				return "";
			}

			IEnumerable<GraphNode> CycleNodes = GetNodesInCycle();

			return String.Join(", ", CycleNodes.Select(Node => NodeToString(Node.Data)));
		}

		/// <summary>
		/// Graph node's state wrt the traversal.
		/// </summary>
		private enum Mark
		{
			None = 0,
			InProgress = 1,
			Done = 2
		}

		/// <summary>
		/// Internal helper representing a graph node (wrapping the user-provided node and storing internal state).
		/// </summary>
		private class GraphNode
		{
			/// <summary>
			/// User-provided node object.
			/// </summary>
			public T Data;

			/// <summary>
			/// List of graph nodes we can visit starting from this one.
			/// </summary>
			public List<GraphNode> Links = new List<GraphNode>();

			/// <summary>
			/// Traversal state (in-progress, done).
			/// </summary>
			public Mark Mark;

			public GraphNode(T Data)
			{
				this.Data = Data;
				Mark = Mark.None;
			}
		}
	}
}
