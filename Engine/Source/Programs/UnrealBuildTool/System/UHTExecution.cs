// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers.Binary;
using System.Collections;
using System.Collections.Generic;
using System.Data;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using System.IO.Hashing;
using System.Linq;
using System.Runtime.InteropServices;
using System.Text;
using System.Text.Json;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.UHT.Utils;
using Microsoft.Extensions.Logging;
using OpenTracing;
using OpenTracing.Util;
using UnrealBuildBase;
using UnrealBuildTool.Modes;

namespace UnrealBuildTool
{
	static class UHTModuleTypeExtensions
	{
		public static UHTModuleType? GetEngineModuleType(this ModuleHostType ModuleType)
		{
			return ModuleType switch
			{
				ModuleHostType.Program => UHTModuleType.Program,

				ModuleHostType.Runtime or 
				ModuleHostType.RuntimeNoCommandlet or 
				ModuleHostType.RuntimeAndProgram or 
				ModuleHostType.CookedOnly or 
				ModuleHostType.ServerOnly or 
				ModuleHostType.ClientOnly or 
				ModuleHostType.ClientOnlyNoCommandlet => UHTModuleType.EngineRuntime,

				ModuleHostType.Developer or 
				ModuleHostType.DeveloperTool => UHTModuleType.EngineDeveloper,

				ModuleHostType.Editor or
				ModuleHostType.EditorNoCommandlet or 
				ModuleHostType.EditorAndProgram => UHTModuleType.EngineEditor,

				ModuleHostType.UncookedOnly => UHTModuleType.EngineUncooked,

				_ => null,
			};
		}

		public static UHTModuleType? GetGameModuleType(this ModuleHostType ModuleType)
		{
			return ModuleType switch
			{
				ModuleHostType.Program => UHTModuleType.Program,

				ModuleHostType.Runtime or 
				ModuleHostType.RuntimeNoCommandlet or 
				ModuleHostType.RuntimeAndProgram or 
				ModuleHostType.CookedOnly or 
				ModuleHostType.ServerOnly or 
				ModuleHostType.ClientOnly or 
				ModuleHostType.ClientOnlyNoCommandlet => UHTModuleType.GameRuntime,

				ModuleHostType.Developer or 
				ModuleHostType.DeveloperTool => UHTModuleType.GameDeveloper,

				ModuleHostType.Editor or 
				ModuleHostType.EditorNoCommandlet or 
				ModuleHostType.EditorAndProgram => UHTModuleType.GameEditor,

				ModuleHostType.UncookedOnly => UHTModuleType.GameUncooked,

				_ => null,
			};
		}
	}

	/// <summary>
	/// Information about a module that needs to be passed to UnrealHeaderTool for code generation
	/// </summary>
	class UHTModuleInfo
	{
		/// <summary>
		/// Module name
		/// </summary>
		public required string ModuleName;

		/// <summary>
		/// Path to the module rules file
		/// </summary>
		public required FileReference ModuleRulesFile;

		/// <summary>
		/// Paths to all potential module source directories (with platform extension directories added in)
		/// </summary>
		public required DirectoryItem[] ModuleDirectories;

		/// <summary>
		/// The include search path for generated headers to include other headers
		/// </summary>
		public required DirectoryReference[] ModuleIncludePaths;

		/// <summary>
		/// Other modules containing UObject declarations which this module has link dependencies on
		/// </summary>
		public HashSet<string> ModuleDependencies = [];

		/// <summary>
		/// Module type
		/// </summary>
		public required string ModuleType;

		/// <summary>
		/// Overridden package module type to add more flags
		/// </summary>
		public required string OverrideModuleType;

		/// <summary>
		/// Public UObject headers found in the Classes directory (legacy)
		/// </summary>
		public List<FileItem> PublicUObjectClassesHeaders = [];

		/// <summary>
		/// Public headers with UObjects
		/// </summary>
		public List<FileItem> PublicUObjectHeaders = [];

		/// <summary>
		/// Internal headers with UObjects
		/// </summary>
		public List<FileItem> InternalUObjectHeaders = [];

		/// <summary>
		/// Private headers with UObjects
		/// </summary>
		public List<FileItem> PrivateUObjectHeaders = [];

		/// <summary>
		/// Directory containing generated code
		/// </summary>
		public required DirectoryItem GeneratedCodeDirectory;

		/// <summary>
		/// Collection of the module's public defines
		/// </summary>
		public List<string> PublicDefines = [];

		/// <summary>
		/// Path and base filename, without extension, of the .gen files
		/// </summary>
		public string? GeneratedCPPFilenameBase;

		/// <summary>
		/// Version of code generated by UHT
		/// </summary>
		public required EGeneratedCodeVersion GeneratedCodeVersion;

		/// <summary>
		/// Whether this module is read-only
		/// </summary>
		public required bool bIsReadOnly;

		/// <summary>
		/// Whether to exclude the Tests/ folder from UHT generation
		/// </summary>
		public bool bShouldExcludeTestsFolder;

		/// <summary>
		/// Path of the verse generated types 
		/// </summary>
		public required string VersePath;

		/// <summary>
		/// Scope of the verse definitions
		/// </summary>
		public required VerseScope VerseScope;

		/// <summary>
		/// Mount point for UE definitions
		/// </summary>
		public required string VerseMountPoint;

		/// <summary>
		/// Verse module name
		/// </summary>
		public required string VersePackageName;

		/// <summary>
		/// Relative location of the verse code
		/// </summary>
		public required string VerseDirectoryPath;

		/// <summary>
		/// Collection of verse packages this package depends on
		/// </summary>
		public HashSet<string> VersePublicDependencies = [];

		/// <summary>
		/// Collection of verse packages this package depends on
		/// </summary>
		public HashSet<string> VersePrivateDependencies = [];

		/// <summary>
		/// If true, verse scripts were found
		/// </summary>
		public required bool HasVerse;
		
		/// <summary>
		/// If true, autogenerated functions of USTRUCTS are always exported
		/// </summary>
		public required bool AlwaysExportStructs;

		/// <summary>
		/// If true, autogenerated functions of UENUMS are always exported
		/// </summary>
		public required bool AlwaysExportEnums;

		/// <summary>
		/// If true, UE types (i.e. UCLASS) will be allowed in namespaces
		/// </summary>
		public required bool AllowUETypesInNamespaces;

		/// <summary>
		/// If true, minimizes the amount of header files included in .generated.h files, which may require more headers from CoreUObject to be explicitly included.
		/// </summary>
		public required bool MinimizeGeneratedIncludes;

		public UHTModuleInfo()
		{
		}

		[SetsRequiredMembers]
		public UHTModuleInfo(BinaryArchiveReader Reader)
		{
			ModuleName = Reader.ReadString()!;
			ModuleRulesFile = Reader.ReadFileReference();
			ModuleDirectories = Reader.ReadArray<DirectoryItem>(() => Reader.ReadDirectoryItem()!)!;
			ModuleIncludePaths = Reader.ReadArray<DirectoryReference>(Reader.ReadDirectoryReferenceNotNull)!;
			ModuleDependencies = Reader.ReadHashSet(Reader.ReadString)!;
			ModuleType = Reader.ReadString()!;
			OverrideModuleType = Reader.ReadString()!;
			PublicUObjectClassesHeaders = Reader.ReadList(() => Reader.ReadFileItem())!;
			PublicUObjectHeaders = Reader.ReadList(() => Reader.ReadFileItem())!;
			InternalUObjectHeaders = Reader.ReadList(() => Reader.ReadFileItem())!;
			PrivateUObjectHeaders = Reader.ReadList(() => Reader.ReadFileItem())!;
			GeneratedCPPFilenameBase = Reader.ReadString();
			GeneratedCodeDirectory = Reader.ReadDirectoryItem()!;
			GeneratedCodeVersion = (EGeneratedCodeVersion)Reader.ReadInt();
			bIsReadOnly = Reader.ReadBool();
			PublicDefines = Reader.ReadList(() => Reader.ReadString())!;
			VersePath = Reader.ReadString()!;
			VerseScope = (VerseScope)Reader.ReadInt();
			HasVerse = Reader.ReadBool();
			VerseMountPoint = Reader.ReadString()!;
			VersePackageName = Reader.ReadString()!;
			VerseDirectoryPath = Reader.ReadString()!;
			VersePublicDependencies = Reader.ReadHashSet(() => Reader.ReadString())!;
			VersePrivateDependencies = Reader.ReadHashSet(() => Reader.ReadString())!;
			AlwaysExportStructs = Reader.ReadBool();
			AlwaysExportEnums = Reader.ReadBool();
			AllowUETypesInNamespaces = Reader.ReadBool();
			MinimizeGeneratedIncludes = Reader.ReadBool();
		}

		public void Write(BinaryArchiveWriter Writer)
		{
			Writer.WriteString(ModuleName);
			Writer.WriteFileReference(ModuleRulesFile);
			Writer.WriteArray<DirectoryItem>(ModuleDirectories, Writer.WriteDirectoryItem);
			Writer.WriteArray<DirectoryReference>(ModuleIncludePaths, Writer.WriteDirectoryReference);
			Writer.WriteHashSet(ModuleDependencies, Writer.WriteString);
			Writer.WriteString(ModuleType);
			Writer.WriteString(OverrideModuleType);
			Writer.WriteList(PublicUObjectClassesHeaders, Item => Writer.WriteFileItem(Item));
			Writer.WriteList(PublicUObjectHeaders, Item => Writer.WriteFileItem(Item));
			Writer.WriteList(InternalUObjectHeaders, Item => Writer.WriteFileItem(Item));
			Writer.WriteList(PrivateUObjectHeaders, Item => Writer.WriteFileItem(Item));
			Writer.WriteString(GeneratedCPPFilenameBase);
			Writer.WriteDirectoryItem(GeneratedCodeDirectory);
			Writer.WriteInt((int)GeneratedCodeVersion);
			Writer.WriteBool(bIsReadOnly);
			Writer.WriteList(PublicDefines, Item => Writer.WriteString(Item));
			Writer.WriteString(VersePath);
			Writer.WriteInt((int)VerseScope);
			Writer.WriteBool(HasVerse);
			Writer.WriteString(VerseMountPoint);
			Writer.WriteString(VersePackageName);
			Writer.WriteString(VerseDirectoryPath);
			Writer.WriteHashSet(VersePublicDependencies, Item => Writer.WriteString(Item));
			Writer.WriteHashSet(VersePrivateDependencies, Item => Writer.WriteString(Item));
			Writer.WriteBool(AlwaysExportStructs);
			Writer.WriteBool(AlwaysExportEnums);
			Writer.WriteBool(AllowUETypesInNamespaces);
			Writer.WriteBool(MinimizeGeneratedIncludes);
		}

		public override string ToString()
		{
			return ModuleName;
		}
	}

	class UHTModuleHeaderInfo
	{
		public DirectoryItem SourceFolder;
		public List<FileItem> HeaderFiles;
		public bool bUsePrecompiled;

		public UHTModuleHeaderInfo(DirectoryItem SourceFolder, List<FileItem> HeaderFiles, bool bUsePrecompiled)
		{
			this.SourceFolder = SourceFolder;
			this.HeaderFiles = HeaderFiles;
			this.bUsePrecompiled = bUsePrecompiled;
		}

		public UHTModuleHeaderInfo(BinaryArchiveReader Reader)
		{
			SourceFolder = Reader.ReadDirectoryItem()!;
			HeaderFiles = Reader.ReadList(() => Reader.ReadFileItem())!;
			bUsePrecompiled = Reader.ReadBool();
		}

		public void Write(BinaryArchiveWriter Writer)
		{
			Writer.WriteDirectoryItem(SourceFolder);
			Writer.WriteList(HeaderFiles, Item => Writer.WriteFileItem(Item));
			Writer.WriteBool(bUsePrecompiled);
		}
	}

	/// <summary>
	/// This handles all running of the UnrealHeaderTool
	/// </summary>
	class UHTExecution
	{
		static UHTModuleType GetEngineModuleTypeFromDescriptor(ModuleDescriptor Module)
		{
			return Module.Type.GetEngineModuleType() ?? throw new BuildException("Unhandled engine module type {0} for {1}", Module.Type.ToString(), Module.Name);
		}

		static UHTModuleType GetGameModuleTypeFromDescriptor(ModuleDescriptor Module)
		{
			return Module.Type.GetGameModuleType() ?? throw new BuildException("Unhandled game module type {0} for {1}", Module.Type.ToString(), Module.Name);
		}

		/// <summary>
		/// Return the mount point for the given module
		/// </summary>
		/// <param name="Module"></param>
		/// <returns></returns>
		/// <exception cref="BuildException"></exception>
		public static string GetVerseMountPointForModule(UEBuildModule Module)
		{
			if (Module.Rules.Plugin != null)
			{
				if (Module.bHasVerse && !Module.Rules.Plugin.Descriptor.bCanContainVerse)
				{
					throw new BuildException("Module '{0}' has an associated Verse directory but its containing plugin '{1}' does not specify \"CanContainVerse\": true.", Module.Name, Module.Rules.Plugin.Name);
				}

				return Module.Rules.Plugin.Name;
			}
			// We currently allow Verse only in plugins and programs
			if (Module.bHasVerse && Module.Rules.Context.DefaultUHTModuleType.GetValueOrDefault(UHTModuleType.Program) != UHTModuleType.Program)
			{
				throw new BuildException("Module '{0}' has an associated Verse directory but it is neither in a plugin nor program.", Module.Name);
			}

			return Module.Rules.Context.Scope.Name == "Project" ? "Game" : "Engine";
		}

		/// <summary>
		/// Gets the module type for a given rules object
		/// </summary>
		/// <param name="RulesObject">The rules object</param>
		/// <param name="ProjectDescriptor">Descriptor for the project being built</param>
		/// <returns>The module type</returns>
		static UHTModuleType GetModuleType(ModuleRules RulesObject, ProjectDescriptor? ProjectDescriptor)
		{
			ModuleRulesContext Context = RulesObject.Context;
			if (Context.bClassifyAsGameModuleForUHT)
			{
				if (RulesObject.Type == ModuleRules.ModuleType.External)
				{
					return UHTModuleType.GameThirdParty;
				}
				if (Context.DefaultUHTModuleType.HasValue)
				{
					return Context.DefaultUHTModuleType.Value;
				}
				if (RulesObject.Plugin != null)
				{
					ModuleDescriptor? Module = RulesObject.Plugin.Descriptor.Modules?.FirstOrDefault(x => x.Name == RulesObject.Name);
					if (Module != null)
					{
						return GetGameModuleTypeFromDescriptor(Module);
					}
				}
				if (ProjectDescriptor != null && ProjectDescriptor.Modules != null)
				{
					ModuleDescriptor? Module = ProjectDescriptor.Modules.FirstOrDefault(x => x.Name == RulesObject.Name);
					if (Module != null)
					{
						return Module.Type.GetGameModuleType() ?? UHTModuleType.GameRuntime;
					}
				}
				return UHTModuleType.GameRuntime;
			}
			else
			{
				if (RulesObject.Type == ModuleRules.ModuleType.External)
				{
					return UHTModuleType.EngineThirdParty;
				}
				if (Context.DefaultUHTModuleType.HasValue)
				{
					return Context.DefaultUHTModuleType.Value;
				}
				if (RulesObject.Plugin != null)
				{
					string PluginName = !RulesObject.IsTestModule ? RulesObject.Name : TargetDescriptor.GetTestedName(RulesObject.Name);
					ModuleDescriptor? Module = RulesObject.Plugin.Descriptor.Modules?.FirstOrDefault(x => x.Name == PluginName);
					if (Module != null)
					{
						return GetEngineModuleTypeFromDescriptor(Module);
					}
				}
				throw new BuildException("Unable to determine UHT module type for {0}", RulesObject.File);
			}
		}

		/// <summary>
		/// Find all the headers under the given base directory, excluding any other platform folders with the option to exclude particular test folders.
		/// </summary>
		/// <param name="ModuleBaseDirectory">Base module directory</param>
		/// <param name="BaseDirectory">Base directory to search</param>
		/// <param name="ExcludeFolders">Array of folders to exclude</param>
		/// <param name="Headers">Receives the list of headers that was found</param>
		/// <param name="bShouldExcludeTestsFolder">Whether to exclude the Tests/ folder from the module's generated files</param>
		static void FindHeadersTest(DirectoryItem ModuleBaseDirectory, DirectoryItem BaseDirectory, IReadOnlySet<string> ExcludeFolders, List<FileItem> Headers, bool bShouldExcludeTestsFolder = false)
		{
			if (BaseDirectory.TryGetFile(".ubtignore", out FileItem? OutIgnoreFile))
			{
				return;
			}

			if (bShouldExcludeTestsFolder)
			{
				// Skip the <Module Base Directory>/Tests folder
				if (BaseDirectory.Location.Equals(DirectoryReference.Combine(ModuleBaseDirectory.Location, "Tests")))
				{
					return;
				}
			}

			// Check for all the headers in this folder
			Headers.AddRange(BaseDirectory.EnumerateFiles().Where((fi) => fi.HasExtension(".h")));

			foreach (DirectoryItem SubDirectory in BaseDirectory.EnumerateDirectories())
			{
				if (!ExcludeFolders.Contains(SubDirectory.Name))
				{
					FindHeadersTest(ModuleBaseDirectory, SubDirectory, ExcludeFolders, Headers, bShouldExcludeTestsFolder);
				}
			}
		}

		/// <summary>
		/// Find all the headers under the given base directory, excluding any other platform folders.
		/// </summary>
		/// <param name="BaseDirectory">Base directory to search</param>
		/// <param name="ExcludeFolders">Array of folders to exclude</param>
		/// <param name="Headers">Receives the list of headers that was found</param>
		static void FindHeaders(DirectoryItem BaseDirectory, IReadOnlySet<string> ExcludeFolders, List<FileItem> Headers)
		{
			if (BaseDirectory.TryGetFile(".ubtignore", out FileItem? OutIgnoreFile))
			{
				return;
			}

			// Check for all the headers in this folder
			Headers.AddRange(BaseDirectory.EnumerateFiles().Where((fi) => fi.HasExtension(".h")));

			foreach (DirectoryItem SubDirectory in BaseDirectory.EnumerateDirectories())
			{
				if (!ExcludeFolders.Contains(SubDirectory.Name))
				{
					FindHeaders(SubDirectory, ExcludeFolders, Headers);
				}
			}
		}

		public static void SetupUObjectModules(
			IEnumerable<UEBuildModuleCPP> ModulesToGenerateHeadersFor,
			UnrealTargetPlatform Platform, 
			ProjectDescriptor? ProjectDescriptor, 
			List<UHTModuleInfo> UObjectModules,
			List<UHTModuleHeaderInfo> UObjectModuleHeaders,
			DirectoryReference ExecutableDirectory,
			EGeneratedCodeVersion GeneratedCodeVersion, 
			SourceFileMetadataCache MetadataCache)
		{
			// Find the type of each module
			Dictionary<UEBuildModuleCPP, UHTModuleType> ModuleToType = [];
			foreach (UEBuildModuleCPP Module in ModulesToGenerateHeadersFor)
			{
				ModuleToType[Module] = GetModuleType(Module.Rules, ProjectDescriptor);
			}

			// Sort modules by type, then by dependency
			List<UEBuildModuleCPP> ModulesSortedByType = [.. ModulesToGenerateHeadersFor.OrderBy(c => ModuleToType[c])];
			ModulesSortedByType = [.. UEBuildModule.StableTopologicalSort([.. ModulesSortedByType.Cast<UEBuildModule>()]).Cast<UEBuildModuleCPP>()];

			// Create a module to index lookup
			Dictionary<UEBuildModule, int> ModuleToIndex = [];
			for (int Idx = 0; Idx < ModulesSortedByType.Count; Idx++)
			{
				ModuleToIndex.Add(ModulesSortedByType[Idx], Idx);
			}

			// We neeed the base dir to make a relative path for the Verse source location
			DirectoryReference BaseDir = ExecutableDirectory;
			string? RelativeBaseDir = UEBuildTarget.GetRelativeBaseDir(ExecutableDirectory, Platform);
			if (RelativeBaseDir != null)
			{
				// Copy the logic from UEBuildTarget.SetupGlobalEnvironment ~ln 3645 that defines UE_RELATIVE_BASE_DIR to make the BaseDir relative to the Engine directory
				// This ensures that the source file gathering can always use FPlatformProcess:BaseDir reliably since it has UE_RELATIVE_BASE_DIR baked in
				BaseDir = DirectoryReference.Combine(ExecutableDirectory, RelativeBaseDir);
			}

			// Create the info for each module in parallel
			UHTModuleInfo[] ModuleInfoArray = new UHTModuleInfo[ModulesSortedByType.Count];

			{
				ParallelQueue Queue = new();
				IReadOnlySet<string> ExcludedFolders = UEBuildPlatform.GetBuildPlatform(Platform).GetExcludedFolderNames();
				for (int Idx = 0; Idx < ModulesSortedByType.Count; Idx++)
				{
					UEBuildModuleCPP Module = ModulesSortedByType[Idx];

					DirectoryItem GeneratedCodeDirectory = DirectoryItem.GetItemByDirectoryReference(Module.GeneratedCodeDirectoryUHT!);

					DirectoryReference[] ModuleIncludePaths = [.. Module.PublicIncludePaths.Union(Module.InternalIncludePaths).Union(Module.PrivateIncludePaths)];
					HashSet<string> ModuleDependencies = [];
					foreach (UEBuildModule OtherModule in Module.GetDependencies(false, false))
					{
						if (ModuleToIndex.ContainsKey(OtherModule))
						{
							ModuleDependencies.Add(OtherModule.Name);	
						}
					}

					string VerseMountPoint = String.Empty;
					string VersePackageName = String.Empty;
					string VerseDirectoryPath = String.Empty;
					if (Module.bHasVerse)
					{
						DirectoryReference SourceDirectory = DirectoryReference.Combine(Module.Rules.Directory, "Verse");
						VerseMountPoint = GetVerseMountPointForModule(Module);
						VersePackageName = $"{VerseMountPoint}/{Module.Name}";
						VerseDirectoryPath = SourceDirectory.MakeRelativeTo(BaseDir).Replace("\\", "/", StringComparison.Ordinal);
					}

					UHTModuleInfo Info = new()
					{
						ModuleName = Module.Name,
						ModuleRulesFile = Module.RulesFile,
						ModuleDirectories = Module.ModuleDirectories,
						ModuleIncludePaths = ModuleIncludePaths,
						ModuleDependencies = ModuleDependencies,
						ModuleType = ModuleToType[Module].ToString(),
						GeneratedCodeDirectory = GeneratedCodeDirectory,
						GeneratedCodeVersion = GeneratedCodeVersion,
						bIsReadOnly = Module.Rules.bUsePrecompiled,
						OverrideModuleType = Module.Rules.OverridePackageType.ToString(),
						VersePath = Module.Rules.VersePath ?? "",
						VerseScope = Module.Rules.VerseScope,
						HasVerse = Module.bHasVerse,
						VerseMountPoint = VerseMountPoint,
						VersePackageName = VersePackageName,
						VerseDirectoryPath = VerseDirectoryPath,
						AlwaysExportStructs = Module.Rules.bAlwaysExportStructs,
						AlwaysExportEnums = Module.Rules.bAlwaysExportEnums,
						AllowUETypesInNamespaces = Module.Rules.bAllowUETypesInNamespaces,
						MinimizeGeneratedIncludes = Module.Rules.bMinimizeGeneratedIncludes,
					};
					ModuleInfoArray[Idx] = Info;

					if (Module.Rules.Target.IsTestTarget)
					{
						Info.bShouldExcludeTestsFolder = Module.Rules.Target.SelectedTestModuleNames.Count > 0 && !Module.Rules.Target.SelectedTestModuleNames.Contains(Module.Name);
						Queue.Enqueue(() => SetupUObjectModuleTest(Info, ExcludedFolders, MetadataCache, Queue));
					}
					else
					{
						Queue.Enqueue(() => SetupUObjectModule(Info, ExcludedFolders, MetadataCache, Queue));
					}
				}
				Queue.Drain();
			}

			// Gather all the verse dependencies
			BitArray ModuleHasBeenGathered = new(ModulesSortedByType.Count, false);
			for (int Idx = 0; Idx < ModulesSortedByType.Count; Idx++)
			{
				GatherVerseDependencies(Idx, ModulesSortedByType, ModuleInfoArray, ModuleToIndex, ModuleHasBeenGathered);
			}

			// Filter out all the modules with reflection data
			for (int Idx = 0; Idx < ModulesSortedByType.Count; Idx++)
			{
				UEBuildModuleCPP Module = ModulesSortedByType[Idx];
				UHTModuleInfo Info = ModuleInfoArray[Idx];
				Info.PublicDefines.AddRange(Module.PublicDefinitions);

				if (Info.PublicUObjectClassesHeaders.Count > 0 || Info.PrivateUObjectHeaders.Count > 0 || Info.PublicUObjectHeaders.Count > 0 || Info.InternalUObjectHeaders.Count > 0)
				{
					Module.bHasUObjects = true;

					// If we've got this far and there are no source files then it's likely we're installed and ignoring
					// engine files, so we don't need a .gen.cpp either
					DirectoryReference GeneratedCodeDirectoryUHT = Module.GeneratedCodeDirectoryUHT!;
					Info.GeneratedCPPFilenameBase = Path.Combine(GeneratedCodeDirectoryUHT.FullName, Info.ModuleName) + ".gen";
					if (!Module.Rules.bUsePrecompiled)
					{
						Module.GeneratedCppDirectories ??= [];
						Module.GeneratedCppDirectories.Add(GeneratedCodeDirectoryUHT.FullName);
					}

					UObjectModules.Add(Info);

					DirectoryItem ModuleDirectoryItem = DirectoryItem.GetItemByDirectoryReference(Module.ModuleDirectory);

					List<FileItem> ReflectedHeaderFiles =
					[
						.. Info.PublicUObjectClassesHeaders,
						.. Info.PublicUObjectHeaders,
						.. Info.InternalUObjectHeaders,
						.. Info.PrivateUObjectHeaders,
					];
					UObjectModuleHeaders.Add(new UHTModuleHeaderInfo(ModuleDirectoryItem, ReflectedHeaderFiles, Module.Rules.bUsePrecompiled));
				}
			}

			// Set Module.bHasUObjects for any IncludePathModules not already processed.
			// This is necessary to keep include paths consistent between targets built with -AllModules and without
			// Note that PublicIncludePathModules can be recursive so we need to traverse the entire chain
			List<UEBuildModuleCPP> IncludePathModules = [];
			CollectModulesOnlyIncluded(ModulesToGenerateHeadersFor, IncludePathModules);
			if (IncludePathModules.Count > 0)
			{
				Dictionary<UEBuildModuleCPP, UHTModuleInfo> IncludePathInfo = [];

				using ThreadPoolWorkQueue Queue = new();
				IReadOnlySet<string> ExcludedFolders = UEBuildPlatform.GetBuildPlatform(Platform).GetExcludedFolderNames();
				foreach (UEBuildModuleCPP Module in IncludePathModules)
				{
					Queue.Enqueue(() =>
					{
						foreach (DirectoryItem ModuleDirectoryItem in Module.ModuleDirectories)
						{
							List<FileItem> HeaderFiles = [];
							if (Module.Rules.Target.IsTestTarget)
							{
								FindHeadersTest(DirectoryItem.GetItemByDirectoryReference(Module.ModuleDirectory), ModuleDirectoryItem, ExcludedFolders, HeaderFiles, Module.Rules.Target.SelectedTestModuleNames.Count > 0 && !Module.Rules.Target.SelectedTestModuleNames.Contains(Module.Name));
							}
							else
							{
								FindHeaders(ModuleDirectoryItem, ExcludedFolders, HeaderFiles);
							}

							if (HeaderFiles.Any(x => MetadataCache.ContainsReflectionMarkup(x)))
							{
								Module.bHasUObjects = true;
								break;
							}
						}
					});
				}
			}
		}

		static UHTModuleInfo GatherVerseDependencies(int Idx, List<UEBuildModuleCPP> ModulesSortedByType, UHTModuleInfo[] ModuleInfoArray, Dictionary<UEBuildModule, int> ModuleToIndex, BitArray ModuleHasBeenGathered)
		{
			UHTModuleInfo ModuleInfo = ModuleInfoArray[Idx];
			if (ModuleHasBeenGathered[Idx])
			{
				return ModuleInfo;
			}
			ModuleHasBeenGathered[Idx] = true;

			UEBuildModuleCPP BuildModule = ModulesSortedByType[Idx];
			if (BuildModule.bHasVerse)
			{
				GatherVerseDependencies(ModulesSortedByType, ModuleInfoArray, ModuleToIndex, ModuleHasBeenGathered, BuildModule.PublicDependencyModules, ModuleInfo.VersePublicDependencies);
				GatherVerseDependencies(ModulesSortedByType, ModuleInfoArray, ModuleToIndex, ModuleHasBeenGathered, BuildModule.PrivateDependencyModules, ModuleInfo.VersePrivateDependencies);
			}
			return ModuleInfo;
		}

		static void GatherVerseDependencies(List<UEBuildModuleCPP> ModulesSortedByType, UHTModuleInfo[] ModuleInfoArray, Dictionary<UEBuildModule, int> ModuleToIndex, BitArray ModuleHasBeenGathered, 
			List<UEBuildModule>? InDependencies, HashSet<string> OutDependencies)
		{
			if (InDependencies != null)
			{
				foreach (UEBuildModule Dependency in InDependencies)
				{
					if (ModuleToIndex.TryGetValue(Dependency, out int GatherIdx))
					{
						UHTModuleInfo GatherModuleInfo = GatherVerseDependencies(GatherIdx, ModulesSortedByType, ModuleInfoArray, ModuleToIndex, ModuleHasBeenGathered);
						if (GatherModuleInfo.HasVerse)
						{
							OutDependencies.Add(GatherModuleInfo.VersePackageName);
						}
						OutDependencies.UnionWith(GatherModuleInfo.VersePublicDependencies); // This is correct, private inherits public
					}
				}
			}
		}

		static void CollectModulesOnlyIncluded(HashSet<UEBuildModuleCPP> HandledModules, List<UEBuildModule> IncludedModules, List<UEBuildModuleCPP> OutList)
		{
			foreach (UEBuildModule pi in IncludedModules)
			{
				if (pi is UEBuildModuleCPP IncludedModule)
				{
					if (IncludedModule.bHasUObjects)
					{
						continue;
					}

					if (!HandledModules.Add(IncludedModule))
					{
						continue;
					}

					OutList.Add(IncludedModule);

					if (IncludedModule.PublicIncludePathModules != null)
					{
						CollectModulesOnlyIncluded(HandledModules, IncludedModule.PublicIncludePathModules, OutList);
					}
				}
			}
		}

		static void CollectModulesOnlyIncluded(IEnumerable<UEBuildModuleCPP> ModulesToGenerateHeadersFor, List<UEBuildModuleCPP> OutList)
		{
			HashSet<UEBuildModuleCPP> HandledModules = [];
			foreach (UEBuildModuleCPP Module in ModulesToGenerateHeadersFor)
			{
				HandledModules.Add(Module);
			}

			foreach (UEBuildModuleCPP Module in ModulesToGenerateHeadersFor)
			{
				if (Module.PublicIncludePathModules != null)
				{
					CollectModulesOnlyIncluded(HandledModules, Module.PublicIncludePathModules, OutList);
				}
				if (Module.PrivateIncludePathModules != null)
				{
					CollectModulesOnlyIncluded(HandledModules, Module.PrivateIncludePathModules, OutList);
				}
			}
		}

		static void SetupUObjectModuleTest(UHTModuleInfo ModuleInfo, IReadOnlySet<string> ExcludedFolders, SourceFileMetadataCache MetadataCache, ParallelQueue Queue)
		{
			foreach (DirectoryItem ModuleDirectory in ModuleInfo.ModuleDirectories)
			{
				List<FileItem> HeaderFiles = [];
				FindHeadersTest(ModuleInfo.ModuleDirectories[0], ModuleDirectory, ExcludedFolders, HeaderFiles, ModuleInfo.bShouldExcludeTestsFolder);

				foreach (FileItem HeaderFile in HeaderFiles)
				{
					Queue.Enqueue(() => SetupUObjectModuleHeader(ModuleInfo, HeaderFile, MetadataCache));
				}
			}
		}

		static void SetupUObjectModule(UHTModuleInfo ModuleInfo, IReadOnlySet<string> ExcludedFolders, SourceFileMetadataCache MetadataCache, ParallelQueue Queue)
		{
			foreach (DirectoryItem ModuleDirectory in ModuleInfo.ModuleDirectories)
			{
				List<FileItem> HeaderFiles = [];
				FindHeaders(ModuleDirectory, ExcludedFolders, HeaderFiles);

				foreach (FileItem HeaderFile in HeaderFiles)
				{
					Queue.Enqueue(() => SetupUObjectModuleHeader(ModuleInfo, HeaderFile, MetadataCache));
				}
			}
		}

		static void SetupUObjectModuleHeader(UHTModuleInfo ModuleInfo, FileItem HeaderFile, SourceFileMetadataCache MetadataCache)
		{
			// Check to see if we know anything about this file.  If we have up-to-date cached information about whether it has
			// UObjects or not, we can skip doing a test here.
			if (MetadataCache.ContainsReflectionMarkup(HeaderFile))
			{
				lock (ModuleInfo)
				{
					bool bFoundHeaderLocation = false;
					foreach (DirectoryItem ModuleDirectory in ModuleInfo.ModuleDirectories)
					{
						if (HeaderFile.Location.IsUnderDirectory(DirectoryReference.Combine(ModuleDirectory.Location, "Classes")))
						{
							ModuleInfo.PublicUObjectClassesHeaders.Add(HeaderFile);
							bFoundHeaderLocation = true;
						}
						else if (HeaderFile.Location.IsUnderDirectory(DirectoryReference.Combine(ModuleDirectory.Location, "Public")))
						{
							ModuleInfo.PublicUObjectHeaders.Add(HeaderFile);
							bFoundHeaderLocation = true;
						}
						else if (HeaderFile.Location.IsUnderDirectory(DirectoryReference.Combine(ModuleDirectory.Location, "Internal")))
						{
							ModuleInfo.InternalUObjectHeaders.Add(HeaderFile);
							bFoundHeaderLocation = true;
						}
					}
					if (!bFoundHeaderLocation)
					{
						ModuleInfo.PrivateUObjectHeaders.Add(HeaderFile);
					}
				}
			}
		}

		private static FileInfo GetInitGenCppFileInfo(string ModuleName, string ModuleGeneratedCodeDirectory) =>
			new(Path.Combine(ModuleGeneratedCodeDirectory, ModuleName + ".init.gen.cpp"));

		/// <summary>
		/// Gets the timestamp of CoreUObject.gen.cpp file.
		/// </summary>
		/// <returns>Last write time of CoreUObject.gen.cpp or DateTime.MaxValue if it doesn't exist.</returns>
		private static DateTime GetCoreGeneratedTimestampUtc(string ModuleName, string ModuleGeneratedCodeDirectory)
		{
			// In Installed Builds, we don't check the timestamps on engine headers.  Default to a very old date.
			if (Unreal.IsEngineInstalled())
			{
				return DateTime.MinValue;
			}

			// Otherwise look for CoreUObject.init.gen.cpp
			FileInfo CoreGeneratedFileInfo = GetInitGenCppFileInfo(ModuleName, ModuleGeneratedCodeDirectory);
			if (CoreGeneratedFileInfo.Exists)
			{
				return CoreGeneratedFileInfo.LastWriteTimeUtc;
			}

			// Doesn't exist, so use a 'newer that everything' date to force rebuild headers.
			return DateTime.MaxValue;
		}

		/// <summary>
		/// Gets the timestamp of the Version.h header file. Throws an exception if not found.
		/// </summary>
		/// <returns>Last write time of Version.h</returns>
		private static DateTime GetEngineVersionHeaderTimestampUtc()
		{
			// In Installed Builds, we don't check the timestamps on engine headers.  Default to a very old date.
			if (Unreal.IsEngineInstalled())
			{
				return DateTime.MinValue;
			}

			string EngineVersionHeaderPath = Path.Combine(Unreal.EngineSourceDirectory.FullName, "Runtime", "Launch", "Resources", "Version.h");
			FileInfo EngineVersionHeaderFileInfo = new(EngineVersionHeaderPath);
			if (EngineVersionHeaderFileInfo.Exists)
			{
				return EngineVersionHeaderFileInfo.LastWriteTimeUtc;
			}

			// The Version.h header doesn't exist somehow (maybe it was moved).
			throw new BuildException("Could not find engine version header, expected: {0}", EngineVersionHeaderPath);
		}

		/// <summary>
		/// Checks the class header files and determines if generated UObject code files are out of date in comparison.
		/// </summary>
		/// <param name="timestampsFile">Modules that we generate headers for</param>
		/// <param name="UObjectModules">Modules that we generate headers for</param>
		/// <param name="HeaderToolTimestampUtc">Timestamp for UHT</param>
		/// <param name="Logger">Logger for output</param>
		/// <returns>True if the code files are out of date</returns>
		private static bool AreGeneratedCodeFilesOutOfDate(TimestampsFile timestampsFile, List<UHTModuleInfo> UObjectModules, DateTime HeaderToolTimestampUtc, ILogger Logger)
		{
			// Get CoreUObject.init.gen.cpp timestamp.  If the source files are older than the CoreUObject generated code, we'll
			// need to regenerate code for the module
			DateTime? CoreGeneratedTimestampUtc = null;
			{
				// Find the CoreUObject module
				foreach (UHTModuleInfo Module in UObjectModules)
				{
					if (Module.ModuleName.Equals("CoreUObject", StringComparison.OrdinalIgnoreCase))
					{
						CoreGeneratedTimestampUtc = GetCoreGeneratedTimestampUtc(Module.ModuleName, Module.GeneratedCodeDirectory.FullName);
						break;
					}
				}
				if (CoreGeneratedTimestampUtc == null)
				{
					throw new BuildException("Could not find CoreUObject in list of all UObjectModules");
				}
			}

			// Load timestamps file
			timestampsFile.Load(Logger);

			// Also get the Version.h timestamp.  If it is newer than the last UHT run, it could affect preprocessor conditionals
			// that check for the engine version
			DateTime EngineVersionHeaderTimestampUtc = GetEngineVersionHeaderTimestampUtc();

			int invalidCounter = 0;

			Parallel2.ForEach(UObjectModules, Module =>
			{
				if (invalidCounter != 0)
				{
					return;
				}

				// If we're using a precompiled engine, skip checking timestamps for modules that are under the engine directory
				if (Module.bIsReadOnly)
				{
					return;
				}

				// Make sure we have an existing init.gen.cpp for generated code. If not, then we definitely need to generate code.
				// We can't check the generated code directory for existence as this is now pre-created in PrepopulateDirectoryCache.
				FileInfo GenCppFileInfo = GetInitGenCppFileInfo(Module.ModuleName, Module.GeneratedCodeDirectory.FullName);
				if (!GenCppFileInfo.Exists)
				{
					// Generated code directory is missing entirely!
					if (Interlocked.Increment(ref invalidCounter) == 1)
					{
						Logger.LogDebug("UnrealHeaderTool needs to run because no generated code directory was found for module {ModuleName}", Module.ModuleName);
					}
					return;
				}

				if (!timestampsFile.GetModuleInfo(Module.ModuleName, out UInt128 moduleOldHash, out long moduleLastUpdateUtc))
				{
					if (Interlocked.Increment(ref invalidCounter) == 1)
					{
						Logger.LogDebug("UnrealHeaderTool needs to run because module {ModuleName} was not found in timestamps file", Module.ModuleName);
					}
					return;
				}

				// Make sure the last UHT run completed after UnrealHeaderTool.exe was compiled last, and after the CoreUObject headers were touched last.
				DateTime SavedTimestampUtc = new(moduleLastUpdateUtc);
				if (HeaderToolTimestampUtc > SavedTimestampUtc)
				{
					// Generated code is older than UnrealHeaderTool.exe.  Out of date!
					if (Interlocked.Increment(ref invalidCounter) == 1)
					{
						Logger.LogDebug("UnrealHeaderTool needs to run because UnrealHeaderTool timestamp ({Time}) is later than timestamp for module {ModuleName} ({ModuleTime})", HeaderToolTimestampUtc.ToLocalTime(), Module.ModuleName, SavedTimestampUtc.ToLocalTime());
					}
					return;
				}
				if (CoreGeneratedTimestampUtc > SavedTimestampUtc)
				{
					// Generated code is older than CoreUObject headers.  Out of date!
					if (Interlocked.Increment(ref invalidCounter) == 1)
					{
						Logger.LogDebug("UnrealHeaderTool needs to run because CoreUObject timestamp ({Time}) is newer than timestamp for module {ModuleName} ({ModuleTime})", CoreGeneratedTimestampUtc.Value.ToLocalTime(), Module.ModuleName, SavedTimestampUtc.ToLocalTime());
					}
					return;
				}
				if (EngineVersionHeaderTimestampUtc > SavedTimestampUtc)
				{
					// Generated code is older than engine version header.  Out of date!
					if (Interlocked.Increment(ref invalidCounter) == 1)
					{
						Logger.LogDebug("UnrealHeaderTool needs to run because engine version header timestamp ({Time}) is newer than timestamp for module {ModuleName} ({ModuleTime})", EngineVersionHeaderTimestampUtc.ToLocalTime(), Module.ModuleName, SavedTimestampUtc.ToLocalTime());
					}
					return;
				}

				// Has the .build.cs file changed since we last generated headers successfully?
				FileInfo ModuleRulesFile = new(Module.ModuleRulesFile.FullName);
				if (!ModuleRulesFile.Exists || ModuleRulesFile.LastWriteTimeUtc > SavedTimestampUtc)
				{
					if (Interlocked.Increment(ref invalidCounter) == 1)
					{
						Logger.LogDebug("UnrealHeaderTool needs to run because SavedTimestamp is older than the rules file ({ModuleModuleRulesFile}) for module {ModuleModuleName}", Module.ModuleRulesFile, Module.ModuleName);
					}
					return;
				}

				UInt128 moduleHash = CalculateHash(Module);

				if (moduleHash != moduleOldHash)
				{
					if (Interlocked.Increment(ref invalidCounter) == 1)
					{
						Logger.LogDebug("UnrealHeaderTool needs to run because module {ModuleName} had different hash than what was found in timestamps file", Module.ModuleName);
					}
					return;
				}

				// Iterate over our UObjects headers and figure out if any of them have changed
				List <FileItem> AllUObjectHeaders =
				[
					.. Module.PublicUObjectClassesHeaders,
					.. Module.PublicUObjectHeaders,
					.. Module.InternalUObjectHeaders,
					.. Module.PrivateUObjectHeaders,
				];

				foreach (FileItem HeaderFile in AllUObjectHeaders)
				{
					DateTime HeaderFileTimestampUtc = HeaderFile.LastWriteTimeUtc;

					// Has the source header changed since we last generated headers successfully?
					if (HeaderFileTimestampUtc > SavedTimestampUtc)
					{
						if (Interlocked.Increment(ref invalidCounter) == 1)
						{
							Logger.LogDebug("UnrealHeaderTool needs to run because SavedTimestamp is older than HeaderFileTimestamp ({HeaderFileAbsolutePath}) for module {ModuleModuleName}", HeaderFile.AbsolutePath, Module.ModuleName);
						}
						return;
					}
				}
			});

			return invalidCounter != 0;
		}

		/// <summary>
		/// Determines if any external dependencies for generated code is out of date
		/// </summary>
		/// <param name="ExternalDependenciesFile">Path to the external dependencies file</param>
		/// <returns>True if any external dependencies are out of date</returns>
		private static bool AreExternalDependenciesOutOfDate(FileReference ExternalDependenciesFile)
		{
			if (!FileReference.Exists(ExternalDependenciesFile))
			{
				return true;
			}

			DateTime LastWriteTime = File.GetLastWriteTimeUtc(ExternalDependenciesFile.FullName);

			string[] Lines = File.ReadAllLines(ExternalDependenciesFile.FullName);
			foreach (string Line in Lines)
			{
				string ExternalDependencyFile = Line.Trim();
				if (ExternalDependencyFile.Length > 0)
				{
					if (!File.Exists(ExternalDependencyFile) || File.GetLastWriteTimeUtc(ExternalDependencyFile) > LastWriteTime)
					{
						return true;
					}
				}
			}

			return false;
		}

		/// <summary>
		/// Return the file reference for the UHT manifest file
		/// </summary>
		/// <param name="Makefile">Input makefile</param>
		/// <param name="TargetName">Name of the target</param>
		/// <returns>Manifest file name</returns>
		public static FileReference GetUHTModuleInfoFileName(TargetMakefile Makefile, string TargetName)
		{
			return FileReference.Combine(Makefile.ProjectIntermediateDirectoryNoArch.ParentDirectory!, TargetName + ".uhtmanifest");
		}

		/// <summary>
		/// Return the file reference for an intermediate file containing the UHT settings last used to build a given target
		/// </summary>
		/// <param name="Makefile">Input makefile</param>
		/// <param name="TargetName">Name of the target</param>
		/// <returns>Manifest file name</returns>
		public static FileReference GetUHTSettingsFileName(TargetMakefile Makefile, string TargetName)
		{
			return FileReference.Combine(Makefile.ProjectIntermediateDirectoryNoArch.ParentDirectory!, TargetName + ".uhtsettings");
		}

		/// <summary>
		/// Return the file reference for the UHT deps file
		/// </summary>
		/// <param name="ModuleInfoFileName">Manifest info file name</param>
		/// <returns>UHT dependency file name</returns>
		public static FileReference GetUHTDepsFileName(FileReference ModuleInfoFileName)
		{
			return ModuleInfoFileName.ChangeExtension(".deps");
		}

		/// <summary>
		/// Convert the makefile to a UHTManifest object
		/// </summary>
		/// <param name="Makefile">Input makefile</param>
		/// <param name="TargetName">Name of the target</param>
		/// <param name="DepsFileName">Name of the dependencies file.</param>
		/// <returns>Output UHT manifest</returns>
		public static UHTManifest CreateUHTManifest(TargetMakefile Makefile, string TargetName, FileReference DepsFileName)
		{
			List<UHTManifest.Module> Modules = [];
			foreach (UHTModuleInfo UObjectModule in Makefile.UObjectModules)
			{
				List<string> VerseDependencies = [.. UObjectModule.VersePublicDependencies.Union(UObjectModule.VersePrivateDependencies)];
				VerseDependencies.Sort(StringComparer.Ordinal);

				Modules.Add(
					new UHTManifest.Module
					{
						Name = UObjectModule.ModuleName,
						ModuleType = (UHTModuleType)Enum.Parse(typeof(UHTModuleType), UObjectModule.ModuleType),
						OverrideModuleType = (EPackageOverrideType)Enum.Parse(typeof(EPackageOverrideType), UObjectModule.OverrideModuleType),
						BaseDirectory = UObjectModule.ModuleDirectories[0].FullName,
						IncludePaths = [.. UObjectModule.ModuleIncludePaths.Select(IncludePath => IncludePath.FullName)],
						OutputDirectory = Path.GetDirectoryName(UObjectModule.GeneratedCPPFilenameBase)!,
						ClassesHeaders = [.. UObjectModule.PublicUObjectClassesHeaders.Select((Header) => Header.AbsolutePath)],
						PublicHeaders = [.. UObjectModule.PublicUObjectHeaders.Select((Header) => Header.AbsolutePath)],
						InternalHeaders = [.. UObjectModule.InternalUObjectHeaders.Select((Header) => Header.AbsolutePath)],
						PrivateHeaders = [.. UObjectModule.PrivateUObjectHeaders.Select((Header) => Header.AbsolutePath)],
						ModuleDependencies = [.. UObjectModule.ModuleDependencies ],
						PublicDefines = UObjectModule.PublicDefines,
						GeneratedCPPFilenameBase = UObjectModule.GeneratedCPPFilenameBase,
						SaveExportedHeaders = !UObjectModule.bIsReadOnly,
						GeneratedCodeVersion = UObjectModule.GeneratedCodeVersion,
						VersePath = UObjectModule.VersePath,
						VerseScope = (UHTVerseScope)Enum.Parse(typeof(UHTVerseScope), UObjectModule.VerseScope.ToString()),
						HasVerse = UObjectModule.HasVerse,
						VerseMountPoint = UObjectModule.VerseMountPoint,
						VersePackageName = UObjectModule.VersePackageName,
						VerseDirectoryPath = UObjectModule.VerseDirectoryPath,
						VerseDependencies = VerseDependencies,
						AlwaysExportStructs = UObjectModule.AlwaysExportStructs,
						AlwaysExportEnums = UObjectModule.AlwaysExportEnums,
						AllowUETypesInNamespaces = UObjectModule.AllowUETypesInNamespaces,
						MinimizeGeneratedIncludes = UObjectModule.MinimizeGeneratedIncludes,
					});
			}

			List<string> UhtPlugins = [];
			if (Makefile.EnabledUhtPlugins != null)
			{
				foreach (FileReference UhtPlugin in Makefile.EnabledUhtPlugins)
				{
					UhtPlugins.Add(UhtPlugin.FullName);
				}
			}

			UHTManifest Manifest = new()
			{
				TargetName = TargetName,
				IsGameTarget = Makefile.TargetType != TargetType.Program,
				RootLocalPath = Unreal.RootDirectory.FullName,
				ExternalDependenciesFile = DepsFileName.FullName,
				TargetSettings = Makefile.UHTTargetSettings,
				Modules = Modules,
				UhtPlugins = UhtPlugins,
			};

			return Manifest;
		}

		/// <summary>
		/// Write the manifest to disk
		/// </summary>
		/// <param name="Makefile">Input makefile</param>
		/// <param name="TargetName">Name of the target</param>
		/// <param name="ModuleInfoFileName">Destination file name.  If not supplied, it will be generated.</param>
		/// <param name="DepsFileName">Name of the dependencies file.  If not supplied, it will be generated.</param>
		public static void WriteUHTManifest(TargetMakefile Makefile, string TargetName, FileReference ModuleInfoFileName, FileReference DepsFileName)
		{
			// @todo ubtmake: Optimization: Ideally we could avoid having to generate this data in the case where UHT doesn't even need to run!  Can't we use the existing copy?  (see below use of Manifest)
			UHTManifest Manifest = CreateUHTManifest(Makefile, TargetName, DepsFileName);
			Directory.CreateDirectory(ModuleInfoFileName.Directory.FullName);

			int retry = 0;
			while (true)
			{
				try
				{
					System.IO.File.WriteAllText(ModuleInfoFileName.FullName, JsonSerializer.Serialize(Manifest, new JsonSerializerOptions { WriteIndented = true }));
					break;
				}
				catch (IOException) // I have no idea what could cause this. Something is holding on to the file but I can't see what. and it goes away when waiting a little bit
				{
					if (retry++ > 5)
					{
						throw;
					}
					System.Threading.Thread.Sleep(500);
				}
			}
		}

		/// <summary>
		/// Will prepopulate DirectoryInfo cache with files that will be written.
		/// This can be combined with ExecuteHeaderToolIfNecessary and WaitForExecution.
		/// </summary>
		/// <param name="moduleInfos"></param>
		public static void PrepopulateDirectoryCache(List<UHTModuleInfo> moduleInfos)
		{
			Parallel2.ForEach(moduleInfos, mi =>
			{
				List<string> outputFiles = [];
				outputFiles.Add(mi.ModuleName + ".init.gen.cpp");
				outputFiles.Add(mi.ModuleName + "Classes.cpp");

				foreach (FileItem? f in mi.PublicUObjectClassesHeaders.Concat(mi.PublicUObjectHeaders).Concat(mi.InternalUObjectHeaders).Concat(mi.PrivateUObjectHeaders))
				{
					string fileName = f.Location.GetFileNameWithoutExtension();
					outputFiles.Add(fileName + ".generated.h");
					outputFiles.Add(fileName + ".gen.cpp");
				}

				mi.GeneratedCodeDirectory.PrepopulateCache(outputFiles);
			});
		}

		static UInt128 CalculateHash(UHTModuleInfo mi)
		{
			XxHash128 xx = new();
			List<FileItem> files = new(mi.PublicUObjectClassesHeaders.Concat(mi.PublicUObjectHeaders).Concat(mi.InternalUObjectHeaders).Concat(mi.PrivateUObjectHeaders));
			files.Sort(FileItem.SortByPath);
			foreach (FileItem file in files)
			{
				xx.Append(MemoryMarshal.AsBytes(file.FullName.AsSpan()));
			}
			byte[] hash16 = xx.GetHashAndReset();
			ulong lo = BinaryPrimitives.ReadUInt64LittleEndian(hash16.AsSpan()[..8]);
			ulong hi = BinaryPrimitives.ReadUInt64LittleEndian(hash16.AsSpan()[8..]);
			return ((UInt128)hi << 64) | lo;
		}

		class TimestampsFile
		{
			internal TimestampsFile(FileReference location)
			{
				_location = location;
			}

			const uint CurrentVersion = 2;

			internal void Load(ILogger logger)
			{
				try
				{
					if (FileReference.Exists(_location))
					{
						using FileStream stream = FileReference.Open(_location, FileMode.Open);
						using BinaryReader reader = new(stream);
						uint version = reader.ReadUInt32();
						if (version != CurrentVersion)
						{
							logger.LogInformation("UHT timestamps file version mismatch");
							return;
						}

						int moduleCount = reader.ReadInt32();
						for (int i = 0; i < moduleCount; i++)
						{
							string moduleName = reader.ReadString();
							ulong hashA = reader.ReadUInt64();
							UInt128 hashB = reader.ReadUInt64();
							UInt128 hash = (hashB << 64) + hashA;
							long lastUpdateUtc = reader.ReadInt64();
							_modules.Add(moduleName, (hash, lastUpdateUtc));
						}
					}
				}
				catch
				{
				}
			}

			internal bool GetModuleInfo(string module, out UInt128 hash, out long lastUpdateUtc)
			{
				if (_modules.TryGetValue(module, out (UInt128 hash, long lastUpdateUtc) entry))
				{
					hash = entry.hash;
					lastUpdateUtc = entry.lastUpdateUtc;
					return true;
				}
				hash = 0;
				lastUpdateUtc = 0;
				return false;
			}

			internal void Update(List<UHTModuleInfo> modules)
			{
				DateTime now = DateTime.UtcNow;
				foreach (UHTModuleInfo module in modules)
				{
					_modules[module.ModuleName] = (CalculateHash(module), now.Ticks);
				}
			}

			internal void Save()
			{
				using FileStream stream = FileReference.Open(_location, FileMode.Create, FileAccess.Write);
				using BinaryWriter writer = new(stream);
				writer.Write(CurrentVersion);
				writer.Write(_modules.Count);
				foreach (KeyValuePair<string, (UInt128 hash, long lastUpdateUtc)> kv in _modules)
				{
					writer.Write(kv.Key);
					writer.Write((ulong)kv.Value.hash);
					writer.Write((ulong)(kv.Value.hash >> 64));
					writer.Write(kv.Value.lastUpdateUtc);
				}
			}

			readonly FileReference _location;
			readonly SortedDictionary<string, (UInt128 hash, long lastUpdateUtc)> _modules = [];
		}

		/// <summary>
		/// Builds and runs the header tool and touches the header directories.
		/// Performs any early outs if headers need no changes, given the UObject modules, tool path, game name, and configuration
		/// </summary>
		public static (Task run, Task timestamp) ExecuteHeaderToolIfNecessary(BuildConfiguration BuildConfiguration, FileReference? ProjectFile, TargetMakefile Makefile, string TargetName, ILogger Logger)
		{
			bool runTimestamp = false;

			TimestampsFile timestampsFile = new(FileReference.Combine(Makefile.ProjectIntermediateDirectoryNoArch.ParentDirectory!, TargetName + ".uhttimestamps"));

			Task runTask = new Task(() =>
			{
				if (ProgressWriter.bWriteMarkup)
				{
					Logger.LogInformation("@progress push 5%");
				}
				using (ProgressWriter Progress = new("Generating code...", false, Logger))
				{
					using ITimelineEvent uhtManifestScope = Timeline.ScopeEvent("UHTManifest");

					string RootLocalPath = Unreal.RootDirectory.FullName;

					// Get the path to the assemblies we care about
					string UhtAssemblyPath = typeof(UhtSession).Assembly.Location;

					// Get a composite date/time for critical assemblies being used
					DateTime CompositeTimestamp = UhtUtilities.GetLatestAssemblyDateTime(Makefile.EnabledUhtPlugins?.Select(x => x.FullName));

					bool bUHTNeedsToRun = false;

					// Check explicit settings passed to UHT - this info is also saved in the manifest but we load just the settings here
					FileReference UHTSettingsFileName = GetUHTSettingsFileName(Makefile, TargetName);
					if (!FileReference.Exists(UHTSettingsFileName))
					{
						bUHTNeedsToRun = true;
					}
					else
					{
						using (BinaryArchiveReader reader = new(UHTSettingsFileName))
						{
							// Note that even if default/omitted values cause the settings to compare equal here, the UHT module version check below 
							// will ensure that we rerun UHT when new properties are added to UHTTargetSettings
							try
							{
								UHTTargetSettings settings = new(reader);
								if (settings != Makefile.UHTTargetSettings)
								{
									bUHTNeedsToRun = true;
								}
							}
							catch (Exception)
							{
								bUHTNeedsToRun = true;
							}
						}
					}

					// ensure the headers are up to date
					if (BuildConfiguration.bForceHeaderGeneration)
					{
						bUHTNeedsToRun = true;
					}
					else if (!bUHTNeedsToRun && AreGeneratedCodeFilesOutOfDate(timestampsFile, Makefile.UObjectModules, CompositeTimestamp, Logger))
					{
						bUHTNeedsToRun = true;
					}

					// Check we're not using a different version of UHT
					FileReference ModuleInfoFileName = GetUHTModuleInfoFileName(Makefile, TargetName);
					FileReference ToolInfoFile = ModuleInfoFileName.ChangeExtension(".uhtpath");
					if (!bUHTNeedsToRun)
					{
						if (!FileReference.Exists(ToolInfoFile))
						{
							bUHTNeedsToRun = true;
						}
						else if (FileReference.ReadAllText(ToolInfoFile) != UhtAssemblyPath)
						{
							bUHTNeedsToRun = true;
						}
					}

					// Get the file containing dependencies for the generated code
					FileReference ExternalDependenciesFile = GetUHTDepsFileName(ModuleInfoFileName);
					if (AreExternalDependenciesOutOfDate(ExternalDependenciesFile))
					{
						bUHTNeedsToRun = true;
					}

					if (bUHTNeedsToRun)
					{
						Progress.Write(1, 3);

						WriteUHTManifest(Makefile, TargetName, ModuleInfoFileName, ExternalDependenciesFile);
						string ActualTargetName = String.IsNullOrEmpty(TargetName) ? "UE5" : TargetName;
						Logger.LogDebug("Parsing headers for {ActualTargetName}", ActualTargetName);

						// Generate the command line
						List<string> CmdArgs = [
							(ProjectFile != null) ? ProjectFile.FullName : TargetName,
							ModuleInfoFileName.FullName,
							"-WarningsAsErrors",
						];

						if (Unreal.IsEngineInstalled())
						{
							CmdArgs.Add("-installed");
						}

						if (BuildConfiguration.bFailIfGeneratedCodeChanges)
						{
							CmdArgs.Add("-FailIfGeneratedCodeChanges");
						}

						if (Makefile.UHTAdditionalArguments != null)
						{
							foreach (string Arg in Makefile.UHTAdditionalArguments)
							{
								if (Arg[0] == '"' && Arg[^1] == '"')
								{
									CmdArgs.Add(Arg[1..^1]);
								}
								else
								{
									CmdArgs.Add(Arg);
								}
							}
						}
						CommandLineArguments Arguments = new([.. CmdArgs]);

						StringBuilder CmdLine = new();
						foreach (string Arg in CmdArgs)
						{
							CmdLine.AppendArgument(Arg);
						}

						uhtManifestScope.Finish();

						Logger.LogDebug("  Running Internal UnrealHeaderTool {CmdLine}", CmdLine);

						using ITimelineEvent uhtExecute = Timeline.ScopeEvent("UHT");

						// Run UHT
						IScope Timer = GlobalTracer.Instance.BuildSpan("Executing UnrealHeaderTool").StartActive();
						UnrealHeaderToolMode UHTTool = new();

						CompilationResult UHTResult = (CompilationResult)UHTTool.ExecuteAsync(Arguments, Logger).Result;
						Timer.Span.Finish();

						uhtExecute.Dispose();

						if (UHTResult != CompilationResult.Succeeded)
						{
							// On Linux and Mac, the shell will return 128+signal number exit codes if UHT gets a signal (e.g. crashes or is interrupted)
							if ((BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Linux ||
								BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Mac) &&
								(int)(UHTResult) >= 128
								)
							{
								// SIGINT is 2, so 128 + SIGINT is 130
								UHTResult = ((int)(UHTResult) == 130) ? CompilationResult.Canceled : CompilationResult.CrashOrAssert;
							}

							if (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Win64 &&
								(int)(UHTResult) < 0)
							{
								Logger.LogError("UnrealHeaderTool failed with exit code 0x{Result:X} - check that Unreal Engine prerequisites are installed.", (int)UHTResult);
							}

							throw new CompilationResultException(UHTResult);
						}

						// Update the tool info file
						DirectoryReference.CreateDirectory(ToolInfoFile.Directory);
						FileReference.WriteAllText(ToolInfoFile, UhtAssemblyPath);

						// Update saved UHT settings
						using (BinaryArchiveWriter writer = new(UHTSettingsFileName))
						{
							Makefile.UHTTargetSettings.Write(writer);
						}
					}
					else
					{
						uhtManifestScope.Finish();

						Logger.LogDebug("Generated code is up to date.");
					}

					Progress.Write(2, 3);

					runTimestamp = true;

					Progress.Write(3, 3);
				}
				if (ProgressWriter.bWriteMarkup)
				{
					Logger.LogInformation("@progress pop");
				}
			});
			Task timestampTask = runTask.ContinueWith((prev) =>
			{
				if (runTimestamp)
				{
					using (GlobalTracer.Instance.BuildSpan("UHTExecution.UpdateTimestamps()").StartActive())
					{
						using ITimelineEvent uhtExecute = Timeline.ScopeEvent("UHTTimestamps");
						timestampsFile.Update(Makefile.UObjectModules);
						timestampsFile.Save();
					}
				}
			}, TaskScheduler.Current);

			runTask.Start();

			return (runTask, timestampTask);
		}

		/// <summary>
		/// Will wait for execution task to finish and the reset cached directories
		/// </summary>
		/// <param name="executionTask"></param>
		/// <param name="Makefile"></param>
		public static void WaitForExecution(Task executionTask, TargetMakefile Makefile)
		{
			using ITimelineEvent uhtManifestScope = Timeline.ScopeEvent("UHTWait");
			executionTask.Wait();
			ResetCachedHeaderInfo(Makefile.UObjectModules);
		}

		static void ResetCachedHeaderInfo(List<UHTModuleInfo> UObjectModules)
		{
			foreach (UHTModuleInfo ModuleInfo in UObjectModules)
			{
				ModuleInfo.GeneratedCodeDirectory.ResetCachedInfo();
			}
		}
	}
}
