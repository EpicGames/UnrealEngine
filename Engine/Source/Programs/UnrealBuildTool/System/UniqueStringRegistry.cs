// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Threading;

namespace UnrealBuildTool
{
	/// <summary>
	/// Maps a unique string to an integer
	/// </summary>
	public class UniqueStringRegistry
	{
		// protect the InstanceMap
		private readonly Lock LockObject = new();

		// holds a mapping of string name to single instance
		private Dictionary<string, int> StringToInstanceMap = new Dictionary<string, int>(StringComparer.OrdinalIgnoreCase);

		// holds a mapping of string aliases to original string name
		private Dictionary<string, string> AliasToStringMap = new Dictionary<string, string>(StringComparer.OrdinalIgnoreCase);

		/// <summary>
		/// Default constructor
		/// </summary>
		public UniqueStringRegistry()
		{
		}

		/// <summary>
		/// See if string is in registry
		/// </summary>
		/// <returns>True if string is present in registry</returns>
		public bool HasString(string Name)
		{
			return StringToInstanceMap.ContainsKey(Name);
		}

		/// <summary>
		/// Add string if missing and/or lookup
		/// </summary>
		/// <returns>String instance id</returns>
		public int FindOrAddByName(string Name)
		{
			// look for existing one
			if (!StringToInstanceMap.TryGetValue(Name, out int Instance))
			{
				lock (LockObject)
				{
					if (!StringToInstanceMap.TryGetValue(Name, out Instance))
					{
						// copy over the dictionary and add, so that other threads can keep reading out of the old one until it's time
						Dictionary<string, int> NewStringToInstanceMap = new Dictionary<string, int>(StringToInstanceMap, StringComparer.OrdinalIgnoreCase);

						// make and add a new instance number
						Instance = StringToInstanceMap.Count;
						NewStringToInstanceMap[Name] = Instance;

						// replace the class's map
						StringToInstanceMap = NewStringToInstanceMap;
					}
				}
			}
			return Instance;
		}

		/// <summary>
		/// Get list of strings
		/// </summary>
		/// <returns>String list</returns>
		public string[] GetStringNames()
		{
			return StringToInstanceMap.Keys.ToArray();
		}

		/// <summary>
		/// Get list of string ids
		/// </summary>
		/// <returns>String id list</returns>
		public int[] GetStringIds()
		{
			return StringToInstanceMap.Values.ToArray();
		}

		/// <summary>
		/// Get string given instance id
		/// </summary>
		/// <returns>String</returns>
		public string GetStringForId(int Id)
		{
			return StringToInstanceMap.First(x => x.Value == Id).Key;
		}

		/// <summary>
		/// See if alias exists in registry
		/// </summary>
		/// <returns>True if alias exists</returns>
		public bool HasAlias(string Alias)
		{
			return AliasToStringMap.ContainsKey(Alias);
		}

		/// <summary>
		/// Get instance id of alias
		/// </summary>
		/// <returns>Instance id of alias</returns>
		public int FindExistingAlias(string Alias)
		{
			string? Name;
			if (!AliasToStringMap.TryGetValue(Alias, out Name) || Name == null)
			{
				throw new BuildException($"Alias {Alias} not found");
			}

			return FindOrAddByName(Name);
		}

		/// <summary>
		/// Add alias if missing and/or lookup
		/// </summary>
		/// <returns>Instance id of alias</returns>
		public int FindOrAddAlias(string Alias, string OriginalName)
		{
			string? Name;
			if (!AliasToStringMap.TryGetValue(Alias, out Name))
			{
				lock (LockObject)
				{
					// copy over the dictionary and add, so that other threads can keep reading out of the old one until it's time
					Dictionary<string, string> NewAliasToStringMap = new Dictionary<string, string>(AliasToStringMap, StringComparer.OrdinalIgnoreCase);

					// add a new alias
					NewAliasToStringMap[Alias] = OriginalName;

					// replace the instance map
					AliasToStringMap = NewAliasToStringMap;
				}
				Name = OriginalName;
			}
			else
			{
				if (Name != OriginalName || Name == null)
				{
					throw new BuildException($"{Alias} is already an alias for {Name}. Can't associate with {OriginalName}");
				}
			}

			return FindOrAddByName(Name);
		}
	}
}