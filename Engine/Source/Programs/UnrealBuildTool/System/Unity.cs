// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using EpicGames.Core;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	class Unity
	{
		/// <summary>
		/// Prefix used for all dynamically created Unity modules
		/// </summary>
		public const string ModulePrefix = "Module.";

		/// <summary>
		/// Constant used to estimate file size for generated files
		/// </summary>
		/// <remarks>
		/// When sampled, the median size for a generated file going into a Unity module was ~7.3KB.
		/// 4KB is rounding down to the closest power of two, as generated code has been observed to
		/// compile faster than non-generated code of equivalent size.
		/// </remarks>
		public const long PlaceholderFileSize = 4 * 1024;

		/// <summary>
		/// Safe way to get file length
		/// </summary>
		static long FileLength(FileItem file)
		{
			// Temp: Currently there is a race between unity generation and UHT file generation that can cause files to move across Unity TUs
			// based on whether they've been generated or not yet, which can cause errors about modified build files if multiple targets
			// generate unity sets for the same intermediates (e.g. DebugGame, Development).
			// Until this is fixed properly, we'll always use the placeholder size to make this deterministic.
			if (file.Location.ContainsName("Intermediate", 0))
			{
				return PlaceholderFileSize;
			}
			// Files outside of an Intermediate directory must never be placeholder.
			return file.Length;
		}

		/// <summary>
		/// A class which represents a list of files and the sum of their lengths.
		/// </summary>
		public class FileCollection
		{
			public List<FileItem> Files { get; private set; }
			public List<FileItem> VirtualFiles { get; private set; }
			public long TotalLength { get; private set; }

			/// The length of this file collection, plus any additional virtual space needed for bUseAdapativeUnityBuild.
			/// See the comment above AddVirtualFile() below for more information.
			public long VirtualLength { get; private set; }

			public FileCollection()
			{
				Files = new List<FileItem>();
				VirtualFiles = new List<FileItem>();
				TotalLength = 0;
				VirtualLength = 0;
			}

			public void AddFile(FileItem File)
			{
				Files.Add(File);

				long Length = FileLength(File);
				TotalLength += Length;
				VirtualLength += Length;
			}

			/// <summary>
			/// Doesn't actually add a file, but instead reserves space.  This is used with "bUseAdaptiveUnityBuild", to prevent
			/// other compiled unity blobs in the module's numbered set from having to be recompiled after we eject source files
			/// one of that module's unity blobs.  Basically, it can prevent dozens of files from being recompiled after the first
			/// time building after your working set of source files changes
			/// </summary>
			/// <param name="File">The virtual file to add to the collection</param>
			public void AddVirtualFile(FileItem File)
			{
				VirtualFiles.Add(File);
				VirtualLength += File.Length;
			}
		}

		/// <summary>
		/// A class for building up a set of unity files.  You add files one-by-one using AddFile then call EndCurrentUnityFile to finish that one and
		/// (perhaps) begin a new one.
		/// </summary>
		public class UnityFileBuilder
		{
			private List<FileCollection> UnityFiles;
			private FileCollection CurrentCollection;
			private readonly int SplitLength;

			/// <summary>
			/// Constructs a new UnityFileBuilder.
			/// </summary>
			/// <param name="InSplitLength">The accumulated length at which to automatically split a unity file, or -1 to disable automatic splitting.</param>
			public UnityFileBuilder(int InSplitLength)
			{
				UnityFiles = new List<FileCollection>();
				CurrentCollection = new FileCollection();
				SplitLength = InSplitLength;
			}

			/// <summary>
			/// Adds a file to the current unity file.  If splitting is required and the total size of the
			/// unity file exceeds the split limit, then a new file is automatically started.
			/// </summary>
			/// <param name="File">The file to add.</param>
			public void AddFile(FileItem File)
			{
				if (SplitLength != -1 && FileLength(File) > SplitLength)
				{
					EndCurrentUnityFile();
				}
				CurrentCollection.AddFile(File);
				if (SplitLength != -1 && CurrentCollection.VirtualLength > SplitLength)
				{
					EndCurrentUnityFile();
				}
			}

			/// <summary>
			/// Doesn't actually add a file, but instead reserves space, then splits the unity blob normally as if it
			/// was a real file that was added.  See the comment above FileCollection.AddVirtualFile() for more info.
			/// </summary>
			/// <param name="File">The file to add virtually.  Only the size of the file is tracked.</param>
			public void AddVirtualFile(FileItem File)
			{
				if (SplitLength != -1 && FileLength(File) > SplitLength)
				{
					EndCurrentUnityFile();
				}
				CurrentCollection.AddVirtualFile(File);
				if (SplitLength != -1 && CurrentCollection.VirtualLength > SplitLength)
				{
					EndCurrentUnityFile();
				}
			}

			/// <summary>
			/// Starts a new unity file.  If the current unity file contains no files, this function has no effect, i.e. you will not get an empty unity file.
			/// </summary>
			public void EndCurrentUnityFile()
			{
				if (CurrentCollection.Files.Count == 0)
				{
					return;
				}

				UnityFiles.Add(CurrentCollection);
				CurrentCollection = new FileCollection();
			}

			/// <summary>
			/// Returns the list of built unity files.  The UnityFileBuilder is unusable after this.
			/// </summary>
			/// <returns></returns>
			public List<FileCollection> GetUnityFiles()
			{
				EndCurrentUnityFile();

				List<FileCollection> Result = UnityFiles;

				// Null everything to ensure that failure will occur if you accidentally reuse this object.
				CurrentCollection = null!;
				UnityFiles = null!;

				return Result;
			}
		}

		/// <summary>
		/// Given a set of source files, generates another set of source files that #include all the original
		/// files, the goal being to compile the same code in fewer translation units.
		/// The "unity" files are written to the IntermediateDirectory.
		/// </summary>
		/// <param name="Target">The target we're building</param>
		/// <param name="SourceFiles">The source files to #include.</param>
		/// <param name="HeaderFiles">The header files that might correspond to the source files.</param>
		/// <param name="CompileEnvironment">The environment that is used to compile the source files.</param>
		/// <param name="WorkingSet">Interface to query files which belong to the working set</param>
		/// <param name="BaseName">Base name to use for the Unity files</param>
		/// <param name="IntermediateDirectory">Intermediate directory for unity source files</param>
		/// <param name="Graph">The makefile being built</param>
		/// <param name="SourceFileToUnityFile">Receives a mapping of source file to unity file</param>
		/// <param name="NormalFiles">Receives the files to compile using the normal configuration.</param>
		/// <param name="AdaptiveFiles">Receives the files to compile using the adaptive unity configuration.</param>
		/// <param name="NumIncludedBytesPerUnitySource">An approximate number of bytes of source code to target for inclusion in a single unified source file.</param>
		public static void GenerateUnitySource(
			ReadOnlyTargetRules Target,
			List<FileItem> SourceFiles,
			List<FileItem> HeaderFiles,
			CppCompileEnvironment CompileEnvironment,
			ISourceFileWorkingSet WorkingSet,
			string BaseName,
			DirectoryReference IntermediateDirectory,
			IActionGraphBuilder Graph,
			Dictionary<FileItem, FileItem> SourceFileToUnityFile,
			out List<FileItem> NormalFiles,
			out List<FileItem> AdaptiveFiles,
			int NumIncludedBytesPerUnitySource)
		{
			List<FileItem> NewSourceFiles = new List<FileItem>();

			// Figure out size of all input files combined. We use this to determine whether to use larger unity threshold or not.
			long TotalBytesInSourceFiles = SourceFiles.Sum(F => FileLength(F));

			// We have an increased threshold for unity file size if, and only if, all files fit into the same unity file. This
			// is beneficial when dealing with PCH files. The default PCH creation limit is X unity files so if we generate < X 
			// this could be fairly slow and we'd rather bump the limit a bit to group them all into the same unity file.
			// Optimization only makes sense if PCH files are enabled.
			bool bForceIntoSingleUnityFile = Target.bStressTestUnity || (TotalBytesInSourceFiles < NumIncludedBytesPerUnitySource * 2 && Target.bUsePCHFiles);

			// Even if every single file in the module appears in the working set adaptive unity should still be used even if it's slower.
			GetAdaptiveFiles(Target, SourceFiles, HeaderFiles, CompileEnvironment, WorkingSet, IntermediateDirectory, Graph, out NormalFiles, out AdaptiveFiles);

			// Build the list of unity files.
			List<FileCollection> AllUnityFiles;
			{
				// Sort the incoming file paths lexicographically, so there will be consistency in unity blobs across multiple machines.
				// Note that we're relying on this not only sorting files within each directory, but also the directories
				// themselves, so the whole list of file paths is the same across computers.
				// Case-insensitive file path compare, because you never know what is going on with local file systems.
				List<FileItem> SortedSourceFiles = [.. SourceFiles];
				SortedSourceFiles.Sort((A, B) =>
				{
					// Generated files from UHT need to be first in the list because they implement templated functions that aren't
					// declared in the header but are required to link. If they are placed later in the list, you will see
					// compile errors because the templated function is instantiated but is defined later in the same translation unit
					// which results in 'error C2908: explicit specialization; '*****' has already been instantiated'
					bool bAIsGenerated = A.AbsolutePath.EndsWith(".gen.cpp", StringComparison.Ordinal) || CompileEnvironment.FileMatchesExtraGeneratedCPPTypes(A.AbsolutePath);
					bool bBIsGenerated = B.AbsolutePath.EndsWith(".gen.cpp", StringComparison.Ordinal) || CompileEnvironment.FileMatchesExtraGeneratedCPPTypes(B.AbsolutePath);
					if (bAIsGenerated != bBIsGenerated)
					{
						return bAIsGenerated && !bBIsGenerated ? -1 : 1;
					}

					// Sort oversized files to the end of the list so they will be placed into their own unity file
					if (!bForceIntoSingleUnityFile)
					{
						bool bAIsOversized = FileLength(A) > NumIncludedBytesPerUnitySource;
						bool bBIsOversized = FileLength(B) > NumIncludedBytesPerUnitySource;
						if (bAIsOversized != bBIsOversized)
						{
							return !bAIsOversized && bBIsOversized ? -1 : 1;
						}
					}

					return String.Compare(A.AbsolutePath, B.AbsolutePath, StringComparison.OrdinalIgnoreCase);
				});

				HashSet<FileItem> AdaptiveFileSet = [.. AdaptiveFiles];
				UnityFileBuilder SourceUnityFileBuilder = new(bForceIntoSingleUnityFile ? -1 : NumIncludedBytesPerUnitySource);
				foreach (FileItem SourceFile in SortedSourceFiles)
				{
					if (!bForceIntoSingleUnityFile && SourceFile.AbsolutePath.Contains(".GeneratedWrapper.", StringComparison.InvariantCultureIgnoreCase))
					{
						NewSourceFiles.Add(SourceFile);
					}

					// When adaptive unity is enabled, go ahead and exclude any source files that we're actively working with
					if (AdaptiveFileSet.Contains(SourceFile))
					{
						// Let the unity file builder know about the file, so that we can retain the existing size of the unity blobs.
						// This won't actually make the source file part of the unity blob, but it will keep track of how big the
						// file is so that other existing unity blobs from the same module won't be invalidated. This prevents much
						// longer compile times the first time you build after your working file set changes.
						SourceUnityFileBuilder.AddVirtualFile(SourceFile);
					}
					else
					{
						// Compile this file as part of the unity blob
						SourceUnityFileBuilder.AddFile(SourceFile);
					}
				}

				AllUnityFiles = SourceUnityFileBuilder.GetUnityFiles();
			}

			// Create a set of CPP files that combine smaller CPP files into larger compilation units, along with the corresponding 
			// actions to compile them.
			int CurrentUnityFileCount = 0;
			foreach (FileCollection UnityFile in AllUnityFiles)
			{
				++CurrentUnityFileCount;

				FileItem FirstFile = UnityFile.Files.FirstOrDefault() ?? UnityFile.VirtualFiles.First();
#pragma warning disable CA1308 // Normalize strings to uppercase - File extensions are safe to lowercase under normal usage
				string UnityExt = FirstFile.Location.GetExtension().ToLowerInvariant();
#pragma warning restore CA1308
				StringWriter OutputUnitySourceWriter = new();

				OutputUnitySourceWriter.WriteLine($"// This file is automatically generated at compile-time to include some subset of the user-created {UnityExt.Trim('.')} files.");

				// Determine unity file path name
				string UnitySourceFileName = AllUnityFiles.Count > 1
					? $"{ModulePrefix}{BaseName}.{CurrentUnityFileCount}{UnityExt}"
					: $"{ModulePrefix}{BaseName}{UnityExt}";
				FileReference UnitySourceFilePath = FileReference.Combine(IntermediateDirectory, UnitySourceFileName);

				HashSet<FileItem> InlinedSourceFilesInUnity = [];

				// Add source files to the unity file
				foreach (FileItem SourceFile in UnityFile.Files)
				{
					string SourceFileString = SourceFile.AbsolutePath;
					if (CompileEnvironment.RootPaths.GetVfsOverlayPath(SourceFile.Location, out string? vfsPath))
					{
						SourceFileString = vfsPath;
					}
					else if (SourceFile.Location.IsUnderDirectory(Unreal.RootDirectory))
					{
						SourceFileString = SourceFile.Location.MakeRelativeTo(Unreal.EngineSourceDirectory);
					}
					OutputUnitySourceWriter.WriteLine("#include \"{0}\"", SourceFileString.Replace('\\', '/'));

					if (CompileEnvironment.FileInlineSourceMap.TryGetValue(SourceFile, out HashSet<FileItem>? InlinedFiles))
					{
						InlinedSourceFilesInUnity.UnionWith(InlinedFiles);
					}
					InlinedSourceFilesInUnity.Add(SourceFile);
				}

				// Write the unity file to the intermediate folder.
				FileItem UnitySourceFile = Graph.CreateIntermediateTextFile(UnitySourceFilePath, OutputUnitySourceWriter.ToString());
				NewSourceFiles.Add(UnitySourceFile);

				// Store all the inlined gen.cpp files
				CompileEnvironment.FileInlineSourceMap[UnitySourceFile] = InlinedSourceFilesInUnity;

				// Store the mapping of source files to unity files in the makefile
				foreach (FileItem SourceFile in UnityFile.Files)
				{
					SourceFileToUnityFile[SourceFile] = UnitySourceFile;
				}
				foreach (FileItem SourceFile in UnityFile.VirtualFiles)
				{
					SourceFileToUnityFile[SourceFile] = UnitySourceFile;
				}
			}

			NormalFiles = NewSourceFiles;
		}

		public static void GetAdaptiveFiles(
			ReadOnlyTargetRules Target,
			List<FileItem> CPPFiles,
			List<FileItem> HeaderFiles,
			CppCompileEnvironment CompileEnvironment,
			ISourceFileWorkingSet WorkingSet,
			DirectoryReference IntermediateDirectory,
			IActionGraphBuilder Graph,
			out List<FileItem> NormalFiles,
			out List<FileItem> AdaptiveFiles)
		{
			NormalFiles = new List<FileItem>();
			AdaptiveFiles = new List<FileItem>();

			if (!Target.bUseAdaptiveUnityBuild || Target.bStressTestUnity)
			{
				NormalFiles = CPPFiles;
				return;
			}

			HashSet<FileItem> HeaderFilesInWorkingSet = new HashSet<FileItem>(HeaderFiles.Where(WorkingSet.Contains));

			// Figure out which uniquely-named header files are in the working set.
			// Unique names are important to avoid ambiguity about which header a source file includes.
			Dictionary<string, FileItem> NameToHeaderFileInWorkingSet = new Dictionary<string, FileItem>();
			List<string> DuplicateHeaderNames = new List<string>();
			HashSet<string> HeaderNames = new HashSet<string>();
			foreach (FileItem HeaderFile in HeaderFiles)
			{
				string HeaderFileName = HeaderFile.Location.GetFileName();
				if (!HeaderNames.Add(HeaderFileName))
				{
					DuplicateHeaderNames.Add(HeaderFileName);
				}
				else if (HeaderFilesInWorkingSet.Contains(HeaderFile))
				{
					NameToHeaderFileInWorkingSet[HeaderFileName] = HeaderFile;
				}
			}
			foreach (string Name in DuplicateHeaderNames)
			{
				NameToHeaderFileInWorkingSet.Remove(Name);
			}

			HashSet<FileItem> UnhandledHeaderFilesInWorkingSet = new(HeaderFilesInWorkingSet);

			// Add source files to the adaptive set if they or their first included header are in the working set.
			foreach (FileItem CPPFile in CPPFiles)
			{
				bool bHeaderInWorkingSet = false;
				if (CPPFile.Name.EndsWith(".gen.cpp", StringComparison.Ordinal))
				{
					// Ignore these files, does not need to be in adapt 
				}
				else if (CompileEnvironment.MetadataCache.GetFirstInclude(CPPFile) is string FirstInclude &&
					NameToHeaderFileInWorkingSet.TryGetValue(Path.GetFileName(FirstInclude), out FileItem? HeaderFile))
				{
					bHeaderInWorkingSet = true;
					UnhandledHeaderFilesInWorkingSet.Remove(HeaderFile);
				}

				bool bAdaptive = (bHeaderInWorkingSet || WorkingSet.Contains(CPPFile)) && !CompileEnvironment.FileMatchesExtraGeneratedCPPTypes(CPPFile.FullName);
				List<FileItem> Files = bAdaptive ? AdaptiveFiles : NormalFiles;
				Files.Add(CPPFile);
			}

			// Add adaptive files to the working set that will invalidate the makefile if it changes.
			foreach (FileItem File in AdaptiveFiles)
			{
				Graph.AddFileToWorkingSet(File);
			}

			// We also need to add the headers since we don't want to invalidate makefile if they are changing
			foreach (FileItem File in HeaderFilesInWorkingSet)
			{
				Graph.AddFileToWorkingSet(File);
			}

			// Add header files in the working set to the adaptive files if they are not the first include of a source file.
			if (Target.bAdaptiveUnityCompilesHeaderFiles)
			{
				foreach (FileItem HeaderFile in UnhandledHeaderFilesInWorkingSet)
				{
					// If file is skipped by header units it means they can't be compiled by themselves and we must skip them too
					if (CompileEnvironment.MetadataCache.GetHeaderUnitType(HeaderFile) != HeaderUnitType.Valid)
					{
						continue;
					}

					StringWriter OutputHeaderCPPWriter = new StringWriter();
					OutputHeaderCPPWriter.WriteLine("// This file is automatically generated at compile-time to include a modified header file.");
					OutputHeaderCPPWriter.WriteLine($"#include \"{HeaderFile.AbsolutePath.Replace('\\', '/')}\"");

					string HeaderCPPFileName = $"{HeaderFile.Location.GetFileNameWithoutExtension()}.h.cpp";
					FileReference HeaderCPPFilePath = FileReference.Combine(IntermediateDirectory, HeaderCPPFileName);

					AdaptiveFiles.Add(Graph.CreateIntermediateTextFile(HeaderCPPFilePath, OutputHeaderCPPWriter.ToString()));
				}
			}

			HashSet<FileItem> CandidateAdaptiveFiles = new HashSet<FileItem>();
			CandidateAdaptiveFiles.UnionWith(CPPFiles);
			CandidateAdaptiveFiles.UnionWith(HeaderFiles);
			CandidateAdaptiveFiles.ExceptWith(AdaptiveFiles);
			CandidateAdaptiveFiles.ExceptWith(HeaderFilesInWorkingSet);

			foreach (FileItem File in CandidateAdaptiveFiles)
			{
				Graph.AddCandidateForWorkingSet(File);
			}
		}
	}
}
