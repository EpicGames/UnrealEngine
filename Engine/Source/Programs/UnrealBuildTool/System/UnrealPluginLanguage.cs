// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using System.Linq;
using System.Text;
using System.Xml;
using System.Xml.Linq;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/* UnrealPluginLanguage (UPL) is a simple XML-based language for manipulating XML and returning
	 * strings.  It contains an <init> section which is evaluated once per architecture before any
	 * other sections.  The state is maintained and carried forward to the next section evaluated
	 * so the order the sections are executed matters.
	 * 
	 * While UPL is a general system for modifying and quering XML it is specifically used to allow
	 * plug-ins to affect the global configuration of the package that they are a part of. For
	 * example, this allows a plug-in to modify an Android's APK AndroidManfiest.xml file or an
	 * IOS IPA's plist file. UBT will also query a plug-in's UPL xml file for strings to be included
	 * in files that must be common to the package such as some .java files on Android.
	 * 
	 * If you need to see the instructions executed in your plugin context add the following to
	 * enable tracing:
	 * 
	 *	<trace enable="true"/>
	 *	
	 * After this instuction all the nodes actually executed in your context will be written to the
	 * log until you do a <trace enable="false"/>.  You can also get a dump of all the variables in
	 * your context with this command:
	 * 
	 *	<dumpvars/>
	 * 
	 * Bool, Int, and String variable types are supported.  Any attribute may reference a variable
	 * and will be replaced with the string equivalent before evaluation using this syntax:
	 * 
	 *	$B(name) = boolean variable "name"'s value
	 *	$I(name) = integer variable "name"'s value
	 *	$S(name) = string variable "name"'s value
	 *	$E(name) = element variable "name"'s value
	 * 
	 * The following variables are initialized automatically:
	 * 
	 *	$S(Output) = the output returned for evaluating the section (initialized to Input)
	 *	$S(UnrealArch) = target Unreal arch (arm32, arm64, x86, x64)
	 *	$S(Architecture) = target architecture (armeabi-armv7a, arm64-v8a, x86, x86_64)
	 *	$S(PluginDir) = directory the XML file was loaded from
	 *	$S(EngineDir) = engine directory
	 *	$S(BuildDir) = project's platform appropriate build directory (within the Intermediate folder)
	 *	$S(Configuration) = configuration type (Debug, DebugGame, Development, Test, Shipping)
	 *	$B(Distribution) = true if distribution build
	 *	$B(IsEmbedded) = true if project compiled for embedded use [Android-only]
	 *	$I(EngineMajorVersion) = major version of engine (ex. 4)
	 *	$I(EngineMinorVersion) = minor version of engine (ex. 21)
	 *	$I(EnginePatchVersion) = patch version of engine (ex. 0)
	 *	$S(EngineVersion) = engine version string (ex. "4.21.0")
	 * 
	 * Note: with the exception of the above variables, all are in the context of the plugin to
	 * prevent namespace collision; trying to set a new value to any of the above, with the
	 * exception of Output, will only affect the current context.
	 * 
	 * The following nodes allow manipulation of variables:
	 * 
	 *	<setBool result="" value=""/>
	 *	<setInt result="" value=""/>
	 *	<setString result="" value=""/>
	 *	<setElement result="" value=""/>
	 *	<setElement result="" value="" text=""/>
	 *	<setElement result="" xml=""/>
	 *	
	 * <setElement> with value creates an empty XML element with the tag set to value.
	 * <setElement> with value and text creates an XML element with tag set to value of unparsed text.
	 * <setElement> with xml will parse the XML provided.  Remember to escape any special characters!
	 * 
	 * Variables may also be set from a property in an ini file:
	 * 
	 *	<setBoolFromProperty result="" ini="" section="" property="" default=""/>
	 *	<setBoolFromPropertyContains result="" ini="" section="" property="" default="" contains=""/>
	 *	<setIntFromProperty result="" ini="" section="" property="" default=""/>
	 *	<setIntFromPropertyArrayNum result="" ini="" section="" property=""/>
	 *	<setStringFromProperty result="" ini="" section="" property="" default=""/>
	 *	<setStringFromPropertyArray result="" ini="" section="" property="" index="" default=""/>
     *	
     * Strings may also be set from an environment variable using the <setStringFromEnvVar> node.
	 * The environment variable must be specified as the 'value' attribute and wrapped in a pair
	 * of '%' characters.
     *	
     *	<setStringFromEnvVar result="" value=""/>
	 *	
	 * You can also check if a specific environment variable is defined (again, wrapped in '%' characters):
	 * 
	 *  <setBoolEnvVarDefined result="" value=""/>
	 *  
	 * A general example for using environment variable nodes:
	 * 
	 *  <setBoolEnvVarDefined result="bHasNDK" value="%NDKROOT%"/>
	 *  <setStringFromEnvVar result="NDKPath" value="%NDKROOT%"/>
	 * 
	 * Boolean variables may also be set to the result of applying operators:
	 * 
	 *	<setBoolNot result="" source=""/>
	 *	<setBoolAnd result="" arg1="" arg2=""/>
	 *	<setBoolOr result="" arg1="" arg2=""/>
	 *	<setBoolIsEqual result="" arg1="" arg2=""/>
	 *	<setBoolIsLess result="" arg1="" arg2=""/>
	 *	<setBoolIsLessEqual result="" arg1="" arg2=""/>
	 *	<setBoolIsGreater result="" arg1="" arg2=""/>
	 *	<setBoolIsGreaterEqual result="" arg1="" arg2=""/>
	 *	<setBoolFromPropertyContains result="" ini="" section="" property="" contains=""/>
	 * 
	 * Integer variables may use these arithmetic operations: 
	 * 
	 *	<setIntAdd result="" arg1="" arg2=""/>
	 *	<setIntSubtract result="" arg1="" arg2=""/>
	 *	<setIntMultiply result="" arg1="" arg2=""/>
	 *	<setIntDivide result="" arg1="" arg2=""/>
	 * 
	 * Strings are manipulated with the following:
	 * 
 	 *	<setStringAdd result="" arg1="" arg2=""/>
	 *	<setStringSubstring result="" source="" start="" length=""/>
	 *	<setStringSubstringAfterFind result="" source="" find="" length="" default=""/>
	 *	<setStringReplace result="" source="" find="" with=""/>
	 *	<setStringToLower result="" source=""/>
	 *	<setStringToUpper result="" source=""/>
	 *	
	 * <setStringSubstringAfterFind> is equivalent to calling setStringSubstring with "start" set to
	 * the index of the character immediately following the find string. Passing -1 for length (or
	 * omitting it) will return the entire string to the right of the find string.
	 * 
	 * String length may be retrieved with:
	 * 
	 *	<setIntLength result="" source=""/>
	 * 
	 * The index of a search string may be found in source with:
	 * 
	 *	<setIntFindString result="" source="" find=""/>
	 *	
	 * The following shortcut string comparisons may also be used instead of using <setIntFindString>
	 * and checking the result:
	 * 
	 *	<setBoolStartsWith result="" source="" find=""/>
	 *	<setBoolEndsWith result="" source="" find=""/>
	 *	<setBoolContains result="" source="" find=""/>
	 *	
	 * Messages are written to the log with this node:
	 *
	 *	<log text=""/>
	 * 
	 * Conditional execution uses the following form:
	 * 
	 *	<if condition="">
	 *		<true>
	 *			<!-- executes if boolean variable in condition is true -->
	 *		</true>
	 *		<false>
	 *			<!-- executes if boolean variable in condition is false -->
	 *		</false>
	 *	</if>
	 * 
	 * The <true> and <false> blocks are optional.  The condition must be in a boolean variable.
	 * The boolean operator nodes may be combined to create a final state for more complex
	 * conditions:
	 * 
	 *	<setBoolNot result="notDistribution" source="$B(Distribution)/>
	 *	<setBoolIsEqual result="isX86" arg1="$S(Architecture)" arg2="x86"/>
	 *	<setBoolIsEqual result="isX86_64" arg2="$S(Architecture)" arg2="x86_64/">
	 *	<setBoolOr result="isIntel" arg1="$B(isX86)" arg2="$B(isX86_64)"/>
	 *	<setBoolAnd result="intelAndNotDistribution" arg1="$B(isIntel)" arg2="$B(notDistribution)"/>
	 *	<if condition="intelAndNotDistribution">
	 *		<true>
	 *			<!-- do something for Intel if not a distribution build -->
	 *		</true>
	 *	</if>
	 * 
	 * Note the "isIntel" could also be done like this:
	 * 
	 *	<setStringSubstring result="subarch" source="$S(Architecture)" start="0" length="3"/>
	 *	<setBoolIsEqual result="isIntel" arg1="$S(subarch)" arg2="x86"/>
	 * 
	 * Two shortcut nodes are available for conditional execution:
	 * 
	 *	<isArch arch="armeabi-armv7">
	 *		<!-- do stuff -->
	 *	</isArch>
	 * 
	 * is the equivalent of:
	 * 
	 *	<setBoolIsEqual result="temp" arg1="$S(Architecture)" arg2="armeabi-armv7">
	 *	<if condition="temp">
	 *		<true>
	 *			<!-- do stuff -->
	 *		</true>
	 *	</if>
	 *	
	 * and
	 * 
	 *	<isDistribution>
	 *		<!-- do stuff -->
	 *	</isDistribution>
	 *	
	 * is the equivalent of:
	 * 
	 *	<if condition="Distribution">
	 *		<!-- do stuff -->
	 *	</if>
	 * 
	 * Execution may be stopped with:
	 * 
	 *	<return/>
	 *	 
	 * Loops may be created using these nodes:
	 * 
	 *	<while condition="">
	 *		<!-- do stuff -->
	 *	</while>
	 *	
	 *	<break/>
	 *	<continue/>
	 *	
	 * The <while> body will execute until the condition is false or a <break/> is hit.  The
	 * <continue/> will restart execution of the loop if the condition is still true or exit.
	 * 
	 * Note: <break/> outside a <while> body will act the same as <return/>
	 * 
	 * Here is an example loop which writes 1 to 5 to the log, skipping 3.  Note the update of the
	 * while condition should be done before the continue otherwise it may not exit.
	 * 
	 *	<setInt result="index" value="0"/>
	 *	<setBool result="loopRun" value="true"/>
	 *	<while condition="loopRun">
	 *		<setIntAdd result="index" arg1="$I(index)" arg2="1"/>
	 *		<setBoolIsLess result="loopRun" arg1="$I(index)" arg2="5"/>
	 *		<setBoolIsEqual result="indexIs3" arg1="$I(index)" arg2="3"/>
	 *		<if condition="indexIs3">
	 *			<true>
	 *				<continue/>
	 *			</true>
	 *		</if>
	 *		<log text="$I(index)"/>
	 *	</while>
	 *	
	 * It is possible to use variable replacement in generating the result variable
	 * name as well.  This makes the creation of arrays in loops possible:
	 * 
	 *	<setString result="array_$I(index)" value="element $I(index) in array"/>
	 *	
	 * This may be retrieved using the following (value is treated as the variable
	 * name):
	 * 
	 *	<setStringFrom result="out" value="array_$I(index)"/>
	 *	
	 * For boolean and integer types, you may use <setBoolFrom/> and <setIntFrom/>.
	 * 
	 * Nodes for inserting text into the section are as follows:
	 * 
	 *	<insert> body </insert>
	 *	<insertNewline/>
	 *	<insertValue value=""/>
	 *	<loadLibrary name="" failmsg=""/>
	 *	
	 * The first one will insert either text or nodes into the returned section
	 * string.  Please note you must use escaped characters for:
	 * 
	 *	< = &lt;
	 *	> = &gt;
	 *	& = &amp;
	 * 
	 *	<insertValue value=""/> evaluates variables in value before insertion.  If value contains
	 * double quote ("), you must escape it with &quot;.
	 * 
	 *	<loadLibrary name="" failmsg=""/> is a shortcut to insert a system.LoadLibrary try/catch
	 * block with an optional logged message for failure to load case.
	 * 
	 * You can do a search and replace in the Output with:
	 * 
	 *	<replace find="" with=""/>
	 * 
	 * Note you can also manipulate the actual $S(Output) directly, the above are more efficient:
	 * 
	 *	<setStringAdd result="Input" arg1="$S(Output)" arg2="sample\n"/>
	 *	<setStringReplace result="Input" source="$S(Output)" find=".LAUNCH" with=".INFO"/>
	 *	
	 * XML manipulation uses the following nodes:
	 * 
	 *	<addElement tag="" name=""/>
	 *	<addElements tag=""> body </addElements>
	 *	<removeElement tag=""/>
	 *	<setStringFromTag result="" tag=""/>
	 *	<setStringFromAttribute result="" tag="" name=""/>
	 *	<setStringFromTagText result="" tag=""/>
	 *	<addAttribute tag="" name="" value=""/>
	 *	<removeAttribute tag="" name=""/>
	 *	<loopElements tag=""> instructions </loopElements>
	 *	
	 * The current element is referenced with tag="$".  Element variables are referenced with $varname
	 * since using $E(varname) will be expanded to the string equivalent of the XML.
	 * 
	 * addElement, addElements, and removeElement by default are applied to all matching tags.  An
	 * optional once="true" attribute may be added to only apply to first matching tag.
	 *
	 * <uses-permission>, <uses-feature>, and <uses-library> are updated with:
	 * 
	 *	<addPermission android:name="" .. />
	 *	<addFeature android:name="" .. />
	 *	<addLibrary android:name="" .. />
	 *
	 * Any attributes in the above commands are copied to the element added to the manifest so you
	 * can do the following, for example:
	 * 
	 *	<addFeature android:name="android.hardware.usb.host" android:required="true"/>
	 *	
	 * Finally, these nodes allow copying of files useful for staging jar and so files:
	 * 
	 *	<copyFile src="" dst="" force=""/>
	 *	<copyDir src="" dst="" force=""/>
	 *
	 * If force is false the file(s) are replaced only if length or timestamp don't match.  Default is true.
	 *	
	 * The following should be used as the base for the src and dst paths:
	 * 
	 *	$S(PluginDir) = directory the XML file was loaded from
 	 *	$S(EngineDir) = engine directory
	 *	$S(BuildDir) = project's platform appropriate build directory
	 *	
	 * While it is possible to write outside the APK directory, it is not recommended.
	 * 
	 * If you must remove files (like development-only files from distribution builds) you can
	 * use this node:
	 * 
	 *	<deleteFiles filespec="" recursive=""/>
	 *	
	 * It is restricted to only removing files from the BuildDir. If recursive is true it will search all subfolders.
	 * Here is example usage to remove the Oculus Signature Files (osig) from the assets directory:
	 * 
	 *	<deleteFiles filespec="assets/oculussig_*"/>
	 *	
	 * The following sections are evaluated during the packaging or deploy stages:
	 * 
	 * ** For all platforms **
	 *	<!-- init section is always evaluated once per architecture -->
	 * 	<init> </init>
	 * 
	 * ** Android Specific sections **
	 * 	<!-- optional updates applied to AndroidManifest.xml -->
	 * 	<androidManifestUpdates> </androidManifestUpdates>
	 * 	
	 * 	<!-- optional additions to proguard -->
	 * 	<proguardAdditions>	</proguardAdditions>
	 * 	
	 * 	<!-- optional AAR imports additions -->
	 * 	<AARImports> </AARImports>
	 * 	
	 * 	<!-- optional base build.gradle additions -->
	 * 	<baseBuildGradleAdditions>  </baseBuildGradleAdditions>
	 *
	 *	<!-- optional base build.gradle buildscript additions -->
	 *	<buildscriptGradleAdditions>  </buildscriptGradleAdditions>
	 *	
	 * 	<!-- optional app build.gradle additions -->
	 * 	<buildGradleAdditions>  </buildGradleAdditions>
	 * 	
	 * 	<!-- optional additions to generated build.xml before ${sdk.dir}/tools/ant/build.xml import -->
	 * 	<buildXmlPropertyAdditions> </buildXmlPropertyAdditions>
	 *
	 * 	<!-- optional files or directories to copy or delete from Intermediate/Android/APK before ndk-build -->
	 * 	<prebuildCopies> </prebuildCopies>
	 * 	
	 * 	<!-- optional files or directories to copy or delete from Intermediate/Android/APK after ndk-build -->
	 * 	<resourceCopies> </resourceCopies>
	 * 	
	 * 	<!-- optional files or directories to copy or delete from Intermediate/Android/APK before Gradle -->
	 * 	<gradleCopies> </gradleCopies>
	 * 	
	 * 	<!-- optional properties to add to gradle.properties -->
	 * 	<gradleProperties> </gradleProperties>
	 *
 	 * 	<!-- optional parameters to add to Gradle commandline (prefix with a space or will run into previous parameter(s)) -->
	 * 	<gradleParameters> </gradleParameters>
	 *
	 *  <!-- optional minimum SDK API level required -->
	 *  <minimumSDKAPI> </minimumSDKAPI>
	 * 	
	 * 	<!-- optional additions to the GameActivity imports in GameActivity.java -->
	 * 	<gameActivityImportAdditions> </gameActivityImportAdditions>
	 *
	 * 	<!-- optional additions to the GameActivity after imports in GameActivity.java -->
	 *  <gameActivityPostImportAdditions> </gameActivityPostImportAdditions>
	 *  
	 * 	<!-- optional additions to the GameActivity class implements in GameActivity.java (end each line with a comma) -->
	 * 	<gameActivityImplementsAdditions> </gameActivityImplementsAdditions>
	 *
	 * 	<!-- optional additions to the GameActivity class body in GameActivity.java -->
	 * 	<gameActivityClassAdditions> </gameActivityOnClassAdditions>
	 * 	
	 * 	<!-- optional additions to GameActivity onCreate metadata reading in GameActivity.java -->
	 * 	<gameActivityReadMetadata> </gameActivityReadMetadata>
	 * 
	 * 	<!-- optional additions to the very earliest point in GameActivity onCreate in GameActivity.java -->
	 *	<gameActivityOnCreateBeginningAdditions> </gameActivityOnCreateBeginningAdditions>
	 *	
	 *	<!-- optional additions to GameActivity onCreate in GameActivity.java -->
	 *	<gameActivityOnCreateAdditions> </gameActivityOnCreateAdditions>
	 * 	
	 * 	<!-- optional additions to GameActivity onDestroy in GameActivity.java -->
	 * 	<gameActivityOnDestroyAdditions> </gameActivityOnDestroyAdditions>
	 * 	
	 * 	<!-- optional additions to GameActivity onConfigurationChanged in GameActivity.java -->
	 * 	<gameActivityonConfigurationChangedAdditions> </gameActivityonConfigurationChangedAdditions>
	 * 	
	 * 	<!-- optional additions to GameActivity onStart in GameActivity.java -->
	 * 	<gameActivityOnStartAdditions> </gameActivityOnStartAdditions>
	 * 	
	 * 	<!-- optional additions to GameActivity onStop in GameActivity.java -->
	 * 	<gameActivityOnStopAdditions> </gameActivityOnStopAdditions>
	 * 	
	 * 	<!-- optional additions to GameActivity onPause in GameActivity.java -->
	 * 	<gameActivityOnPauseAdditions> </gameActivityOnPauseAdditions>
	 * 	
	 * 	<!-- optional additions to GameActivity onResume in GameActivity.java -->
	 *	<gameActivityOnResumeAdditions>	</gameActivityOnResumeAdditions>
	 *	
	 * 	<!-- optional additions to GameActivity onTopResumeActivity in GameActivity.java -->
	 *	<gameActivityOnTopResumeActivityAdditions>	</gameActivityOnTopResumeActivityAdditions>
	 *	
	 * 	<!-- optional additions to GameActivity onNewIntent in GameActivity.java -->
	 *	<gameActivityOnNewIntentAdditions> </gameActivityOnNewIntentAdditions>
	 *	
	 * 	<!-- optional additions to GameActivity onActivityResult in GameActivity.java -->
	 * 	<gameActivityOnActivityResultAdditions>	</gameActivityOnActivityResultAdditions>
	 * 	
	 * 	<!-- optional additional interfaces that GameApplication can implement in GameApplication.java (must start with a comma and list interfaces using commas) -->
	 * 	<gameApplicationImplementsAdditions> </gameApplicationImplementsAdditions>
	 * 	
	 *	<!-- optional additions to GameApplication onCreate in GameApplication.java (all processes) -->
	 *	<gameApplicationOnCreateAdditionsAlways> </gameApplicationOnCreateAdditionsAlways>
	 *  
	 *	<!-- optional additions to GameApplication onCreate in GameApplication.java (non-PSO processes) -->
	 *	<gameApplicationOnCreateAdditions> </gameApplicationOnCreateAdditions>
	 *  
	 * 	<!-- optional libraries to load in GameActivity.java before libUnreal.so -->
	 * 	<soLoadLibrary>	</soLoadLibrary>
	 * 	
	 * 
	 * Here is the complete list of supported nodes:
	 * 
	 * <isArch arch="">
	 * <isDistribution>
	 * <if> => <true> / <false>
	 * <while condition="">
	 * <return/>
	 * <break/>
	 * <continue/>
	 * <log text=""/>
	 * <insert> </insert>
	 * <insertValue value=""/>
	 * <replace find="" with=""/>
	 * <copyFile src="" dst=""/>
	 * <copyDir src="" dst=""/>
	 * <loadLibrary name="" failmsg=""/>
	 * <setBool result="" value=""/>
	 * <setBoolEnvVarDefined result="" value=""/>
	 * <setBoolFrom result="" value=""/>
	 * <setBoolFromProperty result="" ini="" section="" property="" default=""/>
	 * <setBoolFromPropertyContains result="" ini="" section="" property="" contains=""/>
	 * <setBoolNot result="" source=""/>
	 * <setBoolAnd result="" arg1="" arg2=""/>
	 * <setBoolOr result="" arg1="" arg2=""/>
	 * <setBoolIsEqual result="" arg1="" arg2=""/>
	 * <setBoolIsLess result="" arg1="" arg2=""/>
	 * <setBoolIsLessEqual result="" arg1="" arg2=""/>
	 * <setBoolIsGreater result="" arg1="" arg2=""/>
	 * <setBoolIsGreaterEqual result="" arg1="" arg2=""/>
	 * <setInt result="" value=""/>
	 * <setIntFrom result="" value=""/>
	 * <setIntFromProperty result="" ini="" section="" property="" default=""/>
	 * <setIntFromPropertyArrayNum result="" ini="" section="" property=""/>
	 * <setIntAdd result="" arg1="" arg2=""/>
	 * <setIntSubtract result="" arg1="" arg2=""/>
	 * <setIntMultiply result="" arg1="" arg2=""/>
	 * <setIntDivide result="" arg1="" arg2=""/>
	 * <setIntLength result="" source=""/>
	 * <setIntFindString result="" source="" find=""/>
	 * <setString result="" value=""/>
	 * <setStringFrom result="" value=""/>
     * <setStringFromEnvVar result="" value=""/>
	 * <setStringFromProperty result="" ini="" section="" property="" default=""/>
	 * <setStringFromPropertyArray result="" ini="" section="" property="" index="" default=""/>
	 * <setStringAdd result="" arg1="" arg2=""/>
	 * <setStringSubstring result="" source="" index="" length=""/>
	 * <setStringSubstringAfterFind result="" source="" find="" length="" default=""/>
	 * <setStringReplace result="" source="" find="" with=""/>
	 * <setStringToLower result="" source=""/>
	 * <setStringToUpper result="" source=""/>
	 * 
	 */

	// Helper struct to initialize aliased namespaces
	struct AliasedXMLNamespace
	{
		public string Alias { get; set; }
		public string Url { get; set; }

		public override string ToString()
		{
			return String.Format("xmlns:{0}=\"{1}\"", Alias, Url);
		}
	}

	class UnrealPluginLanguage
	{
		/** The merged XML program to run */
		private readonly XDocument XDoc;

		/** XML namespace */
		private readonly XNamespace XMLDefaultNameSpace;
		private readonly Dictionary<string, XNamespace> XMLNameSpaceAliases = new Dictionary<string, XNamespace>();
		private readonly string XMLRootDefinition;

		private UnrealTargetPlatform TargetPlatform;

		/** Trace flag to enable debugging */
		private static bool bGlobalTrace = false;
		private static bool bGlobalTraceFilename = false;

		/** Project file reference */
		private readonly FileReference? ProjectFile;

		private static XDocument XMLDummy = XDocument.Parse("<manifest></manifest>");

		private class UPLContext
		{
			public string Filename;

			/** Variable state */
			public Dictionary<string, bool> BoolVariables;
			public Dictionary<string, int> IntVariables;
			public Dictionary<string, string> StringVariables;
			public Dictionary<string, XElement> ElementVariables;

			/** Local context trace */
			public bool bTrace;
			public bool bTraceFilename;

			public UPLContext(string inFilename, (string UEArch, string NativeArch) Arch, string PluginDir)
			{
				Filename = inFilename;

				BoolVariables = new Dictionary<string, bool>();
				IntVariables = new Dictionary<string, int>();
				StringVariables = new Dictionary<string, string>();
				ElementVariables = new Dictionary<string, XElement>();

				if (PluginDir == null || String.IsNullOrEmpty(PluginDir))
				{
					PluginDir = ".";
				}
				StringVariables["PluginDir"] = PluginDir.Replace("\\", "/", StringComparison.Ordinal);
				StringVariables["AbsPluginDir"] = Path.GetFullPath(PluginDir).Replace("\\", "/", StringComparison.Ordinal);

				StringVariables["UnrealArch"] = Arch.UEArch;
				StringVariables["Architecture"] = Arch.NativeArch;

				bTrace = false;
				bTraceFilename = false;
			}
		}

		private readonly UPLContext GlobalContext;
		private readonly Dictionary<string, UPLContext> Contexts;
		private int ContextIndex;
#pragma warning disable IDE0044 // Make field readonly - this field appears intended to be mutable.
		private string? LastError;
#pragma warning restore IDE0044
		private readonly ILogger Logger;

		// First entry in InXMLNameSpaceAliases will be used as default namespace
		public UnrealPluginLanguage(FileReference? InProjectFile, List<string> InXMLFiles, List<(string UEArch, string NativeArch)> InArchs, List<AliasedXMLNamespace>? InXMLNameSpaceAliases, UnrealTargetPlatform InTargetPlatform, ILogger InLogger)
		{
			ProjectFile = InProjectFile;
			Logger = InLogger;

			LastError = null;

			Contexts = new Dictionary<string, UPLContext>();
			GlobalContext = new UPLContext("", ("", ""), "");
			ContextIndex = 0;
			
			if (InXMLNameSpaceAliases != null && InXMLNameSpaceAliases.Count > 0)
			{
				XMLDefaultNameSpace = InXMLNameSpaceAliases[0].Url;

				StringBuilder RootNamespaces = new StringBuilder();
				foreach (AliasedXMLNamespace AliasToNamespace in InXMLNameSpaceAliases)
				{
					XMLNameSpaceAliases[AliasToNamespace.Alias] = AliasToNamespace.Url;
					RootNamespaces.AppendFormat("xmlns:{0}=\"{1}\" ", AliasToNamespace.Alias, AliasToNamespace.Url);
				}

				XMLRootDefinition = String.Join(" ", RootNamespaces);
			}
			else
			{
				XMLDefaultNameSpace = "";
				XMLRootDefinition = "";
			}
			TargetPlatform = InTargetPlatform;

			string PathPrefix = Path.GetFileName(Directory.GetCurrentDirectory()).Equals("Source", StringComparison.Ordinal) ? ".." : "Engine";

			XDoc = XDocument.Parse("<root " + XMLRootDefinition + "></root>");
			foreach (string Basename in InXMLFiles)
			{
				string Filename = Path.Combine(PathPrefix, Basename.Replace("\\", "/", StringComparison.Ordinal));
				Logger.LogInformation("UPL: {FileName}", Filename);
				if (File.Exists(Filename))
				{
					string PluginDir = Path.GetDirectoryName(Filename)!;
					try
					{
						XDocument MergeDoc = XDocument.Load(Filename, LoadOptions.SetLineInfo);
						MergeXML(MergeDoc, Filename, PluginDir, InArchs);
					}
					catch (Exception e)
					{
						LastError = String.Format("Unreal Plugin file {0} parsing failed! {1}", Filename, e);
						Logger.LogError("\n{LastError}", LastError);
					}
				}
				else
				{
					LastError = String.Format("Unreal Plugin file {0} missing!", Filename);
					Logger.LogError("\n{LastError}", LastError);
					Logger.LogInformation("\nCWD: {Cwd}", Directory.GetCurrentDirectory());
				}
			}
		}

		public string? GetLastError() { return LastError; }

		public static bool GetTrace() { return bGlobalTrace; }
		public static void SetTrace() { bGlobalTrace = true; }
		public static void ClearTrace() { bGlobalTrace = false; }
		public static bool GetTraceFilename() { return bGlobalTraceFilename; }
		public static void SetTraceFilename() { bGlobalTraceFilename = true; }
		public static void ClearTraceFilename() { bGlobalTraceFilename = false; }

		public string GetUPLHash()
		{
			return ContentHash.MD5(XDoc.ToString()).ToString();
		}

		public bool MergeXML(XDocument MergeDoc, string Filename, string PluginDir, List<(string UEArch, string NativeArch)> Archs)
		{
			if (MergeDoc == null)
			{
				return false;
			}

			// create a context for each architecture
			ContextIndex++;
			foreach ((string UEArch, string NativeArch) Arch in Archs)
			{
				UPLContext Context = new UPLContext(Filename, Arch, PluginDir);
				Contexts[Arch.NativeArch + "_" + ContextIndex] = Context;
			}

			// line numbers are lost in the merge so add them as an attribute for later tracing of warnings/errors
			foreach (XElement Element in MergeDoc.Root!.Descendants())
			{
				if (((IXmlLineInfo)Element).HasLineInfo())
				{
					Element.Add(new XAttribute("__line", ((IXmlLineInfo)Element).LineNumber));
				}
			}

			// merge in the nodes
			foreach (XElement Element in MergeDoc.Root!.Elements())
			{
				XElement? Parent = XDoc.Root!.Element(Element.Name);
				if (Parent != null)
				{
					XElement Entry = new XElement("Context", new XAttribute("index", ContextIndex.ToString()));
					Entry.Add(Element.Elements());
					Parent.Add(Entry);
				}
				else
				{
					XElement Entry = new XElement("Context", new XAttribute("index", ContextIndex.ToString()));
					Entry.Add(Element.Elements());
					XElement Base = new XElement(Element.Name);
					Base.Add(Entry);
					XDoc.Root.Add(Base);
				}
			}

			return true;
		}

		public void SaveXML(string Filename)
		{
			XDoc?.Save(Filename);
		}

		private static string DumpContext(UPLContext Context)
		{
			StringBuilder Text = new StringBuilder();
			foreach (KeyValuePair<string, bool> Variable in Context.BoolVariables)
			{
				Text.AppendLine(String.Format("\tbool {0} = {1}", Variable.Key, Variable.Value ? "true" : "false"));
			}
			foreach (KeyValuePair<string, int> Variable in Context.IntVariables)
			{
				Text.AppendLine(String.Format("\tint {0} = {1}", Variable.Key, Variable.Value));
			}
			foreach (KeyValuePair<string, string> Variable in Context.StringVariables)
			{
				Text.AppendLine(String.Format("\tstring {0} = {1}", Variable.Key, Variable.Value));
			}
			foreach (KeyValuePair<string, XElement> Variable in Context.ElementVariables)
			{
				Text.AppendLine(String.Format("\telement {0} = {1}", Variable.Key, Variable.Value));
			}
			return Text.ToString();
		}

		public string DumpVariables()
		{
			string Result = "Global Context:\n" + DumpContext(GlobalContext);
			foreach (KeyValuePair<string, UPLContext> Context in Contexts)
			{
				Result += "Context " + Context.Key + ": " + Context.Value.StringVariables["PluginDir"] + "\n" + DumpContext(Context.Value);
			}
			return Result;
		}

		private bool GetCondition(UPLContext Context, XElement Node, string? Condition, out bool Result)
		{
			Result = false;

			if (Condition == null)
			{
				return false;
			}

			if (!Context.BoolVariables.TryGetValue(Condition, out Result))
			{
				if (!GlobalContext.BoolVariables.TryGetValue(Condition, out Result))
				{
					Logger.LogWarning("\nMissing condition '{Condition}' in '{Node}' (skipping instruction)", Condition, TraceNodeString(Context, Node));
					return false;
				}
			}
			return true;
		}

		private string ExpandVariables(UPLContext Context, string InputString)
		{
			string Result = InputString;
			for (int Idx = Result.IndexOf("$B(", StringComparison.Ordinal); Idx != -1; Idx = Result.IndexOf("$B(", Idx, StringComparison.Ordinal))
			{
				// Find the end of the variable name
				int EndIdx = Result.IndexOf(')', Idx + 3);
				if (EndIdx == -1)
				{
					break;
				}

				// Extract the variable name from the string
				string Name = Result.Substring(Idx + 3, EndIdx - (Idx + 3));

				// Find the value for it, either from the dictionary or the environment block
				bool Value;
				if (!Context.BoolVariables.TryGetValue(Name, out Value))
				{
					if (!GlobalContext.BoolVariables.TryGetValue(Name, out Value))
					{
						Idx = EndIdx + 1;
						continue;
					}
				}

				// Replace the variable, or skip past it
				Result = Result.Substring(0, Idx) + (Value ? "true" : "false") + Result.Substring(EndIdx + 1);
			}
			for (int Idx = Result.IndexOf("$I(", StringComparison.Ordinal); Idx != -1; Idx = Result.IndexOf("$I(", Idx, StringComparison.Ordinal))
			{
				// Find the end of the variable name
				int EndIdx = Result.IndexOf(')', Idx + 3);
				if (EndIdx == -1)
				{
					break;
				}

				// Extract the variable name from the string
				string Name = Result.Substring(Idx + 3, EndIdx - (Idx + 3));

				// Find the value for it, either from the dictionary or the environment block
				int Value;
				if (!Context.IntVariables.TryGetValue(Name, out Value))
				{
					if (!GlobalContext.IntVariables.TryGetValue(Name, out Value))
					{
						Idx = EndIdx + 1;
						continue;
					}
				}

				// Replace the variable, or skip past it
				Result = Result.Substring(0, Idx) + Value.ToString() + Result.Substring(EndIdx + 1);
			}
			for (int Idx = Result.IndexOf("$S(", StringComparison.Ordinal); Idx != -1; Idx = Result.IndexOf("$S(", Idx, StringComparison.Ordinal))
			{
				// Find the end of the variable name
				int EndIdx = Result.IndexOf(')', Idx + 3);
				if (EndIdx == -1)
				{
					break;
				}

				// Extract the variable name from the string
				string Name = Result.Substring(Idx + 3, EndIdx - (Idx + 3));

				// Find the value for it, either from the dictionary or the environment block
				string? Value;
				if (!Context.StringVariables.TryGetValue(Name, out Value))
				{
					if (!GlobalContext.StringVariables.TryGetValue(Name, out Value))
					{
						Idx = EndIdx + 1;
						continue;
					}
				}

				// Replace the variable, or skip past it
				Result = Result.Substring(0, Idx) + Value + Result.Substring(EndIdx + 1);
			}
			for (int Idx = Result.IndexOf("$E(", StringComparison.Ordinal); Idx != -1; Idx = Result.IndexOf("$E(", Idx, StringComparison.Ordinal))
			{
				// Find the end of the variable name
				int EndIdx = Result.IndexOf(')', Idx + 3);
				if (EndIdx == -1)
				{
					break;
				}

				// Extract the variable name from the string
				string Name = Result.Substring(Idx + 3, EndIdx - (Idx + 3));

				// Find the value for it, either from the dictionary or the environment block
				XElement? Value;
				if (!Context.ElementVariables.TryGetValue(Name, out Value))
				{
					if (!GlobalContext.ElementVariables.TryGetValue(Name, out Value))
					{
						Idx = EndIdx + 1;
						continue;
					}
				}

				// Replace the variable, or skip past it
				Result = Result.Substring(0, Idx) + Value + Result.Substring(EndIdx + 1);
			}
			return Result;
		}

		private static string TraceNodeString(UPLContext Context, XElement Node, bool ShowFilename = true)
		{
			XAttribute? LineAttrib = null;
			string Result = Node.Name.ToString();
			foreach (XAttribute Attrib in Node.Attributes())
			{
				string AttribStr = Attrib.ToString();
				if (AttribStr.StartsWith("__line=", StringComparison.Ordinal))
				{
					LineAttrib = Attrib;
				}
				else
				{
					Result += " " + AttribStr;
				}
			}

			if (ShowFilename)
			{
				Result += ", File: " + Context.Filename;
			}
			if (LineAttrib != null)
			{
				Result += ", Line: " + LineAttrib.Value;
			}

			return Result;
		}

		private static bool StringToBool(string? Input)
		{
			if (Input == null)
			{
				return false;
			}
			return !(
				Input.Equals("0", StringComparison.OrdinalIgnoreCase) ||
				Input.Equals("false", StringComparison.OrdinalIgnoreCase) ||
				Input.Equals("off", StringComparison.OrdinalIgnoreCase) ||
				Input.Equals("no", StringComparison.OrdinalIgnoreCase)
			);
		}

		private int StringToInt(UPLContext Context, XElement Node, string? Input)
		{
			if (!Int32.TryParse(Input, out int Result))
			{
				Logger.LogWarning("\nInvalid integer '{Input}' in '{Node}' (defaulting to 0)", Input, TraceNodeString(Context, Node));
			}
			return Result;
		}

		[return: NotNullIfNotNull("Fallback")]
		private string? GetAttribute(UPLContext Context, XElement Node, string AttributeName, bool bExpand = true, bool bRequired = true, string? Fallback = null)
		{
			XAttribute? Attribute = Node.Attribute(AttributeName);
			if (Attribute == null)
			{
				if (bRequired)
				{
					Logger.LogWarning("\nMissing attribute '{Attribute}' in '{Node}' (skipping instruction)", AttributeName, TraceNodeString(Context, Node));
				}
				return Fallback;
			}
			string Result = Attribute.Value;
			return bExpand ? ExpandVariables(Context, Result) : Result;
		}

		[return: NotNullIfNotNull("Fallback")]
		private string? GetAttributeWithNamespace(UPLContext Context, XElement Node, XNamespace Namespace, string AttributeName, bool bExpand = true, bool bRequired = true, string? Fallback = null)
		{
			XAttribute? Attribute = Node.Attribute(Namespace + AttributeName);
			if (Attribute == null)
			{
				if (bRequired)
				{
					Logger.LogWarning("\nMissing attribute '{Attribute}' in '{Node}' (skipping instruction)", AttributeName, TraceNodeString(Context, Node));
				}
				return Fallback;
			}
			string Result = Attribute.Value;
			return bExpand ? ExpandVariables(Context, Result) : Result;
		}

		private static Dictionary<string, ConfigCacheIni_UPL>? ConfigCache = null;

		private ConfigCacheIni_UPL GetConfigCacheIni_UPL(string baseIniName)
		{
			ConfigCache ??= new Dictionary<string, ConfigCacheIni_UPL>();
			if (!ConfigCache.TryGetValue(baseIniName, out ConfigCacheIni_UPL? config))
			{
				// note: use our own ConfigCacheIni since EngineConfiguration.cs only parses RequiredSections!
				config = ConfigCacheIni_UPL.CreateConfigCacheIni_UPL(TargetPlatform, baseIniName, DirectoryReference.FromFile(ProjectFile), Logger);
				ConfigCache.Add(baseIniName, config);
			}
			return config;
		}

		private static bool FilesAreDifferent(string SourceFilename, string DestFilename, ILogger Logger)
		{
			// source must exist
			FileInfo SourceInfo = new FileInfo(SourceFilename);
			if (!SourceInfo.Exists)
			{
				Logger.LogInformation("File {SourceFilename} does not exist", SourceFilename);
				return false;
			}

			// different if destination doesn't exist
			FileInfo DestInfo = new FileInfo(DestFilename);
			if (!DestInfo.Exists)
			{
				return true;
			}

			// file lengths differ?
			if (SourceInfo.Length != DestInfo.Length)
			{
				return true;
			}

			// validate timestamps
			TimeSpan Diff = DestInfo.LastWriteTimeUtc - SourceInfo.LastWriteTimeUtc;
			if (Diff.TotalSeconds < -1 || Diff.TotalSeconds > 1)
			{
				return true;
			}

			// could check actual bytes just to be sure, but good enough
			return false;
		}

		private static void CopyFileDirectory(string SourceDir, string DestDir, ILogger Logger, bool bForce = false)
		{
			if (!Directory.Exists(SourceDir))
			{
				return;
			}

			string[] Files = Directory.GetFiles(SourceDir, "*.*", SearchOption.AllDirectories);
			foreach (string Filename in Files)
			{
				// make the dst filename with the same structure as it was in SourceDir
				string DestFilename = Path.Combine(DestDir, Utils.MakePathRelativeTo(Filename, SourceDir));

				if (bForce || FilesAreDifferent(Filename, DestFilename, Logger))
				{
					if (File.Exists(DestFilename))
					{
						File.SetAttributes(DestFilename, FileAttributes.Normal);
						File.Delete(DestFilename);
					}

					// make the subdirectory if needed
					string DestSubdir = Path.GetDirectoryName(DestFilename)!;
					if (!Directory.Exists(DestSubdir))
					{
						Directory.CreateDirectory(DestSubdir);
					}

					File.Copy(Filename, DestFilename);

					// remove any read only flags and keep timestamp
					FileInfo DestFileInfo = new FileInfo(DestFilename);
					DestFileInfo.Attributes &= ~FileAttributes.ReadOnly;
					File.SetLastWriteTimeUtc(DestFilename, File.GetLastWriteTimeUtc(Filename));
				}
			}
		}

		private static void DeleteFiles(string Filespec, bool Recursive, ILogger Logger)
		{
			string BaseDir = Path.GetDirectoryName(Filespec)!;
			string Mask = Path.GetFileName(Filespec);

			if (!Directory.Exists(BaseDir))
			{
				return;
			}

			string[] Files = Directory.GetFiles(BaseDir, Mask, Recursive ? SearchOption.AllDirectories : SearchOption.TopDirectoryOnly);
			foreach (string Filename in Files)
			{
				File.SetAttributes(Filename, FileAttributes.Normal);
				File.Delete(Filename);
				Logger.LogInformation("\nDeleted file {Filename}", Filename);
			}
		}

		XNamespace? TrimNamespaceAliasFromName(ref string Name)
		{
			int Index = Name.IndexOf(':', StringComparison.Ordinal);
			if (Index >= 0)
			{
				XNamespace? Namespace;
				string NamespaceAlias = Name.Substring(0, Index);
				if (!XMLNameSpaceAliases.TryGetValue(NamespaceAlias, out Namespace))
				{
					Namespace = XMLDefaultNameSpace;
				}
				Name = Name.Substring(Index + 1);
				return Namespace;
			}
			else
			{
				return null;
			}
		}
		private void AddAttribute(XElement Element, string Name, string Value)
		{
			XNamespace? XMLNameSpace = TrimNamespaceAliasFromName(ref Name);
			XAttribute? Attribute = Element.Attribute(XMLNameSpace != null? XMLNameSpace + Name : Name);

			if (Attribute != null)
			{
				Attribute.SetValue(Value);
			}
			else
			{
				Element.Add(new XAttribute(XMLNameSpace != null ? XMLNameSpace + Name : Name, Value));
			}
		}

		private void RemoveAttribute(XElement Element, string Name)
		{
			XNamespace? XMLNameSpace = TrimNamespaceAliasFromName(ref Name);
			XAttribute? Attribute = Element.Attribute(XMLNameSpace != null ? XMLNameSpace + Name : Name);

			Attribute?.Remove();
		}

		private static void AddElements(XElement Target, XElement Source)
		{
			if (Source.HasElements)
			{
				foreach (XElement Index in Source.Elements())
				{
					//					if (Target.Element(Index.Name) == null)
					{
						Target.Add(Index);
					}
				}
			}
		}

		public string ProcessPluginNode(string Architecture, string NodeName, string Input)
		{
			return ProcessPluginNode(Architecture, NodeName, Input, ref XMLDummy);
		}

		public string ProcessPluginNode(string Architecture, string NodeName, string Input, ref XDocument XMLWork)
		{
			// add all instructions to execution list
			Stack<XElement> ExecutionStack = new Stack<XElement>();
			XElement? StartNode = XDoc.Root!.Element(NodeName);
			if (StartNode != null)
			{
				foreach (XElement Instruction in StartNode.Elements().Reverse())
				{
					ExecutionStack.Push(Instruction);
				}
			}

			if (ExecutionStack.Count == 0)
			{
				return Input;
			}

			Stack<UPLContext> ContextStack = new Stack<UPLContext>();
			UPLContext CurrentContext = GlobalContext;

			// update Output in global context
			GlobalContext.StringVariables["Output"] = Input;

			Stack<XElement> ElementStack = new Stack<XElement>();
			XElement CurrentElement = XMLWork.Elements().First();

			// run the instructions
			while (ExecutionStack.Count > 0)
			{
				XElement Node = ExecutionStack.Pop();
				if (bGlobalTrace || CurrentContext.bTrace)
				{
					Logger.LogInformation("Execute: '{Node}'", TraceNodeString(CurrentContext, Node, bGlobalTraceFilename || CurrentContext.bTraceFilename));
				}
				switch (Node.Name.ToString())
				{
					case "trace":
						{
							string? TraceFilename = GetAttribute(CurrentContext, Node, "filename");
							if (TraceFilename != null)
							{
								CurrentContext.bTraceFilename = StringToBool(TraceFilename);
							}
							string? Enable = GetAttribute(CurrentContext, Node, "enable");
							if (Enable != null)
							{
								CurrentContext.bTrace = StringToBool(Enable);
								if (!bGlobalTrace && CurrentContext.bTrace)
								{
									Logger.LogInformation("Context: '{Context}' using Architecture='{Arch}', NodeName='{NodeName}', Input='{Input}'", CurrentContext.StringVariables["PluginDir"], Architecture, NodeName, Input);
								}
							}
						}
						break;

					case "dumpvars":
						{
							if (!bGlobalTrace && !CurrentContext.bTrace)
							{
								Logger.LogInformation("Context: '{Context}' using Architecture='{Arch}', NodeName='{NodeName}', Input='{Input}'", CurrentContext.StringVariables["PluginDir"], Architecture, NodeName, Input);
							}
							Logger.LogInformation("Variables:\n{Variables}", DumpContext(CurrentContext));
						}
						break;

					case "Context":
						{
							ContextStack.Push(CurrentContext);
							string? index = GetAttribute(CurrentContext, Node, "index");
							CurrentContext = Contexts[Architecture + "_" + index];
							ExecutionStack.Push(new XElement("PopContext"));
							foreach (XElement instruction in Node.Elements().Reverse())
							{
								ExecutionStack.Push(instruction);
							}

							if (bGlobalTrace || CurrentContext.bTrace)
							{
								Logger.LogInformation("Context: '{Context}' using Architecture='{Arch}', NodeName='{NodeName}', Input='{Input}'", CurrentContext.StringVariables["PluginDir"], Architecture, NodeName, Input);
							}
						}
						break;

					case "PopContext":
						{
							CurrentContext = ContextStack.Pop();
						}
						break;

					case "PopElement":
						{
							CurrentElement = ElementStack.Pop();
						}
						break;

					case "isArch":
						{
							string? arch = GetAttribute(CurrentContext, Node, "arch");
							if (arch != null && arch.Equals(Architecture, StringComparison.Ordinal))
							{
								foreach (XElement instruction in Node.Elements().Reverse())
								{
									ExecutionStack.Push(instruction);
								}
							}
						}
						break;

					case "isDistribution":
						{
							if (GetCondition(CurrentContext, Node, "Distribution", out bool Result))
							{
								if (Result)
								{
									foreach (XElement Instruction in Node.Elements().Reverse())
									{
										ExecutionStack.Push(Instruction);
									}
								}
							}
						}
						break;

					case "if":
						{
							bool Result;
							if (GetCondition(CurrentContext, Node, GetAttribute(CurrentContext, Node, "condition"), out Result))
							{
								XElement? ResultNode = Node.Element(Result ? "true" : "false");
								if (ResultNode != null)
								{
									foreach (XElement Instruction in ResultNode.Elements().Reverse())
									{
										ExecutionStack.Push(Instruction);
									}
								}
							}
						}
						break;

					case "while":
						{
							bool Result;
							if (GetCondition(CurrentContext, Node, GetAttribute(CurrentContext, Node, "condition"), out Result))
							{
								if (Result)
								{
									IEnumerable<XElement> ResultNode = Node.Elements();
									if (ResultNode != null)
									{
										ExecutionStack.Push(Node);
										foreach (XElement Instruction in ResultNode.Reverse())
										{
											ExecutionStack.Push(Instruction);
										}
									}
								}
							}
						}
						break;

					case "return":
						{
							while (ExecutionStack.Count > 0)
							{
								ExecutionStack.Pop();
							}
						}
						break;

					case "break":
						{
							// remove up to while (or acts like a return if outside by removing everything)
							while (ExecutionStack.Count > 0)
							{
								Node = ExecutionStack.Pop();
								if (Node.Name.ToString().Equals("while", StringComparison.Ordinal))
								{
									break;
								}
							}
						}
						break;

					case "continue":
						{
							// remove up to while (or acts like a return if outside by removing everything)
							while (ExecutionStack.Count > 0)
							{
								Node = ExecutionStack.Pop();
								if (Node.Name.ToString().Equals("while", StringComparison.Ordinal))
								{
									ExecutionStack.Push(Node);
									break;
								}
							}
						}
						break;

					case "log":
						{
							string? Text = GetAttribute(CurrentContext, Node, "text");
							if (Text != null)
							{
								Logger.LogInformation("{Message}", Text);
							}
						}
						break;

					case "loopElements":
						{
							string? Tag = GetAttribute(CurrentContext, Node, "tag");
							ElementStack.Push(CurrentElement);
							ExecutionStack.Push(new XElement("PopElement"));
							IEnumerable<XElement> WorkList = (Tag == "$") ? CurrentElement.Elements().Reverse() : CurrentElement.Descendants(Tag).Reverse();
							foreach (XElement WorkNode in WorkList)
							{
								foreach (XElement Instruction in Node.Elements().Reverse())
								{
									ExecutionStack.Push(Instruction);
								}
								ElementStack.Push(WorkNode);
								ExecutionStack.Push(new XElement("PopElement"));
							}
						}
						break;

					case "addAttribute":
						{
							string? Tag = GetAttribute(CurrentContext, Node, "tag");
							string? Name = GetAttribute(CurrentContext, Node, "name");
							string? Value = GetAttribute(CurrentContext, Node, "value");
							if (Tag != null && Name != null && Value != null)
							{
								if (Tag.StartsWith('$'))
								{
									XElement? Target = CurrentElement;
									if (Tag.Length > 1)
									{
										if (!CurrentContext.ElementVariables.TryGetValue(Tag.Substring(1), out Target))
										{
											if (!GlobalContext.ElementVariables.TryGetValue(Tag.Substring(1), out Target))
											{
												Logger.LogWarning("Missing element variable '{Tag}' in '{Node}' (skipping instruction)", Tag, TraceNodeString(CurrentContext, Node));
												continue;
											}
										}
									}
									AddAttribute(Target!, Name, Value);
								}
								else
								{
									if (CurrentElement.Name.ToString().Equals(Tag, StringComparison.Ordinal))
									{
										AddAttribute(CurrentElement, Name, Value);
									}
									foreach (XElement WorkNode in CurrentElement.Descendants(Tag))
									{
										AddAttribute(WorkNode, Name, Value);
									}
								}
							}
						}
						break;

					case "removeAttribute":
						{
							string? Tag = GetAttribute(CurrentContext, Node, "tag");
							string? Name = GetAttribute(CurrentContext, Node, "name");
							if (Tag != null && Name != null)
							{
								if (Tag.StartsWith('$'))
								{
									XElement? Target = CurrentElement;
									if (Tag.Length > 1)
									{
										if (!CurrentContext.ElementVariables.TryGetValue(Tag.Substring(1), out Target))
										{
											if (!GlobalContext.ElementVariables.TryGetValue(Tag.Substring(1), out Target))
											{
												Logger.LogInformation("\nMissing element variable '{Tag}' in '{Node}' (skipping instruction)", Tag, TraceNodeString(CurrentContext, Node));
												continue;
											}
										}
									}
									RemoveAttribute(Target, Name);
								}
								else
								{
									if (CurrentElement.Name.ToString().Equals(Tag, StringComparison.Ordinal))
									{
										RemoveAttribute(CurrentElement, Name);
									}
									foreach (XElement WorkNode in CurrentElement.Descendants(Tag))
									{
										RemoveAttribute(WorkNode, Name);
									}
								}
							}
						}
						break;

					case "addPermission":
						{
							string? Name = GetAttributeWithNamespace(CurrentContext, Node, XMLDefaultNameSpace, "name");
							if (Name != null)
							{
								// make sure it isn't already added
								bool bFound = false;
								foreach (XElement Element in XMLWork.Descendants("uses-permission"))
								{
									XAttribute? Attribute = Element.Attribute(XMLDefaultNameSpace + "name");
									if (Attribute != null)
									{
										if (Attribute.Value == Name)
										{
											bFound = true;
											break;
										}
									}
								}

								// add it if not found
								if (!bFound)
								{
									// Get the attributes and apply any variable expansion needed
									List<XAttribute> AttributeList = Node.Attributes().ToList();
									foreach (XAttribute Attribute in AttributeList)
									{
										string NewValue = ExpandVariables(CurrentContext, Attribute.Value);
										Attribute.SetValue(NewValue);
									}

									XMLWork.Element("manifest")?.Add(new XElement("uses-permission", AttributeList));
								}
							}
						}
						break;

					case "removePermission":
						{
							string? Name = GetAttributeWithNamespace(CurrentContext, Node, XMLDefaultNameSpace, "name");
							if (Name != null)
							{
								foreach (XElement Element in XMLWork.Descendants("uses-permission"))
								{
									XAttribute? Attribute = Element.Attribute(XMLDefaultNameSpace + "name");
									if (Attribute != null)
									{
										if (Attribute.Value == Name)
										{
											Element.Remove();
											break;
										}
									}
								}
							}
						}
						break;

					case "addFeature":
						{
							string? Name = GetAttributeWithNamespace(CurrentContext, Node, XMLDefaultNameSpace, "name");
							if (Name != null)
							{
								// make sure it isn't already added
								bool bFound = false;
								foreach (XElement Element in XMLWork.Descendants("uses-feature"))
								{
									XAttribute? Attribute = Element.Attribute(XMLDefaultNameSpace + "name");
									if (Attribute != null)
									{
										if (Attribute.Value == Name)
										{
											bFound = true;
											break;
										}
									}
								}

								// add it if not found
								if (!bFound)
								{
									// Get the attributes and apply any variable expansion needed
									List<XAttribute> AttributeList = Node.Attributes().ToList();
									foreach (XAttribute Attribute in AttributeList)
									{
										string NewValue = ExpandVariables(CurrentContext, Attribute.Value);
										Attribute.SetValue(NewValue);
									}

									XMLWork.Element("manifest")?.Add(new XElement("uses-feature", AttributeList));
								}
							}
						}
						break;

					case "removeFeature":
						{
							string? Name = GetAttributeWithNamespace(CurrentContext, Node, XMLDefaultNameSpace, "name");
							if (Name != null)
							{
								foreach (XElement Element in XMLWork.Descendants("uses-feature"))
								{
									XAttribute? Attribute = Element.Attribute(XMLDefaultNameSpace + "name");
									if (Attribute != null)
									{
										if (Attribute.Value == Name)
										{
											Element.Remove();
											break;
										}
									}
								}
							}
						}
						break;

					case "addLibrary":
						{
							string? Name = GetAttributeWithNamespace(CurrentContext, Node, XMLDefaultNameSpace, "name");
							if (Name != null)
							{
								// make sure it isn't already added
								bool bFound = false;
								foreach (XElement Element in XMLWork.Descendants("uses-library"))
								{
									XAttribute? Attribute = Element.Attribute(XMLDefaultNameSpace + "name");
									if (Attribute != null)
									{
										if (Attribute.Value == Name)
										{
											bFound = true;
											break;
										}
									}
								}

								// add it if not found
								if (!bFound)
								{
									// Get the attributes and apply any variable expansion needed
									List<XAttribute> AttributeList = Node.Attributes().ToList();
									foreach (XAttribute Attribute in AttributeList)
									{
										string NewValue = ExpandVariables(CurrentContext, Attribute.Value);
										Attribute.SetValue(NewValue);
									}

									XMLWork.Element("manifest")?.Element("application")?.Add(new XElement("uses-library", AttributeList));
								}
							}
						}
						break;

					case "removeLibrary":
						{
							string? Name = GetAttributeWithNamespace(CurrentContext, Node, XMLDefaultNameSpace, "name");
							if (Name != null)
							{
								foreach (XElement Element in XMLWork.Descendants("uses-library"))
								{
									XAttribute? Attribute = Element.Attribute(XMLDefaultNameSpace + "name");
									if (Attribute != null)
									{
										if (Attribute.Value == Name)
										{
											Element.Remove();
											break;
										}
									}
								}
							}
						}
						break;

					case "removeElement":
						{
							string? Tag = GetAttribute(CurrentContext, Node, "tag");
							bool bOnce = StringToBool(GetAttribute(CurrentContext, Node, "once", true, false));
							if (Tag != null)
							{
								if (Tag == "$")
								{
									XElement Parent = CurrentElement.Parent!;
									CurrentElement.Remove();
									CurrentElement = Parent;
								}
								else
								{
									// use a list since Remove() may modify it
									foreach (XElement Element in XMLWork.Descendants(Tag).ToList())
									{
										Element.Remove();
										if (bOnce)
										{
											break;
										}
									}
								}
							}
						}
						break;

					case "addElement":
						{
							string? Tag = GetAttribute(CurrentContext, Node, "tag");
							string? Name = GetAttribute(CurrentContext, Node, "name");
							bool bOnce = StringToBool(GetAttribute(CurrentContext, Node, "once", true, false));
							if (Tag != null && Name != null)
							{
								XElement? Element;
								if (!CurrentContext.ElementVariables.TryGetValue(Name, out Element))
								{
									if (!GlobalContext.ElementVariables.TryGetValue(Name, out Element))
									{
										Logger.LogWarning("Missing element variable '{Name}' in '{Node}' (skipping instruction)", Name, TraceNodeString(CurrentContext, Node));
										continue;
									}
								}
								if (Tag.StartsWith('$'))
								{
									XElement? Target = CurrentElement;
									if (Tag.Length > 1)
									{
										if (!CurrentContext.ElementVariables.TryGetValue(Tag.Substring(1), out Target))
										{
											if (!GlobalContext.ElementVariables.TryGetValue(Tag.Substring(1), out Target))
											{
												Logger.LogWarning("Missing element variable '{Name}' in '{Node}' (skipping instruction)", Tag, TraceNodeString(CurrentContext, Node));
												continue;
											}
										}
									}
									Target.Add(new XElement(Element));
								}
								else
								{
									if (CurrentElement.Name.ToString().Equals(Tag, StringComparison.Ordinal))
									{
										CurrentElement.Add(new XElement(Element));
									}

									// make sure we don't recurse forever if Tag is in Element
									List<XElement> AddSet = new List<XElement>();
									foreach (XElement WorkNode in CurrentElement.Descendants(Tag))
									{
										AddSet.Add(WorkNode);
										if (bOnce)
										{
											break;
										}
									}
									foreach (XElement WorkNode in AddSet)
									{
										WorkNode.Add(new XElement(Element));
									}
								}
							}
						}
						break;

					case "addElements":
						{
							string? Tag = GetAttribute(CurrentContext, Node, "tag");
							bool bOnce = StringToBool(GetAttribute(CurrentContext, Node, "once", true, false));
							if (Tag != null)
							{
								if (Tag.StartsWith('$'))
								{
									XElement? Target = CurrentElement;
									if (Tag.Length > 1)
									{
										if (!CurrentContext.ElementVariables.TryGetValue(Tag.Substring(1), out Target))
										{
											if (!GlobalContext.ElementVariables.TryGetValue(Tag.Substring(1), out Target))
											{
												Logger.LogWarning("Missing element variable '{Tag}' in '{Node}' (skipping instruction)", Tag, TraceNodeString(CurrentContext, Node));
												continue;
											}
										}
									}
									AddElements(Target, Node);
								}
								else
								{
									if (CurrentElement.Name.ToString().Equals(Tag, StringComparison.Ordinal))
									{
										AddElements(CurrentElement, Node);
									}

									// make sure we don't recurse forever if Tag is in Node
									List<XElement> AddSet = new List<XElement>();
									foreach (XElement WorkNode in CurrentElement.Descendants(Tag))
									{
										AddSet.Add(WorkNode);
										if (bOnce)
										{
											break;
										}
									}
									foreach (XElement WorkNode in AddSet)
									{
										AddElements(WorkNode, Node);
									}
								}
							}
						}
						break;

					case "insert":
						{
							if (Node.HasElements)
							{
								foreach (XElement Element in Node.Elements())
								{
									string Value = Element.ToString().Replace(" " + XMLRootDefinition + " ", "", StringComparison.Ordinal);
									GlobalContext.StringVariables["Output"] += Value + "\n";
								}
							}
							else
							{
								string Value = Node.Value.ToString();

								// trim trailing tabs
								int Index = Value.Length;
								while (Index > 0 && Value[Index - 1] == '\t')
								{
									Index--;
								}
								if (Index < Value.Length)
								{
									Value = Value.Substring(0, Index);
								}

								// trim leading newlines
								Index = 0;
								while (Index < Value.Length && Value[Index] == '\n')
								{
									Index++;
								}

								if (Index < Value.Length)
								{
									GlobalContext.StringVariables["Output"] += Value.Substring(Index);
								}
							}
						}
						break;

					case "insertNewline":
						GlobalContext.StringVariables["Output"] += "\n";
						break;

					case "insertValue":
						{
							string? Value = GetAttribute(CurrentContext, Node, "value");
							if (Value != null)
							{
								GlobalContext.StringVariables["Output"] += Value;
							}
						}
						break;

					case "replace":
						{
							string? Find = GetAttribute(CurrentContext, Node, "find");
							string? With = GetAttribute(CurrentContext, Node, "with");
							if (Find != null && With != null)
							{
								GlobalContext.StringVariables["Output"] = GlobalContext.StringVariables["Output"].Replace(Find, With, StringComparison.Ordinal);
							}
						}
						break;

					case "copyFile":
						{
							string? Src = GetAttribute(CurrentContext, Node, "src");
							string? Dst = GetAttribute(CurrentContext, Node, "dst");
							bool bForce = StringToBool(GetAttribute(CurrentContext, Node, "force", true, false, "true"));
							if (Src != null && Dst != null)
							{
								if (File.Exists(Src))
								{
									// check to see if newer than last time we copied
									if (bForce || FilesAreDifferent(Src, Dst, Logger))
									{
										if (File.Exists(Dst))
										{
											File.SetAttributes(Dst, FileAttributes.Normal);
											File.Delete(Dst);
										}
										Directory.CreateDirectory(Path.GetDirectoryName(Dst)!);
										File.Copy(Src, Dst, true);
										Logger.LogInformation("File {Src} copied to {Dst}", Src, Dst);

										// remove any read only flags and keep timestamp
										FileInfo DestFileInfo = new FileInfo(Dst);
										DestFileInfo.Attributes &= ~FileAttributes.ReadOnly;
										File.SetLastWriteTimeUtc(Dst, File.GetLastWriteTimeUtc(Src));
									}
								}
								else
								{
									Logger.LogInformation("File {Src} does not exist, not copied!", Src);
								}
							}
						}
						break;

					case "copyDir":
						{
							string? Src = GetAttribute(CurrentContext, Node, "src");
							string? Dst = GetAttribute(CurrentContext, Node, "dst");
							bool bForce = StringToBool(GetAttribute(CurrentContext, Node, "force", true, false, "true"));
							if (Src != null && Dst != null)
							{
								CopyFileDirectory(Src, Dst, Logger, bForce);
								Logger.LogInformation("\nDirectory {Src} copied to {Dst} ({Force})", Src, Dst, bForce);
							}
						}
						break;

					case "deleteFiles":
						{
							string? Filespec = GetAttribute(CurrentContext, Node, "filespec");
							bool bRecursive = StringToBool(GetAttribute(CurrentContext, Node, "recursive", true, false, "false"));
							if (Filespec != null)
							{
								if (Filespec.Contains(':', StringComparison.Ordinal) || Filespec.Contains("..", StringComparison.Ordinal))
								{
									Logger.LogInformation("\nFilespec {FileSpec} not allowed; ignored.", Filespec);
								}
								else
								{
									// force relative to BuildDir (and only from global context so someone doesn't try to be clever)
									DeleteFiles(Path.Combine(GlobalContext.StringVariables["BuildDir"], Filespec), bRecursive, Logger);
								}
							}
						}
						break;

					case "loadLibrary":
						{
							string? Name = GetAttribute(CurrentContext, Node, "name");
							string? FailMsg = GetAttribute(CurrentContext, Node, "failmsg", true, false);
							if (Name != null)
							{
								string Work = "\t\ttry\n" +
											"\t\t{\n" +
											"\t\t\tSystem.loadLibrary(\"" + Name + "\");\n" +
											"\t\t}\n" +
											"\t\tcatch (java.lang.UnsatisfiedLinkError e)\n" +
											"\t\t{\n";
								if (FailMsg != null)
								{
									Work += "\t\t\tLog.debug(e.toString());\n";
									Work += "\t\t\tLog.debug(\"" + FailMsg + "\");\n";
								}
								GlobalContext.StringVariables["Output"] += Work + "\t\t}\n";
							}
						}
						break;

					case "setBool":
						{
							string? Result = GetAttribute(CurrentContext, Node, "result");
							string Value = GetAttribute(CurrentContext, Node, "value", true, false, "false");
							if (Result != null)
							{
								CurrentContext.BoolVariables[Result] = StringToBool(Value);
							}
						}
						break;

					case "setBoolEnvVarDefined":
						{
							string? Result = GetAttribute(CurrentContext, Node, "result");
							string? Value = GetAttribute(CurrentContext, Node, "value");
							if (Result != null)
							{
								CurrentContext.BoolVariables[Result] = (Value != null && Environment.ExpandEnvironmentVariables(Value).Length > 0);
							}
						}
						break;

					case "setBoolFrom":
						{
							string? Result = GetAttribute(CurrentContext, Node, "result");
							string Value = GetAttribute(CurrentContext, Node, "value", true, false, "false");
							if (Result != null)
							{
								Value = ExpandVariables(CurrentContext, "$B(" + Value + ")");
								CurrentContext.BoolVariables[Result] = StringToBool(Value);
							}
						}
						break;

					case "setBoolFromProperty":
						{
							string? Result = GetAttribute(CurrentContext, Node, "result");
							string? Ini = GetAttribute(CurrentContext, Node, "ini");
							string? Section = GetAttribute(CurrentContext, Node, "section");
							string? Property = GetAttribute(CurrentContext, Node, "property");
							string DefaultVal = GetAttribute(CurrentContext, Node, "default", true, false, "false");
							if (Result != null && Ini != null && Section != null && Property != null)
							{
								bool Value = StringToBool(DefaultVal);

								ConfigCacheIni_UPL ConfigIni = GetConfigCacheIni_UPL(Ini);
								if (ConfigIni != null)
								{
									if (!ConfigIni.GetBool(Section, Property, out Value))
									{
										Value = StringToBool(DefaultVal);
									}
								}
								CurrentContext.BoolVariables[Result] = Value;
							}
						}
						break;

					case "setBoolFromPropertyContains":
						{
							string? Result = GetAttribute(CurrentContext, Node, "result");
							string? Ini = GetAttribute(CurrentContext, Node, "ini");
							string? Section = GetAttribute(CurrentContext, Node, "section");
							string? Property = GetAttribute(CurrentContext, Node, "property");
							string DefaultVal = GetAttribute(CurrentContext, Node, "default", true, false, "false");
							string Contains = GetAttribute(CurrentContext, Node, "contains", true, true, "");
							if (Result != null && Ini != null && Section != null && Property != null)
							{
								bool Value = StringToBool(DefaultVal);

								ConfigCacheIni_UPL ConfigIni = GetConfigCacheIni_UPL(Ini);
								if (ConfigIni != null)
								{
									List<string>? StringList;
									if (ConfigIni.GetArray(Section, Property, out StringList))
									{
										Value = false;
										foreach (string Entry in StringList)
										{
											if (Entry.Equals(Contains, StringComparison.Ordinal))
											{
												Value = true;
												break;
											}
										}
									}
								}
								CurrentContext.BoolVariables[Result] = Value;
							}
						}
						break;

					case "setBoolContains":
						{
							string? Result = GetAttribute(CurrentContext, Node, "result");
							string Source = GetAttribute(CurrentContext, Node, "source", true, false, "");
							string Find = GetAttribute(CurrentContext, Node, "find", true, false, "");
							if (Result != null)
							{
								CurrentContext.BoolVariables[Result] = Source.Contains(Find, StringComparison.Ordinal);
							}
						}
						break;

					case "setBoolStartsWith":
						{
							string? Result = GetAttribute(CurrentContext, Node, "result");
							string Source = GetAttribute(CurrentContext, Node, "source", true, false, "");
							string Find = GetAttribute(CurrentContext, Node, "find", true, false, "");
							if (Result != null)
							{
								CurrentContext.BoolVariables[Result] = Source.StartsWith(Find, StringComparison.Ordinal);
							}
						}
						break;

					case "setBoolEndsWith":
						{
							string? Result = GetAttribute(CurrentContext, Node, "result");
							string Source = GetAttribute(CurrentContext, Node, "source", true, false, "");
							string Find = GetAttribute(CurrentContext, Node, "find", true, false, "");
							if (Result != null)
							{
								CurrentContext.BoolVariables[Result] = Source.EndsWith(Find, StringComparison.Ordinal);
							}
						}
						break;

					case "setBoolNot":
						{
							string? Result = GetAttribute(CurrentContext, Node, "result");
							string Source = GetAttribute(CurrentContext, Node, "source", true, false, "false");
							if (Result != null)
							{
								CurrentContext.BoolVariables[Result] = !StringToBool(Source);
							}
						}
						break;

					case "setBoolAnd":
						{
							string? Result = GetAttribute(CurrentContext, Node, "result");
							string Arg1 = GetAttribute(CurrentContext, Node, "arg1", true, false, "false");
							string Arg2 = GetAttribute(CurrentContext, Node, "arg2", true, false, "false");
							if (Result != null)
							{
								CurrentContext.BoolVariables[Result] = StringToBool(Arg1) && StringToBool(Arg2);
							}
						}
						break;

					case "setBoolOr":
						{
							string? Result = GetAttribute(CurrentContext, Node, "result");
							string Arg1 = GetAttribute(CurrentContext, Node, "arg1", true, false, "false");
							string Arg2 = GetAttribute(CurrentContext, Node, "arg2", true, false, "false");
							if (Result != null)
							{
								CurrentContext.BoolVariables[Result] = StringToBool(Arg1) || StringToBool(Arg2);
							}
						}
						break;

					case "setBoolIsEqual":
						{
							string? Result = GetAttribute(CurrentContext, Node, "result");
							string Arg1 = GetAttribute(CurrentContext, Node, "arg1", true, false, "");
							string Arg2 = GetAttribute(CurrentContext, Node, "arg2", true, false, "");
							if (Result != null)
							{
								CurrentContext.BoolVariables[Result] = Arg1.Equals(Arg2, StringComparison.Ordinal);
							}
						}
						break;

					case "setBoolIsLess":
						{
							string? Result = GetAttribute(CurrentContext, Node, "result");
							string? Arg1 = GetAttribute(CurrentContext, Node, "arg1", true, false);
							string? Arg2 = GetAttribute(CurrentContext, Node, "arg2", true, false);
							if (Result != null)
							{
								CurrentContext.BoolVariables[Result] = (StringToInt(CurrentContext, Node, Arg1) < StringToInt(CurrentContext, Node, Arg2));
							}
						}
						break;

					case "setBoolIsLessEqual":
						{
							string? Result = GetAttribute(CurrentContext, Node, "result");
							string? Arg1 = GetAttribute(CurrentContext, Node, "arg1", true, false);
							string? Arg2 = GetAttribute(CurrentContext, Node, "arg2", true, false);
							if (Result != null)
							{
								CurrentContext.BoolVariables[Result] = (StringToInt(CurrentContext, Node, Arg1) <= StringToInt(CurrentContext, Node, Arg2));
							}
						}
						break;

					case "setBoolIsGreater":
						{
							string? Result = GetAttribute(CurrentContext, Node, "result");
							string? Arg1 = GetAttribute(CurrentContext, Node, "arg1", true, false);
							string? Arg2 = GetAttribute(CurrentContext, Node, "arg2", true, false);
							if (Result != null)
							{
								CurrentContext.BoolVariables[Result] = (StringToInt(CurrentContext, Node, Arg1) > StringToInt(CurrentContext, Node, Arg2));
							}
						}
						break;

					case "setBoolIsGreaterEqual":
						{
							string? Result = GetAttribute(CurrentContext, Node, "result");
							string? Arg1 = GetAttribute(CurrentContext, Node, "arg1", true, false);
							string? Arg2 = GetAttribute(CurrentContext, Node, "arg2", true, false);
							if (Result != null)
							{
								CurrentContext.BoolVariables[Result] = (StringToInt(CurrentContext, Node, Arg1) >= StringToInt(CurrentContext, Node, Arg2));
							}
						}
						break;

					case "setBoolFileExists":
						{
							string? Result = GetAttribute(CurrentContext, Node, "result");
							string? FilePath = GetAttribute(CurrentContext, Node, "file");
							if (Result != null && FilePath != null)
							{
								CurrentContext.BoolVariables[Result] = File.Exists(FilePath);
							}
						}
						break;

					case "setInt":
						{
							string? Result = GetAttribute(CurrentContext, Node, "result");
							string Value = GetAttribute(CurrentContext, Node, "value", true, false, "0");
							if (Result != null)
							{
								CurrentContext.IntVariables[Result] = StringToInt(CurrentContext, Node, Value);
							}
						}
						break;

					case "setIntFrom":
						{
							string? Result = GetAttribute(CurrentContext, Node, "result");
							string Value = GetAttribute(CurrentContext, Node, "value", true, false, "0");
							if (Result != null)
							{
								Value = ExpandVariables(CurrentContext, "$I(" + Value + ")");
								CurrentContext.IntVariables[Result] = StringToInt(CurrentContext, Node, Value);
							}
						}
						break;

					case "setIntFromProperty":
						{
							string? Result = GetAttribute(CurrentContext, Node, "result");
							string? Ini = GetAttribute(CurrentContext, Node, "ini");
							string? Section = GetAttribute(CurrentContext, Node, "section");
							string? Property = GetAttribute(CurrentContext, Node, "property");
							string DefaultVal = GetAttribute(CurrentContext, Node, "default", true, false, "0");
							if (Result != null && Ini != null && Section != null && Property != null)
							{
								int DefaultInt = StringToInt(CurrentContext, Node, DefaultVal);
								int Value = DefaultInt;

								ConfigCacheIni_UPL ConfigIni = GetConfigCacheIni_UPL(Ini);
								if (ConfigIni != null)
								{
									if (!ConfigIni.GetInt32(Section, Property, out Value))
									{
										Value = DefaultInt;
									}
								}
								CurrentContext.IntVariables[Result] = Value;
							}
						}
						break;

					case "setIntFromPropertyArrayNum":
						{
							string? Result = GetAttribute(CurrentContext, Node, "result");
							string? Ini = GetAttribute(CurrentContext, Node, "ini");
							string? Section = GetAttribute(CurrentContext, Node, "section");
							string? Property = GetAttribute(CurrentContext, Node, "property");
							if (Result != null && Ini != null && Section != null && Property != null)
							{
								int Value = 0;

								ConfigCacheIni_UPL ConfigIni = GetConfigCacheIni_UPL(Ini);
								if (ConfigIni != null)
								{
									List<string>? StringList;
									if (ConfigIni.GetArray(Section, Property, out StringList))
									{
										Value = StringList.Count;
									}
								}
								CurrentContext.IntVariables[Result] = Value;
							}
						}
						break;

					case "setIntAdd":
						{
							string? Result = GetAttribute(CurrentContext, Node, "result");
							string Arg1 = GetAttribute(CurrentContext, Node, "arg1", true, false, "0");
							string Arg2 = GetAttribute(CurrentContext, Node, "arg2", true, false, "0");
							if (Result != null)
							{
								CurrentContext.IntVariables[Result] = StringToInt(CurrentContext, Node, Arg1) + StringToInt(CurrentContext, Node, Arg2);
							}
						}
						break;

					case "setIntSubtract":
						{
							string? Result = GetAttribute(CurrentContext, Node, "result");
							string Arg1 = GetAttribute(CurrentContext, Node, "arg1", true, false, "0");
							string Arg2 = GetAttribute(CurrentContext, Node, "arg2", true, false, "0");
							if (Result != null)
							{
								CurrentContext.IntVariables[Result] = StringToInt(CurrentContext, Node, Arg1) - StringToInt(CurrentContext, Node, Arg2);
							}
						}
						break;

					case "setIntMultiply":
						{
							string? Result = GetAttribute(CurrentContext, Node, "result");
							string Arg1 = GetAttribute(CurrentContext, Node, "arg1", true, false, "1");
							string Arg2 = GetAttribute(CurrentContext, Node, "arg2", true, false, "1");
							if (Result != null)
							{
								CurrentContext.IntVariables[Result] = StringToInt(CurrentContext, Node, Arg1) * StringToInt(CurrentContext, Node, Arg2);
							}
						}
						break;

					case "setIntDivide":
						{
							string? Result = GetAttribute(CurrentContext, Node, "result");
							string Arg1 = GetAttribute(CurrentContext, Node, "arg1", true, false, "1");
							string Arg2 = GetAttribute(CurrentContext, Node, "arg2", true, false, "1");
							if (Result != null)
							{
								int Denominator = StringToInt(CurrentContext, Node, Arg2);
								if (Denominator == 0)
								{
									CurrentContext.IntVariables[Result] = StringToInt(CurrentContext, Node, Arg1);
								}
								else
								{
									CurrentContext.IntVariables[Result] = StringToInt(CurrentContext, Node, Arg1) / Denominator;
								}
							}
						}
						break;

					case "setIntLength":
						{
							string? Result = GetAttribute(CurrentContext, Node, "result");
							string Source = GetAttribute(CurrentContext, Node, "source", true, false, "");
							if (Result != null)
							{
								CurrentContext.IntVariables[Result] = Source.Length;
							}
						}
						break;

					case "setIntFindString":
						{
							string? Result = GetAttribute(CurrentContext, Node, "result");
							string Source = GetAttribute(CurrentContext, Node, "source", true, false, "");
							string Find = GetAttribute(CurrentContext, Node, "find", true, false, "");
							if (Result != null)
							{
								CurrentContext.IntVariables[Result] = Source.IndexOf(Find, StringComparison.Ordinal);
							}
						}
						break;

					case "setString":
						{
							string? Result = GetAttribute(CurrentContext, Node, "result");
							string Value = GetAttribute(CurrentContext, Node, "value", true, false, "");
							if (Result != null)
							{
								if (Result == "Output")
								{
									GlobalContext.StringVariables["Output"] = Value;
								}
								else
								{
									CurrentContext.StringVariables[Result] = Value;
								}
							}
						}
						break;

					case "setStringFrom":
						{
							string? Result = GetAttribute(CurrentContext, Node, "result");
							string Value = GetAttribute(CurrentContext, Node, "value", true, false, "0");
							if (Result != null)
							{
								Value = ExpandVariables(CurrentContext, "$S(" + Value + ")");
								CurrentContext.StringVariables[Result] = Value;
							}
						}
						break;

					case "setStringFromEnvVar":
						{
							string? Result = GetAttribute(CurrentContext, Node, "result");
							string? Value = GetAttribute(CurrentContext, Node, "value");
							if (Result != null && Value != null)
							{
								CurrentContext.StringVariables[Result] = Environment.ExpandEnvironmentVariables(Value);
							}
						}
						break;

					case "setStringFromTag":
						{
							string? Result = GetAttribute(CurrentContext, Node, "result");
							string Tag = GetAttribute(CurrentContext, Node, "tag", true, false, "$");
							if (Result != null)
							{
								XElement? Element = CurrentElement;
								if (Tag.StartsWith('$'))
								{
									if (Tag.Length > 1)
									{
										if (!CurrentContext.ElementVariables.TryGetValue(Tag.Substring(1), out Element))
										{
											if (!GlobalContext.ElementVariables.TryGetValue(Tag.Substring(1), out Element))
											{
												Logger.LogWarning("Missing element variable '{Tag}' in '{Node}' (skipping instruction)", Tag, TraceNodeString(CurrentContext, Node));
												continue;
											}
										}
									}
								}
								CurrentContext.StringVariables[Result] = Element.Name.ToString();
							}
						}
						break;

					case "setStringFromAttribute":
						{
							string? Result = GetAttribute(CurrentContext, Node, "result");
							string? Tag = GetAttribute(CurrentContext, Node, "tag");
							string? Name = GetAttribute(CurrentContext, Node, "name");
							if (Result != null && Tag != null && Name != null)
							{
								XElement? Element = CurrentElement;
								if (Tag.StartsWith('$'))
								{
									if (Tag.Length > 1)
									{
										if (!CurrentContext.ElementVariables.TryGetValue(Tag.Substring(1), out Element))
										{
											if (!GlobalContext.ElementVariables.TryGetValue(Tag.Substring(1), out Element))
											{
												Logger.LogWarning("Missing element variable '{Tag}' in '{Node}' (skipping instruction)", Tag, TraceNodeString(CurrentContext, Node));
												continue;
											}
										}
									}
								}

								XNamespace? XMLNameSpace = TrimNamespaceAliasFromName(ref Name);
								XAttribute? Attribute = Element.Attribute(XMLNameSpace != null ? XMLNameSpace + Name : Name);

								CurrentContext.StringVariables[Result] = (Attribute != null) ? Attribute.Value : "";
							}
						}
						break;

					case "setStringFromTagText":
						{
							string? Result = GetAttribute(CurrentContext, Node, "result");
							string Tag = GetAttribute(CurrentContext, Node, "tag", true, false, "$");
							if (Result != null)
							{
								XElement? Element = CurrentElement;
								if (Tag.StartsWith('$'))
								{
									if (Tag.Length > 1)
									{
										if (!CurrentContext.ElementVariables.TryGetValue(Tag.Substring(1), out Element))
										{
											if (!GlobalContext.ElementVariables.TryGetValue(Tag.Substring(1), out Element))
											{
												Logger.LogWarning("Missing element variable '{Tag}' in '{Node}' (skipping instruction)", Tag, TraceNodeString(CurrentContext, Node));
												continue;
											}
										}
									}
								}

								if (Element.Value == null)
								{
									Logger.LogWarning("Expected text in element '{Element}' in '{Node}' but found none (skipping instruction)", Element.Name.ToString(), TraceNodeString(CurrentContext, Node));
									continue;
								}

								CurrentContext.StringVariables[Result] = Element.Value;
							}
						}
						break;

					case "setStringFromProperty":
						{
							string? Result = GetAttribute(CurrentContext, Node, "result");
							string? Ini = GetAttribute(CurrentContext, Node, "ini");
							string? Section = GetAttribute(CurrentContext, Node, "section");
							string? Property = GetAttribute(CurrentContext, Node, "property");
							string DefaultVal = GetAttribute(CurrentContext, Node, "default", true, false, "");
							if (Result != null && Ini != null && Section != null && Property != null)
							{
								string Value = DefaultVal;

								ConfigCacheIni_UPL ConfigIni = GetConfigCacheIni_UPL(Ini);
								if (ConfigIni != null)
								{
									if (!ConfigIni.GetString(Section, Property, out Value))
									{
										// If the string was not found in the config, Value will have been set to an empty string
										// Set it back to the DefaultVal
										Value = DefaultVal;
									}
								}
								if (Result == "Output")
								{
									GlobalContext.StringVariables["Output"] = Value;
								}
								else
								{
									CurrentContext.StringVariables[Result] = Value;
								}
							}
						}
						break;

					case "setStringFromPropertyArray":
						{
							string? Result = GetAttribute(CurrentContext, Node, "result");
							string? Ini = GetAttribute(CurrentContext, Node, "ini");
							string? Section = GetAttribute(CurrentContext, Node, "section");
							string? Property = GetAttribute(CurrentContext, Node, "property");
							string? IndexStr = GetAttribute(CurrentContext, Node, "index", true);
							string DefaultVal = GetAttribute(CurrentContext, Node, "default", true, false, "");
							if (Result != null && Ini != null && Section != null && Property != null && IndexStr != null)
							{
								string Value = DefaultVal;

								ConfigCacheIni_UPL ConfigIni = GetConfigCacheIni_UPL(Ini);
								if (ConfigIni != null)
								{
									List<string>? StringList;
									if (ConfigIni.GetArray(Section, Property, out StringList))
									{
										int Index = StringToInt(CurrentContext, Node, IndexStr);
										if (Index >= 0 && Index < StringList.Count)
										{
											Value = StringList.ElementAt(Index);
										}
									}
								}
								if (Result == "Output")
								{
									GlobalContext.StringVariables["Output"] = Value;
								}
								else
								{
									CurrentContext.StringVariables[Result] = Value;
								}
							}
						}
						break;

					case "setStringAdd":
						{
							string? Result = GetAttribute(CurrentContext, Node, "result");
							string Arg1 = GetAttribute(CurrentContext, Node, "arg1", true, false, "");
							string Arg2 = GetAttribute(CurrentContext, Node, "arg2", true, false, "");
							if (Result != null)
							{
								string Value = Arg1 + Arg2;
								if (Result == "Output")
								{
									GlobalContext.StringVariables["Output"] = Value;
								}
								else
								{
									CurrentContext.StringVariables[Result] = Value;
								}
							}
						}
						break;

					case "setStringSubstring":
						{
							string? Result = GetAttribute(CurrentContext, Node, "result");
							string Source = GetAttribute(CurrentContext, Node, "source", true, false, "");
							string Start = GetAttribute(CurrentContext, Node, "start", true, false, "0");
							string Length = GetAttribute(CurrentContext, Node, "length", true, false, "0");
							if (Result != null && Source != null)
							{
								int Index = StringToInt(CurrentContext, Node, Start);
								int Count = StringToInt(CurrentContext, Node, Length);
								Index = (Index < 0) ? 0 : (Index > Source.Length) ? Source.Length : Index;
								Count = (Index + Count > Source.Length) ? Source.Length - Index : Count;
								string Value = Source.Substring(Index, Count);
								if (Result == "Output")
								{
									GlobalContext.StringVariables["Output"] = Value;
								}
								else
								{
									CurrentContext.StringVariables[Result] = Value;
								}
							}
						}
						break;

					case "setStringSubstringAfterFind":
						{
							string? Result = GetAttribute(CurrentContext, Node, "result");
							string Source = GetAttribute(CurrentContext, Node, "source", true, false, "");
							string Find = GetAttribute(CurrentContext, Node, "find", true, false, "");
							string Length = GetAttribute(CurrentContext, Node, "length", true, false, "-1");
							string DefaultVal = GetAttribute(CurrentContext, Node, "default", true, false, "");
							if (Result != null)
							{
								string Value = DefaultVal;
								int Index = Source.IndexOf(Find, StringComparison.Ordinal);
								if (Index != -1)
								{
									Index += Find.Length;
									int Count = StringToInt(CurrentContext, Node, Length);
									Count = Count == -1 || (Index + Count > Source.Length) ? Source.Length - Index : Count;
									Value = Source.Substring(Index, Count);
								}
								if (Result == "Output")
								{
									GlobalContext.StringVariables["Output"] = Value;
								}
								else
								{
									CurrentContext.StringVariables[Result] = Value;
								}
							}
						}
						break;

					case "setStringReplace":
						{
							string? Result = GetAttribute(CurrentContext, Node, "result");
							string Source = GetAttribute(CurrentContext, Node, "source", true, false, "");
							string? Find = GetAttribute(CurrentContext, Node, "find");
							string With = GetAttribute(CurrentContext, Node, "with", true, false, "");
							if (Result != null && Find != null)
							{
								string Value = Source.Replace(Find, With, StringComparison.Ordinal);
								if (Result == "Output")
								{
									GlobalContext.StringVariables["Output"] = Value;
								}
								else
								{
									CurrentContext.StringVariables[Result] = Value;
								}
							}
						}
						break;

					case "setStringToLower":
						{
							string? Result = GetAttribute(CurrentContext, Node, "result");
							string Source = GetAttribute(CurrentContext, Node, "source", true, false, "");
							if (Result != null)
							{
#pragma warning disable CA1308 // Normalize strings to uppercase - specifically implementing a method of lowercasing a string
								string Value = Source.ToLowerInvariant();
#pragma warning restore CA1308
								if (Result == "Output")
								{
									GlobalContext.StringVariables["Output"] = Value;
								}
								else
								{
									CurrentContext.StringVariables[Result] = Value;
								}
							}
						}
						break;

					case "setStringToUpper":
						{
							string? Result = GetAttribute(CurrentContext, Node, "result");
							string Source = GetAttribute(CurrentContext, Node, "source", true, false, "");
							if (Result != null)
							{
								string Value = Source.ToUpperInvariant();
								if (Result == "Output")
								{
									GlobalContext.StringVariables["Output"] = Value;
								}
								else
								{
									CurrentContext.StringVariables[Result] = Value;
								}
							}
						}
						break;

					case "setElement":
						{
							string? Result = GetAttribute(CurrentContext, Node, "result");
							string? Value = GetAttribute(CurrentContext, Node, "value", true, false);
							string? Text = GetAttribute(CurrentContext, Node, "text", true, false);
							string? Parse = GetAttribute(CurrentContext, Node, "xml", true, false);
							if (Result != null)
							{
								if (Value != null)
								{
									XNamespace? XMLNameSpace = TrimNamespaceAliasFromName(ref Value);
									XElement NewElement = new XElement(XMLNameSpace != null ? XMLNameSpace + Value : Value);
									if (Text != null)
									{
										NewElement.Value = Text;
									}
									CurrentContext.ElementVariables[Result] = NewElement;
								}
								else if (Parse != null)
								{
									try
									{
										CurrentContext.ElementVariables[Result] = XElement.Parse(Parse);
									}
									catch (Exception e)
									{
										Logger.LogError(e, "XML parsing {Parse} failed! {Ex} (skipping instruction)", Parse, e);
									}
								}
							}
						}
						break;

					default:
						Logger.LogWarning("Unknown command: {Name}", Node.Name);
						break;
				}
			}

			return GlobalContext.StringVariables["Output"];
		}

		public void Init(IEnumerable<string> Architectures, bool bDistribution, string EngineDirectory, string BuildDirectory, string ProjectDirectory, string Configuration, bool bIsEmbedded,
			bool bPerArchBuildDir = false, Dictionary<string, string>? ArchRemapping = null)
		{
			GlobalContext.BoolVariables["Distribution"] = bDistribution;
			GlobalContext.BoolVariables["IsEmbedded"] = bIsEmbedded;
			GlobalContext.StringVariables["Configuration"] = Configuration;

			GlobalContext.StringVariables["EngineDir"] = EngineDirectory.Replace("\\", "/", StringComparison.Ordinal);
			GlobalContext.StringVariables["BuildDir"] = BuildDirectory.Replace("\\", "/", StringComparison.Ordinal);
			GlobalContext.StringVariables["ProjectDir"] = ProjectDirectory.Replace("\\", "/", StringComparison.Ordinal);

			ReadOnlyBuildVersion Version = ReadOnlyBuildVersion.Current;
			GlobalContext.IntVariables["EngineMajorVersion"] = Version.MajorVersion;
			GlobalContext.IntVariables["EngineMinorVersion"] = Version.MinorVersion;
			GlobalContext.IntVariables["EnginePatchVersion"] = Version.PatchVersion;
			GlobalContext.IntVariables["EngineChangelist"] = Version.Changelist;
			GlobalContext.StringVariables["EngineVersion"] = Version.MajorVersion.ToString() + "." + Version.MinorVersion.ToString() + "." + Version.PatchVersion.ToString();
			GlobalContext.StringVariables["EngineBranchName"] = Version.BranchName ?? "";

			if (GlobalContext.StringVariables["EngineDir"].Length < 1)
			{
				GlobalContext.StringVariables["EngineDir"] = "./";
			}
			if (GlobalContext.StringVariables["BuildDir"].Length < 1)
			{
				GlobalContext.StringVariables["BuildDir"] = "./";
			}
			if (GlobalContext.StringVariables["ProjectDir"].Length < 1)
			{
				GlobalContext.StringVariables["ProjectDir"] = "./";
			}

			GlobalContext.StringVariables["AbsEngineDir"] = Path.GetFullPath(GlobalContext.StringVariables["EngineDir"]).Replace("\\", "/", StringComparison.Ordinal);
			GlobalContext.StringVariables["AbsBuildDir"] = Path.GetFullPath(GlobalContext.StringVariables["BuildDir"]).Replace("\\", "/", StringComparison.Ordinal);
			GlobalContext.StringVariables["AbsProjectDir"] = Path.GetFullPath(GlobalContext.StringVariables["ProjectDir"]).Replace("\\", "/", StringComparison.Ordinal);

			foreach (string Arch in Architectures)
			{
				if (bPerArchBuildDir)
				{
					string ActiveArch = Arch;
					if (ArchRemapping != null && ArchRemapping.ContainsKey(ActiveArch))
					{
						ActiveArch = ArchRemapping[ActiveArch];
					}
					string ArchBuildDirectory = Path.Combine(GlobalContext.StringVariables["BuildDir"], ActiveArch.Replace("-", "_", StringComparison.Ordinal));
					string ArchBuildDir = ArchBuildDirectory.Replace("\\", "/", StringComparison.Ordinal);
					string ArchAbsBuildDir = Path.GetFullPath(ArchBuildDir).Replace("\\", "/", StringComparison.Ordinal);

					// add it to all the architecture contexts (overrides global context)
					for (int Index = 1; Index <= ContextIndex; Index++)
					{
						UPLContext ArchContext = Contexts[Arch + "_" + Index];
						ArchContext.StringVariables["BuildDir"] = ArchBuildDir;
						ArchContext.StringVariables["AbsBuildDir"] = ArchAbsBuildDir;
					}
				}

				Logger.LogInformation("UPL Init: {Arch}", Arch);
				ProcessPluginNode(Arch, "init", "");
			}

			if (bGlobalTrace)
			{
				Logger.LogInformation("\nVariables:\n{Variables}", DumpVariables());
			}
		}

		public void SetGlobalContextVariable(string VariableName, bool Value)
		{
			if (VariableName != null)
			{
				GlobalContext.BoolVariables[VariableName] = Value;
			}
		}

		public void SetGlobalContextVariable(string VariableName, int Value)
		{
			if (VariableName != null)
			{
				GlobalContext.IntVariables[VariableName] = Value;
			}
		}

		public void SetGlobalContextVariable(string VariableName, string Value)
		{
			if (VariableName != null && Value != null)
			{
				GlobalContext.StringVariables[VariableName] = Value;
			}
		}
	}

	/// <summary>
	/// Equivalent of FConfigCacheIni_UPL. Parses ini files.  This version reads ALL sections since ConfigCacheIni_UPL does NOT
	/// </summary>
	class ConfigCacheIni_UPL
	{
		/// <summary>
		/// Exception when parsing ini files
		/// </summary>
		public class IniParsingException : Exception
		{
			public IniParsingException(string Message)
				: base(Message)
			{ }
			public IniParsingException(string Format, params object[] Args)
				: base(String.Format(Format, Args))
			{ }
		}

		/// <summary>
		/// command class for being able to create config caches over and over without needing to read the ini files
		/// </summary>
		public class Command
		{
			public string? TrimmedLine;
		}

		class SectionCommand : Command
		{
			public FileReference? Filename;
			public int LineIndex;
		}

		class KeyValueCommand : Command
		{
			public string? Key;
			public string? Value;
			public ParseAction LastAction;
		}

		// cached ini files
		static readonly Dictionary<string, List<Command>> FileCache = new Dictionary<string, List<Command>>();
		static readonly Dictionary<string, ConfigCacheIni_UPL> IniCache = new Dictionary<string, ConfigCacheIni_UPL>();

		// static creation functions for ini files
		public static ConfigCacheIni_UPL CreateConfigCacheIni_UPL(UnrealTargetPlatform Platform, string BaseIniName, DirectoryReference? ProjectDirectory, ILogger Logger, DirectoryReference? EngineDirectory = null)
		{
			if (EngineDirectory == null)
			{
				EngineDirectory = Unreal.EngineDirectory;
			}

			// build the new ini and cache it for later re-use
			string Key = ConfigHierarchy.GetIniPlatformName(Platform) + BaseIniName + EngineDirectory.FullName + (ProjectDirectory != null ? ProjectDirectory.FullName : "");
			if (!IniCache.ContainsKey(Key))
			{
				IniCache.Add(Key, new ConfigCacheIni_UPL(Platform, BaseIniName, ProjectDirectory, Logger));
			}
			return IniCache[Key];
		}

		/// <summary>
		/// List of values (or a single value)
		/// </summary>
		public class IniValues : List<string>
		{
			public IniValues()
			{
			}
			public IniValues(IniValues Other)
				: base(Other)
			{
			}
			public override string ToString()
			{
				return String.Join(",", ToArray());
			}
		}

		/// <summary>
		/// Ini section (map of keys and values)
		/// </summary>
		public class IniSection : Dictionary<string, IniValues>
		{
			public IniSection()
				: base(StringComparer.InvariantCultureIgnoreCase)
			{ }
			public IniSection(IniSection Other)
				: this()
			{
				foreach (KeyValuePair<string, IniValues> Pair in Other)
				{
					Add(Pair.Key, new IniValues(Pair.Value));
				}
			}
			public override string ToString()
			{
				return "IniSection";
			}
		}

		/// <summary>
		/// True if we are loading a hierarchy of config files that should be merged together
		/// </summary>
		readonly bool bIsMergingConfigs = false;

		/// <summary>
		/// All sections parsed from ini file
		/// </summary>
		readonly Dictionary<string, IniSection> Sections;

		private ConfigCacheIni_UPL()
		{
			Sections = new Dictionary<string, IniSection>(StringComparer.InvariantCultureIgnoreCase);
		}

		/// <summary>
		/// Constructor. Parses a single ini file. No Platform settings, no engine hierarchy. Do not use this with ini files that have hierarchy!
		/// </summary>
		/// <param name="Filename">The ini file to load</param>
		/// <param name="Logger"></param>
		public ConfigCacheIni_UPL(FileReference Filename, ILogger Logger) : this()
		{
			bIsMergingConfigs = false;
			ParseIniFile(Filename, Logger);
		}

		/// <summary>
		/// Constructor. Parses ini hierarchy for the specified project.  No Platform settings.
		/// </summary>
		/// <param name="BaseIniName">Ini name (Engine, Editor, etc)</param>
		/// <param name="ProjectDirectory">Project path</param>
		/// <param name="Logger">Logger for </param>
		public ConfigCacheIni_UPL(string BaseIniName, string? ProjectDirectory, ILogger Logger)
			: this(BuildHostPlatform.Current.Platform, BaseIniName, ProjectDirectory, Logger)
		{
		}

		/// <summary>
		/// Constructor. Parses ini hierarchy for the specified project.  No Platform settings.
		/// </summary>
		/// <param name="BaseIniName">Ini name (Engine, Editor, etc)</param>
		/// <param name="ProjectDirectory">Project path</param>
		/// <param name="Logger">Logger for output</param>
		public ConfigCacheIni_UPL(string BaseIniName, DirectoryReference ProjectDirectory, ILogger Logger)
			: this(BuildHostPlatform.Current.Platform, BaseIniName, ProjectDirectory, Logger)
		{
		}

		/// <summary>
		/// Constructor. Parses ini hierarchy for the specified platform and project.
		/// </summary>
		/// <param name="ProjectDirectory">Project path</param>
		/// <param name="Platform">Target platform</param>
		/// <param name="BaseIniName">Ini name (Engine, Editor, etc)</param>
		/// <param name="Logger"></param>
		public ConfigCacheIni_UPL(UnrealTargetPlatform Platform, string BaseIniName, string? ProjectDirectory, ILogger Logger)
			: this(Platform, BaseIniName, ProjectDirectory == null ? null : new DirectoryReference(ProjectDirectory), Logger)
		{
		}

		/// <summary>
		/// Constructor. Parses ini hierarchy for the specified platform and project.
		/// </summary>
		/// <param name="ProjectDirectory">Project path</param>
		/// <param name="Platform">Target platform</param>
		/// <param name="BaseIniName">Ini name (Engine, Editor, etc)</param>
		/// <param name="Logger"></param>
		public ConfigCacheIni_UPL(UnrealTargetPlatform Platform, string BaseIniName, DirectoryReference? ProjectDirectory, ILogger Logger) : this()
		{
			bIsMergingConfigs = true;

			foreach (FileReference IniFileName in EnumerateConfigFileLocations(Platform, BaseIniName, ProjectDirectory))
			{
				if (FileReference.Exists(IniFileName))
				{
					ParseIniFile(IniFileName, Logger);
				}
			}
		}

		/// <summary>
		/// Finds a section in INI
		/// </summary>
		/// <param name="SectionName"></param>
		/// <returns>Found section or null</returns>
		public IniSection? FindSection(string SectionName)
		{
			IniSection? Section;
			Sections.TryGetValue(SectionName, out Section);
			return Section;
		}

		/// <summary>
		/// Finds values associated with the specified key (does not copy the list)
		/// </summary>
		private bool GetList(string SectionName, string Key, [NotNullWhen(true)] out IniValues? Value)
		{
			Value = null;

			IniSection? Section = FindSection(SectionName);
			if (Section == null)
			{
				return false;
			}

			if (Section.TryGetValue(Key, out Value) && Value != null)
			{
				return true;
			}

			return false;
		}

		/// <summary>
		/// Gets all values associated with the specified key
		/// </summary>
		/// <param name="SectionName">Section where the key is located</param>
		/// <param name="Key">Key name</param>
		/// <param name="Value">Copy of the list containing all values associated with the specified key</param>
		/// <returns>True if the key exists</returns>
		public bool GetArray(string SectionName, string Key, [NotNullWhen(true)] out List<string>? Value)
		{
			Value = null;
			if (GetList(SectionName, Key, out IniValues? ValueList) && ValueList != null)
			{
				Value = new List<string>(ValueList);
				return true;
			}
			return false;
		}

		/// <summary>
		/// Gets a single string value associated with the specified key.
		/// </summary>
		/// <param name="SectionName">Section name</param>
		/// <param name="Key">Key name</param>
		/// <param name="Value">Value associated with the specified key. If the key has more than one value, only the first one is returned</param>
		/// <returns>True if the key exists</returns>
		public bool GetString(string SectionName, string Key, out string Value)
		{
			Value = String.Empty;
			IniValues? ValueList;
			bool Result = GetList(SectionName, Key, out ValueList);
			if (Result && ValueList != null && ValueList.Count > 0)
			{
				Value = ValueList[0];
				Result = true;
			}
			else
			{
				Result = false;
			}
			return Result;
		}

		/// <summary>
		/// Gets a single bool value associated with the specified key.
		/// </summary>
		/// <param name="SectionName">Section name</param>
		/// <param name="Key">Key name</param>
		/// <param name="Value">Value associated with the specified key. If the key has more than one value, only the first one is returned</param>
		/// <returns>True if the key exists</returns>
		public bool GetBool(string SectionName, string Key, out bool Value)
		{
			Value = false;
			string TextValue;
			bool Result = GetString(SectionName, Key, out TextValue);
			if (Result)
			{
				// C# Boolean type expects "False" or "True" but since we're not case sensitive, we need to suppor that manually
				if (String.Equals(TextValue, "true", StringComparison.OrdinalIgnoreCase) || String.Equals(TextValue, "1", StringComparison.Ordinal))
				{
					Value = true;
				}
				else if (String.Equals(TextValue, "false", StringComparison.OrdinalIgnoreCase) || String.Equals(TextValue, "0", StringComparison.Ordinal))
				{
					Value = false;
				}
				else
				{
					// Failed to parse
					Result = false;
				}
			}
			return Result;
		}

		/// <summary>
		/// Gets a single Int32 value associated with the specified key.
		/// </summary>
		/// <param name="SectionName">Section name</param>
		/// <param name="Key">Key name</param>
		/// <param name="Value">Value associated with the specified key. If the key has more than one value, only the first one is returned</param>
		/// <returns>True if the key exists</returns>
		public bool GetInt32(string SectionName, string Key, out int Value)
		{
			Value = 0;
			string TextValue;
			bool Result = GetString(SectionName, Key, out TextValue);
			if (Result)
			{
				Result = Int32.TryParse(TextValue, out Value);
			}
			return Result;
		}

		/// <summary>
		/// Gets a single GUID value associated with the specified key.
		/// </summary>
		/// <param name="SectionName">Section name</param>
		/// <param name="Key">Key name</param>
		/// <param name="Value">Value associated with the specified key. If the key has more than one value, only the first one is returned</param>
		/// <returns>True if the key exists</returns>
		public bool GetGUID(string SectionName, string Key, out Guid Value)
		{
			Value = Guid.Empty;
			string TextValue;
			bool Result = GetString(SectionName, Key, out TextValue);
			if (Result)
			{
				string HexString = "";
				if (TextValue.Contains("A=", StringComparison.Ordinal) && TextValue.Contains("B=", StringComparison.Ordinal) && TextValue.Contains("C=", StringComparison.Ordinal) && TextValue.Contains("D=", StringComparison.Ordinal))
				{
					char[] Separators = new char[] { '(', ')', '=', ',', ' ', 'A', 'B', 'C', 'D' };
					string[] ComponentValues = TextValue.Split(Separators, StringSplitOptions.RemoveEmptyEntries);
					if (ComponentValues.Length == 4)
					{
						for (int ComponentIndex = 0; ComponentIndex < 4; ComponentIndex++)
						{
							int IntegerValue;
							Result &= Int32.TryParse(ComponentValues[ComponentIndex], out IntegerValue);
							HexString += IntegerValue.ToString("X8");
						}
					}
				}
				else
				{
					HexString = TextValue;
				}

				try
				{
					Value = Guid.ParseExact(HexString, "N");
					Result = true;
				}
				catch (Exception)
				{
					Result = false;
				}
			}
			return Result;
		}

		/// <summary>
		/// Gets a single float value associated with the specified key.
		/// </summary>
		/// <param name="SectionName">Section name</param>
		/// <param name="Key">Key name</param>
		/// <param name="Value">Value associated with the specified key. If the key has more than one value, only the first one is returned</param>
		/// <returns>True if the key exists</returns>
		public bool GetSingle(string SectionName, string Key, out float Value)
		{
			Value = 0.0f;
			string TextValue;
			bool Result = GetString(SectionName, Key, out TextValue);
			if (Result)
			{
				Result = Single.TryParse(TextValue, out Value);
			}
			return Result;
		}

		/// <summary>
		/// Gets a single double value associated with the specified key.
		/// </summary>
		/// <param name="SectionName">Section name</param>
		/// <param name="Key">Key name</param>
		/// <param name="Value">Value associated with the specified key. If the key has more than one value, only the first one is returned</param>
		/// <returns>True if the key exists</returns>
		public bool GetDouble(string SectionName, string Key, out double Value)
		{
			Value = 0.0;
			string TextValue;
			bool Result = GetString(SectionName, Key, out TextValue);
			if (Result)
			{
				Result = Double.TryParse(TextValue, out Value);
			}
			return Result;
		}

		private static bool ExtractPath(string Source, out string Path)
		{
			int start = Source.IndexOf('"', StringComparison.Ordinal);
			int end = Source.LastIndexOf('"');
			if (start != 1 && end != -1 && start < end)
			{
				++start;
				Path = Source.Substring(start, end - start);
				return true;
			}
			else
			{
				Path = "";
			}

			return false;
		}

		public bool GetPath(string SectionName, string Key, out string Value)
		{
			string temp;
			if (GetString(SectionName, Key, out temp))
			{
				return ExtractPath(temp, out Value);
			}
			else
			{
				Value = "";
			}

			return false;
		}

		/// <summary>
		/// List of actions that can be performed on a single line from ini file
		/// </summary>
		enum ParseAction
		{
			None,
			New,
			Add,
			Remove
		}

		/// <summary>
		/// Checks what action should be performed on a single line from ini file
		/// </summary>
		private ParseAction GetActionForLine(ref string Line)
		{
			if (String.IsNullOrEmpty(Line) || Line.StartsWith(';') || Line.StartsWith("//", StringComparison.Ordinal))
			{
				return ParseAction.None;
			}
			else if (Line.StartsWith('-'))
			{
				Line = Line.Substring(1).TrimStart();
				return ParseAction.Remove;
			}
			else if (Line.StartsWith('+'))
			{
				Line = Line.Substring(1).TrimStart();
				return ParseAction.Add;
			}
			else
			{
				// We use Add rather than New when we're not merging config files together in order 
				// to mimic the behavior of the C++ config cache when loading a single file
				return (bIsMergingConfigs) ? ParseAction.New : ParseAction.Add;
			}
		}

		/// <summary>
		/// Loads and parses ini file.
		/// </summary>
		public void ParseIniFile(FileReference Filename, ILogger Logger)
		{
			string[]? IniLines = null;
			List<Command>? Commands = null;
			if (!FileCache.ContainsKey(Filename.FullName))
			{
				try
				{
					IniLines = File.ReadAllLines(Filename.FullName);
					Commands = new List<Command>();
					FileCache.Add(Filename.FullName, Commands);
				}
				catch (Exception ex)
				{
					Logger.LogInformation("Error reading ini file: {Filename} Exception: {Ex}", Filename, ex.Message);
				}
			}
			else
			{
				Commands = FileCache[Filename.FullName];
			}
			if (IniLines != null && Commands != null)
			{
				IniSection? CurrentSection = null;

				// Line Index for exceptions
				int LineIndex = 1;
				bool bMultiLine = false;
				string SingleValue = "";
				string Key = "";
				ParseAction LastAction = ParseAction.None;

				// Parse each line
				foreach (string Line in IniLines)
				{
					string TrimmedLine = Line.Trim();
					// Multiline value support
					bool bWasMultiLine = bMultiLine;
					bMultiLine = TrimmedLine.EndsWith('\\');
					if (bMultiLine)
					{
						TrimmedLine = TrimmedLine.Substring(0, TrimmedLine.Length - 1).TrimEnd();
					}
					if (!bWasMultiLine)
					{
						if (TrimmedLine.StartsWith('['))
						{
							CurrentSection = FindOrAddSection(TrimmedLine, Filename, LineIndex);
							LastAction = ParseAction.None;
							if (CurrentSection != null)
							{
								SectionCommand Command = new SectionCommand();
								Command.Filename = Filename;
								Command.LineIndex = LineIndex;
								Command.TrimmedLine = TrimmedLine;
								Commands.Add(Command);
							}
						}
						else
						{
							if (LastAction != ParseAction.None)
							{
								throw new IniParsingException("Parsing new key/value pair when the previous one has not yet been processed ({0}, {1}) in {2}, line {3}: {4}", Key, SingleValue, Filename, LineIndex, TrimmedLine);
							}
							// Check if the line is empty or a comment, also remove any +/- markers
							LastAction = GetActionForLine(ref TrimmedLine);
							if (LastAction != ParseAction.None)
							{
								/*								if (CurrentSection == null)
																{
																	throw new IniParsingException("Trying to parse key/value pair that doesn't belong to any section in {0}, line {1}: {2}", Filename, LineIndex, TrimmedLine);
																}*/
								ParseKeyValuePair(TrimmedLine, Filename, LineIndex, out Key, out SingleValue);
							}
						}
					}
					if (bWasMultiLine)
					{
						SingleValue += TrimmedLine;
					}
					if (!bMultiLine && LastAction != ParseAction.None && CurrentSection != null)
					{
						ProcessKeyValuePair(CurrentSection, Key, SingleValue, LastAction);
						KeyValueCommand Command = new KeyValueCommand();
						Command.Key = Key;
						Command.Value = SingleValue;
						Command.LastAction = LastAction;
						Commands.Add(Command);
						LastAction = ParseAction.None;
						SingleValue = "";
						Key = "";
					}
					else if (CurrentSection == null)
					{
						LastAction = ParseAction.None;
					}
					LineIndex++;
				}
			}
			else if (Commands != null)
			{
				IniSection? CurrentSection = null;

				// run each command
				for (int Idx = 0; Idx < Commands.Count; ++Idx)
				{
					Command Command = Commands[Idx];
					if (Command is SectionCommand SectionCommand)
					{
						CurrentSection = FindOrAddSection(SectionCommand.TrimmedLine!, SectionCommand.Filename!, SectionCommand.LineIndex);
					}
					else if (Command is KeyValueCommand KeyValueCommand)
					{
						ProcessKeyValuePair(CurrentSection!, KeyValueCommand.Key!, KeyValueCommand.Value!, KeyValueCommand.LastAction);
					}
				}
			}
		}

		/// <summary>
		/// Splits a line into key and value
		/// </summary>
		private static void ParseKeyValuePair(string TrimmedLine, FileReference Filename, int LineIndex, out string Key, out string Value)
		{
			int AssignIndex = TrimmedLine.IndexOf('=', StringComparison.Ordinal);
			if (AssignIndex < 0)
			{
				throw new IniParsingException("Failed to find value when parsing {0}, line {1}: {2}", Filename, LineIndex, TrimmedLine);
			}
			Key = TrimmedLine.Substring(0, AssignIndex).Trim();
			if (String.IsNullOrEmpty(Key))
			{
				throw new IniParsingException("Empty key when parsing {0}, line {1}: {2}", Filename, LineIndex, TrimmedLine);
			}
			Value = TrimmedLine.Substring(AssignIndex + 1).Trim();
			if (Value.StartsWith('\"'))
			{
				// Remove quotes
				int QuoteEnd = Value.LastIndexOf('\"');
				if (QuoteEnd == 0)
				{
					throw new IniParsingException("Mismatched quotes when parsing {0}, line {1}: {2}", Filename, LineIndex, TrimmedLine);
				}
				Value = Value.Substring(1, Value.Length - 2);
			}
		}

		/// <summary>
		/// Processes parsed key/value pair
		/// </summary>
		private static void ProcessKeyValuePair(IniSection CurrentSection, string Key, string SingleValue, ParseAction Action)
		{
			switch (Action)
			{
				case ParseAction.New:
					{
						// New/replace
						IniValues? Value;
						if (CurrentSection.TryGetValue(Key, out Value) == false)
						{
							Value = new IniValues();
							CurrentSection.Add(Key, Value);
						}
						Value.Clear();
						Value.Add(SingleValue);
					}
					break;
				case ParseAction.Add:
					{
						IniValues? Value;
						if (CurrentSection.TryGetValue(Key, out Value) == false)
						{
							Value = new IniValues();
							CurrentSection.Add(Key, Value);
						}
						Value.Add(SingleValue);
					}
					break;
				case ParseAction.Remove:
					{
						IniValues? Value;
						if (CurrentSection.TryGetValue(Key, out Value))
						{
							int ExistingIndex = Value.FindIndex(X => (String.Equals(SingleValue, X, StringComparison.OrdinalIgnoreCase)));
							if (ExistingIndex >= 0)
							{
								Value.RemoveAt(ExistingIndex);
							}
						}
					}
					break;
			}
		}

		/// <summary>
		/// Finds an existing section or adds a new one
		/// </summary>
		private IniSection FindOrAddSection(string TrimmedLine, FileReference Filename, int LineIndex)
		{
			int SectionEndIndex = TrimmedLine.IndexOf(']', StringComparison.Ordinal);
			if (SectionEndIndex < 1)
			{
				throw new IniParsingException("Mismatched brackets when parsing section name in {0}, line {1}: {2}", Filename, LineIndex, TrimmedLine);
			}

			// comment could follow the ] but will just be trimmed out
			string SectionName = TrimmedLine.Substring(1, SectionEndIndex - 1);
			if (String.IsNullOrEmpty(SectionName))
			{
				throw new IniParsingException("Empty section name when parsing {0}, line {1}: {2}", Filename, LineIndex, TrimmedLine);
			}
			{
				IniSection? CurrentSection;
				if (Sections.TryGetValue(SectionName, out CurrentSection) == false)
				{
					CurrentSection = new IniSection();
					Sections.Add(SectionName, CurrentSection);
				}
				return CurrentSection;
			}
		}

		/// <summary>
		/// Returns the ordered list of INI files that should be merged for the given
		/// platform + project, matching the hierarchy used by C# build code
		/// (ConfigHierarchy / ConfigCache). This keeps UPL and C# readers in agreement
		/// on which files override which — including platform Base*.ini, Generated*.ini,
		/// Custom/{CUSTOMCONFIG}/*.ini, LimitedAccess restricted layers, platform
		/// extensions ({EXTENGINE}/{EXTPROJECT}) and the DataDrivenPlatformInfo
		/// IniParentChain.
		/// </summary>
		static IEnumerable<FileReference> EnumerateConfigFileLocations(UnrealTargetPlatform Platform, string BaseIniName, DirectoryReference? ProjectDirectory)
		{
			ConfigHierarchyType Type = Enum.Parse<ConfigHierarchyType>(BaseIniName, ignoreCase: true);
			return ConfigHierarchy.EnumerateConfigFileLocations(Type, ProjectDirectory, Platform, CustomConfig: String.Empty, IncludePluginsForTargetType: null);
		}
	}
}
