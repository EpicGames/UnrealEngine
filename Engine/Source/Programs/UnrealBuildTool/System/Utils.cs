// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using System.Linq;
using System.Runtime.InteropServices;
using System.Text;
using System.Text.RegularExpressions;
using System.Xml;
using System.Xml.Serialization;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using Microsoft.Win32.SafeHandles;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	/// <summary>
	/// Utility functions
	/// </summary>
#pragma warning disable CA1724 // Type names should not match namespaces
	public static class Utils
#pragma warning restore CA1724
	{
		/// <summary>
		/// Searches for a flag in a set of command-line arguments.
		/// </summary>
		public static bool ParseCommandLineFlag(string[] Arguments, string FlagName, out int ArgumentIndex)
		{
			// Find an argument with the given name.
			for (ArgumentIndex = 0; ArgumentIndex < Arguments.Length; ArgumentIndex++)
			{
				string Argument = Arguments[ArgumentIndex].ToUpperInvariant();
				if (Argument == FlagName.ToUpperInvariant())
				{
					return true;
				}
			}
			return false;
		}

		/// <summary>
		/// Regular expression to match $(ENV) and/ or %ENV% environment variables.
		/// </summary>
		static readonly Regex EnvironmentVariableRegex = new Regex(@"\$\((.*?)\)|\%(.*?)\%", RegexOptions.None);

		/// <summary>
		/// Resolves $(ENV) and/ or %ENV% to the value of the environment variable in the passed in string.
		/// </summary>
		/// <param name="InString">String to resolve environment variable in.</param>
		/// <returns>String with environment variable expanded/ resolved.</returns>
		public static string ResolveEnvironmentVariable(string InString)
		{
			string Result = InString;

			// Try to find $(ENV) substring.
			Match M = EnvironmentVariableRegex.Match(InString);

			// Iterate over all matches, resolving the match to an environment variable.
			while (M.Success)
			{
				// Convoluted way of stripping first and last character and '(' in the case of $(ENV) to get to ENV
				string EnvironmentVariable = M.ToString();
				if (EnvironmentVariable.StartsWith('$') && EnvironmentVariable.EndsWith(')'))
				{
					EnvironmentVariable = EnvironmentVariable.Substring(1, EnvironmentVariable.Length - 2).Replace("(", "", StringComparison.Ordinal);
				}

				if (EnvironmentVariable.StartsWith('%') && EnvironmentVariable.EndsWith('%'))
				{
					EnvironmentVariable = EnvironmentVariable.Substring(1, EnvironmentVariable.Length - 2);
				}

				// Resolve environment variable.
				Result = Result.Replace(M.ToString(), Environment.GetEnvironmentVariable(EnvironmentVariable), StringComparison.Ordinal);

				// Move on to next match. Multiple environment variables are handled correctly by regexp.
				M = M.NextMatch();
			}

			return Result;
		}

		/// <summary>
		/// Expands variables in $(VarName) format in the given string. Variables are retrieved from the given dictionary, or through the environment of the current process.
		/// Any unknown variables are ignored.
		/// </summary>
		/// <param name="InputString">String to search for variable names</param>
		/// <param name="AdditionalVariables">Lookup of variable names to values</param>
		/// <param name="bUseAdditionalVariablesOnly">If true, then Environment.GetEnvironmentVariable will not be used if the var is not found in AdditionalVariables</param>
		/// <returns>String with all variables replaced</returns>
		public static string ExpandVariables(string InputString, Dictionary<string, string>? AdditionalVariables = null, bool bUseAdditionalVariablesOnly = false)
		{
			string Result = InputString;
			for (int Idx = Result.IndexOf("$(", StringComparison.Ordinal); Idx != -1; Idx = Result.IndexOf("$(", Idx, StringComparison.Ordinal))
			{
				// Find the end of the variable name
				int EndIdx = Result.IndexOf(')', Idx + 2);
				if (EndIdx == -1)
				{
					break;
				}

				// Extract the variable name from the string
				string Name = Result.Substring(Idx + 2, EndIdx - (Idx + 2));

				// Find the value for it, either from the dictionary or the environment block
				string? Value = null;
				if (AdditionalVariables == null || !AdditionalVariables.TryGetValue(Name, out Value))
				{
					if (bUseAdditionalVariablesOnly == false)
					{
						Value = Environment.GetEnvironmentVariable(Name);
					}
					if (Value == null)
					{
						Idx = EndIdx + 1;
						continue;
					}
				}

				// Replace the variable, or skip past it
				Result = Result.Substring(0, Idx) + Value + Result.Substring(EndIdx + 1);
			}
			return Result;
		}

		/// <summary>
		/// Makes sure path can be used as a command line param (adds quotes if it contains spaces)
		/// </summary>
		/// <param name="InPath">Path to convert</param>
		/// <returns></returns>
		public static string MakePathSafeToUseWithCommandLine(string InPath)
		{
			// just always quote paths if they aren't already
			if (InPath[0] != '\"')
			{
				InPath = "\"" + InPath + "\"";
			}
			return InPath;
		}

		/// <summary>
		/// Makes sure path can be used as a command line param (adds quotes if it contains spaces)
		/// </summary>
		/// <param name="InPath">Path to convert</param>
		/// <returns></returns>
		public static string MakePathSafeToUseWithCommandLine(FileReference InPath)
		{
			return MakePathSafeToUseWithCommandLine(InPath.FullName);
		}

		/// <summary>
		/// Escapes whitespace in the given command line argument with a backslash. Used on Unix-like platforms for command line arguments in shell commands.
		/// </summary>
		/// <param name="Argument">The argument to escape </param>
		/// <returns>Escaped shell argument</returns>
		public static string EscapeShellArgument(string Argument)
		{
			return Argument.Replace(" ", "\\ ", StringComparison.Ordinal);
		}

		/// <summary>
		/// This is a faster replacement of File.ReadAllText. Code snippet based on code
		/// and analysis by Sam Allen
		/// http://dotnetperls.com/Content/File-Handling.aspx
		/// </summary>
		/// <param name="SourceFile"> Source file to fully read and convert to string</param>
		/// <returns>Textual representation of file.</returns>
		public static string ReadAllText(string SourceFile)
		{
			using (StreamReader Reader = new StreamReader(SourceFile, System.Text.Encoding.UTF8))
			{
				return Reader.ReadToEnd();
			}
		}

		/// <summary>
		/// Reads the specified environment variable
		/// </summary>
		/// <param name="VarName"> the environment variable to read</param>
		/// <param name="bDefault">the default value to use if missing</param>
		/// <returns>the value of the environment variable if found and the default value if missing</returns>
		public static bool GetEnvironmentVariable(string VarName, bool bDefault)
		{
			string? Value = Environment.GetEnvironmentVariable(VarName);
			if (Value != null)
			{
				// Convert the string to its boolean value
				return Convert.ToBoolean(Value);
			}
			return bDefault;
		}

		/// <summary>
		/// Reads the specified environment variable
		/// </summary>
		/// <param name="VarName"> the environment variable to read</param>
		/// <param name="Default">the default value to use if missing</param>
		/// <returns>the value of the environment variable if found and the default value if missing</returns>
		public static string GetStringEnvironmentVariable(string VarName, string Default)
		{
			string? Value = Environment.GetEnvironmentVariable(VarName);
			if (Value != null)
			{
				return Value;
			}
			return Default;
		}

		/// <summary>
		/// Reads the specified environment variable
		/// </summary>
		/// <param name="VarName"> the environment variable to read</param>
		/// <param name="Default">the default value to use if missing</param>
		/// <returns>the value of the environment variable if found and the default value if missing</returns>
		public static double GetEnvironmentVariable(string VarName, double Default)
		{
			string? Value = Environment.GetEnvironmentVariable(VarName);
			if (Value != null)
			{
				return Convert.ToDouble(Value);
			}
			return Default;
		}

		/// <summary>
		/// Reads the specified environment variable
		/// </summary>
		/// <param name="VarName"> the environment variable to read</param>
		/// <param name="Default">the default value to use if missing</param>
		/// <returns>the value of the environment variable if found and the default value if missing</returns>
		public static string GetEnvironmentVariable(string VarName, string Default)
		{
			string? Value = Environment.GetEnvironmentVariable(VarName);
			if (Value != null)
			{
				return Value;
			}
			return Default;
		}

		/// <summary>
		/// Try to launch a local process, and produce a friendly error message if it fails.
		/// </summary>
		public static int RunLocalProcess(Process LocalProcess)
		{
			int ExitCode = -1;

			// release all process resources
			using (LocalProcess)
			{
				LocalProcess.StartInfo.UseShellExecute = false;
				LocalProcess.StartInfo.RedirectStandardOutput = true;
				LocalProcess.StartInfo.RedirectStandardError = true;

				try
				{
					// Start the process up and then wait for it to finish
					LocalProcess.Start();
					LocalProcess.BeginOutputReadLine();
					LocalProcess.BeginErrorReadLine();
					LocalProcess.WaitForExit();
					ExitCode = LocalProcess.ExitCode;
				}
				catch (Exception ex)
				{
					throw new BuildException(ex, "Failed to start local process for action (\"{0}\"): {1} {2}", ex.Message, LocalProcess.StartInfo.FileName, LocalProcess.StartInfo.Arguments);
				}
			}

			return ExitCode;
		}

		/// <summary>
		/// Runs a local process and pipes the output to the log
		/// </summary>
		public static int RunLocalProcessAndLogOutput(ProcessStartInfo StartInfo, ILogger Logger)
		{
			Process LocalProcess = new Process();
			LocalProcess.StartInfo = StartInfo;
			LocalProcess.OutputDataReceived += (Sender, Args) => { LocalProcessOutput(Args, false, Logger); };
			LocalProcess.ErrorDataReceived += (Sender, Args) => { LocalProcessOutput(Args, true, Logger); };
			return RunLocalProcess(LocalProcess);
		}

		/// <summary>
		/// Output a line of text from a local process. Implemented as a separate function to give a useful function name in the UAT log prefix.
		/// </summary>
		static void LocalProcessOutput(DataReceivedEventArgs Args, bool bIsError, ILogger Logger)
		{
			if (Args != null && Args.Data != null)
			{
				if (bIsError)
				{
					Logger.LogError("{Message}", Args.Data.TrimEnd());
				}
				else
				{
					Logger.LogInformation("{Message}", Args.Data.TrimEnd());
				}
			}
		}

		/// <summary>
		/// Runs a local process and pipes the output to a file
		/// </summary>
		public static int RunLocalProcessAndPrintfOutput(ProcessStartInfo StartInfo, ILogger Logger)
		{
			string AppName = Path.GetFileNameWithoutExtension(StartInfo.FileName);
			string LogFilenameBase = String.Format("{0}_{1}", AppName, DateTime.Now.ToString("yyyy.MM.dd-HH.mm.ss"));
			string LogDir = Path.Combine(Unreal.EngineDirectory.FullName, "Programs", "AutomationTool", "Saved", "Logs");
			string LogFilename = "";
			for (int Attempt = 1; Attempt < 100; ++Attempt)
			{
				try
				{
					if (!Directory.Exists(LogDir))
					{
						string? IniPath = UnrealBuildTool.GetRemoteIniPath();
						if (String.IsNullOrEmpty(IniPath))
						{
							break;
						}

						LogDir = Path.Combine(IniPath, "Saved", "Logs");
						if (!Directory.Exists(LogDir) && !Directory.CreateDirectory(LogDir).Exists)
						{
							break;
						}
					}

					string LogFilenameBaseToCreate = LogFilenameBase;
					if (Attempt > 1)
					{
						LogFilenameBaseToCreate += "_" + Attempt;
					}
					LogFilenameBaseToCreate += ".txt";
					string LogFilenameToCreate = Path.Combine(LogDir, LogFilenameBaseToCreate);
					if (File.Exists(LogFilenameToCreate))
					{
						continue;
					}
					File.CreateText(LogFilenameToCreate).Close();
					LogFilename = LogFilenameToCreate;
					break;
				}
				catch (IOException)
				{
					//fatal error, let report to console
					break;
				}
			}

			DataReceivedEventHandler Output = (object sender, DataReceivedEventArgs Args) =>
			{
				if (Args != null && Args.Data != null)
				{
					string data = Args.Data.TrimEnd();
					if (String.IsNullOrEmpty(data))
					{
						return;
					}

					if (!String.IsNullOrEmpty(LogFilename))
					{
						File.AppendAllLines(LogFilename, data.Split('\n'));
					}
					else
					{
						Logger.LogInformation("{Output}", data);
					}
				}
			};
			Process LocalProcess = new Process();
			LocalProcess.StartInfo = StartInfo;
			LocalProcess.OutputDataReceived += Output;
			LocalProcess.ErrorDataReceived += Output;
			int ExitCode = RunLocalProcess(LocalProcess);
			if (ExitCode != 0 && !String.IsNullOrEmpty(LogFilename))
			{
				Logger.LogError("Process \'{AppName}\' failed. Details are in \'{LogFilename}\'", AppName, LogFilename);
			}

			return ExitCode;
		}

		/// <summary>
		/// Runs a local process and pipes the output to the log
		/// </summary>
		public static int RunLocalProcessAndLogOutput(string Command, string Args, ILogger Logger)
		{
			return RunLocalProcessAndLogOutput(new ProcessStartInfo(Command, Args), Logger);
		}

		/// <summary>
		/// Runs a command line process, and returns simple StdOut output. This doesn't handle errors or return codes
		/// </summary>
		/// <returns>The entire StdOut generated from the process as a single trimmed string</returns>
		/// <param name="Command">Command to run</param>
		/// <param name="Args">Arguments to Command</param>
		public static string RunLocalProcessAndReturnStdOut(string Command, string Args) => RunLocalProcessAndReturnStdOut(Command, Args, null);

		/// <summary>
		/// Runs a command line process, and returns simple StdOut output. This doesn't handle errors or return codes
		/// </summary>
		/// <returns>The entire StdOut generated from the process as a single trimmed string</returns>
		/// <param name="Command">Command to run</param>
		/// <param name="Args">Arguments to Command</param>
		/// <param name="Logger">Logger for output</param>
		public static string RunLocalProcessAndReturnStdOut(string Command, string Args, ILogger? Logger)
		{
			return RunLocalProcessAndReturnStdOut(Command, Args, Logger, out _);
		}

		/// <summary>
		/// Runs a command line process, and returns simple StdOut output.
		/// </summary>
		/// <returns>The entire StdOut generated from the process as a single trimmed string</returns>
		/// <param name="Command">Command to run</param>
		/// <param name="Args">Arguments to Command</param>
		/// <param name="ExitCode">The return code from the process after it exits</param>
		public static string RunLocalProcessAndReturnStdOut(string Command, string? Args, out int ExitCode) => RunLocalProcessAndReturnStdOut(Command, Args, null, out ExitCode);

		/// <summary>
		/// Runs a command line process, and returns simple StdOut output.
		/// </summary>
		/// <returns>The entire StdOut generated from the process as a single trimmed string</returns>
		/// <param name="Command">Command to run</param>
		/// <param name="Args">Arguments to Command</param>
		/// <param name="ExitCode">The return code from the process after it exits</param>
		/// <param name="LogOutput">Whether to also log standard output and standard error</param>
		public static string RunLocalProcessAndReturnStdOut(string Command, string? Args, out int ExitCode, bool LogOutput) => RunLocalProcessAndReturnStdOut(Command, Args, LogOutput ? Log.Logger : null, out ExitCode);

		/// <summary>
		/// Runs a command line process, and returns simple StdOut output.
		/// </summary>
		/// <returns>The entire StdOut generated from the process as a single trimmed string</returns>
		/// <param name="Command">Command to run</param>
		/// <param name="Args">Arguments to Command</param>
		/// <param name="Logger">Logger for output. No output if null.</param>
		/// <param name="ExitCode">The return code from the process after it exits</param>
		public static string RunLocalProcessAndReturnStdOut(string Command, string? Args, ILogger? Logger, out int ExitCode)
		{
			// Process Arguments follow windows conventions in .NET Core
			// Which means single quotes ' are not considered quotes.
			// see https://github.com/dotnet/runtime/issues/29857
			// also see UE-102580
			// for rules see https://docs.microsoft.com/en-us/cpp/cpp/main-function-command-line-args
			Args = Args?.Replace('\'', '\"') ?? String.Empty;

			ProcessStartInfo StartInfo = new ProcessStartInfo(Command, Args);
			StartInfo.UseShellExecute = false;
			StartInfo.RedirectStandardInput = true;
			StartInfo.RedirectStandardOutput = true;
			StartInfo.RedirectStandardError = true;
			StartInfo.CreateNoWindow = true;
			StartInfo.StandardOutputEncoding = Encoding.UTF8;

			string FullOutput = "";
			string ErrorOutput = "";
			using (Process LocalProcess = Process.Start(StartInfo)!)
			{
				StreamReader OutputReader = LocalProcess.StandardOutput;
				// trim off any extraneous new lines, helpful for those one-line outputs
				FullOutput = OutputReader.ReadToEnd().Trim();

				StreamReader ErrorReader = LocalProcess.StandardError;
				// trim off any extraneous new lines, helpful for those one-line outputs
				ErrorOutput = ErrorReader.ReadToEnd().Trim();
				if (Logger != null)
				{
					if (FullOutput.Length > 0)
					{
						Logger.LogInformation("{Output}", FullOutput);
					}

					if (ErrorOutput.Length > 0)
					{
						Logger.LogError("{Output}", ErrorOutput);
					}
				}

				LocalProcess.WaitForExit();
				ExitCode = LocalProcess.ExitCode;
			}

			// trim off any extraneous new lines, helpful for those one-line outputs
			if (ErrorOutput.Length > 0)
			{
				if (FullOutput.Length > 0)
				{
					FullOutput += Environment.NewLine;
				}
				FullOutput += ErrorOutput;
			}
			return FullOutput;
		}

		/// <summary>
		/// Find all the platforms in a given class
		/// </summary>
		/// <param name="Class">Class of platforms to return</param>
		/// <returns>Array of platforms in the given class</returns>
		public static UnrealTargetPlatform[] GetPlatformsInClass(UnrealPlatformClass Class)
		{
			return s_PlatformsInClass.TryGetValue(Class, out Lazy<UnrealTargetPlatform[]>? LazyArray)
				? LazyArray.Value
				: throw new ArgumentException($"'{(int)Class}' is not a valid value for {typeof(UnrealPlatformClass).Name}");

		}
		static readonly Dictionary<UnrealPlatformClass, Lazy<UnrealTargetPlatform[]>> s_PlatformsInClass = new()
		{
			{ UnrealPlatformClass.All, new(UnrealTargetPlatform.GetValidPlatforms) },
			{ UnrealPlatformClass.Desktop, new(() => [.. UEBuildPlatform.GetPlatformsInGroup(UnrealPlatformGroup.Desktop)]) },
			{ UnrealPlatformClass.Editor, new(() => [UnrealTargetPlatform.Win64, UnrealTargetPlatform.Linux, UnrealTargetPlatform.Mac]) },
			{ UnrealPlatformClass.Server, new(() => [UnrealTargetPlatform.Win64, UnrealTargetPlatform.Linux, UnrealTargetPlatform.LinuxArm64, UnrealTargetPlatform.Mac]) },
		};

		/// <summary>
		/// Retrieve the list of platforms in this group (if any)
		/// </summary>
		public static List<UnrealTargetPlatform> GetPlatformsInGroup(UnrealPlatformGroup Group)
		{
			return UEBuildPlatform.GetPlatformsInGroup(Group);
		}

		/// <summary>
		/// Given a list of supported platforms, returns a list of names of platforms that should not be supported
		/// </summary>
		/// <param name="SupportedPlatforms">List of supported platforms</param>
		/// <param name="bIncludeUnbuildablePlatforms">If true, add platforms that are present but not available for compiling</param>
		/// <param name="Logger"></param>
		/// 
		/// <returns>List of unsupported platforms in string format</returns>
		public static List<string> MakeListOfUnsupportedPlatforms(List<UnrealTargetPlatform> SupportedPlatforms, bool bIncludeUnbuildablePlatforms, ILogger Logger)
		{
			// Make a list of all platform name strings that we're *not* currently compiling, to speed
			// up file path comparisons later on
			List<string> OtherPlatformNameStrings = new List<string>();
			{
				List<UnrealPlatformGroup> SupportedGroups = new List<UnrealPlatformGroup>();

				// look at each group to see if any supported platforms are in it
				foreach (UnrealPlatformGroup Group in UnrealPlatformGroup.GetValidGroups())
				{
					// get the list of platforms registered to this group, if any
					List<UnrealTargetPlatform> Platforms = UEBuildPlatform.GetPlatformsInGroup(Group);
					if (Platforms != null)
					{
						// loop over each one
						foreach (UnrealTargetPlatform Platform in Platforms)
						{
							// if it's a compiled platform, then add this group to be supported
							if (SupportedPlatforms.Contains(Platform))
							{
								SupportedGroups.Add(Group);
							}
						}
					}
				}

				// loop over groups one more time, anything NOT in SupportedGroups is now unsupported, and should be added to the output list
				foreach (UnrealPlatformGroup Group in UnrealPlatformGroup.GetValidGroups())
				{
					if (SupportedGroups.Contains(Group) == false)
					{
						OtherPlatformNameStrings.Add(Group.ToString());
					}
				}

				foreach (UnrealTargetPlatform CurPlatform in UnrealTargetPlatform.GetValidPlatforms())
				{
					bool ShouldConsider = true;

					// If we have a platform and a group with the same name, don't add the platform
					// to the other list if the same-named group is supported.  This is a lot of
					// lines because we need to do the comparisons as strings.
					string CurPlatformString = CurPlatform.ToString();
					foreach (UnrealPlatformGroup Group in UnrealPlatformGroup.GetValidGroups())
					{
						if (Group.ToString().Equals(CurPlatformString, StringComparison.Ordinal))
						{
							ShouldConsider = false;
							break;
						}
					}

					// Don't add our current platform to the list of platform sub-directory names that
					// we'll skip source files for
					if (ShouldConsider && !SupportedPlatforms.Contains(CurPlatform))
					{
						OtherPlatformNameStrings.Add(CurPlatform.ToString());
					}
					// if a platform isn't available to build, then return it 
					else if (bIncludeUnbuildablePlatforms && !UEBuildPlatform.IsPlatformAvailable(CurPlatform))
					{
						OtherPlatformNameStrings.Add(CurPlatform.ToString());
					}
				}

				return OtherPlatformNameStrings;
			}
		}

		/// <summary>
		/// Takes a path string and makes all of the path separator characters consistent. Also removes unnecessary multiple separators.
		/// </summary>
		/// <param name="FilePath">File path with potentially inconsistent slashes</param>
		/// <param name="UseDirectorySeparatorChar">The directory separator to use</param>
		/// <returns>File path with consistent separators</returns>
		public static string CleanDirectorySeparators(string FilePath, char UseDirectorySeparatorChar = '\0')
		{
			StringBuilder? CleanPath = null;
			if (UseDirectorySeparatorChar == '\0')
			{
				UseDirectorySeparatorChar = Path.DirectorySeparatorChar;
			}
			char PrevC = '\0';
			// Don't check for double separators until we run across a valid dir name. Paths that start with '//' or '\\' can still be valid.			
			bool bCanCheckDoubleSeparators = false;
			for (int Index = 0; Index < FilePath.Length; ++Index)
			{
				char C = FilePath[Index];
				if (C == '/' || C == '\\')
				{
					if (C != UseDirectorySeparatorChar)
					{
						C = UseDirectorySeparatorChar;
						CleanPath ??= new StringBuilder(FilePath.Substring(0, Index), FilePath.Length);
					}

					if (bCanCheckDoubleSeparators && C == PrevC)
					{
						CleanPath ??= new StringBuilder(FilePath.Substring(0, Index), FilePath.Length);
						continue;
					}
				}
				else
				{
					// First non-separator character, safe to check double separators
					bCanCheckDoubleSeparators = true;
				}

				CleanPath?.Append(C);
				PrevC = C;
			}
			return CleanPath != null ? CleanPath.ToString() : FilePath;
		}

		/// <summary>
		/// Correctly collapses any ../ or ./ entries in a path.
		/// </summary>
		/// <param name="InPath">The path to be collapsed</param>
		/// <returns>true if the path could be collapsed, false otherwise.</returns>
		public static string CollapseRelativeDirectories(string InPath)
		{
			string LocalString = InPath;
			bool bHadBackSlashes = false;
			// look to see what kind of slashes we had
			if (LocalString.Contains('\\', StringComparison.Ordinal))
			{
				LocalString = LocalString.Replace("\\", "/", StringComparison.Ordinal);
				bHadBackSlashes = true;
			}

			string ParentDir = "/..";
			int ParentDirLength = ParentDir.Length;

			for (; ; )
			{
				// An empty path is finished
				if (String.IsNullOrEmpty(LocalString))
				{
					break;
				}

				// Consider empty paths or paths which start with .. or /.. as invalid
				if (LocalString.StartsWith("..", StringComparison.Ordinal) || LocalString.StartsWith(ParentDir, StringComparison.Ordinal))
				{
					return InPath;
				}

				// If there are no "/.."s left then we're done
				int Index = LocalString.IndexOf(ParentDir, StringComparison.Ordinal);
				if (Index == -1)
				{
					break;
				}

				int PreviousSeparatorIndex = Index;
				for (; ; )
				{
					// Find the previous slash
					PreviousSeparatorIndex = Math.Max(0, LocalString.LastIndexOf('/', PreviousSeparatorIndex - 1));

					// Stop if we've hit the start of the string
					if (PreviousSeparatorIndex == 0)
					{
						break;
					}

					// Stop if we've found a directory that isn't "/./"
					if ((Index - PreviousSeparatorIndex) > 1 && (LocalString[PreviousSeparatorIndex + 1] != '.' || LocalString[PreviousSeparatorIndex + 2] != '/'))
					{
						break;
					}
				}

				// If we're attempting to remove the drive letter, that's illegal
				int Colon = LocalString.IndexOf(':', PreviousSeparatorIndex);
				if (Colon >= 0 && Colon < Index)
				{
					return InPath;
				}

				LocalString = LocalString.Substring(0, PreviousSeparatorIndex) + LocalString.Substring(Index + ParentDirLength);
			}

			LocalString = LocalString.Replace("./", "", StringComparison.Ordinal);

			// restore back slashes now
			if (bHadBackSlashes)
			{
				LocalString = LocalString.Replace("/", "\\", StringComparison.Ordinal);
			}

			// and pass back out
			return LocalString;
		}

		/// <summary>
		/// Given a file path and a directory, returns a file path that is relative to the specified directory
		/// </summary>
		/// <param name="SourcePath">File path to convert</param>
		/// <param name="RelativeToDirectory">The directory that the source file path should be converted to be relative to.  If this path is not rooted, it will be assumed to be relative to the current working directory.</param>
		/// <param name="AlwaysTreatSourceAsDirectory">True if we should treat the source path like a directory even if it doesn't end with a path separator</param>
		/// <returns>Converted relative path</returns>
		public static string MakePathRelativeTo(string SourcePath, string RelativeToDirectory, bool AlwaysTreatSourceAsDirectory = false)
		{
			if (String.IsNullOrEmpty(RelativeToDirectory))
			{
				// Assume CWD
				RelativeToDirectory = ".";
			}

			string AbsolutePath = SourcePath;
			if (!Path.IsPathRooted(AbsolutePath))
			{
				AbsolutePath = Path.GetFullPath(SourcePath);
			}
			bool SourcePathEndsWithDirectorySeparator = AbsolutePath.EndsWith(Path.DirectorySeparatorChar) || AbsolutePath.EndsWith(Path.AltDirectorySeparatorChar);
			if (AlwaysTreatSourceAsDirectory && !SourcePathEndsWithDirectorySeparator)
			{
				AbsolutePath += Path.DirectorySeparatorChar;
			}

			Uri AbsolutePathUri = new Uri(AbsolutePath);

			string AbsoluteRelativeDirectory = RelativeToDirectory;
			if (!Path.IsPathRooted(AbsoluteRelativeDirectory))
			{
				AbsoluteRelativeDirectory = Path.GetFullPath(AbsoluteRelativeDirectory);
			}

			// Make sure the directory has a trailing directory separator so that the relative directory that
			// MakeRelativeUri creates doesn't include our directory -- only the directories beneath it!
			if (!AbsoluteRelativeDirectory.EndsWith(Path.DirectorySeparatorChar) && !AbsoluteRelativeDirectory.EndsWith(Path.AltDirectorySeparatorChar))
			{
				AbsoluteRelativeDirectory += Path.DirectorySeparatorChar;
			}

			// Convert to URI form which is where we can make the relative conversion happen
			Uri AbsoluteRelativeDirectoryUri = new Uri(AbsoluteRelativeDirectory);

			// Ask the URI system to convert to a nicely formed relative path, then convert it back to a regular path string
			Uri UriRelativePath = AbsoluteRelativeDirectoryUri.MakeRelativeUri(AbsolutePathUri);
			string RelativePath = Uri.UnescapeDataString(UriRelativePath.ToString()).Replace('/', Path.DirectorySeparatorChar);

			// If we added a directory separator character earlier on, remove it now
			if (!SourcePathEndsWithDirectorySeparator && AlwaysTreatSourceAsDirectory && RelativePath.EndsWith(Path.DirectorySeparatorChar))
			{
				RelativePath = RelativePath.Substring(0, RelativePath.Length - 1);
			}

			return RelativePath;
		}

		/*
		 * Read and write classes with xml specifiers
		 */
		private static void UnknownAttributeDelegate(object? sender, XmlAttributeEventArgs e)
		{
		}

		private static void UnknownNodeDelegate(object? sender, XmlNodeEventArgs e)
		{
		}

		/// <summary>
		/// Reads a class using XML serialization
		/// </summary>
		/// <typeparam name="T">The type to read</typeparam>
		/// <param name="FileName">The XML file to read from</param>
		/// <param name="Logger">Logger for output</param>
		/// <returns>New deserialized instance of type T</returns>
		public static T ReadClass<T>(string FileName, ILogger Logger) where T : new()
		{
			T Instance = new T();
			StreamReader? XmlStream = null;
			try
			{
				// Get the XML data stream to read from
				XmlStream = new StreamReader(FileName);

				// Creates an instance of the XmlSerializer class so we can read the settings object
				XmlSerializer Serialiser = new XmlSerializer(typeof(T));
				// Add our callbacks for unknown nodes and attributes
				Serialiser.UnknownNode += new XmlNodeEventHandler(UnknownNodeDelegate);
				Serialiser.UnknownAttribute += new XmlAttributeEventHandler(UnknownAttributeDelegate);

				// Create an object graph from the XML data
				using XmlReader XmlReader = XmlReader.Create(XmlStream);
				Instance = (T)Serialiser.Deserialize(XmlReader)!;
			}
			catch (Exception E)
			{
				Logger.LogInformation("{Output}", E.Message);
			}
			finally
			{
				// Done with the file so close it
				XmlStream?.Close();
			}

			return Instance;
		}

		/// <summary>
		/// Serialize an object to an XML file
		/// </summary>
		/// <typeparam name="T">Type of the object to serialize</typeparam>
		/// <param name="Data">Object to write</param>
		/// <param name="FileName">File to write to</param>
		/// <param name="DefaultNameSpace">Default namespace for the output elements</param>
		/// <param name="Logger">Logger for output</param>
		/// <returns>True if the file was written successfully</returns>
		public static bool WriteClass<T>(T Data, string FileName, string DefaultNameSpace, ILogger Logger)
		{
			try
			{
				FileInfo Info = new FileInfo(FileName);
				if (Info.Exists)
				{
					Info.IsReadOnly = false;
				}

				// Make sure the output directory exists
				Directory.CreateDirectory(Path.GetDirectoryName(FileName)!);

				XmlSerializerNamespaces EmptyNameSpace = new();
				EmptyNameSpace.Add("", DefaultNameSpace);

				using StreamWriter XmlStream = new(FileName, false, Encoding.Unicode);
				XmlSerializer Serialiser = new(typeof(T));

				// Add our callbacks for unknown nodes and attributes
				Serialiser.UnknownNode += new XmlNodeEventHandler(UnknownNodeDelegate);
				Serialiser.UnknownAttribute += new XmlAttributeEventHandler(UnknownAttributeDelegate);

				Serialiser.Serialize(XmlStream, Data, EmptyNameSpace);
				return true;
			}
			catch (Exception E)
			{
				Logger.LogInformation("{Message}", E.Message);
				return false;
			}
		}

		/// <summary>
		/// Removes multi-dot extensions from a filename (i.e. *.automation.csproj)
		/// </summary>
		/// <param name="Filename">Filename to remove the extensions from</param>
		/// <returns>Clean filename.</returns>
		public static string GetFilenameWithoutAnyExtensions(string Filename)
		{
			Filename = Path.GetFileName(Filename);

			int DotIndex = Filename.IndexOf('.', StringComparison.Ordinal);
			if (DotIndex == -1)
			{
				return Filename; // No need to copy string
			}
			else
			{
				return Filename.Substring(0, DotIndex);
			}
		}

		/// <summary>
		/// Returns Filename with path but without extension.
		/// </summary>
		/// <param name="Filename">Filename</param>
		/// <returns>Path to the file with its extension removed.</returns>
		public static string GetPathWithoutExtension(string Filename)
		{
			if (!String.IsNullOrEmpty(Path.GetExtension(Filename)))
			{
				return Path.Combine(Path.GetDirectoryName(Filename)!, Path.GetFileNameWithoutExtension(Filename));
			}
			else
			{
				return Filename;
			}
		}

		/// <summary>
		/// Returns true if the specified file's path is located under the specified directory, or any of that directory's sub-folders.  Does not care whether the file or directory exist or not.  This is a simple string-based check.
		/// </summary>
		/// <param name="FilePath">The path to the file</param>
		/// <param name="Directory">The directory to check to see if the file is located under (or any of this directory's subfolders)</param>
		/// <returns></returns>
		public static bool IsFileUnderDirectory(string FilePath, string Directory)
		{
			string DirectoryPathPlusSeparator = Path.GetFullPath(Directory);
			if (!DirectoryPathPlusSeparator.EndsWith(Path.DirectorySeparatorChar))
			{
				DirectoryPathPlusSeparator += Path.DirectorySeparatorChar;
			}
			return Path.GetFullPath(FilePath).StartsWith(DirectoryPathPlusSeparator, StringComparison.OrdinalIgnoreCase);
		}

		/// <summary>
		/// Checks if given type implements given interface.
		/// </summary>
		/// <typeparam name="TInterface">Interface to check.</typeparam>
		/// <param name="TestType">Type to check.</param>
		/// <returns>True if TestType implements TInterface. False otherwise.</returns>
		public static bool ImplementsInterface<TInterface>(Type TestType)
		{
			return Array.IndexOf(TestType.GetInterfaces(), typeof(TInterface)) != -1;
		}

		/// <summary>
		/// Returns the User Settings Directory path. This matches FPlatformProcess::UserSettingsDir().
		/// NOTE: This function may return null. Some accounts (eg. the SYSTEM account on Windows) do not have a personal folder, and Jenkins
		/// runs using this account by default.
		/// </summary>
		[Obsolete("Deprecated in UE5.8. Replace with Unreal.UserSettingDirectory")]
		public static DirectoryReference? GetUserSettingDirectory() => Unreal.UserSettingDirectory;

		/// <summary>
		/// Gets the number of logical cores. We use this rather than Environment.ProcessorCount when possible to handle machines with > 64 cores (the single group limit available to the .NET framework).
		/// </summary>
		/// <returns>The number of logical cores.</returns>
		[Obsolete("Deprecated in UE5.8. Use SystemUtils.GetLogicalProcessorCount() from EpicGames.Core instead.")]
		public static int GetLogicalProcessorCount() => SystemUtils.GetLogicalProcessorCount();

		/// <summary>
		/// Gets the number of physical cores, excluding hyper threading.
		/// </summary>
		/// <returns>The number of physical cores, or -1 if it could not be obtained</returns>
		[Obsolete("Deprecated in UE5.8. Use SystemUtils.GetPhysicalProcessorCount() from EpicGames.Core instead.")]
		public static int GetPhysicalProcessorCount() => SystemUtils.GetPhysicalProcessorCount();

		/// <summary>
		/// Gets if the processor has asymmetrical cores (Windows only)
		/// </summary>
		/// <returns></returns>
		[Obsolete("Deprecated in UE5.8. Use SystemUtils.IsAsymmetricalProcessor() from EpicGames.Core instead.")]
		public static bool IsAsymmetricalProcessor() => SystemUtils.IsAsymmetricalProcessor();

		/// <summary>
		/// Gets the total memory bytes available, based on what is known to the garbage collector.
		/// </summary>
		/// <remarks>This will return a max of 2GB for a 32bit application.</remarks>
		/// <returns>The total memory available, in bytes.</returns>
		[Obsolete("Deprecated in UE5.8. Use SystemUtils.GetAvailableMemoryBytes() from EpicGames.Core instead.")]
		public static long GetAvailableMemoryBytes() => SystemUtils.GetAvailableMemoryBytes();

		/// <summary>
		/// Gets the total system memory in bytes based on what is known to the garbage collector.
		/// </summary>
		/// <remarks>This will return a max of 2GB for a 32bit application.</remarks>
		/// <returns>The total system memory, in bytes.</returns>
		[Obsolete("Deprecated in UE5.8. Use SystemUtils.GetTotalSystemMemoryBytes() from EpicGames.Core instead.")]
		public static long GetTotalSystemMemoryBytes() => SystemUtils.GetTotalSystemMemoryBytes();

		/// <summary>
		/// Gets the total memory bytes free, based on what is known to the garbage collector.
		/// </summary>
		/// <remarks>This will return a max of 2GB for a 32bit application.</remarks>
		/// <returns>The total memory free, in bytes.</returns>
		[Obsolete("Deprecated in UE5.8. Use SystemUtils.GetFreeMemoryBytes() from EpicGames.Core instead.")]
		public static long GetFreeMemoryBytes() => SystemUtils.GetFreeMemoryBytes();

		/// <summary>
		/// Gives the current CPU utilization.
		/// </summary>
		/// <param name="Utilization">Percentage of CPU utilization currently.</param>
		/// <returns>Whether or not it was successful in getting the CPU utilization.</returns>
		[Obsolete("Deprecated in UE5.8. Use SystemUtils.GetTotalCpuUtilization() from EpicGames.Core instead.")]
		public static bool GetTotalCpuUtilization(out float Utilization) => SystemUtils.GetTotalCpuUtilization(out Utilization);

		/// <summary>
		/// Determines the maximum number of actions to execute in parallel, taking into account the resources available on this machine.
		/// </summary>
		/// <param name="MaxProcessorCount">How many actions to execute in parallel. When 0 a default will be chosen based on system resources</param>
		/// <param name="ProcessorCountMultiplier">Physical processor count multiplier for local execution. Can be below 1 to reserve CPU for other tasks.</param>
		/// <param name="ConsiderLogicalCores">Consider logical cores when determing max actions to execute in parallel. Unused if ProcessorCountMultiplier is not 1.0.</param>
		/// <param name="MemoryPerActionBytes">Limit max number of actions based on total system memory.</param>
		/// <returns>Max number of actions to execute in parallel</returns>
		public static int GetMaxActionsToExecuteInParallel(int MaxProcessorCount, double ProcessorCountMultiplier, bool ConsiderLogicalCores, long MemoryPerActionBytes)
		{
			// If non-zero then this is the number of actions that can be executed in parallel. This 
			// matches the BuildConfiguration.MaxLocalActions documentation and the <= 5.1 behavior
			if (MaxProcessorCount != 0)
			{
				Log.TraceInformationOnce($"Executing up to {MaxProcessorCount} actions based on MaxProcessorCount override");
				return MaxProcessorCount;
			}

			// Get the number of logical processors
			int NumLogicalCores = SystemUtils.GetLogicalProcessorCount();

			// Use WMI to figure out physical cores, excluding hyper threading.
			int NumPhysicalCores = SystemUtils.GetPhysicalProcessorCount();
			if (NumPhysicalCores == -1)
			{
				NumPhysicalCores = NumLogicalCores;
			}

			Log.TraceInformationOnce($"Determining max actions to execute in parallel ({NumPhysicalCores} physical cores, {NumLogicalCores} logical cores)");

			// The number of actions to execute in parallel is trying to keep the CPU busy enough in presence of I/O stalls.
			int MaxActionsToExecuteInParallel;
			if (ProcessorCountMultiplier != 1.0)
			{
				// The CPU has more logical cores than physical ones, aka uses hyper-threading. 
				// Use multiplier if provided
				MaxActionsToExecuteInParallel = (int)(NumPhysicalCores * ProcessorCountMultiplier);

				// make sure we don't try to run more actions than cores we have
				MaxActionsToExecuteInParallel = Math.Min(MaxActionsToExecuteInParallel, Math.Max(NumLogicalCores, NumPhysicalCores));

				Log.TraceInformationOnce($"  Requested {ProcessorCountMultiplier} process count multiplier: limiting max parallel actions to {MaxActionsToExecuteInParallel}");
			}
			else if (ConsiderLogicalCores && NumLogicalCores > NumPhysicalCores)
			{
				Log.TraceInformationOnce($"  Executing up to {NumLogicalCores} processes, one per logical core");
				MaxActionsToExecuteInParallel = NumLogicalCores;
			}
			// kick off a task per physical core - evidence suggests that, in general, using more cores does not yield significantly better throughput
			else
			{
				Log.TraceInformationOnce($"  Executing up to {NumPhysicalCores} processes, one per physical core");
				MaxActionsToExecuteInParallel = NumPhysicalCores;
			}

			// Limit number of actions to execute if the system is memory starved.
			if (MemoryPerActionBytes > 0)
			{
				// The OS needs enough memory to serve all the action processes that will be spawned. Historically this check limited
				// actions based on free memory to ensure action processes weren't forced to use swap, but there's strong evidence that
				// limiting based on total system memory and relying on the OS to swap out other processes to make room is fine.
				// That said only Mac has been extensively tested here in this scenario, and does tend to have faster disk access
				// than other platforms. So for now we'll change Mac and leave Windows/Linux using the old behavior
				long AvailableMemoryBytes = OperatingSystem.IsMacOS() ? SystemUtils.GetTotalSystemMemoryBytes() : SystemUtils.GetFreeMemoryBytes();

				if (AvailableMemoryBytes != -1)
				{
					int TotalMemoryActions = Convert.ToInt32(AvailableMemoryBytes / MemoryPerActionBytes);
					if (TotalMemoryActions < MaxActionsToExecuteInParallel)
					{
						MaxActionsToExecuteInParallel = Math.Max(1, Math.Min(MaxActionsToExecuteInParallel, TotalMemoryActions));
						Log.TraceInformationOnce($"  Requested {StringUtils.FormatBytesString(MemoryPerActionBytes)} memory per action, {StringUtils.FormatBytesString(AvailableMemoryBytes)} available: limiting max parallel actions to {MaxActionsToExecuteInParallel}");
					}
				}
			}

			return MaxActionsToExecuteInParallel;
		}

		/// <summary>
		/// Executes a list of custom build step scripts
		/// </summary>
		/// <param name="ScriptFiles">List of script files to execute</param>
		/// <param name="Logger">Logger for output</param>
		/// <returns>True if the steps succeeded, false otherwise</returns>
		public static void ExecuteCustomBuildSteps(FileReference[] ScriptFiles, ILogger Logger)
		{
			foreach (FileReference ScriptFile in ScriptFiles)
			{
				using ITimelineEvent _ = Timeline.ScopeEvent(ScriptFile.GetFileName());

				ProcessStartInfo StartInfo = new ProcessStartInfo();
				StartInfo.FileName = BuildHostPlatform.Current.Shell.FullName;

				if (BuildHostPlatform.Current.ShellType == ShellType.Cmd)
				{
					StartInfo.Arguments = String.Format("/C \"{0}\"", ScriptFile.FullName);
				}
				else
				{
					StartInfo.Arguments = String.Format("\"{0}\"", ScriptFile.FullName);
				}

				int ReturnCode = Utils.RunLocalProcessAndLogOutput(StartInfo, Logger);
				if (ReturnCode != 0)
				{
					throw new BuildException("Custom build step {0} {1} terminated with exit code {2}", StartInfo.FileName, StartInfo.Arguments, ReturnCode);
				}
			}

			if (ScriptFiles.Length > 0)
			{
				using ITimelineEvent _ = Timeline.ScopeEvent("ResetCachedInfo");

				// We have to invalidate all cached file info after running the scripts, because we don't know what may have changed.
				DirectoryItem.ResetAllCachedInfo_SLOW();
			}
		}

		/// <summary>
		/// Parses a command line into a list of arguments
		/// </summary>
		/// <param name="CommandLine">The command line to parse</param>
		/// <returns>List of output arguments</returns>
		public static List<string> ParseArgumentList(string CommandLine)
		{
			List<string> Arguments = new List<string>();

			StringBuilder CurrentArgument = new StringBuilder();
			for (int Idx = 0; Idx < CommandLine.Length; Idx++)
			{
				if (!Char.IsWhiteSpace(CommandLine[Idx]))
				{
					CurrentArgument.Clear();

					bool bInQuotes = false;
					for (; Idx < CommandLine.Length; Idx++)
					{
						if (CommandLine[Idx] == '\"')
						{
							bInQuotes ^= true;
						}
						else if (CommandLine[Idx] == ' ' && !bInQuotes)
						{
							break;
						}
						else
						{
							CurrentArgument.Append(CommandLine[Idx]);
						}
					}

					Arguments.Add(CurrentArgument.ToString());
				}
			}

			return Arguments;
		}

		/// <summary>
		/// Try to parse the project file from the command line
		/// </summary>
		/// <param name="Arguments">The command line arguments</param>
		/// <param name="Logger">Logger for output</param>
		/// <param name="ProjectFile">The project file that was parsed</param>
		/// <param name="MarkArgumentAsUsed">Marks the Project argument as used if set to true.</param>
		/// <returns>True if the project file was parsed, false otherwise</returns>
		public static bool TryParseProjectFileArgument(CommandLineArguments Arguments, ILogger Logger, [NotNullWhen(true)] out FileReference? ProjectFile, bool MarkArgumentAsUsed = true)
		{
			string? CandidateProjectPath = null;

			// look for -project=<path>, if it does not exist check arguments for anything that has .uproject in it
			if (!Arguments.TryGetValue("-Project=", out CandidateProjectPath) )
			{
				Logger.LogDebug("No project argument provided while trying to parse Project File. Attempting fallback parsing to attempt to find a Project File.");

				// Go through the argument list and try to match poorly (or well..) formed arguments like 
				// EngineTest, EngineTest.uproject
				// Collaboration/FooProject
				// by checking for those in the native project list, and checking for common -Target="...... "SomeProjectFile.uproject\"" strings in the multi-target scenario.
				List<FileReference> TargetProjectFiles = new List<FileReference>();
				for (int Idx = 0; Idx < Arguments.Count; Idx++)
				{
					if (!Arguments[Idx].StartsWith('-') && Arguments[Idx].EndsWith(".uproject", StringComparison.OrdinalIgnoreCase))
					{
						CandidateProjectPath = Arguments[Idx];

						if (MarkArgumentAsUsed)
						{
							Arguments.MarkAsUsed(Idx);
						}
						break;
					}
					else if (Arguments[Idx].StartsWith("-Target", StringComparison.OrdinalIgnoreCase)) // Potentially a -Target="...." aggregate argument
					{
						Logger.LogDebug("-Target argument detected in original command line args. Will attempt to subparse in search of ProjectFile.");

						FileReference? targetProjectFile = null;
						CommandLineArguments subCommand = new CommandLineArguments(ParseArgumentList(Arguments[Idx]).ToArray());

						if (Arguments[Idx].Contains("-Project", StringComparison.OrdinalIgnoreCase)) //If there's a -Project argument, just recurse and let the rest capture. Ignore any other sources within this -Target="..."
						{
							TryParseProjectFileArgument(subCommand, Logger, out targetProjectFile, MarkArgumentAsUsed);
						}
						else // Just extract the Target, and attempt to get a project for it.
						{
							string? potentialTarget;
							if (subCommand.TryGetValue("-Target=", out potentialTarget))
							{
								NativeProjects.TryGetProjectForTarget(potentialTarget, Logger, out targetProjectFile);
							}
						}

						if(targetProjectFile != null)
						{
							TargetProjectFiles.Add(targetProjectFile);
						}
					}
					else if (!Arguments[Idx].StartsWith("-Target", StringComparison.OrdinalIgnoreCase)) // Assume it *could* just be a raw target; this amortizes to a cheap dictionary check now.
					{
						FileReference? targetProjectFile = null;
						if (NativeProjects.TryGetProjectForTarget(Arguments[Idx], Logger, out targetProjectFile) && targetProjectFile != null)
						{
							TargetProjectFiles.Add(targetProjectFile);
						}
					}
				}
				
				if (TargetProjectFiles.Count > 0 && TargetProjectFiles.GroupBy(x => x.FullName.ToUpperInvariant()).Count() == 1)
				{
					// We can return early as we have already resolved a file reference.
					ProjectFile = TargetProjectFiles[0];
					Logger.LogDebug("Resolved project argument {CandidateProjectPath} to {CandidateProjectFile} through -Target resolution.", ProjectFile, ProjectFile.ToNormalizedPath());

					return true;
				}
				else if (TargetProjectFiles.Count > 0 && TargetProjectFiles.GroupBy(x => x.FullName.ToUpperInvariant()).Count() > 1)
				{
					Logger.LogInformation("Multiple unique uprojects provided. Cannot determine a single project file to use from command line: will not apply a project scope BuildConfiguration.xml.");
				}
			}

			// We have a project file either via -project= or because there was something called .uproject in the arg list
			// so now validate it
			if (!String.IsNullOrEmpty(CandidateProjectPath))
			{
				FileReference? CandidateProjectFile = FileReference.FindCorrectCase(new FileReference(CandidateProjectPath));

				// if the path doesn't exist then check native paths (ueprojectdirs)
				if (!FileReference.Exists(CandidateProjectFile))
				{
					// clean everything the user provided to just the name and make sure it has the expected extension
					string ProjectName = CandidateProjectFile.ChangeExtension("uproject").GetFileName();

					// check native project paths (uprojectdirs)
					IEnumerable<FileReference> NativeProjectFiles = NativeProjects.EnumerateProjectFiles(Logger);

					CandidateProjectFile = NativeProjectFiles.Where(F => F.GetFileName().Equals(ProjectName, StringComparison.OrdinalIgnoreCase)).FirstOrDefault();
				}

				if (CandidateProjectFile == null || !FileReference.Exists(CandidateProjectFile))
				{
					// if we didn't find anything then throw an error as the user explicitly provided a uproject
					throw new Exception(String.Format("Unable to find project file based on argument {0}", CandidateProjectPath));
				}

				Logger.LogDebug("Resolved project argument {CandidateProjectPath} to {CandidateProjectFile}", CandidateProjectPath, CandidateProjectFile);
				ProjectFile = CandidateProjectFile;
				return true;
			}

			FileReference? InstalledProjectFile = Unreal.GetInstalledProjectFile();
			if (InstalledProjectFile != null)
			{
				ProjectFile = InstalledProjectFile;
				return true;
			}

			ProjectFile = null;
			return false;
		}

		/// <summary>
		/// Formats a list of arguments as a command line, inserting quotes as necessary
		/// </summary>
		/// <param name="Arguments">List of arguments to format</param>
		/// <returns>Command line string</returns>
		public static string FormatCommandLine(List<string> Arguments)
		{
			StringBuilder CommandLine = new StringBuilder();
			foreach (string Argument in Arguments)
			{
				if (CommandLine.Length > 0)
				{
					CommandLine.Append(' ');
				}

				if (!Argument.Contains(' ', StringComparison.Ordinal))
				{
					CommandLine.Append(Argument);
				}
				else
				{
					int EqualsIdx = Argument.IndexOf('=', StringComparison.Ordinal);
					if (EqualsIdx != -1 && Argument[0] == '-')
					{
						CommandLine.Append(Argument, 0, EqualsIdx + 1);
						CommandLine.Append('\"');
						CommandLine.Append(Argument, EqualsIdx + 1, Argument.Length - (EqualsIdx + 1));
						CommandLine.Append('\"');
					}
					else
					{
						CommandLine.Append('\"');
						CommandLine.Append(Argument);
						CommandLine.Append('\"');
					}
				}
			}
			return CommandLine.ToString();
		}

		/// <summary>
		/// Writes a file if the contents have changed
		/// </summary>
		/// <param name="Location">Location of the file</param>
		/// <param name="Contents">New contents of the file</param>
		/// <param name="Logger">Logger for output</param>
		internal static void WriteFileIfChanged(FileReference Location, string Contents, ILogger Logger)
		{
			WriteFileIfChanged(Location, Contents, StringComparison.Ordinal, Logger);
		}

		/// <summary>
		/// Writes a file if the contents have changed
		/// </summary>
		/// <param name="Location">Location of the file</param>
		/// <param name="Contents">New contents of the file</param>
		/// <param name="Comparison">The type of string comparison to use</param>
		/// <param name="Logger">Logger for output</param>
		internal static void WriteFileIfChanged(FileReference Location, string Contents, StringComparison Comparison, ILogger Logger)
		{
			FileItem FileItem = FileItem.GetItemByFileReference(Location);
			WriteFileIfChanged(FileItem, Contents, Comparison, Logger);
		}

		/// <summary>
		/// Writes a file if the contents have changed
		/// </summary>
		/// <param name="Location">Location of the file</param>
		/// <param name="ContentLines">New contents of the file</param>
		/// <param name="Logger">Logger for output</param>
		internal static void WriteFileIfChanged(FileReference Location, IEnumerable<string> ContentLines, ILogger Logger)
		{
			WriteFileIfChanged(Location, ContentLines, StringComparison.Ordinal, Logger);
		}

		/// <summary>
		/// Writes a file if the contents have changed
		/// </summary>
		/// <param name="Location">Location of the file</param>
		/// <param name="ContentLines">New contents of the file</param>
		/// <param name="Comparison">The type of string comparison to use</param>
		/// <param name="Logger">Logger for output</param>
		internal static void WriteFileIfChanged(FileReference Location, IEnumerable<string> ContentLines, StringComparison Comparison, ILogger Logger)
		{
			FileItem FileItem = FileItem.GetItemByFileReference(Location);
			WriteFileIfChanged(FileItem, ContentLines, Comparison, Logger);
		}

		/// <summary>
		/// Record each file that has been requested written, with the number of times the file has been written
		/// </summary>
		static readonly Dictionary<FileReference, (int WriteRequestCount, int ActualWriteCount)> WriteFileIfChangedRecord = new Dictionary<FileReference, (int, int)>();

		internal static FileReference? WriteFileIfChangedTrace = null;
		internal static string WriteFileIfChangedContext = "";

		static void RecordWriteFileIfChanged(FileReference File, bool bNew, bool bChanged, ILogger Logger)
		{
			int NewWriteRequestCount = 1;
			int NewActualWriteCount = bChanged ? 1 : 0;

			bool bOverrideLogEventType = FileReference.Equals(WriteFileIfChangedTrace, File);
			LogLevel OverrideType = LogLevel.Information;

			string Prefix = "";
			if (bOverrideLogEventType)
			{
				Prefix = "[TraceWrites] ";
			}

			string Context = "";
			if (!String.IsNullOrEmpty(WriteFileIfChangedContext))
			{
				Context = $" ({WriteFileIfChangedContext})";
			}

			lock (WriteFileIfChangedRecord)
			{
				if (WriteFileIfChangedRecord.TryGetValue(File, out (int WriteRequestCount, int ActualWriteCount) WriteRecord))
				{
					// Unexepected that a file is getting written more than once during a single execution

					NewWriteRequestCount += WriteRecord.WriteRequestCount;
					NewActualWriteCount += WriteRecord.ActualWriteCount;

					if (WriteRecord.ActualWriteCount == 0)
					{
						if (bNew)
						{
							Logger.Log(bOverrideLogEventType ? OverrideType : LogLevel.Warning,
								"{Prefix}Writing a file that previously existed was not overwritten and then removed: \"{File}\"{Context}", Prefix, File, Context);
						}
						else
						{
							if (bChanged)
							{
								Logger.Log(bOverrideLogEventType ? OverrideType : LogLevel.Warning,
									"{Prefix}Writing a file that previously was not written \"{File}\"{Context}", Prefix, File, Context);
							}
							else
							{
								if (bOverrideLogEventType)
								{
									Logger.Log(OverrideType,
										"{Prefix}Not writing a file that was previously not written: \"{File}\"{Context}", Prefix, File, Context);
								}
							}
						}
					}
					else
					{
						if (bNew)
						{
							Logger.Log(bOverrideLogEventType ? OverrideType : LogLevel.Warning,
								"{Prefix}Re-writing a file that was previously written and then removed: \"{File}\"{Context}", Prefix, File, Context);
						}
						else
						{
							if (bChanged)
							{
								Logger.Log(bOverrideLogEventType ? OverrideType : LogLevel.Warning,
									"{Prefix}Re-writing a file that was previously written: \"{File}\"{Context}", Prefix, File, Context);
							}
							else
							{
								if (bOverrideLogEventType)
								{
									Logger.Log(OverrideType,
										"{Prefix}Not writing a file that was previously written: \"{File}\"{Context}", Prefix, File, Context);
								}
							}
						}
					}
				}
				else
				{
					if (FileReference.Equals(WriteFileIfChangedTrace, File))
					{
						if (bNew)
						{
							Logger.LogInformation("{Prefix}Writing new file: \"{File}\"{Context}", Prefix, File, Context);
						}
						else
						{
							if (bChanged)
							{
								Logger.LogInformation("{Prefix}Writing changed file: \"{File}\"{Context}", Prefix, File, Context);
							}
							else
							{
								Logger.LogInformation("{Prefix}Not writing unchanged file: \"{File}\"{Context}", Prefix, File, Context);
							}
						}
					}
				}

				WriteFileIfChangedRecord[File] = (NewWriteRequestCount, NewActualWriteCount);
			}
		}

		internal static void LogWriteFileIfChangedActivity(ILogger Logger)
		{
			int TotalRequests = 0;
			int TotalWrites = 0;
			foreach ((int Requested, int Actual) in WriteFileIfChangedRecord.Values)
			{
				TotalRequests += Requested;
				TotalWrites += Actual;
			}

			Logger.LogDebug("WriteFileIfChanged() wrote {TotalWrites} changed files of {TotalRequests} requested writes.", TotalWrites, TotalRequests);
		}

		/// <summary>
		/// Writes a file if the contents have changed
		/// </summary>
		/// <param name="FileItem">Location of the file</param>
		/// <param name="Contents">New contents of the file</param>
		/// <param name="Logger">Logger for output</param>
		internal static void WriteFileIfChanged(FileItem FileItem, string Contents, ILogger Logger)
		{
			WriteFileIfChanged(FileItem, Contents, StringComparison.Ordinal, Logger);
		}

		/// <summary>
		/// Writes a file if the contents have changed
		/// </summary>
		/// <param name="FileItem">Location of the file</param>
		/// <param name="Contents">New contents of the file</param>
		/// <param name="Comparison">The type of string comparison to use</param>
		/// <param name="Logger">Logger for output</param>
		internal static void WriteFileIfChanged(FileItem FileItem, string Contents, StringComparison Comparison, ILogger Logger)
		{
			// Only write the file if its contents have changed.
			FileReference Location = FileItem.Location;
			if (!FileItem.Exists)
			{
				DirectoryReference.CreateDirectory(Location.Directory);
				FileReference.WriteAllText(Location, Contents, GetEncodingForString(Contents));
				FileItem.ResetCachedInfo();

				RecordWriteFileIfChanged(FileItem.Location, bNew: true, bChanged: true, Logger);
			}
			else
			{
				string CurrentContents = Utils.ReadAllText(FileItem.FullName);
				if (!String.Equals(CurrentContents, Contents, Comparison))
				{
					FileReference BackupFile = new FileReference(FileItem.FullName + ".old");
					try
					{
						Logger.LogDebug("Updating {File}: contents have changed. Saving previous version to {BackupFile}.", FileItem.Location, BackupFile);
						FileReference.Delete(BackupFile);
						FileReference.Move(Location, BackupFile);
					}
					catch (Exception Ex)
					{
						Logger.LogWarning("Unable to rename {FileItem} to {BackupFile}", FileItem, BackupFile);
						Logger.LogDebug(Ex, "{Ex}", ExceptionUtils.FormatExceptionDetails(Ex));
					}
					FileReference.WriteAllText(Location, Contents, GetEncodingForString(Contents));
					FileItem.ResetCachedInfo();

					RecordWriteFileIfChanged(FileItem.Location, bNew: false, bChanged: true, Logger);
				}
				else
				{
					RecordWriteFileIfChanged(FileItem.Location, bNew: false, bChanged: false, Logger);
				}
			}
		}

		/// <summary>
		/// Writes a file if the contents have changed
		/// </summary>
		/// <param name="FileItem">Location of the file</param>
		/// <param name="ContentLines">New contents of the file</param>
		/// <param name="Logger">Logger for output</param>
		internal static void WriteFileIfChanged(FileItem FileItem, IEnumerable<string> ContentLines, ILogger Logger)
		{
			WriteFileIfChanged(FileItem, ContentLines, StringComparison.Ordinal, Logger);
		}

		/// <summary>
		/// Writes a file if the contents have changed
		/// </summary>
		/// <param name="FileItem">Location of the file</param>
		/// <param name="ContentLines">New contents of the file</param>
		/// <param name="Comparison">The type of string comparison to use</param>
		/// <param name="Logger">Logger for output</param>
		internal static void WriteFileIfChanged(FileItem FileItem, IEnumerable<string> ContentLines, StringComparison Comparison, ILogger Logger)
		{
			string Contents = String.Join(Environment.NewLine, ContentLines);
			WriteFileIfChanged(FileItem, Contents, Comparison, Logger);
		}

		/// <summary>
		/// Determines the appropriate encoding for a string: either ASCII or UTF-8.
		/// If the string length is equivalent to the encoded length, then no non-ASCII characters were present in the string.
		/// Don't write BOM as it messes with clang when loading response files.
		/// </summary>
		/// <param name="Str">The string to test.</param>
		/// <returns>Either System.Text.Encoding.ASCII or System.Text.Encoding.UTF8, depending on whether or not the string contains non-ASCII characters.</returns>
		private static Encoding GetEncodingForString(string Str)
		{
			// If the string length is equivalent to the encoded length, then no non-ASCII characters were present in the string.
			// Don't write BOM as it messes with clang when loading response files.
			return (Encoding.UTF8.GetByteCount(Str) != Str.Length) ? new UTF8Encoding(false) : Encoding.ASCII;
		}

		/// <summary>
		/// Creates a hard link at Path pointing to PathToTarget.
		/// If that fails, copies the file at PathToTarget to Path if it's different, using Length and LastWriteTimeUtc.
		/// </summary>
		/// <param name="Path">Path to create the hard link/copy file at.</param>
		/// <param name="PathToTarget">Path to which the hard link should point to/copy from.</param>
		/// <returns>True if the files were different.</returns>
		internal static bool HardLinkOrCopyIfDifferent(string Path, string PathToTarget)
		{
			FileInfo Info = new(Path), TargetInfo = new(PathToTarget);

			if (Info.Exists)
			{
				if (OperatingSystem.IsWindows())
				{
					using (SafeFileHandle Handle = File.OpenHandle(Path), HandleToTarget = File.OpenHandle(PathToTarget))
					{
						if (!Windows.GetFileInformationByHandle(Handle, out Windows.BY_HANDLE_FILE_INFORMATION FileInformation) || !Windows.GetFileInformationByHandle(HandleToTarget, out Windows.BY_HANDLE_FILE_INFORMATION FileInformationToTarget))
						{
							throw Marshal.GetExceptionForHR(Marshal.GetHRForLastWin32Error())!;
						}

						if (FileInformation.dwVolumeSerialNumber == FileInformationToTarget.dwVolumeSerialNumber && FileInformation.nFileIndexHigh == FileInformationToTarget.nFileIndexHigh && FileInformation.nFileIndexLow == FileInformationToTarget.nFileIndexLow)
						{
							return false;
						}
					}
				}

				// Allow 1s slop in case one of the files is on a FAT volume, and the other is not
				if (Info.Length == TargetInfo.Length && Math.Abs((Info.LastWriteTimeUtc - TargetInfo.LastWriteTimeUtc).TotalSeconds) <= 1)
				{
					return false;
				}

				Info.Delete();
			}

			if (OperatingSystem.IsWindows() && Windows.CreateHardLink(Path, PathToTarget))
			{
				return true;
			}

			Info = TargetInfo.CopyTo(Path);

			Info.LastWriteTimeUtc = TargetInfo.LastWriteTimeUtc;
			return true;
		}

		/// <summary>
		/// Creates a hard/symbolic link at Path pointing to PathToTarget.
		/// If that fails, copies the file at PathToTarget to Path if it's different, using Length and LastWriteTimeUtc.
		/// </summary>
		/// <param name="Path">Path to create the hard/symbolic link/copy file at.</param>
		/// <param name="PathToTarget">Path to which the hard/symbolic link should point to/copy from.</param>
		/// <returns>True if the files were different.</returns>
		internal static bool LinkOrCopyIfDifferent(string Path, string PathToTarget)
		{
			FileInfo Info = new(Path), TargetInfo = new(PathToTarget);

			if (Info.Exists)
			{
				if (Info.LinkTarget == null)
				{
					if (OperatingSystem.IsWindows())
					{
						using (SafeFileHandle Handle = File.OpenHandle(Path), HandleToTarget = File.OpenHandle(PathToTarget))
						{
							if (!Windows.GetFileInformationByHandle(Handle, out Windows.BY_HANDLE_FILE_INFORMATION FileInformation) || !Windows.GetFileInformationByHandle(HandleToTarget, out Windows.BY_HANDLE_FILE_INFORMATION FileInformationToTarget))
							{
								throw Marshal.GetExceptionForHR(Marshal.GetHRForLastWin32Error())!;
							}

							if (FileInformation.dwVolumeSerialNumber == FileInformationToTarget.dwVolumeSerialNumber && FileInformation.nFileIndexHigh == FileInformationToTarget.nFileIndexHigh && FileInformation.nFileIndexLow == FileInformationToTarget.nFileIndexLow)
							{
								return false;
							}
						}
					}

					// Allow 1s slop in case one of the files is on a FAT volume, and the other is not
					if (Info.Length == TargetInfo.Length && Math.Abs((Info.LastWriteTimeUtc - TargetInfo.LastWriteTimeUtc).TotalSeconds) <= 1)
					{
						return false;
					}
				}
				else if (Info.LinkTarget == TargetInfo.FullName)
				{
					// Allow 1s slop in case one of the files is on a FAT volume, and the other is not
					if (Math.Abs((Info.LastWriteTimeUtc - TargetInfo.LastWriteTimeUtc).TotalSeconds) <= 1)
					{
						return false;
					}

					Info.LastWriteTimeUtc = TargetInfo.LastWriteTimeUtc;
					return true;
				}

				Info.Delete();
			}

			if (OperatingSystem.IsWindows() && Windows.CreateHardLink(Path, PathToTarget))
			{
				return true;
			}

			try
			{
				Info.CreateAsSymbolicLink(TargetInfo.FullName);
			}
			catch (IOException)
			{
				Info = TargetInfo.CopyTo(Path);
			}

			Info.LastWriteTimeUtc = TargetInfo.LastWriteTimeUtc;
			return true;
		}

		internal static class Windows
		{
			[DllImport("Kernel32.dll", CharSet = CharSet.Auto, SetLastError = true)]
			public static extern bool CreateHardLink(string lpFileName, string lpExistingFileName, [Optional] nuint lpSecurityAttributes);

			[StructLayout(LayoutKind.Sequential)]
			public struct FILETIME
			{
				public uint dwLowDateTime;
				public uint dwHighDateTime;
			}

			[StructLayout(LayoutKind.Sequential)]
			public struct BY_HANDLE_FILE_INFORMATION
			{
				public uint dwFileAttributes;
				public FILETIME ftCreationTime;
				public FILETIME ftLastAccessTime;
				public FILETIME ftLastWriteTime;
				public uint dwVolumeSerialNumber;
				public uint nFileSizeHigh;
				public uint nFileSizeLow;
				public uint nNumberOfLinks;
				public uint nFileIndexHigh;
				public uint nFileIndexLow;
			}

			[DllImport("Kernel32.dll", SetLastError = true)]
			public static extern bool GetFileInformationByHandle(SafeFileHandle hFile, out BY_HANDLE_FILE_INFORMATION lpFileInformation);
		}
	}
}
