// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Runtime.InteropServices;
using System.Text;
using EpicGames.Core;
using Microsoft.CodeAnalysis;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	abstract class ISPCToolChain : UEToolChain
	{
		public ISPCToolChain(ILogger InLogger) : base(InLogger)
		{
		}

		protected FileReference? ProjectFile = null;
		protected bool bMergeModules = false;
		protected bool bAllowUbaCompression = false;
		protected ReadOnlyTargetRules? TargetRules = null;

		/// <inheritdoc/>
		public override void GetVersionInfo(List<string> Lines)
		{
			base.GetVersionInfo(Lines);

			if (TargetRules?.bCompileISPC == true)
			{
				string version = GetISPCHostCompilerVersion(BuildHostPlatform.Current.Platform).ReplaceLineEndings(" ");
				string path = GetISPCHostCompilerPath(BuildHostPlatform.Current.Platform);
				Lines.Add($"Using ISPC compiler ({path})");
				Lines.Add($"  {version}");
			}
		}

		public override void SetUpGlobalEnvironment(ReadOnlyTargetRules Target, CppCompileEnvironment GlobalCompileEnvironment, LinkEnvironment GlobalLinkEnvironment)
		{
			base.SetUpGlobalEnvironment(Target, GlobalCompileEnvironment, GlobalLinkEnvironment);
			ProjectFile = Target.ProjectFile;
			bMergeModules = Target.bMergeModules;
			bAllowUbaCompression = Target.bAllowUbaCompression;
			TargetRules = Target;
		}

		public override void GetExternalDependencies(HashSet<FileItem> ExternalDependencies)
		{
			base.GetExternalDependencies(ExternalDependencies);

			ExternalDependencies.Add(FileItem.GetItemByPath(GetISPCHostCompilerPath(BuildHostPlatform.Current.Platform)));
			string? ISPCHostBytecodeCompilerPath = GetISPCHostBytecodeCompilerPath(BuildHostPlatform.Current.Platform);
			if (ISPCHostBytecodeCompilerPath != null)
			{
				ExternalDependencies.Add(FileItem.GetItemByPath(ISPCHostBytecodeCompilerPath));
			}

			if (TargetRules != null)
			{
				UEBuildPlatformSDK? SDK = UEBuildPlatformSDK.GetSDKForPlatform(TargetRules.Platform.ToString());
				ExternalDependencies.UnionWith(SDK?.ExternalDependencies ?? []);
			}
		}

		/// <summary>
		/// Get CPU Instruction set targets for ISPC.
		/// </summary>
		/// <param name="Platform">Which OS platform to target.</param>
		/// <param name="Arch">Which architecture inside an OS platform to target. Only used for Android currently.</param>
		/// <returns>List of instruction set targets passed to ISPC compiler</returns>
		public virtual List<string> GetISPCCompileTargets(UnrealTargetPlatform Platform, UnrealArch Arch)
		{
			List<string> ISPCTargets = [];

			if (UEBuildPlatform.IsPlatformInGroup(Platform, UnrealPlatformGroup.Windows) ||
				UEBuildPlatform.IsPlatformInGroup(Platform, UnrealPlatformGroup.Unix) ||
				UEBuildPlatform.IsPlatformInGroup(Platform, UnrealPlatformGroup.Apple))
			{
				if (Arch.bIsX64 || Arch == UnrealArch.Arm64ec)
				{
					ISPCTargets.AddRange(["avx512skx-i32x8", "avx2", "avx", "sse4"]);
				}
				else
				{
					ISPCTargets.Add("neon");
				}
			}
			else if (Platform == UnrealTargetPlatform.Android)
			{
				if (Arch == UnrealArch.X64)
				{
					ISPCTargets.Add("sse4");
				}
				else if (Arch == UnrealArch.Arm64)
				{
					ISPCTargets.Add("neon");
				}
				else
				{
					Logger.LogWarning("Invalid Android architecture for ISPC. At least one architecture (arm64, x64) needs to be selected in the project settings to build");
				}
			}
			else
			{
				Logger.LogWarning("Unsupported ISPC platform target!");
			}

			return ISPCTargets;
		}

		/// <summary>
		/// Get OS target for ISPC.
		/// </summary>
		/// <param name="Platform">Which OS platform to target.</param>
		/// <returns>OS string passed to ISPC compiler</returns>
		public virtual string GetISPCOSTarget(UnrealTargetPlatform Platform)
		{
			if (UEBuildPlatform.IsPlatformInGroup(Platform, UnrealPlatformGroup.Windows))
			{
				return "windows";
			}
			else if (UEBuildPlatform.IsPlatformInGroup(Platform, UnrealPlatformGroup.Unix))
			{
				return "linux";
			}
			else if (UEBuildPlatform.IsPlatformInGroup(Platform, UnrealPlatformGroup.Android))
			{
				return "android";
			}
			else if (UEBuildPlatform.IsPlatformInGroup(Platform, UnrealPlatformGroup.IOS))
			{
				return "ios";
			}
			else if (Platform == UnrealTargetPlatform.Mac)
			{
				return "macos";
			}
			
			Logger.LogWarning("Unsupported ISPC platform target!");
			return String.Empty;
		}

		/// <summary>
		/// Get CPU architecture target for ISPC.
		/// </summary>
		/// <param name="Platform">Which OS platform to target.</param>
		/// <param name="Arch">Which architecture inside an OS platform to target. Only used for Android currently.</param>
		/// <returns>Arch string passed to ISPC compiler</returns>
		public virtual string GetISPCArchTarget(UnrealTargetPlatform Platform, UnrealArch Arch)
		{
			if (UEBuildPlatform.IsPlatformInGroup(Platform, UnrealPlatformGroup.Windows) ||
				UEBuildPlatform.IsPlatformInGroup(Platform, UnrealPlatformGroup.Unix) ||
				UEBuildPlatform.IsPlatformInGroup(Platform, UnrealPlatformGroup.Apple) ||
				UEBuildPlatform.IsPlatformInGroup(Platform, UnrealPlatformGroup.Android))
			{
				if (Arch.bIsX64 || Arch == UnrealArch.Arm64ec)
				{
					return "x86-64";
				}
				return "aarch64";
			}
			
			Logger.LogWarning("Unsupported ISPC platform target!");
			return String.Empty;
		}

		/// <summary>
		/// Get CPU target for ISPC.
		/// </summary>
		/// <param name="Platform">Which OS platform to target.</param>
		/// <returns>CPU string passed to ISPC compiler</returns>
		public virtual string? GetISPCCpuTarget(UnrealTargetPlatform Platform)
		{
			return null;  // no specific CPU selected
		}

		/// <summary>
		/// Get host compiler path for ISPC.
		/// </summary>
		/// <param name="HostPlatform">Which OS build platform is running on.</param>
		/// <returns>Path to ISPC compiler</returns>
		public virtual string GetISPCHostCompilerPath(UnrealTargetPlatform HostPlatform)
		{
			string ISPCCompilerPathCommon = Path.Combine(Unreal.EngineSourceDirectory.FullName, "ThirdParty", "Intel", "ISPC", "bin");
			string ISPCArchitecturePath = "";
			string ExeExtension = ".exe";

			if (UEBuildPlatform.IsPlatformInGroup(HostPlatform, UnrealPlatformGroup.Windows))
			{
				ISPCArchitecturePath = "Windows";
			}
			else if (HostPlatform == UnrealTargetPlatform.Linux)
			{
				ISPCArchitecturePath = "Linux";
				ExeExtension = "";
			}
			else if (HostPlatform == UnrealTargetPlatform.Mac)
			{
				ISPCArchitecturePath = "Mac";
				ExeExtension = "";
			}
			else
			{
				Logger.LogWarning("Unsupported ISPC host!");
			}

			return Path.Combine(ISPCCompilerPathCommon, ISPCArchitecturePath, "ispc" + ExeExtension);
		}

		/// <summary>
		/// Get the host bytecode-to-obj compiler path for ISPC. Only used for platforms that support compiling ISPC to LLVM bytecode
		/// </summary>
		/// <param name="HostPlatform">Which OS build platform is running on.</param>
		/// <returns>Path to bytecode to obj compiler</returns>
		public virtual string? GetISPCHostBytecodeCompilerPath(UnrealTargetPlatform HostPlatform)
		{
			// Return null if the platform toolchain doesn't support separate bytecode to obj compilation
			return null;
		}

		static readonly ConcurrentDictionary<string, Lazy<string>> s_ISPCCompilerVersions = [];

		/// <summary>
		/// Returns the version of the ISPC compiler for the specified platform. If GetISPCHostCompilerPath() doesn't return a valid path
		/// this will return a -1 version.
		/// </summary>
		/// <param name="platform">Which OS build platform is running on.</param>
		/// <returns>Version reported by the ISPC compiler</returns>
		public virtual string GetISPCHostCompilerVersion(UnrealTargetPlatform platform)
		{
			string compilerPath = GetISPCHostCompilerPath(platform);
			return s_ISPCCompilerVersions.GetOrAdd(compilerPath, new Lazy<string>(() =>
			{
				if (File.Exists(compilerPath))
				{
					return RunToolAndCaptureOutput(new FileReference(compilerPath), "--version", "(.*)")!;
				}
				else
				{
					Logger.LogWarning("No ISPC compiler at {CompilerPath}", compilerPath);
					return "-1";
				}
			})).Value;
		}

		/// <summary>
		/// Get object file format for ISPC.
		/// </summary>
		/// <param name="Platform">Which OS build platform is running on.</param>
		/// <returns>Object file suffix</returns>
		public virtual string GetISPCObjectFileFormat(UnrealTargetPlatform Platform)
		{
			if (UEBuildPlatform.IsPlatformInGroup(Platform, UnrealPlatformGroup.Windows) ||
				UEBuildPlatform.IsPlatformInGroup(Platform, UnrealPlatformGroup.Unix) ||
				UEBuildPlatform.IsPlatformInGroup(Platform, UnrealPlatformGroup.Apple) ||
				UEBuildPlatform.IsPlatformInGroup(Platform, UnrealPlatformGroup.Android))
			{
				return "obj";
			}

			Logger.LogWarning("Unsupported ISPC platform target!");
			return String.Empty;
		}

		/// <summary>
		/// Get object file suffix for ISPC.
		/// </summary>
		/// <param name="Platform">Which OS build platform is running on.</param>
		/// <returns>Object file suffix</returns>
		public virtual string GetISPCObjectFileSuffix(UnrealTargetPlatform Platform)
		{
			if (UEBuildPlatform.IsPlatformInGroup(Platform, UnrealPlatformGroup.Windows))
			{
				return ".obj";
			}
			else if (UEBuildPlatform.IsPlatformInGroup(Platform, UnrealPlatformGroup.Unix) ||
				UEBuildPlatform.IsPlatformInGroup(Platform, UnrealPlatformGroup.Apple) ||
				UEBuildPlatform.IsPlatformInGroup(Platform, UnrealPlatformGroup.Android))
			{
				return ".o";
			}

			Logger.LogWarning("Unsupported ISPC platform target!");
			return String.Empty;
		}

		private static string EscapeDefinitionForISPC(string Definition)
		{
			// See: https://github.com/ispc/ispc/blob/4ee767560cd752eaf464c124eb7ef1b0fd37f1df/src/main.cpp#L264 for ispc's argument parsing code, which does the following (and does not support escaping):
			// Argument      Parses as 
			// "abc""def"    One agrument:  abcdef
			// "'abc'"       One argument:  'abc'
			// -D"X="Y Z""   Two arguments: -DX=Y and Z
			// -D'X="Y Z"'   One argument:  -DX="Y Z"  (i.e. with quotes in value)
			// -DX="Y Z"     One argument:  -DX=Y Z    (this is what we want on the command line)

			// Assumes that quotes at the start and end of the value string mean that everything between them should be passed on unchanged.

			int DoubleQuoteCount = Definition.Count(c => c == '"');
			bool bHasSingleQuote = Definition.Contains('\'', StringComparison.Ordinal);
			bool bHasSpace = Definition.Contains(' ', StringComparison.Ordinal);

			string Escaped = Definition;

			if (DoubleQuoteCount > 0 || bHasSingleQuote || bHasSpace)
			{
				int EqualsIndex = Definition.IndexOf('=', StringComparison.Ordinal);
				string Name = Definition[0..EqualsIndex];
				string Value = Definition[(EqualsIndex + 1)..];

				string UnquotedValue = Value;

				// remove one layer of quoting, if present
				if (Value.StartsWith('"') && Value.EndsWith('"') && Value.Length != 1)
				{
					UnquotedValue = Value[1..^1];
					DoubleQuoteCount -= 2;
				}

				if (DoubleQuoteCount == 0 && (bHasSingleQuote || bHasSpace))
				{
					Escaped = $"{Name}=\"{UnquotedValue}\"";
				}
				else if (!bHasSingleQuote && (bHasSpace || DoubleQuoteCount > 0))
				{
					// If there are no single quotes, we can use them to quote the value string
					Escaped = $"{Name}='{UnquotedValue}'";
				}
				else
				{
					// Treat all special chars in the value string as needing explicit extra quoting. Thoroughly clumsy.
					StringBuilder Requoted = new();
					foreach (char c in UnquotedValue)
					{
						if (c == '"')
						{
							Requoted.Append("'\"'");
						}
						else if (c == '\'')
						{
							Requoted.Append("\"'\"");
						}
						else if (c == ' ')
						{
							Requoted.Append("\" \"");
						}
						else
						{
							Requoted.Append(c);
						}
					}
					Escaped = $"{Name}={Requoted}";
				}
			}

			return Escaped;
		}

		/// <summary>
		/// Simple hash function for strings (can be chained)
		/// </summary>
		/// <param name="str">The string to hash</param>
		/// <param name="hash">The hash input</param>
		/// <returns>hash of input hash + string</returns>
		static uint GetStringHash(string str, uint hash = 5381)
		{
			for (int idx = 0; idx < str.Length; idx++)
			{
				hash += (hash << 5) + str[idx];
			}
			return hash;
		}

		/// <summary>
		/// Calculates hash bucket based on target, do not call directly
		/// </summary>
		/// <param name="target">Target rules</param>
		/// <returns>bucket hash</returns>
		static uint GetCacheBucket(ReadOnlyTargetRules target)
		{
			IEnumerable<string> strings = [
				target.CacheSalt ?? String.Empty,			// Optional salt
				target.BuildEnvironment.ToString(),			// Unique or Share environment
				target.bMergeModules ? "MM2" : String.Empty // Merged modules will modify all object files
			];
			return strings.Aggregate(GetStringHash(String.Empty), (acc, str) => GetStringHash(str, acc));
		}

		/// <summary>
		/// Calculates hash bucket based on target for ISPC
		/// </summary>
		/// <param name="target">Target rules</param>
		/// <param name="compileEnvironment">Compile environment</param>
		/// <returns>bucket hash</returns>
		static uint GetISPCCacheBucket(ReadOnlyTargetRules? target, CppCompileEnvironment compileEnvironment)
		{
			if (target == null)
			{
				return 0;
			}

			IEnumerable<string> strings = [
				"ISCP",										// Create a separate bucket for ispc
				compileEnvironment.Platform.ToString(),		// Platform
				compileEnvironment.Architecture.ToString(),	// ispc only ever has a single arch
			];
			return strings.Aggregate(GetCacheBucket(target), (acc, str) => GetStringHash(str, acc));
		}

		/// <summary>
		/// Calculates hash bucket based on target and compile environment
		/// </summary>
		/// <param name="target">Target rules</param>
		/// <param name="compileEnvironment">Compile environment</param>
		/// <returns>bucket hash</returns>
		public static uint GetCacheBucket(ReadOnlyTargetRules? target, CppCompileEnvironment compileEnvironment)
		{
			if (target == null)
			{
				return 0;
			}
			IEnumerable<string> strings = [
				compileEnvironment.Platform.ToString(), // Platform
				compileEnvironment.Configuration.ToString(), // Configuration
				target.LinkType.ToString(), // Modular or monolithic (we want this for compiles too in order to end up in same bucket
				!compileEnvironment.Architectures.bIsMultiArch ? compileEnvironment.Architecture.ToString() : compileEnvironment.Architectures.ToString(), // arm64 or x64
				target.StaticAnalyzer != StaticAnalyzer.None && !compileEnvironment.bDisableStaticAnalysis ? target.StaticAnalyzer.ToString() : String.Empty, // If static analyzer
				compileEnvironment.RootPaths.bUseVfs ? "VFS" : String.Empty,
				compileEnvironment.bUseAutoRTFMCompiler ? "AUTORTFM" : String.Empty, // AutoRTFM
				!compileEnvironment.bUseUnity ? "NU" : String.Empty, // Non-unity
			];

			// Optimize settings
			if (compileEnvironment.bPGOProfile)
			{
				strings = [.. strings, "PGOProfile"];
			}
			else if (compileEnvironment.bPGOOptimize)
			{
				strings = [.. strings, "PGOOptimize"];
			}
			else if (compileEnvironment.bAllowLTCG)
			{
				strings = [.. strings, "LTCG"];
			}

			// If vfs is disabled and pch is enabled
			if (!compileEnvironment.RootPaths.bUseVfs && compileEnvironment.PrecompiledHeaderAction != PrecompiledHeaderAction.None)
			{
				strings = [.. strings, .. compileEnvironment.RootPaths.Select(item => item.local.FullName)];
			}
			return strings.Aggregate(GetCacheBucket(target), (acc, str) => GetStringHash(str, acc));
		}

		/// <summary>
		/// Calculates hash bucket based on target and link environment
		/// </summary>
		/// <param name="target">Target rules</param>
		/// <param name="linkEnvironment">Link environment</param>
		/// <returns>bucket hash</returns>
		public static uint GetCacheBucket(ReadOnlyTargetRules? target, LinkEnvironment linkEnvironment)
		{
			if (target == null)
			{
				return 0;
			}

			IEnumerable<string> strings = [
				linkEnvironment.Platform.ToString(), // Platform
				linkEnvironment.Configuration.ToString(), // Configuration
				target.LinkType.ToString(), // Modular or monolithic
				!linkEnvironment.Architectures.bIsMultiArch ? linkEnvironment.Architecture.ToString() : linkEnvironment.Architectures.ToString(), // arm64 or x64
				linkEnvironment.RootPaths.bUseVfs ? "VFS" : String.Empty,
				target.bUseAutoRTFMCompiler ? "AUTORTFM" : String.Empty, // AutoRTFM
				!target.bUseUnityBuild ? "NU" : String.Empty // Non-unity
			];

			// Optimize settings
			if (linkEnvironment.bPGOProfile)
			{
				strings = [.. strings, "PGOProfile"];
			}
			else if (linkEnvironment.bPGOOptimize)
			{
				strings = [.. strings, "PGOOptimize"];
			}
			else if (linkEnvironment.bAllowLTCG)
			{
				strings = [.. strings, "LTCG"];
			}
			return strings.Aggregate(GetCacheBucket(target), (acc, str) => GetStringHash(str, acc));
		}

		protected override CPPOutput GenerateISPCHeaders(CppCompileEnvironment CompileEnvironment, IEnumerable<FileItem> InputFiles, DirectoryReference OutputDir, IActionGraphBuilder Graph)
		{
			CPPOutput Result = new();

			if (!CompileEnvironment.bCompileISPC)
			{
				return Result;
			}

			CppRootPaths RootPaths = CompileEnvironment.RootPaths;

			List<string> CompileTargets = GetISPCCompileTargets(CompileEnvironment.Platform, CompileEnvironment.Architecture);

			List<string> GlobalArguments = [];

			// Build target string. No comma on last
			string TargetString = String.Join(',', CompileTargets);

			string ISPCArch = GetISPCArchTarget(CompileEnvironment.Platform, CompileEnvironment.Architecture);

			// Build target triplet
			GlobalArguments.Add($"--target-os={GetISPCOSTarget(CompileEnvironment.Platform)}");
			GlobalArguments.Add($"--arch={ISPCArch}");
			GlobalArguments.Add($"--target={TargetString}");
			GlobalArguments.Add($"--emit-{GetISPCObjectFileFormat(CompileEnvironment.Platform)}");

			string? CpuTarget = GetISPCCpuTarget(CompileEnvironment.Platform);
			if (!String.IsNullOrEmpty(CpuTarget))
			{
				GlobalArguments.Add($"--cpu={CpuTarget}");
			}

			// PIC is needed for modular builds except on Microsoft platforms, and for android
			if ((CompileEnvironment.bIsBuildingDLL ||
				CompileEnvironment.bIsBuildingLibrary) &&
				!UEBuildPlatform.IsPlatformInGroup(CompileEnvironment.Platform, UnrealPlatformGroup.Microsoft) ||
				UEBuildPlatform.IsPlatformInGroup(CompileEnvironment.Platform, UnrealPlatformGroup.Android))
			{
				GlobalArguments.Add("--pic");
			}

			// Include paths. Don't use AddIncludePath() here, since it uses the full path and exceeds the max command line length.
			// Because ISPC response files don't support white space in arguments, paths with white space need to be passed to the command line directly.
			foreach (DirectoryReference IncludePath in CompileEnvironment.UserIncludePaths)
			{
				GlobalArguments.Add($"-I\"{NormalizeCommandLinePath(IncludePath, RootPaths)}\"");
			}

			// System include paths.
			foreach (DirectoryReference SystemIncludePath in CompileEnvironment.SystemIncludePaths)
			{
				GlobalArguments.Add($"-I\"{NormalizeCommandLinePath(SystemIncludePath, RootPaths)}\"");
			}

			// Preprocessor definitions.
			foreach (string Definition in CompileEnvironment.Definitions)
			{
				// TODO: Causes ISPC compiler to generate a spurious warning about the universal character set
				if (!Definition.Contains("\\\\U", StringComparison.Ordinal) && !Definition.Contains("\\\\u", StringComparison.Ordinal))
				{
					GlobalArguments.Add($"-D{EscapeDefinitionForISPC(Definition)}");
				}
			}

			foreach (FileItem ISPCFile in InputFiles)
			{
				Action CompileAction = Graph.CreateAction(ActionType.Compile);
				CompileAction.RootPaths = CompileEnvironment.RootPaths;

				CompileAction.CommandDescription = $"Generate Header [{ISPCArch}]";
				CompileAction.WorkingDirectory = Unreal.EngineSourceDirectory;
				CompileAction.CommandPath = new FileReference(GetISPCHostCompilerPath(BuildHostPlatform.Current.Platform));
				CompileAction.StatusDescription = Path.GetFileName(ISPCFile.AbsolutePath);
				CompileAction.CommandVersion = GetISPCHostCompilerVersion(BuildHostPlatform.Current.Platform);
				CompileAction.ArtifactMode = ArtifactMode.Enabled;
				CompileAction.CacheBucket = GetISPCCacheBucket(TargetRules, CompileEnvironment);

				CompileAction.bCanExecuteRemotely = true;
				CompileAction.bCanExecuteInUBACrossArchitecture = false;

				if (RuntimeInformation.ProcessArchitecture == Architecture.Arm64)
				{
					CompileAction.bCanExecuteInUBA = false;
				}

				// Disable remote execution to workaround mismatched case on XGE
				CompileAction.bCanExecuteRemotelyWithXGE = false;

				// TODO: Remove, might work
				CompileAction.bCanExecuteRemotelyWithSNDBS = false;

				List<string> Arguments =
				[
					// Add the ISPC obj file as a prerequisite of the action.
					$"\"{NormalizeCommandLinePath(ISPCFile, RootPaths)}\"",
				];

				// Add the ISPC h file to the produced item list.
				FileItem ISPCIncludeHeaderFile = FileItem.GetItemByFileReference(
					FileReference.Combine(
						OutputDir,
						Path.GetFileName(ISPCFile.AbsolutePath) + ".generated.dummy.h"
						)
					);

				// Add the ISPC h files to the produced item list.
				CompileAction.ProducedItems.Add(ISPCIncludeHeaderFile);
				
				// Add the target specific dummy headers to the produced item list.
				// These are unused but should still be tracked.
				if (CompileTargets.Count > 1)
				{
					foreach (string Target in CompileTargets)
					{
						string HeaderTarget = Target;

						if (Target.Contains('-', StringComparison.Ordinal))
						{
							// Remove lane width and gang size from header file name
							HeaderTarget = Target.Split('-')[0];
						}

						FileItem ISPCTargetIncludeHeaderFile = FileItem.GetItemByFileReference(
							FileReference.Combine(
								OutputDir,
								Path.GetFileName(ISPCFile.AbsolutePath) + ".generated.dummy_" + HeaderTarget + ".h"
								)
							);
						CompileAction.ProducedItems.Add(ISPCTargetIncludeHeaderFile);
					}
				}

				// Add the ISPC file to be compiled.
				Arguments.Add($"-h \"{NormalizeCommandLinePath(ISPCIncludeHeaderFile, RootPaths)}\"");

				// Generate the included header dependency list
				FileItem DependencyListFile = FileItem.GetItemByFileReference(FileReference.Combine(OutputDir, Path.GetFileName(ISPCFile.AbsolutePath) + ".txt"));
				Arguments.Add($"-MMM \"{NormalizeCommandLinePath(DependencyListFile, RootPaths)}\"");
				CompileAction.DependencyListFile = DependencyListFile;
				CompileAction.ProducedItems.Add(DependencyListFile);

				Arguments.AddRange(GlobalArguments);

				FileReference ResponseFileName = GetResponseFileName(ISPCIncludeHeaderFile);
				CompileAction.ResponseFileContents = [.. Arguments.Select(x => Utils.ExpandVariables(x))];
				FileItem ResponseFileItem = Graph.CreateIntermediateTextFile(ResponseFileName, CompileAction.ResponseFileContents);
				CompileAction.CommandArguments = $"@\"{NormalizeCommandLinePath(ResponseFileName, RootPaths)}\"";
				CompileAction.PrerequisiteItems.Add(ResponseFileItem);

				// Add the source file and its included files to the prerequisite item list.
				CompileAction.PrerequisiteItems.Add(ISPCFile);

				FileItem ISPCFinalHeaderFile = FileItem.GetItemByFileReference(
					FileReference.Combine(
						OutputDir,
						Path.GetFileName(ISPCFile.AbsolutePath) + ".generated.h"
						)
					);

				// Fix interrupted build issue by copying header after generation completes
				Action CopyAction = Graph.CreateCopyAction(ISPCIncludeHeaderFile, ISPCFinalHeaderFile);
				CopyAction.CommandDescription = $"{CopyAction.CommandDescription} [{ISPCArch}]";
				CopyAction.DeleteItems.Clear();
				CopyAction.PrerequisiteItems.Add(ISPCFile);
				CopyAction.bShouldOutputStatusDescription = false;

				Result.GeneratedHeaderFiles.Add(ISPCFinalHeaderFile);
			}

			return Result;
		}

		protected override CPPOutput CompileISPCFiles(CppCompileEnvironment CompileEnvironment, IEnumerable<FileItem> InputFiles, DirectoryReference OutputDir, IActionGraphBuilder Graph)
		{
			CPPOutput Result = new();

			if (!CompileEnvironment.bCompileISPC)
			{
				return Result;
			}

			CppRootPaths RootPaths = CompileEnvironment.RootPaths;

			List<string> CompileTargets = GetISPCCompileTargets(CompileEnvironment.Platform, CompileEnvironment.Architecture);

			List<string> GlobalArguments = [];

			// Build target string. No comma on last
			string TargetString = "";
			foreach (string Target in CompileTargets)
			{
				if (Target == CompileTargets[^1]) // .Last()
				{
					TargetString += Target;
				}
				else
				{
					TargetString += Target + ",";
				}
			}

			// Build target triplet
			string PlatformObjectFileFormat = GetISPCObjectFileFormat(CompileEnvironment.Platform);
			string ISPCArch = GetISPCArchTarget(CompileEnvironment.Platform, CompileEnvironment.Architecture);

			GlobalArguments.Add($"--target-os={GetISPCOSTarget(CompileEnvironment.Platform)}");
			GlobalArguments.Add($"--arch={ISPCArch}");
			GlobalArguments.Add($"--target={TargetString}");
			GlobalArguments.Add($"--emit-{PlatformObjectFileFormat}");

			string? CpuTarget = GetISPCCpuTarget(CompileEnvironment.Platform);
			if (!String.IsNullOrEmpty(CpuTarget))
			{
				GlobalArguments.Add($"--cpu={CpuTarget}");
			}

			bool bByteCodeOutput = (PlatformObjectFileFormat == "llvm");

			List<string> CommonArgs = [];
			if (CompileEnvironment.Configuration == CppConfiguration.Debug)
			{
				if (CompileEnvironment.Platform == UnrealTargetPlatform.Mac)
				{
					// Turn off debug symbols on Mac due to dsym generation issue
					CommonArgs.Add("-O0");
					// Ideally we would be able to turn on symbols and specify the dwarf version, but that does
					// does not seem to be working currently, ie:
					//    GlobalArguments.Add("-g -O0 --dwarf-version=2");

				}
				else
				{
					CommonArgs.Add("-g -O0");
				}
			}
			else
			{
				CommonArgs.Add("-O3");
			}
			GlobalArguments.AddRange(CommonArgs);

			// PIC is needed for modular builds except on Microsoft platforms, and for android
			if ((CompileEnvironment.bIsBuildingDLL ||
				CompileEnvironment.bIsBuildingLibrary) &&
				!UEBuildPlatform.IsPlatformInGroup(CompileEnvironment.Platform, UnrealPlatformGroup.Microsoft) ||
				UEBuildPlatform.IsPlatformInGroup(CompileEnvironment.Platform, UnrealPlatformGroup.Android))
			{
				GlobalArguments.Add("--pic");
			}

			// Include paths. Don't use AddIncludePath() here, since it uses the full path and exceeds the max command line length.
			foreach (DirectoryReference IncludePath in CompileEnvironment.UserIncludePaths)
			{
				GlobalArguments.Add($"-I\"{NormalizeCommandLinePath(IncludePath, RootPaths)}\"");
			}

			// System include paths.
			foreach (DirectoryReference SystemIncludePath in CompileEnvironment.SystemIncludePaths)
			{
				GlobalArguments.Add($"-I\"{NormalizeCommandLinePath(SystemIncludePath, RootPaths)}\"");
			}

			// Preprocessor definitions.
			foreach (string Definition in CompileEnvironment.Definitions)
			{
				// TODO: Causes ISPC compiler to generate a spurious warning about the universal character set
				if (!Definition.Contains("\\\\U", StringComparison.Ordinal) && !Definition.Contains("\\\\u", StringComparison.Ordinal))
				{
					GlobalArguments.Add($"-D{EscapeDefinitionForISPC(Definition)}");
				}
			}

			foreach (FileItem ISPCFile in InputFiles)
			{
				Action CompileAction = Graph.CreateAction(ActionType.Compile);
				CompileAction.RootPaths = RootPaths;
				CompileAction.CommandDescription = $"Compile [{ISPCArch}]";
				CompileAction.WorkingDirectory = Unreal.EngineSourceDirectory;
				CompileAction.CommandPath = new FileReference(GetISPCHostCompilerPath(BuildHostPlatform.Current.Platform));
				CompileAction.StatusDescription = Path.GetFileName(ISPCFile.AbsolutePath);
				CompileAction.CommandVersion = GetISPCHostCompilerVersion(BuildHostPlatform.Current.Platform).ToString();
				if (bAllowUbaCompression)
				{
					CompileAction.CommandVersion = $"{CompileAction.CommandVersion} Compressed";
				}

				CompileAction.bCanExecuteRemotely = true;
				CompileAction.bCanExecuteInUBACrossArchitecture = false;

				if (RuntimeInformation.ProcessArchitecture == Architecture.Arm64)
				{
					CompileAction.bCanExecuteInUBA = false;
				}

				// Disable remote execution to workaround mismatched case on XGE
				CompileAction.bCanExecuteRemotelyWithXGE = false;

				// TODO: Remove, might work
				CompileAction.bCanExecuteRemotelyWithSNDBS = false;

				CompileAction.ArtifactMode = ArtifactMode.Enabled;
				CompileAction.CacheBucket = GetISPCCacheBucket(TargetRules, CompileEnvironment);

				List<string> Arguments =
				[
					// Add the ISPC file to be compiled.
					$"\"{NormalizeCommandLinePath(ISPCFile, RootPaths)}\"",
				];

				List<FileItem> CompiledISPCObjFiles = [];

				string FileName = Path.GetFileName(ISPCFile.AbsolutePath);

				string CompiledISPCObjFileSuffix = bByteCodeOutput ? ".bc" : GetISPCObjectFileSuffix(CompileEnvironment.Platform);
				foreach (string Target in CompileTargets)
				{
					string ObjTarget = Target;

					if (Target.Contains('-', StringComparison.Ordinal))
					{
						// Remove lane width and gang size from obj file name
						ObjTarget = Target.Split('-')[0];
					}

					FileItem CompiledISPCObjFile;

					if (CompileTargets.Count > 1)
					{
						CompiledISPCObjFile = FileItem.GetItemByFileReference(
						FileReference.Combine(
							OutputDir,
							FileName + "_" + ObjTarget + CompiledISPCObjFileSuffix
							)
						);
					}
					else
					{
						CompiledISPCObjFile = FileItem.GetItemByFileReference(
						FileReference.Combine(
							OutputDir,
							FileName + CompiledISPCObjFileSuffix
							)
						);
					}

					// Add the ISA specific ISPC obj files to the produced item list.
					CompiledISPCObjFiles.Add(CompiledISPCObjFile);
				}

				// Add the common ISPC obj file to the produced item list if it's not already in it
				FileItem CompiledISPCObjFileNoISA = FileItem.GetItemByFileReference(
					FileReference.Combine(
						OutputDir,
						FileName + CompiledISPCObjFileSuffix
						)
					);

				if (CompileTargets.Count > 1)
				{
					CompiledISPCObjFiles.Add(CompiledISPCObjFileNoISA);
				}

				// Add the output ISPC obj file
				Arguments.Add($"-o \"{NormalizeCommandLinePath(CompiledISPCObjFileNoISA, RootPaths)}\"");

				// Generate the timing info
				if (CompileEnvironment.bPrintTimingInfo)
				{
					FileItem TraceFile = FileItem.GetItemByFileReference(new($"{CompiledISPCObjFileNoISA}.json"));
					Arguments.Add("--time-trace");
					CompileAction.ProducedItems.Add(TraceFile);
				}

				Arguments.AddRange(GlobalArguments);

				// Consume the included header dependency list
				FileItem DependencyListFile = FileItem.GetItemByFileReference(FileReference.Combine(OutputDir, FileName + ".txt"));
				CompileAction.DependencyListFile = DependencyListFile;
				CompileAction.PrerequisiteItems.Add(DependencyListFile);

				CompileAction.ProducedItems.UnionWith(CompiledISPCObjFiles);

				FileReference ResponseFileName = GetResponseFileName(CompiledISPCObjFileNoISA);
				CompileAction.ResponseFileContents = [.. Arguments.Select(x => Utils.ExpandVariables(x))];
				FileItem ResponseFileItem = Graph.CreateIntermediateTextFile(ResponseFileName, CompileAction.ResponseFileContents);

				string AdditionalArguments = "";
				// Must be added after response file is created just to make sure it ends up on the command line and not in the response file
				if (!bByteCodeOutput && bMergeModules)
				{
					// EXTRACTEXPORTS can only be interpreted by UBA.. so this action won't build outside uba
					AdditionalArguments = " /EXTRACTEXPORTS";
					CompileAction.ProducedItems.Add(FileItem.GetItemByFileReference(FileReference.Combine(OutputDir, FileName + ".exi")));

					foreach (FileItem file in CompiledISPCObjFiles)
					{
						CompileAction.ProducedItems.Add(FileItem.GetItemByFileReference(file.Location.ChangeExtension(".exi")));
					}
				}

				CompileAction.CommandArguments = $"@\"{NormalizeCommandLinePath(ResponseFileName, RootPaths)}\"{AdditionalArguments}";
				CompileAction.PrerequisiteItems.Add(ResponseFileItem);

				// Add the source file and its included files to the prerequisite item list.
				CompileAction.PrerequisiteItems.Add(ISPCFile);

				if (bByteCodeOutput)
				{
					// If the platform toolchain supports bytecode compilation for ISPC, compile the bytecode object files to actual native object files 
					string? ByteCodeCompilerPath = GetISPCHostBytecodeCompilerPath(BuildHostPlatform.Current.Platform);
					if (ByteCodeCompilerPath != null)
					{
						List<FileItem> FinalObjectFiles = [];
						foreach (FileItem CompiledBytecodeObjFile in CompiledISPCObjFiles)
						{
							string FileNameWithoutExtension = Path.GetFileNameWithoutExtension(CompiledBytecodeObjFile.AbsolutePath);
							FileItem FinalCompiledISPCObjFile = FileItem.GetItemByFileReference(
								FileReference.Combine(
									OutputDir,
									FileNameWithoutExtension + GetISPCObjectFileSuffix(CompileEnvironment.Platform)
									)
								);

							Action PostCompileAction = Graph.CreateAction(ActionType.Compile);

							List<string> PostCompileArgs =
							[
								$"\"{NormalizeCommandLinePath(CompiledBytecodeObjFile, RootPaths)}\"",
								"-c",
								.. CommonArgs,
								$"-o \"{NormalizeCommandLinePath(FinalCompiledISPCObjFile, RootPaths)}\"",
							];

							if (bMergeModules)
							{
								// EXTRACTEXPORTS can only be interpreted by UBA.. so this action won't build outside uba
								AdditionalArguments = " /EXTRACTEXPORTS";
								PostCompileAction.ProducedItems.Add(FileItem.GetItemByFileReference(FileReference.Combine(OutputDir, FileNameWithoutExtension + ".exi")));
							}

							// Write the args to a response file
							FileReference PostCompileResponseFileName = GetResponseFileName(FinalCompiledISPCObjFile);
							PostCompileAction.ResponseFileContents = [.. PostCompileArgs.Select(x => Utils.ExpandVariables(x))];
							FileItem PostCompileResponseFileItem = Graph.CreateIntermediateTextFile(PostCompileResponseFileName, PostCompileAction.ResponseFileContents);
							PostCompileAction.CommandArguments = $"@\"{NormalizeCommandLinePath(PostCompileResponseFileName, RootPaths)}\"{AdditionalArguments}";
							PostCompileAction.PrerequisiteItems.Add(PostCompileResponseFileItem);

							PostCompileAction.PrerequisiteItems.Add(CompiledBytecodeObjFile);
							PostCompileAction.ProducedItems.Add(FinalCompiledISPCObjFile);
							PostCompileAction.CommandDescription = $"CompileByteCode [{ISPCArch}]";
							PostCompileAction.WorkingDirectory = Unreal.EngineSourceDirectory;
							PostCompileAction.CommandPath = new FileReference(ByteCodeCompilerPath);
							PostCompileAction.StatusDescription = Path.GetFileName(ISPCFile.AbsolutePath);
							CompileAction.CommandVersion = GetISPCHostCompilerVersion(BuildHostPlatform.Current.Platform).ToString();
							if (bAllowUbaCompression)
							{
								CompileAction.CommandVersion = $"{CompileAction.CommandVersion} Compressed";
							}

							PostCompileAction.RootPaths = RootPaths;
							PostCompileAction.ArtifactMode = ArtifactMode.Enabled;
							PostCompileAction.CacheBucket = GetISPCCacheBucket(TargetRules, CompileEnvironment);

							// Disable remote execution to workaround mismatched case on XGE
							PostCompileAction.bCanExecuteRemotelyWithXGE = false;

							FinalObjectFiles.Add(FinalCompiledISPCObjFile);
						}
						// Override the output object files
						CompiledISPCObjFiles = FinalObjectFiles;
					}
				}

				Result.ObjectFiles.AddRange(CompiledISPCObjFiles);
			}

			return Result;
		}
	}
}
