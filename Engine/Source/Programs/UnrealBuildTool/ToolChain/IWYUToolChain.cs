// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	class IWYUToolChain : ClangToolChain
	{
		private readonly List<string> CrossCompilingArguments = new();
		private readonly string IWYUMappingFile;
		private readonly string RelativePathToIWYUDirectory = @"Binaries\ThirdParty\IWYU";
		private readonly UnrealTargetPlatform TargetPlatform;
		private readonly ISPCToolChain InnerToolChain;

		public static void ValidateTarget(TargetRules Target)
		{
			Target.DebugInfo = DebugInfoMode.None;
			Target.bDisableLinking = true;
			Target.bUsePCHFiles = false;
			Target.bUseSharedPCHs = false;
			Target.bUseUnityBuild = false;
			Target.bCompileISPC = false;
		}

		public IWYUToolChain(ReadOnlyTargetRules target, ISPCToolChain internalToolChain, ILogger InLogger) : base(ClangToolChainOptions.None, InLogger)
		{
			if (!OperatingSystem.IsWindows())
			{
				throw new BuildException("IWYU only works on win64 x64 platform");
			}
			if (target.Platform != UnrealTargetPlatform.Win64)
			{
				throw new BuildException("IWYU only works on win64 targets. Epic is using UnrealEditor win64 development when running iwyu");
			}

			TargetPlatform = target.Platform;
			InnerToolChain = internalToolChain;

			// set up the path to our toolchains
			IWYUMappingFile = FileReference.Combine(Unreal.EngineDirectory, RelativePathToIWYUDirectory, "ue_mapping.imp").ToNormalizedPath();
			if (IWYUMappingFile == null)
			{
				throw new BuildException("It seems you don't have access to NotForLicensees folder.\n" +
										 "IWYU is not yet released to the public." +
										 "We are working on validating so we can release the modified iwyu exe");
			}
		}

		protected override ClangToolChainInfo GetToolChainInfo()
		{
			if (TargetPlatform == UnrealTargetPlatform.Linux)
			{
				LinuxPlatformSDK PlatformSDK = new LinuxPlatformSDK(Logger);
				DirectoryReference? BaseLinuxPath = PlatformSDK.GetBaseLinuxPathForArchitecture(LinuxPlatform.DefaultHostArchitecture);

				SourceFileMetadataCache MetadataCache = SourceFileMetadataCache.CreateHierarchy(null, Logger);
				CppCompileEnvironment CrossCompileEnvironment = new CppCompileEnvironment(UnrealTargetPlatform.Linux, CppConfiguration.Development, new(LinuxPlatform.DefaultHostArchitecture), MetadataCache);

				CrossCompilingArguments.Add($"--sysroot=\"{NormalizeCommandLinePath(BaseLinuxPath!, CrossCompileEnvironment.RootPaths)}\"");

				FileReference ClangPath = FileReference.Combine(BaseLinuxPath!, "bin", $"clang++{BuildHostPlatform.Current.BinarySuffix}");
				ClangToolChainInfo CompilerToolChainInfo = new ClangToolChainInfo(BaseLinuxPath, ClangPath, null!, Logger);

				// starting with clang 16.x the directory naming changed to include major version only
				string ClangVersionString = (CompilerToolChainInfo.ClangVersion.Major >= 16) ? CompilerToolChainInfo.ClangVersion.Major.ToString() : CompilerToolChainInfo.ClangVersion.ToString();
				DirectoryReference SystemPath = DirectoryReference.Combine(BaseLinuxPath!, "lib", "clang", ClangVersionString, "include");
				CrossCompilingArguments.Add(GetSystemIncludePathArgument(SystemPath, CrossCompileEnvironment));
			}

			FileReference IWYUPath = FileReference.Combine(Unreal.EngineDirectory, RelativePathToIWYUDirectory, @"include-what-you-use.exe");
			//FileReference IWYUPath = FileReference.FromString(@"e:\dev\fn\Engine\Restricted\NotForLicensees\Source\ThirdParty\IWYU\iwyu-0.23\vs_projects\bin\RelWithDebInfo\include-what-you-use.exe");
			return new ClangToolChainInfo(IWYUPath.Directory, IWYUPath, IWYUPath, Logger);
		}

		protected override string GetFileNameFromExtension(string AbsolutePath, string Extension)
		{
			string FileName = Path.GetFileName(AbsolutePath);

			if (Extension == ".o")
			{
				Extension = ".iwyu";
			}

			return FileName + Extension;
		}

        /// <inheritdoc/>
        public override string NormalizeCommandLinePath(FileSystemReference Reference, CppRootPaths RootPath)
		{
			if (RootPath.GetVfsOverlayPath(Reference, out string? VfsPath))
			{
				return VfsPath;
			}
			return Reference.FullName.Replace('\\', '/');
		}

		/// <inheritdoc/>
		public override string NormalizeCommandLinePath(FileItem Item, CppRootPaths RootPath)
		{
			return NormalizeCommandLinePath(Item.Location, RootPath);
		}

		// Code below here is mostly copy-pasted from LinuxToolChain.
		// We didn't inherit from LinuxToolChain on purpose in order to be able to add support for running iwyu against other clang platforms than linux

		private static bool ShouldUseLibcxx()
		{
			// set UE_LINUX_USE_LIBCXX to either 0 or 1. If unset, defaults to 1.
			string? UseLibcxxEnvVarOverride = Environment.GetEnvironmentVariable("UE_LINUX_USE_LIBCXX");
			if (String.IsNullOrEmpty(UseLibcxxEnvVarOverride) || UseLibcxxEnvVarOverride == "1")
			{
				return true;
			}
			return false;
		}

		protected override void GetCompileArguments_Global(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			if (CompileEnvironment.bHasSharedResponseFile)
			{
				return;
			}

			Arguments.Add(GetPreprocessorDefinitionArgument("UE_DIRECT_HEADER_COMPILE=1"));
			Arguments.Add(GetPreprocessorDefinitionArgument("PLATFORM_COMPILER_IWYU=1"));

			Arguments.Add("-Xiwyu --mapping_file=" + IWYUMappingFile);
			Arguments.Add("-Xiwyu --no_default_mappings");
			//Arguments.Add("-Xiwyu --delayed_template_parsing");
			Arguments.Add("-Xiwyu --prefix_header_includes=keep");
			// Arguments.Add("-Xiwyu --transitive_includes_only");  // Since we are building headers separately (not together with their cpp) we don't need this
			// Arguments.Add("-Xiwyu --no_check_matching_header");
			Arguments.Add("-Xiwyu --cxx17ns");

			CompileEnvironment.CppCompileWarnings.UnusedWarningLevel = WarningLevel.Off;

			Arguments.Add("-Wno-delete-incomplete");
			Arguments.Add("-Wno-unknown-warning-option");

			if (TargetPlatform == UnrealTargetPlatform.Win64)
			{
				Arguments.Add("--driver-mode=cl");
				VCToolChain vcToolChain = (VCToolChain)InnerToolChain;
				vcToolChain.AppendCLArguments_GlobalIWYU(CompileEnvironment, Arguments);
			}
			else
			{
				base.GetCompileArguments_Global(CompileEnvironment, Arguments);

				if (TargetPlatform == UnrealTargetPlatform.Linux)
				{
					if (ShouldUseLibcxx())
					{
						Arguments.Add("-nostdinc++");

						LinuxPlatformSDK PlatformSDK = new(Logger);
						DirectoryReference? BaseLinuxPath = PlatformSDK.GetBaseLinuxPathForArchitecture(CompileEnvironment.Architecture);
						if (BaseLinuxPath != null)
						{
							Arguments.Add(GetSystemIncludePathArgument(DirectoryReference.Combine(BaseLinuxPath, "include"), CompileEnvironment));
							Arguments.Add(GetSystemIncludePathArgument(DirectoryReference.Combine(BaseLinuxPath, "include", "c++", "v1"), CompileEnvironment));
						}
					}

					Arguments.Add("-fno-math-errno");               // do not assume that math ops have side effects

					Arguments.Add(GetRTTIFlag(CompileEnvironment)); // flag for run-time type info

					if (true)//CrossCompiling())
					{
						Arguments.Add($"-target {CompileEnvironment.Architecture.LinuxName}");        // Set target triple
						Arguments.AddRange(CrossCompilingArguments);
					}

					if (CompileEnvironment.bHideSymbolsByDefault)
					{
						Arguments.Add("-fvisibility-ms-compat");
						Arguments.Add("-fvisibility-inlines-hidden");
					}
				}
			}
		}

		protected override void GetCompileArguments_WarningsAndErrors(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			if (TargetPlatform == UnrealTargetPlatform.Linux)
			{
				CompileEnvironment.CppCompileWarnings.ShadowVariableWarningLevel = WarningLevel.Off;
				base.GetCompileArguments_WarningsAndErrors(CompileEnvironment, Arguments);
				Arguments.Add("-Wno-undefined-bool-conversion");
				Arguments.Add("-Wno-deprecated-anon-enum-enum-conversion");
				Arguments.Add("-Wno-ambiguous-reversed-operator");
				ClangWarnings.GetHeaderDisabledWarnings(Arguments);
			}
		}

		// IWYU can't link
		public override FileItem LinkFiles(LinkEnvironment LinkEnvironment, bool bBuildImportLibraryOnly, IActionGraphBuilder Graph)
		{
			throw new BuildException("Unable to link with IWYU toolchain.");
		}

		protected override void GetCompileArguments_H(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			if (TargetPlatform == UnrealTargetPlatform.Linux)
			{
				base.GetCompileArguments_H(CompileEnvironment, Arguments);
			}
			else
			{
				((VCToolChain)InnerToolChain).AppendCLArguments_H(CompileEnvironment, Arguments);
			}
		}

		protected override void GetCompileArguments_CPP(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			if (TargetPlatform == UnrealTargetPlatform.Linux)
			{
				base.GetCompileArguments_CPP(CompileEnvironment, Arguments);
			}
			else
			{
				((VCToolChain)InnerToolChain).AppendCLArguments_CPP(CompileEnvironment, Arguments);
			}
		}

		protected override void GetCompileArguments_C(CppCompileEnvironment CompileEnvironment, List<string> Arguments)
		{
			if (TargetPlatform == UnrealTargetPlatform.Linux)
			{
				base.GetCompileArguments_C(CompileEnvironment, Arguments);
			}
			else
			{
				((VCToolChain)InnerToolChain).AppendCLArguments_C(CompileEnvironment, Arguments);
			}
		}

		protected override string GetForceIncludeFileArgument(FileReference ForceIncludeFile, CppCompileEnvironment CompileEnvironment)
		{
			if (TargetPlatform == UnrealTargetPlatform.Linux)
			{
				return base.GetForceIncludeFileArgument(ForceIncludeFile, CompileEnvironment);
			}
			else
			{
				return $"/FI\"{NormalizeCommandLinePath(ForceIncludeFile, CompileEnvironment.RootPaths)}\"";
			}
		}

		protected override FileItem GetCompileArguments_FileType(CppCompileEnvironment CompileEnvironment, FileItem SourceFile, DirectoryReference OutputDir, List<string> Arguments, Action CompileAction, CPPOutput CompileResult)
		{
			FileItem fileItem = base.GetCompileArguments_FileType(CompileEnvironment, SourceFile, OutputDir, Arguments, CompileAction, CompileResult);

			Arguments.Add($"-Xiwyu --write_json_path=\"{fileItem.AbsolutePath.Replace('\\', '/')}\"");
			bool bIsHeaderCpp = SourceFile.HasExtension(".cpp") && SourceFile.Name.EndsWith(".h.cpp", StringComparison.Ordinal);
			if (SourceFile.HasExtension(".cpp") && !bIsHeaderCpp)
			{
				if (CompileEnvironment.FileInlineSourceMap.TryGetValue(SourceFile, out HashSet<FileItem>? InlinedFiles))
				{
					foreach (FileItem InlinedFile in InlinedFiles)
					{
						Arguments.Add($"-Xiwyu --check_also=\"{InlinedFile.FullName.Replace('\\', '/')}\"");
					}
				}

				if (SourceFile.Name != "SingleFile.cpp" && !SourceFile.Name.EndsWith(".gen.cpp", StringComparison.Ordinal))
				{
					foreach (string include in CompileEnvironment.MetadataCache.GetHeaderIncludes(SourceFile))
					{
						if (include.EndsWith(".ipp", StringComparison.Ordinal))
						{
							Arguments.Add($"-Xiwyu --check_also=*/{include.Replace('\\', '/')}");
						}
					}
				}
			}
			else
			{
				// We want to build header files (and .h.cpp wrappers) first
				CompileAction.bIsHighPriority = true;
				Arguments.Add($"-Xiwyu --check_also=*/{SourceFile.Location.GetFileNameWithoutAnyExtensions()}.generated.h");
			}
			return fileItem;
		}

		protected override string GetDepencenciesListFileArgument(FileItem DependencyListFile, CppCompileEnvironment CompileEnvironment)
		{
			string result = base.GetDepencenciesListFileArgument(DependencyListFile, CompileEnvironment);

			if (TargetPlatform == UnrealTargetPlatform.Win64)
			{
				result = result.Replace("-MD", "/clang:-MD", StringComparison.Ordinal).Replace("-MF", "/clang:-MF", StringComparison.Ordinal);
			}
			return result;
		}

		public override void SetEnvironmentVariables() => InnerToolChain.SetEnvironmentVariables();
		public override void GenerateTypeLibraryHeader(CppCompileEnvironment CompileEnvironment, ModuleRules.TypeLibrary TypeLibrary, FileReference OutputFile, FileReference? OutputHeader, IActionGraphBuilder Graph) => InnerToolChain.GenerateTypeLibraryHeader(CompileEnvironment, TypeLibrary, OutputFile, OutputHeader, Graph);
		public override List<string> GetISPCCompileTargets(UnrealTargetPlatform Platform, UnrealArch Arch) => InnerToolChain.GetISPCCompileTargets(Platform, Arch);
		public override string GetISPCOSTarget(UnrealTargetPlatform Platform) => InnerToolChain.GetISPCOSTarget(Platform);
		public override string GetISPCArchTarget(UnrealTargetPlatform Platform, UnrealArch Arch) => InnerToolChain.GetISPCArchTarget(Platform, Arch);
		public override string? GetISPCCpuTarget(UnrealTargetPlatform Platform) => InnerToolChain.GetISPCCpuTarget(Platform);
		public override string GetISPCObjectFileFormat(UnrealTargetPlatform Platform) => InnerToolChain.GetISPCObjectFileFormat(Platform);
		public override string GetISPCObjectFileSuffix(UnrealTargetPlatform Platform) => InnerToolChain.GetISPCObjectFileSuffix(Platform);
	}
}
