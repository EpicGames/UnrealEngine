// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text.RegularExpressions;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	abstract class UEToolChain
	{
		protected readonly ILogger Logger;

		// Return the extension for response files
		public static string ResponseExt => ".rsp";

		public UEToolChain(ILogger InLogger)
		{
			Logger = InLogger;
		}

		/// <summary>
		/// Prepares the environment for building
		/// </summary>
		public virtual void SetEnvironmentVariables()
		{
		}

		/// <summary>
		/// Returns the version info for the toolchain. This will be output before building.
		/// </summary>
		/// <returns>String describing the current toolchain</returns>>
		public virtual void GetVersionInfo(List<string> Lines)
		{
		}

		public virtual void GetExternalDependencies(HashSet<FileItem> ExternalDependencies)
		{
		}

		/// <summary>
		/// Normalize a path for use in a command line, making it relative to Engine/Source if under the root directory
		/// </summary>
		/// <param name="Reference">The FileSystemReference to normalize</param>
		/// <returns>Normalized path as a string</returns>
		protected static string NormalizeCommandLinePath(FileSystemReference Reference)
		{
			string path = Reference.FullName;
			// Try to use a relative path to shorten command line length.
			if (Reference.IsUnderDirectory(Unreal.RootDirectory))
			{
				path = Reference.MakeRelativeTo(Unreal.EngineSourceDirectory);
			}
			if (Path.DirectorySeparatorChar == '\\')
			{
				path = path.Replace("\\", "/", StringComparison.Ordinal);
			}
			return path;
		}

		/// <summary>
		/// Normalize a path for use in a command line, making it relative if under the Root Directory
		/// </summary>
		/// <param name="Item">The FileItem to normalize</param>
		/// <returns>Normalized path as a string</returns>
		protected static string NormalizeCommandLinePath(FileItem Item) => NormalizeCommandLinePath(Item.Location);

		/// <summary>
		/// Normalize a path for use in a command line, making it relative to Engine/Source if under the root directory
		/// </summary>
		/// <param name="Reference">The FileSystemReference to normalize</param>
		/// <param name="RootPaths">The root paths to use for normalization</param>
		/// <returns>Normalized path as a string</returns>
		public virtual string NormalizeCommandLinePath(FileSystemReference Reference, CppRootPaths RootPaths) => 
			RootPaths.GetVfsOverlayPath(Reference, out string? vfsPath) ? vfsPath : NormalizeCommandLinePath(Reference);

		/// <summary>
		/// Normalize a path for use in a command line, making it relative if under the Root Directory
		/// </summary>
		/// <param name="Item">The FileItem to normalize</param>
		/// <param name="RootPaths">The root paths to use for normalization</param>
		/// <returns>Normalized path as a string</returns>
		public virtual string NormalizeCommandLinePath(FileItem Item, CppRootPaths RootPaths) => NormalizeCommandLinePath(Item.Location, RootPaths);

		public static DirectoryReference GetModuleInterfaceDir(DirectoryReference OutputDir)
		{
			return DirectoryReference.Combine(OutputDir, "Ifc");
		}

		// Return the path to the cpp compiler that will be used by this toolchain.
		public virtual FileReference? GetCppCompilerPath()
		{
			return null;
		}

		public virtual FileItem? CopyDebuggerVisualizer(FileItem SourceFile, DirectoryReference IntermediateDirectory, IActionGraphBuilder Graph)
		{
			return null;
		}

		public virtual FileItem? LinkDebuggerVisualizer(FileItem SourceFile, DirectoryReference IntermediateDirectory)
		{
			return null;
		}

		protected abstract CPPOutput CompileCPPFiles(CppCompileEnvironment CompileEnvironment, IEnumerable<FileItem> InputFiles, DirectoryReference OutputDir, string ModuleName, IActionGraphBuilder Graph);

		public CPPOutput CompileAllCPPFiles(CppCompileEnvironment CompileEnvironment, IEnumerable<FileItem> InputFiles, DirectoryReference OutputDir, string ModuleName, IActionGraphBuilder Graph)
		{
			UnrealArchitectureConfig ArchConfig = UnrealArchitectureConfig.ForPlatform(CompileEnvironment.Platform);
			// compile architectures separately if needed
			if (ArchConfig.Mode == UnrealArchitectureMode.SingleTargetCompileSeparately || ArchConfig.Mode == UnrealArchitectureMode.SingleTargetLinkSeparately)
			{
				CPPOutput Result = new();
				foreach (UnrealArch Arch in CompileEnvironment.Architectures.Architectures)
				{
					// determine the output location of intermediates (so, if OutputDir had the arch name in it, like Intermediate/x86+arm64, we would replace it with either emptry string
					// or a single arch name depending on if the platform uses architecture directories for the architecture)
					// @todo Add ArchitectureConfig.RequiresArchitectureFilenames but for directory -- or can we just use GetFolderNameForArch?!?!?
					//					string ArchReplacement = (Arch == ArchitectureWithoutMarkup()) ? "" : ArchConfig.GetFolderNameForArchitecture(Arch);

					string PlatformArchitecturesString = ArchConfig.GetFolderNameForArchitectures(CompileEnvironment.Architectures);
					DirectoryReference ArchOutputDir = new(OutputDir.FullName.Replace(PlatformArchitecturesString, ArchConfig.GetFolderNameForArchitecture(Arch), StringComparison.Ordinal));

					CppCompileEnvironment ArchEnvironment = new(CompileEnvironment, Arch);
					if (ArchEnvironment.UserIncludePaths.Remove(OutputDir))
					{
						ArchEnvironment.UserIncludePaths.Add(ArchOutputDir);
					}
					CPPOutput ArchResult = CompileCPPFiles(ArchEnvironment, InputFiles, ArchOutputDir, ModuleName, Graph);
					Result.Merge(ArchResult, Arch);
				}
				return Result;
			}

			return CompileCPPFiles(CompileEnvironment, InputFiles, OutputDir, ModuleName, Graph);
		}

		public virtual CPPOutput CompileRCFiles(CppCompileEnvironment CompileEnvironment, IEnumerable<FileItem> InputFiles, DirectoryReference OutputDir, IActionGraphBuilder Graph)
		{
			return new();
		}

		public virtual CPPOutput ConvertRCFiles(LinkEnvironment LinkEnvironment, IEnumerable<FileItem> InputFiles, DirectoryReference OutputDir, IActionGraphBuilder Graph)
		{
			CPPOutput Result = new();
			Result.ObjectFiles.AddRange(InputFiles);
			return Result;
		}

		protected abstract CPPOutput CompileISPCFiles(CppCompileEnvironment CompileEnvironment, IEnumerable<FileItem> InputFiles, DirectoryReference OutputDir, IActionGraphBuilder Graph);

		public CPPOutput CompileAllISPCFiles(CppCompileEnvironment CompileEnvironment, IEnumerable<FileItem> InputFiles, DirectoryReference OutputDir, IActionGraphBuilder Graph)
		{
			UnrealArchitectureConfig ArchConfig = UnrealArchitectureConfig.ForPlatform(CompileEnvironment.Platform);
			// compile architectures separately if needed
			if (ArchConfig.Mode == UnrealArchitectureMode.SingleTargetCompileSeparately || ArchConfig.Mode == UnrealArchitectureMode.SingleTargetLinkSeparately)
			{
				CPPOutput Result = new();
				foreach (UnrealArch Arch in CompileEnvironment.Architectures.Architectures)
				{
					string PlatformArchitecturesString = ArchConfig.GetFolderNameForArchitectures(CompileEnvironment.Architectures);
					DirectoryReference ArchOutputDir = new(OutputDir.FullName.Replace(PlatformArchitecturesString, ArchConfig.GetFolderNameForArchitecture(Arch), StringComparison.Ordinal));

					CppCompileEnvironment ArchEnvironment = new(CompileEnvironment, Arch);
					CPPOutput ArchResult = CompileISPCFiles(ArchEnvironment, InputFiles, ArchOutputDir, Graph);
					Result.Merge(ArchResult, Arch);
				}
				return Result;
			}

			return CompileISPCFiles(CompileEnvironment, InputFiles, OutputDir, Graph);
		}

		protected abstract CPPOutput GenerateISPCHeaders(CppCompileEnvironment CompileEnvironment, IEnumerable<FileItem> InputFiles, DirectoryReference OutputDir, IActionGraphBuilder Graph);

		public CPPOutput GenerateAllISPCHeaders(CppCompileEnvironment CompileEnvironment, IEnumerable<FileItem> InputFiles, DirectoryReference OutputDir, IActionGraphBuilder Graph)
		{
			UnrealArchitectureConfig ArchConfig = UnrealArchitectureConfig.ForPlatform(CompileEnvironment.Platform);
			// compile architectures separately if needed
			if (ArchConfig.Mode == UnrealArchitectureMode.SingleTargetCompileSeparately || ArchConfig.Mode == UnrealArchitectureMode.SingleTargetLinkSeparately)
			{
				CPPOutput Result = new();
				foreach (UnrealArch Arch in CompileEnvironment.Architectures.Architectures)
				{
					string PlatformArchitecturesString = ArchConfig.GetFolderNameForArchitectures(CompileEnvironment.Architectures);
					DirectoryReference ArchOutputDir = new(OutputDir.FullName.Replace(PlatformArchitecturesString, ArchConfig.GetFolderNameForArchitecture(Arch), StringComparison.Ordinal));

					CppCompileEnvironment ArchEnvironment = new(CompileEnvironment, Arch);
					CPPOutput ArchResult = GenerateISPCHeaders(ArchEnvironment, InputFiles, ArchOutputDir, Graph);
					Result.Merge(ArchResult, Arch);
				}
				return Result;
			}
			
			return GenerateISPCHeaders(CompileEnvironment, InputFiles, OutputDir, Graph);
		}

		public virtual void GenerateTypeLibraryHeader(CppCompileEnvironment CompileEnvironment, ModuleRules.TypeLibrary TypeLibrary, FileReference OutputFile, FileReference? OutputHeader, IActionGraphBuilder Graph)
		{
			throw new NotSupportedException("This platform does not support type libraries.");
		}

		/// <summary>
		/// Allows a toolchain to decide to create an import library if needed for this Environment
		/// </summary>
		/// <param name="LinkEnvironment"></param>
		/// <param name="Graph"></param>
		/// <returns></returns>
		public virtual FileItem[] LinkImportLibrary(LinkEnvironment LinkEnvironment, IActionGraphBuilder Graph)
		{
			return [];
		}

		public abstract FileItem? LinkFiles(LinkEnvironment LinkEnvironment, bool bBuildImportLibraryOnly, IActionGraphBuilder Graph);
		public virtual FileItem[] LinkAllFiles(LinkEnvironment LinkEnvironment, bool bBuildImportLibraryOnly, IActionGraphBuilder Graph)
		{
			List<FileItem> Result = [];

			// compile architectures separately if needed
			UnrealArchitectureConfig ArchConfig = UnrealArchitectureConfig.ForPlatform(LinkEnvironment.Platform);
			if (ArchConfig.Mode == UnrealArchitectureMode.SingleTargetLinkSeparately)
			{
				foreach (UnrealArch Arch in LinkEnvironment.Architectures.Architectures)
				{
					LinkEnvironment ArchEnvironment = new(LinkEnvironment, Arch);

					// determine the output location of intermediates (so, if OutputDir had the arch name in it, like Intermediate/x86+arm64, we would replace it with either emptry string
					// or a single arch name
					//string ArchReplacement = Arch == ArchitectureWithoutMarkup() ? "" : ArchConfig.GetFolderNameForArchitecture(Arch);
					string PlatformArchitecturesString = ArchConfig.GetFolderNameForArchitectures(LinkEnvironment.Architectures);

					ArchEnvironment.OutputFilePaths = [.. LinkEnvironment.OutputFilePaths.Select(x => new FileReference(x.FullName.Replace(PlatformArchitecturesString, ArchConfig.GetFolderNameForArchitecture(Arch), StringComparison.Ordinal)))];

					FileItem? LinkFile = LinkFiles(ArchEnvironment, bBuildImportLibraryOnly, Graph);
					if (LinkFile != null)
					{
						Result.Add(LinkFile);
					}
				}
			}
			else
			{
				FileItem? LinkFile = LinkFiles(LinkEnvironment, bBuildImportLibraryOnly, Graph);
				if (LinkFile != null)
				{
					Result.Add(LinkFile);
				}
			}
			return [.. Result];
		}

		public virtual IEnumerable<string> GetGlobalCommandLineArgs(CppCompileEnvironment CompileEnvironment)
		{
			return [];
		}

		public virtual IEnumerable<string> GetCPPCommandLineArgs(CppCompileEnvironment CompileEnvironment)
		{
			return [];
		}

		public virtual IEnumerable<string> GetCCommandLineArgs(CppCompileEnvironment CompileEnvironment)
		{
			return [];
		}

		public virtual CppCompileEnvironment CreateSharedResponseFile(CppCompileEnvironment CompileEnvironment, FileReference OutResponseFile, IActionGraphBuilder Graph)
		{
			return CompileEnvironment;
		}

		public virtual void CreateSpecificFileAction(CppCompileEnvironment CompileEnvironment, DirectoryReference SourceDir, DirectoryReference OutputDir, IActionGraphBuilder Graph)
		{
		}

		// Construct a relative path for the intermediate response file
		private static FileReference GetResponseFileName(FileReference Location) => Location.ChangeExtension($"{Location.GetExtension()}{ResponseExt}");

		/// <summary>
		/// Get the name of the response file for the output file
		/// </summary>
		/// <param name="OutputFile"></param>
		/// <returns></returns>
		public static FileReference GetResponseFileName(FileItem OutputFile) => GetResponseFileName(OutputFile.Location);

		/// <summary>
		/// Get the name of the response file for the current linker environment and output file
		/// </summary>
		/// <param name="LinkEnvironment"></param>
		/// <param name="OutputFile"></param>
		/// <returns></returns>
		public static FileReference GetResponseFileName(LinkEnvironment LinkEnvironment, FileItem OutputFile)
		{
			// Construct a relative path for the intermediate response file
			return GetResponseFileName(FileReference.Combine(LinkEnvironment.IntermediateDirectory ?? OutputFile.Directory.Location, OutputFile.Location.GetFileName()));
		}

		public virtual ICollection<FileItem> PostBuild(ReadOnlyTargetRules Target, FileItem Executable, LinkEnvironment ExecutableLinkEnvironment, IActionGraphBuilder Graph)
		{
			return [];
		}

		public virtual ICollection<FileItem> PostBuild(ReadOnlyTargetRules Target, IEnumerable<FileItem> Executables, LinkEnvironment ExecutableLinkEnvironment, IActionGraphBuilder Graph)
		{
			// by default, run PostBuild for exe Exe and merge results
			return [.. Executables.SelectMany(x => PostBuild(Target, x, ExecutableLinkEnvironment, Graph))];
		}

		public virtual void SetUpGlobalEnvironment(ReadOnlyTargetRules Target, CppCompileEnvironment GlobalCompileEnvironment, LinkEnvironment GlobalLinkEnvironment)
		{
			GlobalCompileEnvironment.RootPaths.bUseVfs = Target.bUseVFS;
			GlobalLinkEnvironment.RootPaths.bUseVfs = Target.bUseVFS;

			GlobalCompileEnvironment.RootPaths[CppRootPathFolder.Root] = Unreal.RootDirectory;
			GlobalLinkEnvironment.RootPaths[CppRootPathFolder.Root] = Unreal.RootDirectory;

			DirectoryReference? thinLtoCache = DirectoryReference.FromString(Target.ThinLTOCacheDirectory);
			if (thinLtoCache != null)
			{
				GlobalLinkEnvironment.RootPaths.AddExtraPath(("ThinLTOCache", thinLtoCache.FullName));
				EpicGames.UBA.Utils.RegisterPathHash(thinLtoCache, $"{Target.Platform}-ThinLTOCache");
			}
		}

		public virtual void ModifyBuildProducts(ReadOnlyTargetRules Target, UEBuildBinary Binary, IEnumerable<string> Libraries, IEnumerable<UEBuildBundleResource> BundleResources, Dictionary<FileReference, BuildProductType> BuildProducts)
		{
		}

		public virtual void ModifyTargetReceipt(ReadOnlyTargetRules Target, TargetReceipt Receipt)
		{
		}

		public virtual void FinalizeOutput(ReadOnlyTargetRules Target, TargetMakefileBuilder MakefileBuilder, DirectoryReference ExeDir)
		{
		}

		public virtual void PrepareRuntimeDependencies(List<RuntimeDependency> RuntimeDependencies, Dictionary<FileReference, FileReference> TargetFileToSourceFile, DirectoryReference ExeDir)
		{
		}

		/// <summary>
		/// Adds a build product and its associated debug file to a receipt.
		/// </summary>
		/// <param name="OutputFile">Build product to add</param>
		/// <param name="OutputType">The type of build product</param>
		public virtual bool ShouldAddDebugFileToReceipt(FileReference OutputFile, BuildProductType OutputType)
		{
			return true;
		}

		/// <summary>
		/// Get the file path to a debug file
		/// </summary>
		/// <param name="OutputFile">Binary output file</param>
		/// <param name="DebugExtension">debug file extension</param>
		/// <returns>Path to the debug file</returns>
		public virtual FileReference GetDebugFile(FileReference OutputFile, string DebugExtension)
		{
			//  by default, just change the extension to the debug extension
			return OutputFile.ChangeExtension(DebugExtension);
		}

		public virtual void SetupBundleDependencies(ReadOnlyTargetRules Target, IEnumerable<UEBuildBinary> Binaries, string GameName)
		{
		}

		public virtual string GetSDKVersion()
		{
			return "Not Applicable";
		}

		/// <summary>
		/// This is the extension of the file that is added as extra link object. It can be an obj file but also a dynamic list.. depends on the platform
		/// </summary>
		/// <returns>The extension (without dot)</returns>
		public virtual string GetExtraLinkFileExtension()
		{
			throw new NotSupportedException("This platform does not support merged modules.");
		}

		public virtual IEnumerable<string> GetExtraLinkFileAdditionalSymbols(UEBuildBinaryType binaryType)
		{
			return [];
		}

		/// <summary>
		/// Runs the provided tool and argument. Returns the output, using a rexex capture if one is provided
		/// </summary>
		/// <param name="Command">Full path to the tool to run</param>
		/// <param name="ToolArg">Argument that will be passed to the tool</param>
		/// <param name="Expression">null, or a Regular expression to capture in the output</param>
		/// <returns></returns>
		protected static string? RunToolAndCaptureOutput(FileReference Command, string ToolArg, string? Expression = null)
		{
			string ProcessOutput = Utils.RunLocalProcessAndReturnStdOut(Command.FullName, ToolArg, null);

			if (String.IsNullOrEmpty(Expression))
			{
				return ProcessOutput;
			}

			Match M = Regex.Match(ProcessOutput, Expression);
			return M.Success ? M.Groups[1].ToString() : null;
		}

		/// <summary>
		/// Runs the provided tool and argument and parses the output to retrieve the version
		/// </summary>
		/// <param name="Command">Full path to the tool to run</param>
		/// <param name="VersionArg">Argument that will result in the version string being shown (it's ok this is a byproduct of a command that returns an error)</param>
		/// <param name="VersionExpression">Regular expression to capture the version. By default we look for four integers separated by periods, with the last two optional</param>
		/// <returns></returns>
		public Version RunToolAndCaptureVersion(FileReference Command, string VersionArg, string VersionExpression = @"(\d+\.\d+(\.\d+)?(\.\d+)?)")
		{
			string? ProcessOutput = RunToolAndCaptureOutput(Command, VersionArg, VersionExpression);

			if (Version.TryParse(ProcessOutput, out Version? ToolVersion))
			{
				return ToolVersion;
			}

			Logger.LogWarning("Unable to retrieve version from {Command} {Arg}", Command, VersionArg);

			return new Version(0, 0);
		}
	};
}
