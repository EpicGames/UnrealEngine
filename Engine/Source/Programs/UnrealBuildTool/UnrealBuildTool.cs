// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Reflection;
using System.Text;
using System.Text.RegularExpressions;
using System.Threading.Tasks;
using System.Xml.Linq;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using OpenTracing;
using OpenTracing.Util;
using UnrealBuildBase;

namespace UnrealBuildTool
{
	static class UnrealBuildTool
	{
		/// <summary>
		/// Save the application startup time. This can be used as the timestamp for build makefiles, to determine a base time after which any
		/// modifications should invalidate it.
		/// </summary>
		public static DateTime StartTimeUtc { get; } = DateTime.UtcNow;

		/// <summary>
		/// Whether this is a recursive run of of the application
		/// </summary>
		public static bool IsRecursive = false;

		/// <summary>
		/// Unique id to track this session
		/// </summary>
		public static string SessionIdentifier = Guid.NewGuid().ToString("B");

		/// <summary>
		/// The mode of this instance
		/// </summary>
		public static string CurrentMode = "<uninitialized>";

		/// <summary>
		/// The result of running the application
		/// </summary>
		private static CompilationResult ApplicationResult = CompilationResult.Unknown;

		/// <summary>
		/// The environment at boot time.
		/// </summary>
		public static System.Collections.IDictionary? InitialEnvironment;

		/// <summary>
		/// The Unreal remote tool ini directory.  This should be valid if compiling using a remote server
		/// </summary>
		/// <returns>The directory path</returns>
		public static string? GetRemoteIniPath() => _RemoteIniPath;

		/// <summary>
		/// Set the Unreal remote tool ini directory.
		/// </summary>
		/// <param name="Path"> remote path</param>
		public static void SetRemoteIniPath(string Path) => _RemoteIniPath = Path;

		/// <summary>
		/// The Remote Ini directory.  This should always be valid when compiling using a remote server.
		/// </summary>
		private static string? _RemoteIniPath = null;

		/// <summary>
		/// Get all the valid Modes
		/// </summary>
		/// <returns></returns>
		private static Dictionary<string, ModeDescriptor> GetModes()
		{
			Dictionary<string, ModeDescriptor> modeNameToDescriptor = new(StringComparer.OrdinalIgnoreCase);
			foreach (Type type in Assembly.GetExecutingAssembly().GetTypes().Where(x => x.IsClass && !x.IsAbstract))
			{
				bool isToolMode = type.GetInterfaces().FirstOrDefault(x => x.IsGenericType && x.GetGenericTypeDefinition() == typeof(IToolMode<>)) is Type ToolModeType;
				if (!isToolMode)
				{
					continue;
				}

				string name = (string)type.GetProperty("Name", BindingFlags.Public | BindingFlags.Static)!.GetValue(null)!;
				ToolModeOptions options = (ToolModeOptions)type.GetProperty("Options", BindingFlags.Public | BindingFlags.Static)!.GetValue(null)!;
				modeNameToDescriptor.Add(name, new(type, options));
			}
			return modeNameToDescriptor;
		}

		/// <summary>
		/// Print (incomplete) usage information
		/// </summary>
		private static void PrintUsage()
		{
			IEnumerable<CommandLineAttribute> attributes = [
				.. typeof(GlobalOptions).GetProperties().SelectMany(x => x.GetCustomAttributes<CommandLineAttribute>()),
				.. typeof(GlobalOptions).GetFields().SelectMany(x => x.GetCustomAttributes<CommandLineAttribute>())
			];
			if (!attributes.Any())
			{
				throw new BuildLogEventException("Unable to load properties and fields from {Type}", typeof(GlobalOptions));
			}
			int longestPrefix = attributes.Select(x => x.Prefix?.Length ?? 0).Max();

			Console.WriteLine("Global Options:");
			foreach (CommandLineAttribute Att in attributes)
			{
				if (Att.Prefix != null && Att.Description != null)
				{
					Console.WriteLine($" {Att.Prefix.PadRight(longestPrefix)} :  {Att.Description}");
				}
			}

			Dictionary<string, ModeDescriptor> modes = GetModes();
			if (modes.Count == 0)
			{
				throw new BuildLogEventException("No valid subclasses of {Type} found", typeof(IToolMode<>));
			}

			// Load tool mode summary from xml documentation if available
			XDocument? doc = null;
			try
			{
				FileReference xmlDocPath = new FileReference(Assembly.GetExecutingAssembly().Location).ChangeExtension(".xml");
				doc = FileReference.Exists(xmlDocPath) ? XDocument.Load(xmlDocPath.FullName) : null;
			}
			catch (Exception ex)
			{
				throw new BuildLogEventException(ex, "Documentation malformed for {Assembly}", Assembly.GetExecutingAssembly().Location);
			}

			longestPrefix = modes.Select(x => x.Key.Length).Max();
			Console.WriteLine();
			Console.WriteLine("Available Tool Modes:");
			foreach ((string name, ModeDescriptor descriptor) in modes)
			{
				string summary = doc?.Descendants("member")
					.FirstOrDefault(x => x.Attribute("name")?.Value == $"T:{descriptor.ModeType.FullName}")?
					.Element("summary")?.Value.Trim()
					?? "Summary unavailable";
				Console.WriteLine($" {name.PadRight(longestPrefix)} : {Regex.Replace(summary, @"\s+", " ")}");
			}
			Console.WriteLine();
		}

		/// <summary>
		/// Read extra command-line arguments from an environment variable
		/// Double-quote any argument containing whitespace, as they are split by just that.
		/// </summary>
		/// <returns>Extra arguments</returns>
		private static string[] GetExtraArgsFromEnvVar()
		{
			string? extraArgs = Environment.GetEnvironmentVariable("UBT_EXTRA_ARGS");
			return String.IsNullOrEmpty(extraArgs) ? [] : CommandLineArguments.Split(extraArgs);
		}

		/// <summary>
		/// Event handler for the Console.CancelKeyPress event
		/// </summary>
		/// <param name="sender"></param>
		/// <param name="e"></param>
		private static async void CancelKeyPressAsync(object? sender, ConsoleCancelEventArgs e)
		{
			Console.CancelKeyPress -= CancelKeyPressAsync;
			Console.WriteLine($"UnrealBuildTool: Ctrl-{(e.SpecialKey == ConsoleSpecialKey.ControlC ? "C" : "Break")} pressed. Exiting...");

			// Delay a few seconds to allow for the process to exit normally
			await Task.Delay(2000);

			// While the Ctrl-C handler fixes most instances of a zombie process, we still need to 
			// force an exit from the process to handle _all_ cases.  Ctrl-C should not be a regular event! 
			// Note: this could be a dotnet (6.0.302) on macOS issue.  Recheck with next release if this is still required.
			Environment.Exit(-1);
		}

		/// <summary>
		/// Main entry point. Parses any global options and initializes the logging system, then invokes the appropriate command.
		/// </summary>
		/// <param name="ArgumentsArray">Command line arguments</param>
		/// <returns>Zero on success, non-zero on error</returns>
		private static int Main(string[] ArgumentsArray)
		{
			ToolModeOptions ModeOptions = ToolModeOptions.None;
			FileReference? RunFile = null;
			DirectoryReference? TempDirectory = null;
			IDisposable? Mutex = null;

			ILogger Logger = Log.Logger;

			// When running RunUBT.sh on a Mac we need to install a Ctrl-C handler, or hitting Ctrl-C from a terminal
			// or from cancelling a build within Xcode, can leave a dotnet process in a zombie state. 
			// By putting this in, the Ctrl-C may not be handled immediately, but it shouldn't leave a blocking zombie process
			if (OperatingSystem.IsMacOS())
			{
				Console.CancelKeyPress += CancelKeyPressAsync;
			}

			JsonTracer? Tracer = JsonTracer.TryRegisterAsGlobalTracer();
			// Start the UBT OTel scope as the outermost span, as early as possible.
			// We'll have to manually dispose this as it doesn't fit nicely into the try/catch block below.
			IScope ubtScope = GlobalTracer.Instance.BuildSpan("UnrealBuildTool").StartActive();

			// If command line contains -Session it is a recursive action and we don't want to report this trace to the default trace channel
			string? traceChannel = null;
			string? traceName = null;
			string? traceFile = null;
			if (ArgumentsArray.FirstOrDefault(s => s.Contains("-Session=", StringComparison.Ordinal)) == null)
			{
				FileReference traceFileRef;
				// Setup trace file and backup old one
				if (Unreal.IsBuildMachine())
				{
					traceFileRef = FileReference.Combine(Unreal.EngineProgramSavedDirectory, "AutomationTool", "Saved", "Logs", "Trace.uba");
				}
				else
				{
					traceFileRef = FileReference.Combine(Unreal.EngineProgramSavedDirectory, "UnrealBuildTool", "Trace.uba");
				}
				DirectoryReference.CreateDirectory(traceFileRef.Directory);
				Log.BackupLogFile(traceFileRef);

				traceChannel = "Default";
				traceName = "UnrealBuildTool";
				traceFile = traceFileRef.FullName;
			}

			// Attach timeline to uba trace
			using EpicGames.UBA.ILogger ubaLogger = EpicGames.UBA.ILogger.CreateLogger(Logger, true);
			using EpicGames.UBA.ITrace ubaTrace = EpicGames.UBA.ITrace.Create(traceName, traceFile, true, traceChannel);
			using TimelineEventScope ubaTimelineEventScope = new((name) => ubaTrace.TaskBegin(name, ""), (id) => ubaTrace.TaskEnd(id));

			bool bShowTimeline = false;

			try
			{
				// Start capturing performance info
				Timeline.Start();
				using ITimelineEvent MainScope = Timeline.ScopeEvent("Main");

				ArgumentsArray = ArgumentsArray.Concat(GetExtraArgsFromEnvVar()).ToArray();

				// Parse the command line arguments
				CommandLineArguments Arguments = new CommandLineArguments(ArgumentsArray);

				// Parse the global options
				GlobalOptions Options = new GlobalOptions(Arguments);

				if (
					// Print usage if there are zero arguments provided
					ArgumentsArray.Length == 0

					// Print usage if the user asks for help
					|| Options.GetHelp
					)
				{
					PrintUsage();
					return Options.GetHelp ? (int)CompilationResult.Succeeded : (int)CompilationResult.Canceled;
				}

				// Configure the log system
				Log.OutputLevel = Options.LogOutputLevel;
				Log.IncludeTimestamps = Options.bLogTimestamps;
				Log.IncludeProgramNameWithSeverityPrefix = Options.LogFromMsBuild;

				// Reducing SDK warning events in the log to LogEventType.Log
				if (Options.ShouldSuppressSDKWarnings)
				{
					UEBuildPlatformSDK.bSuppressSDKWarnings = true;
				}

				// Always start capturing logs as early as possible to later copy to a log file if the ToolMode desires it (we have to start capturing before we get the ToolModeOptions below)
				Log.AddStartupTraceListener();

				// Enable threading on logger since it can stall quite a lot when logging happens from multiple threads
				Log.EnableThreaded();

				if (Options.TraceWrites != null)
				{
					Logger.LogInformation("All attempts to write to \"{TraceWrites}\" via WriteFileIfChanged() will be logged", Options.TraceWrites);
					Utils.WriteFileIfChangedTrace = Options.TraceWrites;
				}

				// Configure the progress writer
				ProgressWriter.bWriteMarkup = Options.WriteProgressMarkup;

				// Add the application directory to PATH
				DirectoryReference.AddDirectoryToPath(Unreal.UnrealBuildToolDllPath.Directory);

				// Change the working directory to be the Engine/Source folder. We are likely running from Engine/Binaries/DotNET
				// This is critical to be done early so any code that relies on the current directory being Engine/Source will work.
				DirectoryReference.CreateDirectory(Unreal.EngineSourceDirectory);
				DirectoryReference.SetCurrentDirectory(Unreal.EngineSourceDirectory);

				// Register encodings from Net FW as this is required when using Ionic as we do in multiple toolchains
				Encoding.RegisterProvider(CodePagesEncodingProvider.Instance);

				// Get the type of the mode to execute, using a fast-path for the build mode.
				ModeDescriptor? ModeDescriptor = new(typeof(BuildMode), BuildMode.Options);
				if (Options.Mode != null)
				{
					// Try to get the correct mode
					IReadOnlyDictionary<string, ModeDescriptor> ModeNameToDesciptor = GetModes();
					if (!ModeNameToDesciptor.TryGetValue(Options.Mode, out ModeDescriptor))
					{
						Logger.LogError("No mode named '{Name}'. Available modes are:\n  {ModeList}", Options.Mode, String.Join("\n  ", ModeNameToDesciptor.Keys.Order()));
						return 1;
					}
				}
				CurrentMode = ModeDescriptor.ModeType.Name;

				// Get the options for which systems have to be initialized for this mode
				ModeOptions = ModeDescriptor.Options;

				// if we don't care about the trace listener, toss it now
				if ((ModeOptions & ToolModeOptions.UseStartupTraceListener) == 0)
				{
					Log.RemoveStartupTraceListener();
				}

				// Read the XML configuration files
				if ((ModeOptions & ToolModeOptions.XmlConfig) != 0)
				{
					using (Timeline.ScopeEvent("Apply XmlConfig"))
					using (GlobalTracer.Instance.BuildSpan("XmlConfig.ReadConfigFiles()").StartActive())
					{
						FileReference? XmlConfigCache = Arguments.GetFileReferenceOrDefault("-XmlConfigCache=", null);
						Utils.TryParseProjectFileArgument(Arguments, Logger, out FileReference? ProjectFile, false);
						XmlConfig.ReadConfigFiles(XmlConfigCache, ProjectFile?.Directory, Logger);
					}

					XmlConfig.ApplyTo(Options);
				}

				if (!Options.UseIpv6)
				{
					HttpMessageHandlers.UseIpv6DnsResolving = false;
				}

				// Start prefetching the contents of the engine folder
				if ((ModeOptions & ToolModeOptions.StartPrefetchingEngine) != 0)
				{
					using (GlobalTracer.Instance.BuildSpan("FileMetadataPrefetch.QueueEngineDirectory()").StartActive())
					{
						FileMetadataPrefetch.QueueEngineDirectory();
					}
				}

				Log.BackupLogFiles = Options.BackupLogFiles;
				Log.LogFileBackupCount = Options.LogFileBackupCount;
				bShowTimeline = Options.ShowTimeline;

				// Add the log writer if requested. When building a target, we'll create the writer for the default log file later.
				if (Options.LogFileName != null)
				{
					Log.AddFileWriter("LogTraceListener", Options.LogFileName);
				}

				// Initialize the telemetry service
				if (!String.IsNullOrEmpty(Options.TelemetrySession))
				{
					IsRecursive = true;
					SessionIdentifier = Options.TelemetrySession;
				}

				using (Timeline.ScopeEvent("AddTelemetry"))
				{
					TelemetryService.Get().AddTelemetryConfigProviders(Options.TelemetryProviders.Concat(Options.CmdTelemetryProviders));
					TelemetryService.Get().AddEndpointsFromConfig(Logger);
				}

				// Create a UbtRun file
				try
				{
					DirectoryReference RunsDir = DirectoryReference.Combine(Unreal.EngineDirectory, "Intermediate", "UbtRuns");
					Directory.CreateDirectory(RunsDir.FullName);
					string ModuleFileName = Process.GetCurrentProcess().MainModule?.FileName ?? "";
					if (!String.IsNullOrEmpty(ModuleFileName))
					{
						ModuleFileName = Path.GetFullPath(ModuleFileName);
					}
					FileReference RunFileTemp = FileReference.Combine(RunsDir, $"{Environment.ProcessId}_{ContentHash.MD5(Encoding.UTF8.GetBytes(ModuleFileName.ToUpperInvariant()))}");
					File.WriteAllLines(RunFileTemp.FullName, new string[] { ModuleFileName });
					RunFile = RunFileTemp;
				}
				catch
				{
				}

				// Set an environment variable for the current SessionIdentifier so scripts can access
				if (String.IsNullOrEmpty(Environment.GetEnvironmentVariable("UnrealBuildTool_SessionId")))
				{
					Environment.SetEnvironmentVariable("UnrealBuildTool_SessionId", $"{SessionIdentifier}");
				}

				// Override the temp directory
				try
				{
					// If the temp directory is already overridden from a parent process, do not override again
					if (String.IsNullOrEmpty(Environment.GetEnvironmentVariable("UnrealBuildTool_TMP")))
					{
						DirectoryReference OverrideTempDirectory = new DirectoryReference(Path.Combine(Path.GetTempPath(), "UnrealBuildTool"));
						if (Options.TempDirectory != null)
						{
							if (Directory.Exists(Options.TempDirectory))
							{
								OverrideTempDirectory = new DirectoryReference(Options.TempDirectory);
								if (OverrideTempDirectory.GetDirectoryName() != "UnrealBuildTool")
								{
									OverrideTempDirectory = DirectoryReference.Combine(OverrideTempDirectory, "UnrealBuildTool");
								}
							}
							else
							{
								Logger.LogWarning("Warning: TempDirectory override '{Override}' does not exist, using '{Temp}'", Options.TempDirectory, OverrideTempDirectory.FullName);
							}
						}

						OverrideTempDirectory = DirectoryReference.Combine(OverrideTempDirectory, ContentHash.MD5(Encoding.UTF8.GetBytes(Unreal.UnrealBuildToolDllPath.FullName)).ToString().Substring(0, 8));
						DirectoryReference.CreateDirectory(OverrideTempDirectory);

						Logger.LogDebug("Setting temp directory to '{Path}'", OverrideTempDirectory);
						Environment.SetEnvironmentVariable("UnrealBuildTool_TMP", OverrideTempDirectory.FullName);
						Environment.SetEnvironmentVariable("TMP", OverrideTempDirectory.FullName);
						Environment.SetEnvironmentVariable("TEMP", OverrideTempDirectory.FullName);

						// Deleting the directory is only safe in single instance mode, and only if requested
						if ((ModeOptions & ToolModeOptions.SingleInstance) != 0 && !Options.NoMutex && Options.DeleteTempDirectory)
						{
							Logger.LogDebug("Temp directory '{Path}' will be deleted on exit", OverrideTempDirectory);
							TempDirectory = OverrideTempDirectory;
						}
					}
				}
				catch
				{
				}

				// Acquire a lock for this branch
				if ((ModeOptions & ToolModeOptions.SingleInstance) != 0 && !Options.NoMutex)
				{
					using (GlobalTracer.Instance.BuildSpan("SingleInstanceMutex.Acquire()").StartActive())
					{
						string MutexName = GlobalSingleInstanceMutex.GetUniqueMutexForPath("UnrealBuildTool_Mutex", FileReference.FromString(Assembly.GetExecutingAssembly().Location));
						Mutex = Options.WaitMutex ? SingleInstanceMutex.Acquire(MutexName) : GlobalSingleInstanceMutex.AcquireNowOrThrow(MutexName);
					}
				}

				using (Timeline.ScopeEvent("RegisterPlatforms"))
				{
					// Register all the build platforms
					if ((ModeOptions & ToolModeOptions.BuildPlatforms) != 0)
					{
						using (GlobalTracer.Instance.BuildSpan("UEBuildPlatform.RegisterPlatforms()").StartActive())
						{
							UEBuildPlatform.RegisterPlatforms(false, false, ModeDescriptor.ModeType, ArgumentsArray, Logger);
						}
					}
					if ((ModeOptions & ToolModeOptions.BuildPlatformsHostOnly) != 0)
					{
						using (GlobalTracer.Instance.BuildSpan("UEBuildPlatform.RegisterPlatforms()").StartActive())
						{
							UEBuildPlatform.RegisterPlatforms(false, true, ModeDescriptor.ModeType, ArgumentsArray, Logger);
						}
					}
					if ((ModeOptions & ToolModeOptions.BuildPlatformsForValidation) != 0)
					{
						using (GlobalTracer.Instance.BuildSpan("UEBuildPlatform.RegisterPlatforms()").StartActive())
						{
							UEBuildPlatform.RegisterPlatforms(true, false, ModeDescriptor.ModeType, ArgumentsArray, Logger);
						}
					}
				}

				// Create the appropriate handler
				IToolMode Mode = (IToolMode)Activator.CreateInstance(ModeDescriptor.ModeType)!;

				// Execute the mode
				MainScope.Finish();
				//using ITimelineEvent ExecuteScope = Timeline.ScopeEvent(ModeType.Name);
				int Result = Mode.ExecuteAsync(Arguments, Logger).GetAwaiter().GetResult();
				ApplicationResult = (CompilationResult)Result;
				return Result;
			}
			catch (Exception Ex)
			{
				Ex.LogException(Logger);
				// CompilationResultException is used to return a propagate a specific exit code after an error has occurred.
				ApplicationResult = Ex.GetCompilationResult();
				return (int)ApplicationResult;
			}
			finally
			{
				// Cancel the prefetcher
				using (GlobalTracer.Instance.BuildSpan("FileMetadataPrefetch.Stop()").StartActive())
				{
					try
					{
						FileMetadataPrefetch.Stop();
					}
					catch
					{
					}
				}

				// Uncomment this to output a file that contains all files that UBT has scanned.
				// Useful when investigating why UBT takes time.
				//DirectoryItem.WriteDebugFileWithAllEnumeratedFiles(@"c:\temp\AllFiles.txt");

				using (Timeline.ScopeEvent("TelemetryService.FlushEvents"))
				{
					if (!IsRecursive)
					{
						TelemetryService.Get().RecordEvent(new TelemetryCompletedEvent(ArgumentsArray, StartTimeUtc, ApplicationResult, DateTime.UtcNow));
					}
					// Flush any remaining telemetry events
					TelemetryService.Get().FlushEvents();
				}

				Utils.LogWriteFileIfChangedActivity(Logger);

				// Stop listening to timeline events since Print adds an open <Root> event
				ubaTimelineEventScope.Dispose();

				// Print out all the performance info
				Timeline.Stop();
				bool bShowExecutionTime = ModeOptions.HasFlag(ToolModeOptions.ShowExecutionTime);
				LogLevel ExecutionLogLevel = bShowExecutionTime ? LogLevel.Information : LogLevel.Debug;
				LogLevel TimelineLogLevel = bShowExecutionTime && bShowTimeline && ApplicationResult == CompilationResult.Succeeded ? LogLevel.Information : LogLevel.Debug;

				Timeline.Print(TimeSpan.FromMilliseconds(100.0), TimeSpan.FromMilliseconds(200.0), /*MaxDepth*/ 4, TimelineLogLevel, Logger);

				Logger.Log(ExecutionLogLevel, "");
				if (ApplicationResult == CompilationResult.Succeeded)
				{
					Logger.Log(ExecutionLogLevel, "Result: {ApplicationResult}", ApplicationResult);
				}
				else
				{
					Logger.Log(ExecutionLogLevel, "Result: Failed ({ApplicationResult})", ApplicationResult);
				}
				Logger.Log(ExecutionLogLevel, "Total execution time: {Time:0.00} seconds", Timeline.Elapsed.TotalSeconds);

				Log.DisableThreaded();

				// Make sure we flush the logs however we exit
				Trace.Close();

				// Write any trace logs
				ubtScope.Dispose();
				Tracer?.Flush();

				// Delete the ubt run file
				if (RunFile != null)
				{
					try
					{
						File.Delete(RunFile.FullName);
					}
					catch
					{
					}
				}

				// Remove the the temp subdirectory. TempDirectory will only be set if running in single instance mode when Options.DeleteTempDirectory is enabled
				if (TempDirectory != null)
				{
					try
					{
						DirectoryReference.Delete(TempDirectory, true);
					}
					catch
					{
					}
				}

				// Dispose of the mutex. Must be done last to ensure that another process does not startup and start trying to write to the same log file.
				Mutex?.Dispose();
			}
		}

		private record class ModeDescriptor(Type ModeType, ToolModeOptions Options);
	}
}
