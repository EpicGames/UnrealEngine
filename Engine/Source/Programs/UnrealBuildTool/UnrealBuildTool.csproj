<Project Sdk="Microsoft.NET.Sdk">
  <Import Project="..\Shared\UnrealEngine.csproj.props" />

  <PropertyGroup>
    <OutputType>Exe</OutputType>
    <GenerateAssemblyInfo>false</GenerateAssemblyInfo>
    <GenerateTargetFrameworkAttribute>false</GenerateTargetFrameworkAttribute>
    <ApplicationManifest>app.manifest</ApplicationManifest>
    <SatelliteResourceLanguages>en</SatelliteResourceLanguages>
    <Nullable>enable</Nullable>
    <GenerateDocumentationFile>True</GenerateDocumentationFile>
  </PropertyGroup>

  <PropertyGroup Condition="Exists('..\..\..\Restricted\NotForLicensees\Source\Programs\UnrealBuildTool\ProjectFiles\VProject\VProjectFileGenerator.cs')">
    <DefineConstants>$(DefineConstants);__VPROJECT_AVAILABLE__</DefineConstants>
  </PropertyGroup>

  <ItemGroup>
    <Compile Include="..\Shared\MetaData.cs" Link="Properties\MetaData.cs" />
  </ItemGroup>

  <!-- For regular builds, include only the source files without Link attribute. -->
  <ItemGroup Condition="'$(DesignTimeBuild)' != 'true'">
    <Compile Include="../../../Platforms/*/Source/Programs/UnrealBuildTool/**/*.cs" />
    <Compile Include="../../../Restricted/*/Platforms/*/Source/Programs/UnrealBuildTool/**/*.cs" />
    <Compile Include="../../../Restricted/*/Source/Programs/UnrealBuildTool/**/*.cs" />
  </ItemGroup>

  <!-- For design-time builds (IDE), use Link attribute to organize files in Solution Explorer. -->
  <ItemGroup Condition="'$(DesignTimeBuild)' == 'true'">
    <!-- Platform extension and Restricted source files -->
    <Compile Include="../../../Platforms/*/Source/Programs/UnrealBuildTool/**/*.cs" Link="Platform/$([System.String]::Copy('%(RecursiveDir)').Replace('\', '/').Replace('Source/Programs/UnrealBuildTool/', ''))%(Filename)%(Extension)" />
    <Compile Include="../../../Restricted/*/Platforms/*/Source/Programs/UnrealBuildTool/**/*.cs" Link="Platform/$([System.String]::Copy('%(RecursiveDir)').Replace('\', '/').Replace('Source/Programs/UnrealBuildTool/', ''))%(Filename)%(Extension)" />
    <Compile Include="../../../Restricted/*/Source/Programs/UnrealBuildTool/**/*.cs" Link="Restricted/$([System.String]::Copy('%(RecursiveDir)').Replace('\', '/').Replace('Source/Programs/UnrealBuildTool/', ''))%(Filename)%(Extension)" />

    <!-- SDK config files -->
    <None Include="../../../Platforms/*/Config/*_SDK.json" Link="Platform/%(RecursiveDir)%(Filename)%(Extension)" />
    <None Include="../../../Config/*/*_SDK.json" Link="Platform/%(RecursiveDir)%(Filename)%(Extension)" />

    <!-- Non-compiled files for IDE display only. -->
    <!-- Various batch files and shell scripts that build or run UnrealBuildTool -->
    <None Include="../../../Build/BatchFiles/**/*" Link="BatchFiles/%(RecursiveDir)%(Filename)%(Extension)" />

    <!-- Various BuildConfiguration files -->
    <None Condition="Exists('..\..\..\Restricted\NotForLicensees\Programs\UnrealBuildTool\BuildConfiguration.xml')" Include="..\..\..\Restricted\NotForLicensees\Programs\UnrealBuildTool\BuildConfiguration.xml" Link="BuildConfiguration\BuildConfiguration.xml (Engine NotForLicensees)" />
    <None Condition="Exists('..\..\..\Saved\UnrealBuildTool\BuildConfiguration.xml')" Include="..\..\..\Saved\UnrealBuildTool\BuildConfiguration.xml" Link="BuildConfiguration\BuildConfiguration.xml (Engine Saved)" />
    <None Condition="Exists('$(ProgramData)\Unreal Engine\UnrealBuildTool\BuildConfiguration.xml')" Include="$(ProgramData)\Unreal Engine\UnrealBuildTool\BuildConfiguration.xml" Link="BuildConfiguration\BuildConfiguration.xml (Global ProgramData)" />
    <None Condition="Exists('$(AppData)\Unreal Engine\UnrealBuildTool\BuildConfiguration.xml')" Include="$(AppData)\Unreal Engine\UnrealBuildTool\BuildConfiguration.xml" Link="BuildConfiguration\BuildConfiguration.xml (Global AppData)" />
    <None Condition="Exists('$(LocalAppData)\Unreal Engine\UnrealBuildTool\BuildConfiguration.xml')" Include="$(LocalAppData)\Unreal Engine\UnrealBuildTool\BuildConfiguration.xml" Link="BuildConfiguration\BuildConfiguration.xml (Global LocalAppData)" />
    <None Condition="Exists('$(UserProfile)\Documents\Unreal Engine\UnrealBuildTool\BuildConfiguration.xml')" Include="$(UserProfile)\Documents\Unreal Engine\UnrealBuildTool\BuildConfiguration.xml" Link="BuildConfiguration\BuildConfiguration.xml (Global Documents)" />
    <None Condition="Exists('..\..\..\Intermediate\Build\UnrealBuildTool.Env.BuildConfiguration.xml')" Include="..\..\..\Intermediate\Build\UnrealBuildTool.Env.BuildConfiguration.xml" Link="BuildConfiguration\BuildConfiguration.xml (Environment)" />
  </ItemGroup>

  <ItemGroup>
    <!-- If you add a project reference, BuildUAT.bat/.sh and BuildUBT.bat/.sh need to be updated to also test for those projects.-->
    <ProjectReference Include="..\Shared\EpicGames.Build\EpicGames.Build.csproj" />
    <ProjectReference Include="..\Shared\EpicGames.Core\EpicGames.Core.csproj" />
    <ProjectReference Include="..\Shared\EpicGames.Horde\EpicGames.Horde.csproj" />
    <ProjectReference Include="..\Shared\EpicGames.IoHash\EpicGames.IoHash.csproj" />
    <ProjectReference Include="..\Shared\EpicGames.LogMatchers\EpicGames.LogMatchers.csproj" />
    <ProjectReference Include="..\Shared\EpicGames.OIDC\EpicGames.OIDC.csproj" />
    <ProjectReference Include="..\Shared\EpicGames.Serialization\EpicGames.Serialization.csproj" />
    <ProjectReference Include="..\Shared\EpicGames.UBA\EpicGames.UBA.csproj" />
    <ProjectReference Include="..\Shared\EpicGames.UHT\EpicGames.UHT.csproj" />
  </ItemGroup>

  <ItemGroup>
    <PackageReference Include="Microsoft.CodeAnalysis.CSharp" Version="4.13.0" />
    <PackageReference Include="Microsoft.Extensions.Logging.Console" Version="10.0.6" />
    <PackageReference Include="Microsoft.VisualStudio.Setup.Configuration.Interop" Version="3.14.2075" />
    <PackageReference Include="Microsoft.VisualStudio.SolutionPersistence" Version="1.0.52" />
    <PackageReference Include="OpenTracing" Version="0.12.1" />
    <PackageReference Include="System.CodeDom" Version="10.0.6" />
    <PackageReference Include="System.Management" Version="10.0.6" />
    <PackageReference Include="System.ServiceProcess.ServiceController" Version="10.0.6" />

    <!-- Required by Ionic -->
    <PackageReference Include="System.Security.Permissions" Version="10.0.6" />
  </ItemGroup>

</Project>
