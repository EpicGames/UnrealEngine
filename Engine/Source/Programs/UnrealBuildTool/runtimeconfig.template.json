{
	"configProperties": {
		"System.GC.Server": true,
		"System.GC.Concurrent": true,
		"System.GC.HeapCount": 32,
		"System.Globalization.Invariant": true,
		"System.Globalization.UseNls": true,
		"System.Runtime.TieredCompilation": false,
		"System.Runtime.TieredCompilation.QuickJit": false,
		"System.Runtime.TieredCompilation.QuickJitForLoops": false
	}
}